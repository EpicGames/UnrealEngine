[Startup]
;Enable Shader stages in Niagara
fx.UseShaderStages=1

[CoreRedirects]
+PropertyRedirects=(OldName="HairStrandsAsset",NewName="GroomAsset")
+ClassRedirects=(OldName="HairStrandsActor",NewName="/Script/HairStrandsCore.GroomActor")
+ClassRedirects=(OldName="HairStrandsAsset",NewName="/Script/HairStrandsCore.GroomAsset")
+ClassRedirects=(OldName="HairStrandsComponent",NewName="/Script/HairStrandsCore.GroomComponent",ValueChanges=(("HairStrandsComponent0","GroomComponent0")))
+ClassRedirects=(OldName="/Script/HairStrandsCore.OptimusGroomComponentSource",NewName="/Script/HairStrandsDeformer.OptimusGroomComponentSource")
+ClassRedirects=(OldName="/Script/HairStrandsCore.OptimusGroomDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomDataInterface")
+ClassRedirects=(OldName="/Script/HairStrandsCore.OptimusGroomExecDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomExecDataInterface")
+ClassRedirects=(OldName="/Script/HairStrandsCore.OptimusGroomGuideDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomGuideDataInterface")
+ClassRedirects=(OldName="/Script/HairStrandsCore.OptimusGroomWriteDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomWriteDataInterface")
+ClassRedirects=(OldName="/Script/HairStrandsEditor.GroomAssetImportData",NewName="/Script/HairStrandsCore.GroomAssetImportData")
+ClassRedirects=(OldName="/Script/HairStrandsEditor.GroomImportOptions",NewName="/Script/HairStrandsCore.GroomImportOptions")
+ClassRedirects=(OldName="/Script/HairStrandsNiagara.NiagaraDataInterfaceHairStrands",NewName="/Script/HairStrandsCore.NiagaraDataInterfaceHairStrands")
+ClassRedirects=(OldName="/Script/HairStrandsNiagara.NiagaraDataInterfaceVelocityGrid",NewName="/Script/HairStrandsCore.NiagaraDataInterfaceVelocityGrid")
+ClassRedirects=(OldName="/Script/HairStrandsNiagara.NiagaraDataInterfacePressureGrid",NewName="/Script/HairStrandsCore.NiagaraDataInterfacePressureGrid")
+PropertyRedirects=(OldName="GroomAsset.DeformedSkeletalMesh",NewName="GroomAsset.RiggedSkeletalMesh")
+PropertyRedirects=(OldName="HairDeformationSettings.bEnableDeformation",NewName="HairDeformationSettings.bEnableRigging")
+PropertyRedirects=(OldName="HairGroupsInterpolation.DeformationSettings",NewName="HairGroupsInterpolation.RiggingSettings")
+ClassRedirects=(OldName="OptimusGroomWriteDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomStrandsWriteDataInterface")
+ClassRedirects=(OldName="OptimusGroomDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomStrandsReadDataInterface")
+ClassRedirects=(OldName="OptimusGroomGuideDataInterface",NewName="/Script/HairStrandsDeformer.OptimusGroomGuidesReadDataInterface")
+ClassRedirects=(OldName="OptimusGroomComponentSource",NewName="/Script/HairStrandsDeformer.OptimusGroomAssetComponentSource")