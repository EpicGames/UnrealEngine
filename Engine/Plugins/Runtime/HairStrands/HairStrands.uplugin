{
	"FileVersion": 3,
	"Version": 2,
	"VersionName": "1.0",
	"FriendlyName": "Groom",
	"Description": "Rendering and simulation of grooms",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "HairStrandsCore",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "HairStrandsDeformer",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault",
			"PlatformDenyList": [
				"Android",
				"IOS"
			]
		},
		{
			"Name": "HairStrandsRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HairStrandsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HairCardGeneratorFramework",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HairStrandsDataflow",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HairStrandsSolver",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "ComputeFramework",
			"Enabled": true
		}
	]
}
