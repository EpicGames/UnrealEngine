// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsPack.ush"

uint {DataInterfaceName}_NumValues; 
uint {DataInterfaceName}_BaseValueIndex; 
uint {DataInterfaceName}_TotalValueCount;
uint {DataInterfaceName}_bHasIndexMapping;

StructuredBuffer<float> {DataInterfaceName}_ValueTypedData;

ByteAddressBuffer {DataInterfaceName}_IndexMappingBuffer;

// Utilities function

uint GetAttributeIndex_{DataInterfaceName}(uint GlobalIndex)
{
	const uint LocalIndex = GlobalIndex - {DataInterfaceName}_BaseValueIndex;
	return {DataInterfaceName}_bHasIndexMapping ?
		ReadHairSourceAttributeIndex({DataInterfaceName}_IndexMappingBuffer, LocalIndex, {DataInterfaceName}_bHasIndexMapping) : LocalIndex;
}

// Interface functions

uint ReadNumValues_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalValueCount;
}

uint ReadValueTypedUint_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	return (ValueIndex < {DataInterfaceName}_NumValues) ? (uint){DataInterfaceName}_ValueTypedData[ValueIndex] : 0;
}

bool ReadValueTypedBool_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	return (ValueIndex < {DataInterfaceName}_NumValues) ? (({DataInterfaceName}_ValueTypedData[ValueIndex] == 1.0f) ? true : false) : false;
}

int ReadValueTypedInt_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	return (ValueIndex < {DataInterfaceName}_NumValues) ? (int){DataInterfaceName}_ValueTypedData[ValueIndex] : 0;
}

int2 ReadValueTypedInt2_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 2 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? int2({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1]) : int2(0,0);
}

int3 ReadValueTypedInt3_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 3 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? int3({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1], {DataInterfaceName}_ValueTypedData[ValueOffset+2]) : int3(0,0,0);
}

int4 ReadValueTypedInt4_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 4 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? int4({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1],
					{DataInterfaceName}_ValueTypedData[ValueOffset+2], {DataInterfaceName}_ValueTypedData[ValueOffset+3]) : int4(0,0,0,0);
}

float ReadValueTypedFloat_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	return (ValueIndex < {DataInterfaceName}_NumValues) ? {DataInterfaceName}_ValueTypedData[ValueIndex] : 0.0f;
}

float2 ReadValueTypedFloat2_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);;
	const int ValueOffset = 2 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? float2({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1]) : float2(0.0f,0.0f);

}

float3 ReadValueTypedFloat3_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 3 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? float3({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1], {DataInterfaceName}_ValueTypedData[ValueOffset+2]) : float3(0.0f,0.0f,0.0f);
}

float4 ReadValueTypedFloat4_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 4 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? float4({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1],
					{DataInterfaceName}_ValueTypedData[ValueOffset+2], {DataInterfaceName}_ValueTypedData[ValueOffset+3]) : float4(0.0f,0.0f,0.0f,0.0f);
}

float4x4 ReadValueTypedFloat4x4_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 16 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? float4x4({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1],
						{DataInterfaceName}_ValueTypedData[ValueOffset+2], {DataInterfaceName}_ValueTypedData[ValueOffset+3],
						{DataInterfaceName}_ValueTypedData[ValueOffset+4], {DataInterfaceName}_ValueTypedData[ValueOffset+5],
						{DataInterfaceName}_ValueTypedData[ValueOffset+6], {DataInterfaceName}_ValueTypedData[ValueOffset+7],
						{DataInterfaceName}_ValueTypedData[ValueOffset+8], {DataInterfaceName}_ValueTypedData[ValueOffset+9],
						{DataInterfaceName}_ValueTypedData[ValueOffset+10], {DataInterfaceName}_ValueTypedData[ValueOffset+11],
						{DataInterfaceName}_ValueTypedData[ValueOffset+12], {DataInterfaceName}_ValueTypedData[ValueOffset+13],
						{DataInterfaceName}_ValueTypedData[ValueOffset+14], {DataInterfaceName}_ValueTypedData[ValueOffset+15]) : float4x4(1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f,0.0f,0.0f,0.0f,1.0f);

}

float3x4 ReadValueTypedFloat3x4_{DataInterfaceName}(uint ValueIndex)
{
	ValueIndex = GetAttributeIndex_{DataInterfaceName}(ValueIndex);
	const int ValueOffset = 12 * ValueIndex;
	return (ValueIndex < {DataInterfaceName}_NumValues) ? float3x4({DataInterfaceName}_ValueTypedData[ValueOffset], {DataInterfaceName}_ValueTypedData[ValueOffset+1],
						{DataInterfaceName}_ValueTypedData[ValueOffset+2], {DataInterfaceName}_ValueTypedData[ValueOffset+3],
						{DataInterfaceName}_ValueTypedData[ValueOffset+4], {DataInterfaceName}_ValueTypedData[ValueOffset+5],
						{DataInterfaceName}_ValueTypedData[ValueOffset+6], {DataInterfaceName}_ValueTypedData[ValueOffset+7],
						{DataInterfaceName}_ValueTypedData[ValueOffset+8], {DataInterfaceName}_ValueTypedData[ValueOffset+9],
						{DataInterfaceName}_ValueTypedData[ValueOffset+10], {DataInterfaceName}_ValueTypedData[ValueOffset+11]) : float3x4(1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f);
}



