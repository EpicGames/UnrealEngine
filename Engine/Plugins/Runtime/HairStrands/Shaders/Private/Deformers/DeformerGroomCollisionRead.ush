// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"

uint {DataInterfaceName}_NumVertices; 
uint {DataInterfaceName}_NumTriangles;
uint {DataInterfaceName}_VertexOffset; 
uint {DataInterfaceName}_TriangleOffset;
uint {DataInterfaceName}_TotalNumVertices; 
uint {DataInterfaceName}_TotalNumTriangles;
uint {DataInterfaceName}_BaseVertexIndex; 
uint {DataInterfaceName}_BaseTriangleIndex;

Buffer<float> {DataInterfaceName}_VertexGlobalPositions;
Buffer<uint> {DataInterfaceName}_TriangleVertexIndices;

// Interface functions

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalNumVertices;
}

uint ReadNumTriangles_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalNumTriangles;
}

float3 ReadVertexGlobalPosition_{DataInterfaceName}(uint VertexIndex)
{
	VertexIndex -= {DataInterfaceName}_BaseVertexIndex;
	const uint VertexOffset = 3 * (VertexIndex + {DataInterfaceName}_VertexOffset);
	return (VertexIndex < {DataInterfaceName}_NumVertices) ?
		float3({DataInterfaceName}_VertexGlobalPositions[VertexOffset], {DataInterfaceName}_VertexGlobalPositions[VertexOffset+1], {DataInterfaceName}_VertexGlobalPositions[VertexOffset+2]) : float3(0,0,0);
}

uint3 ReadTriangleVertexIndices_{DataInterfaceName}(uint TriangleIndex)
{
	TriangleIndex -= {DataInterfaceName}_BaseTriangleIndex;
	const uint TriangleOffset = 3*TriangleIndex + {DataInterfaceName}_TriangleOffset;
	return (TriangleIndex < {DataInterfaceName}_NumTriangles) ?
		uint3({DataInterfaceName}_TriangleVertexIndices[TriangleOffset]+{DataInterfaceName}_BaseVertexIndex-{DataInterfaceName}_VertexOffset,
			  {DataInterfaceName}_TriangleVertexIndices[TriangleOffset+1]+{DataInterfaceName}_BaseVertexIndex-{DataInterfaceName}_VertexOffset,
			  {DataInterfaceName}_TriangleVertexIndices[TriangleOffset+2]+{DataInterfaceName}_BaseVertexIndex-{DataInterfaceName}_VertexOffset) : uint3(0,0,0);
}

