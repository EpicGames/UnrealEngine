// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"

int3 {DataInterfaceName}_NumThreads;

int3 ReadNumThreads_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumThreads;
}
