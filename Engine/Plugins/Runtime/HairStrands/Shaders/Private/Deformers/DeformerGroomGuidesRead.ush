// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsVertexFactoryCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsAttributeCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsBindingCommon.ush"

uint {DataInterfaceName}_PointCount; 
uint {DataInterfaceName}_CurveCount;
uint {DataInterfaceName}_BasePointIndex; 
uint {DataInterfaceName}_BaseCurveIndex;
uint {DataInterfaceName}_TotalPointCount; 
uint {DataInterfaceName}_TotalCurveCount;
uint {DataInterfaceName}_ObjectIndex;
float3 {DataInterfaceName}_RestPositionOffset;

ByteAddressBuffer {DataInterfaceName}_PointRestPositions;
ByteAddressBuffer {DataInterfaceName}_CurvePointOffsets;
ByteAddressBuffer {DataInterfaceName}_PointCurveIndices;

float4x4 {DataInterfaceName}_ObjectRestTransform;
float4x4 {DataInterfaceName}_ObjectDeformedTransform;

float3x4 BuildRigidTransform(float4x4 RigidTransform)
{
	return float3x4(RigidTransform[0], RigidTransform[1], RigidTransform[2]);
}
 
#if ENABLE_SKINNED_TRANSFORM == 1
Buffer<float4> {DataInterfaceName}_TriangleRestPositions;
Buffer<float4> {DataInterfaceName}_TriangleDeformedPositions;
Buffer<uint> {DataInterfaceName}_CurveBarycentricCoordinates;
Buffer<uint> {DataInterfaceName}_CurveTriangleIndices;

float3x4 BuildGuidesSkinnedTransform(uint CurveIndex, Buffer<float4> TrianglePositions,
	Buffer<uint> BarycentricCoordinates, Buffer<uint> TriangleIndices)
{
	const float3 BarycentricCoord = UnpackBarycentrics(BarycentricCoordinates[CurveIndex]);
	const uint TriangleIndex = TriangleIndices[CurveIndex];

	const FHairMeshBasis MeshBasis = GetHairMeshBasis(TriangleIndex, TrianglePositions, BarycentricCoord);
	
	return float3x4(MeshBasis.BaseT.x, MeshBasis.BaseB.x, MeshBasis.BaseN.x, MeshBasis.BaseO.x,
					MeshBasis.BaseT.y, MeshBasis.BaseB.y, MeshBasis.BaseN.y, MeshBasis.BaseO.y,
					MeshBasis.BaseT.z, MeshBasis.BaseB.z, MeshBasis.BaseN.z, MeshBasis.BaseO.z);
}
#endif

// Interface functions

uint ReadNumPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalPointCount;
}

uint ReadNumCurves_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalCurveCount;
}

uint ReadObjectIndex_{DataInterfaceName}()
{
	return {DataInterfaceName}_ObjectIndex;
}

float3 ReadPointRestPosition_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return (PointIndex < {DataInterfaceName}_PointCount) ? ReadHairControlPoint({DataInterfaceName}_PointRestPositions, PointIndex, {DataInterfaceName}_RestPositionOffset).Position : float3(0,0,0);
}

float ReadCurvePointOffset_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	return (CurveIndex < {DataInterfaceName}_CurveCount) ? ReadHairCurve({DataInterfaceName}_CurvePointOffsets, CurveIndex).PointIndex + {DataInterfaceName}_BasePointIndex : 0 + {DataInterfaceName}_BasePointIndex;
}

float ReadCurveNumPoints_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	return (CurveIndex < {DataInterfaceName}_CurveCount) ? ReadHairCurve({DataInterfaceName}_CurvePointOffsets, CurveIndex).PointCount : 0;
}

uint ReadPointCurveIndex_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return (PointIndex < {DataInterfaceName}_PointCount) ? ReadHairPointToCurveIndex({DataInterfaceName}_PointCurveIndices, PointIndex) + {DataInterfaceName}_BaseCurveIndex : {DataInterfaceName}_BaseCurveIndex;
}

float3x4 ReadCurveRestTransform_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;	
#if ENABLE_SKINNED_TRANSFORM == 1
		return BuildGuidesSkinnedTransform(CurveIndex, {DataInterfaceName}_TriangleRestPositions,
			{DataInterfaceName}_CurveBarycentricCoordinates, {DataInterfaceName}_CurveTriangleIndices);
#endif

#if ENABLE_SKINNED_TRANSFORM == 0
		return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);
#endif
}

float3x4 ReadCurveDeformedTransform_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
#if ENABLE_SKINNED_TRANSFORM == 1
		return BuildGuidesSkinnedTransform(CurveIndex, {DataInterfaceName}_TriangleDeformedPositions,
			{DataInterfaceName}_CurveBarycentricCoordinates, {DataInterfaceName}_CurveTriangleIndices);
#endif

#if ENABLE_SKINNED_TRANSFORM == 0
		return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);
#endif
}

float3x4 ReadObjectRestTransform_{DataInterfaceName}()
{
	return BuildRigidTransform({DataInterfaceName}_ObjectRestTransform);
}

float3x4 ReadObjectDeformedTransform_{DataInterfaceName}()
{
	return BuildRigidTransform({DataInterfaceName}_ObjectDeformedTransform);
}