// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsVertexFactoryCommon.ush"

uint {DataInterfaceName}_RegisteredIndex; 
uint {DataInterfaceName}_PointCount;
uint {DataInterfaceName}_TotalPointCount;
uint {DataInterfaceName}_BasePointIndex; 

StructuredBuffer<float4> {DataInterfaceName}_DeformedPositionOffset;
ByteAddressBuffer {DataInterfaceName}_PointRestPositions;
RWByteAddressBuffer {DataInterfaceName}_PointDeformedPositions;

// Interface helpers

float3 {DataInterfaceName}_GetDeformedPositionOffset() 
{ 
	return ReadSimPositionOffset({DataInterfaceName}_DeformedPositionOffset,{DataInterfaceName}_RegisteredIndex); 
}

// Interface functions

uint ReadNumPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalPointCount;
}

void WritePointDeformedPosition_{DataInterfaceName}(uint PointIndex, float3 PointPosition)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount)
	{
		const FPackedHairPosition PackedCP = ReadPackedHairPosition({DataInterfaceName}_PointRestPositions, WriteIndex);
		WritePackedHairControlPointPosition({DataInterfaceName}_PointDeformedPositions, WriteIndex, PackedCP, PointPosition, {DataInterfaceName}_GetDeformedPositionOffset());
	}
}
