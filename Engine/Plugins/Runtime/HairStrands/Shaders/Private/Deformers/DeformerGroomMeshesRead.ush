// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"

uint {DataInterfaceName}_NumBones; 

StructuredBuffer<float4> {DataInterfaceName}_BoneTransformMatrices;

// Interface functions

uint ReadNumBones_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumBones;
}

float3x4 ReadBoneTransformMatrix_{DataInterfaceName}(uint BoneIndex)
{
	return (BoneIndex < {DataInterfaceName}_NumBones) ?
		float3x4({DataInterfaceName}_BoneTransformMatrices[3*BoneIndex], {DataInterfaceName}_BoneTransformMatrices[3*BoneIndex+1], {DataInterfaceName}_BoneTransformMatrices[3*BoneIndex+2]) : float3x4(1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f);
}

float3x4 ReadBindTransformMatrix_{DataInterfaceName}(uint BoneIndex)
{
	return (BoneIndex < {DataInterfaceName}_NumBones) ?
		float3x4({DataInterfaceName}_BoneTransformMatrices[({DataInterfaceName}_NumBones + BoneIndex) * 3], 
				 {DataInterfaceName}_BoneTransformMatrices[({DataInterfaceName}_NumBones + BoneIndex) * 3 + 1], 
				 {DataInterfaceName}_BoneTransformMatrices[({DataInterfaceName}_NumBones + BoneIndex) * 3 + 2]) : 
		float3x4(1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f,0.0f,0.0f,0.0f,1.0f,0.0f);
}



