// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"

#ifdef RAYTRACING_SOLVER_BINDING

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/ShadingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"

#ifndef DATA_INTERFACE_ACCELERATION_STRUCTURE_ONCE
#define DATA_INTERFACE_ACCELERATION_STRUCTURE_ONCE 1

struct FAccelerationStructure
{
	RaytracingAccelerationStructure	WorldTLAS;
	int ValidTLAS;
};

#endif // DATA_INTERFACE_ACCELERATION_STRUCTURE_ONCE

RaytracingAccelerationStructure	{DataInterfaceName}_WorldAccelerationStructure;

FAccelerationStructure ReadWorldAccelerationStructure_{DataInterfaceName}()
{
	FAccelerationStructure AccelerationStructure;
	AccelerationStructure.ValidTLAS = true;
	AccelerationStructure.WorldTLAS = {DataInterfaceName}_WorldAccelerationStructure;
	return AccelerationStructure;
}

#endif

uint {DataInterfaceName}_NumSolverPoints;
uint {DataInterfaceName}_NumSolverCurves;
uint {DataInterfaceName}_NumSolverObjects;

uint {DataInterfaceName}_NumDynamicPoints;
uint {DataInterfaceName}_NumDynamicCurves;

uint {DataInterfaceName}_NumKinematicPoints;
uint {DataInterfaceName}_NumKinematicCurves;

float3 {DataInterfaceName}_BoundingBoxMin;
float3 {DataInterfaceName}_BoundingBoxMax;

StructuredBuffer<uint> {DataInterfaceName}_ObjectCurveOffsets;
StructuredBuffer<uint> {DataInterfaceName}_ObjectPointOffsets;

StructuredBuffer<uint> {DataInterfaceName}_ObjectNumCurves;
StructuredBuffer<uint> {DataInterfaceName}_ObjectNumPoints;

StructuredBuffer<int> {DataInterfaceName}_PointObjectIndices;
StructuredBuffer<int> {DataInterfaceName}_CurveObjectIndices;

StructuredBuffer<int> {DataInterfaceName}_DynamicPointIndices;
StructuredBuffer<int> {DataInterfaceName}_DynamicCurveIndices;

StructuredBuffer<int> {DataInterfaceName}_KinematicPointIndices;
StructuredBuffer<int> {DataInterfaceName}_KinematicCurveIndices;

StructuredBuffer<uint> {DataInterfaceName}_ObjectDistanceLods;

// Interface functions

uint ReadNumSolverPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumSolverPoints;
}

uint ReadNumSolverCurves_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumSolverCurves;
}

uint ReadNumSolverObjects_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumSolverObjects;
}

uint ReadNumDynamicPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumDynamicPoints;
}

uint ReadNumDynamicCurves_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumDynamicCurves;
}

uint ReadNumKinematicPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumKinematicPoints;
}

uint ReadNumKinematicCurves_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumKinematicCurves;
}

float3 ReadBoundingBoxMin_{DataInterfaceName}()
{
	return {DataInterfaceName}_BoundingBoxMin;
}

float3 ReadBoundingBoxMax_{DataInterfaceName}()
{
	return {DataInterfaceName}_BoundingBoxMax;
}

uint ReadObjectPointOffset_{DataInterfaceName}(uint ObjectIndex)
{
	return (ObjectIndex < ({DataInterfaceName}_NumSolverObjects+1)) ? {DataInterfaceName}_ObjectPointOffsets[ObjectIndex] : 0;
}

uint ReadObjectCurveOffset_{DataInterfaceName}(uint ObjectIndex)
{
	return (ObjectIndex < ({DataInterfaceName}_NumSolverObjects+1)) ? {DataInterfaceName}_ObjectCurveOffsets[ObjectIndex] : 0;
}

uint ReadObjectNumPoints_{DataInterfaceName}(uint ObjectIndex)
{
	return (ObjectIndex < {DataInterfaceName}_NumSolverObjects) ? {DataInterfaceName}_ObjectNumPoints[ObjectIndex] : 0;
}

uint ReadObjectNumCurves_{DataInterfaceName}(uint ObjectIndex)
{
	return (ObjectIndex < {DataInterfaceName}_NumSolverObjects) ? {DataInterfaceName}_ObjectNumCurves[ObjectIndex] : 0;
}

uint ReadObjectDistanceLod_{DataInterfaceName}(uint ObjectIndex)
{
	return (ObjectIndex < {DataInterfaceName}_NumSolverObjects) ? {DataInterfaceName}_ObjectDistanceLods[ObjectIndex] : 0;
}

int ReadPointObjectIndex_{DataInterfaceName}(uint PointIndex)
{
	return (PointIndex < {DataInterfaceName}_NumSolverPoints)? {DataInterfaceName}_PointObjectIndices[PointIndex] : -1;
}

int ReadCurveObjectIndex_{DataInterfaceName}(uint CurveIndex)
{
	return (CurveIndex < {DataInterfaceName}_NumSolverCurves) ? {DataInterfaceName}_CurveObjectIndices[CurveIndex] : -1;
}

int ReadDynamicPointIndex_{DataInterfaceName}(uint PointIndex)
{
	return (PointIndex < {DataInterfaceName}_NumDynamicPoints) ? {DataInterfaceName}_DynamicPointIndices[PointIndex] : -1;
}

int ReadKinematicPointIndex_{DataInterfaceName}(uint PointIndex)
{
	return (PointIndex < {DataInterfaceName}_NumKinematicPoints) ? {DataInterfaceName}_KinematicPointIndices[PointIndex] : -1;
}

int ReadDynamicCurveIndex_{DataInterfaceName}(uint CurveIndex)
{
	return (CurveIndex < {DataInterfaceName}_NumDynamicCurves) ? {DataInterfaceName}_DynamicCurveIndices[CurveIndex] : -1;
}

int ReadKinematicCurveIndex_{DataInterfaceName}(uint CurveIndex)
{
	return (CurveIndex < {DataInterfaceName}_NumKinematicCurves) ? {DataInterfaceName}_KinematicCurveIndices[CurveIndex] : -1;
}





