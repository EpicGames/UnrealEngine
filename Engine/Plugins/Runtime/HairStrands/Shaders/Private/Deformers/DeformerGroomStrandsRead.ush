// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsVertexFactoryCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsAttributeCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsBindingCommon.ush"

HAIR_STRANDS_INSTANCE_PARAMETERS({DataInterfaceName})

uint {DataInterfaceName}_BasePointIndex;
uint {DataInterfaceName}_BaseCurveIndex;
uint {DataInterfaceName}_BaseGuidePointIndex;
uint {DataInterfaceName}_TotalPointCount;
uint {DataInterfaceName}_TotalCurveCount;

#define HAIR_STRANDS_ATTRIBUTE_ACCESSORS(Name) {DataInterfaceName}_##Name
#include "/Engine/Private/HairStrands/HairStrandsAttributeTemplate.ush"

ByteAddressBuffer {DataInterfaceName}_CurveInterpolationBuffer;
ByteAddressBuffer {DataInterfaceName}_PointInterpolationBuffer;

#if ENABLE_SKINNED_TRANSFORM == 0
float4x4 {DataInterfaceName}_RigidRestTransform;
float4x4 {DataInterfaceName}_RigidDeformedTransform;

float3x4 BuildRigidTransform(float4x4 RigidTransform)
{
	return float3x4(RigidTransform[0], RigidTransform[1], RigidTransform[2]);
}
#endif

#if ENABLE_SKINNED_TRANSFORM == 1
Buffer<float4> {DataInterfaceName}_TriangleRestPositions;
Buffer<float4> {DataInterfaceName}_TriangleDeformedPositions;
Buffer<uint> {DataInterfaceName}_CurveBarycentricCoordinates;
Buffer<uint> {DataInterfaceName}_CurveTriangleIndices;

float3x4 BuildStrandsSkinnedTransform(uint CurveIndex, Buffer<float4> TrianglePositions,
	Buffer<uint> CurveBarycentricCoordinates, Buffer<uint> CurveTriangleIndices)
{
	const float3 BarycentricCoord = UnpackBarycentrics(CurveBarycentricCoordinates[CurveIndex]);
	const uint TriangleIndex = CurveTriangleIndices[CurveIndex];

	const FHairMeshBasis MeshBasis = GetHairMeshBasis(TriangleIndex, TrianglePositions, BarycentricCoord);
	
	return float3x4(MeshBasis.BaseT.x, MeshBasis.BaseB.x, MeshBasis.BaseN.x, MeshBasis.BaseO.x,
					MeshBasis.BaseT.y, MeshBasis.BaseB.y, MeshBasis.BaseN.y, MeshBasis.BaseO.y,
					MeshBasis.BaseT.z, MeshBasis.BaseB.z, MeshBasis.BaseN.z, MeshBasis.BaseO.z);
}
#endif


// Helper functions

FHairControlPoint ReadPointData(uint PointIndex)
{
	const uint MaxControlPoints = {DataInterfaceName}_PointCount;
	if (PointIndex >= MaxControlPoints)
	{
		return (FHairControlPoint)0;
	}

	// Hair deformer work in local space so no need for position offset
	return ReadHairControlPoint(
		{DataInterfaceName}_PositionBuffer,
		PointIndex, 
		{DataInterfaceName}_GetHairInstancePositionOffset(),
		{DataInterfaceName}_Radius, 
		{DataInterfaceName}_RootScale, 
		{DataInterfaceName}_TipScale);
}

FHairCurve ReadCurveData(uint CurveIndex)
{
	const uint MaxCurve = {DataInterfaceName}_CurveCount;
	if (CurveIndex >= MaxCurve)
	{
		return (FHairCurve)0;
	}

	return ReadHairCurve({DataInterfaceName}_CurveBuffer, CurveIndex);
}

// Interface functions

uint ReadNumPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalPointCount;
}

uint ReadNumCurves_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalCurveCount;
}

float3 ReadPointRestPosition_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return ReadPointData(PointIndex).Position;
}

float ReadPointRadius_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return ReadPointData(PointIndex).WorldRadius;
}

float ReadPointCoordU_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return ReadPointData(PointIndex).UCoord;
}

float ReadPointLength_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsDimensions(PointIndex, 0).x;
}

float2 ReadPointRootUV_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsRootUV(PointIndex);
}

float ReadPointSeed_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsSeed(PointIndex);
}

uint ReadPointClumpId_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsClumpID(PointIndex).x;
}

float3 ReadPointColor_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsColor(PointIndex, 0);
}

float ReadPointRoughness_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsRoughness(PointIndex, 0);
}

float ReadPointAO_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return GetHairStrandsAO(PointIndex, 0);
}

uint ReadCurvePointOffset_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	return ReadCurveData(CurveIndex).PointIndex + {DataInterfaceName}_BasePointIndex;
}

uint ReadCurveNumPoints_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	return ReadCurveData(CurveIndex).PointCount;
}

uint ReadPointCurveIndex_{DataInterfaceName}(uint PointIndex)
{
	return {DataInterfaceName}_BaseCurveIndex;
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return ReadHairPointToCurveIndex({DataInterfaceName}_PointToCurveBuffer, PointIndex) + {DataInterfaceName}_BaseCurveIndex;
}

int2 ReadPointGuideIndices_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	// Find the rendering curve index
	const uint CurveIndex = ReadHairPointToCurveIndex({DataInterfaceName}_PointToCurveBuffer, PointIndex);

	// Find the curve and point interpolation data
	const FGuideCurveData CurveData = UnpackGuideCurveData({DataInterfaceName}_CurveInterpolationBuffer, CurveIndex, {DataInterfaceName}_bSingleGuide);
	const FGuidePointData PointData = UnpackGuidePointData({DataInterfaceName}_PointInterpolationBuffer, PointIndex, {DataInterfaceName}_bSingleGuide);

	// Build the sim point index
	const uint2 GuidePointIndices = CurveData.RootPointIndices + PointData.PointIndices;
	return int2(GuidePointIndices.x + {DataInterfaceName}_BaseGuidePointIndex, GuidePointIndices.y + {DataInterfaceName}_BaseGuidePointIndex);
}

float2 ReadPointGuideWeights_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	// Find the rendering curve index
	const uint CurveIndex = ReadHairPointToCurveIndex({DataInterfaceName}_PointToCurveBuffer, PointIndex);

	// Find the curve and point interpolation data
	const FGuideCurveData CurveData = UnpackGuideCurveData({DataInterfaceName}_CurveInterpolationBuffer, CurveIndex, {DataInterfaceName}_bSingleGuide);

	// Build the sim point index
	return  CurveData.CurveWeights;
}

float3x4 ReadCurveRestTransform_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
#if ENABLE_SKINNED_TRANSFORM == 1
		return BuildStrandsSkinnedTransform(CurveIndex, {DataInterfaceName}_TriangleRestPositions,
			{DataInterfaceName}_CurveBarycentricCoordinates, {DataInterfaceName}_CurveTriangleIndices);
#endif

#if ENABLE_SKINNED_TRANSFORM == 0
		return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);
#endif
}

float3x4 ReadCurveDeformedTransform_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
#if ENABLE_SKINNED_TRANSFORM == 1
		return BuildStrandsSkinnedTransform(CurveIndex, {DataInterfaceName}_TriangleDeformedPositions,
			{DataInterfaceName}_CurveBarycentricCoordinates, {DataInterfaceName}_CurveTriangleIndices);
#endif

#if ENABLE_SKINNED_TRANSFORM == 0
		return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);
#endif
}

uint ReadCurveSourceIndex_{DataInterfaceName}(uint CurveIndex)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	return ReadHairSourceCurveIndex({DataInterfaceName}_CurveMappingBuffer, CurveIndex, {DataInterfaceName}_bCurveRemapping);
}

uint ReadPointSourceIndex_{DataInterfaceName}(uint PointIndex)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	return ReadHairSourcePointIndex({DataInterfaceName}_PointMappingBuffer, PointIndex, {DataInterfaceName}_bPointRemapping);
}