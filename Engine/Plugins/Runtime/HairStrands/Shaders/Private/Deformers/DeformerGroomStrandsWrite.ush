// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/ComputeKernelCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsVertexFactoryCommon.ush"

HAIR_STRANDS_INSTANCE_PARAMETERS({DataInterfaceName})

uint {DataInterfaceName}_BasePointIndex;
uint {DataInterfaceName}_BaseCurveIndex;
uint {DataInterfaceName}_TotalPointCount;
uint {DataInterfaceName}_TotalCurveCount;

// Note: resources are explicitly bound here (instance of using HAIR_STRANDS_INSTANCE_PARAMETERS's resources) 
// because we write out data which would be bound as SRV otherwise (position, attributes, ...)
StructuredBuffer<float4> {DataInterfaceName}_DeformedPositionOffset;
ByteAddressBuffer		{DataInterfaceName}_PointRestPositions;
RWByteAddressBuffer		{DataInterfaceName}_OutPointDeformedPositions;
RWByteAddressBuffer 	{DataInterfaceName}_OutPointAttributeBuffer;

// Helper functions

void InternalSetHairStrandsAttributes(uint Index, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, RWByteAddressBuffer InAttributeBuffer, uint In, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	// Adapt index and offset
	const uint ChunkIndex   = Index >> InIndexToChunkDivAsShift;
	Index                  -= ChunkIndex * InChunkElementCount;
	AttributeOffsetInBytes += ChunkIndex * InChunkStrideInBytes;

	// Ensure the reading address is 4-byte aligned
	const uint Address 					= AttributeOffsetInBytes + Index * AttributeStrideInBytes;
	const uint Address_4BytesAligned 	= Address & (~0x3);
	const uint ElementIndex 			= Address - Address_4BytesAligned;

	// Create a mask where the 
	const uint Mask 		  = (1u<<(AttributeStrideInBytes*8u))-1u;
	const uint AttributeMask  = Mask << (ElementIndex*8);
	const uint AttributeValue = In   << (ElementIndex*8);

	// Clear previous value, and then write the value
	InAttributeBuffer.InterlockedAnd(Address_4BytesAligned, ~AttributeMask);
	InAttributeBuffer.InterlockedOr(Address_4BytesAligned, AttributeValue);
}

void InternalSetHairStrandsAttributes32(uint Index, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, RWByteAddressBuffer InAttributeBuffer, uint In, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	// Adapt index and offset
	const uint ChunkIndex   = Index >> InIndexToChunkDivAsShift;
	Index                  -= ChunkIndex * InChunkElementCount;
	AttributeOffsetInBytes += ChunkIndex * InChunkStrideInBytes;

	// Ensure the reading address is 4-byte aligned
	const uint Address = AttributeOffsetInBytes + Index * AttributeStrideInBytes;
	const uint Address_4BytesAligned = Address & (~0x3);

	// Clear previous value, and then write the value
	InAttributeBuffer.Store(Address_4BytesAligned, In);
}

// Interface functions

float3 {DataInterfaceName}_GetDeformedPositionOffset() 
{ 
	return ReadRenPositionOffset({DataInterfaceName}_DeformedPositionOffset,{DataInterfaceName}_RegisteredIndex); 
}

uint ReadNumPoints_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalPointCount;
}

uint ReadNumCurves_{DataInterfaceName}()
{
	return {DataInterfaceName}_TotalCurveCount;
}

void WritePointDeformedPosition_{DataInterfaceName}(uint PointIndex, float3 Position)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount)
	{
		const FPackedHairPosition PackedCP = ReadPackedHairPosition({DataInterfaceName}_PointRestPositions, WriteIndex);
		WritePackedHairControlPointPosition({DataInterfaceName}_OutPointDeformedPositions, WriteIndex, PackedCP, Position, {DataInterfaceName}_GetDeformedPositionOffset());
	}
}

void WritePointRadius_{DataInterfaceName}(uint PointIndex, float Radius)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount)
	{
		const FPackedHairPosition PackedCP = ReadPackedHairPosition({DataInterfaceName}_PointRestPositions, WriteIndex);
		const FPackedHairPosition NewPackedCP = PackHairControlPointRadius(PackedCP, Radius, {DataInterfaceName}_Radius);
		WritePackedHairPosition({DataInterfaceName}_OutPointDeformedPositions, WriteIndex, NewPackedCP);
	}
}

void WritePointPositionRadius_{DataInterfaceName}(uint PointIndex, float4 PositionAndRadius)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount)
	{
		const FPackedHairPosition PackedCP = ReadPackedHairPosition({DataInterfaceName}_PointRestPositions, WriteIndex);
		FPackedHairPosition NewPackedCP = 0;
		NewPackedCP = PackHairControlPointPosition(PackedCP, PositionAndRadius.xyz, {DataInterfaceName}_GetDeformedPositionOffset());
		NewPackedCP = PackHairControlPointRadius(NewPackedCP, PositionAndRadius.w, {DataInterfaceName}_Radius);
		WritePackedHairPosition({DataInterfaceName}_OutPointDeformedPositions, WriteIndex, NewPackedCP);
	}
}

void WriteCurveRootUV_{DataInterfaceName}(uint CurveIndex, float2 RootUV)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	
	const uint WriteIndex = CurveIndex;
	if (WriteIndex < {DataInterfaceName}_CurveCount && HasHairStrandsRootUV())
	{
		InternalSetHairStrandsAttributes32(CurveIndex, HAIR_CURVE_ATTRIBUTE_OFFSET_ROOTUV({DataInterfaceName}_CurveAttributeOffsets), HAIR_CURVE_ATTRIBUTE_STRIDE_ROOTUV, {DataInterfaceName}_OutPointAttributeBuffer, PackHairRootUV(RootUV), {DataInterfaceName}_CurveAttributeIndexToChunkDivAsShift, {DataInterfaceName}_CurveAttributeChunkElementCount, {DataInterfaceName}_CurveAttributeChunkStrideInBytes);
	}
}

void WriteCurveSeed_{DataInterfaceName}(uint CurveIndex, float Seed)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	
	const uint WriteIndex = CurveIndex;
	if (WriteIndex < {DataInterfaceName}_CurveCount)
	{
		InternalSetHairStrandsAttributes(CurveIndex, HAIR_CURVE_ATTRIBUTE_OFFSET_SEED({DataInterfaceName}_CurveAttributeOffsets), HAIR_CURVE_ATTRIBUTE_STRIDE_SEED, {DataInterfaceName}_OutPointAttributeBuffer, PackHairSeed(Seed), {DataInterfaceName}_CurveAttributeIndexToChunkDivAsShift, {DataInterfaceName}_CurveAttributeChunkElementCount, {DataInterfaceName}_CurveAttributeChunkStrideInBytes);
	}
}

void WriteCurveClumpId_{DataInterfaceName}(uint CurveIndex, uint ClumpId)
{
	CurveIndex -= {DataInterfaceName}_BaseCurveIndex;
	
	const uint WriteIndex = CurveIndex;
	if (WriteIndex < {DataInterfaceName}_CurveCount && HasHairStrandsClumpId())
	{
		InternalSetHairStrandsAttributes(CurveIndex, HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID({DataInterfaceName}_CurveAttributeOffsets), HAIR_CURVE_ATTRIBUTE_STRIDE_CLUMPID, {DataInterfaceName}_OutPointAttributeBuffer, PackHairClumpID(ClumpId), {DataInterfaceName}_CurveAttributeIndexToChunkDivAsShift, {DataInterfaceName}_CurveAttributeChunkElementCount, {DataInterfaceName}_CurveAttributeChunkStrideInBytes);
	}
}

void WritePointColor_{DataInterfaceName}(uint PointIndex, float3 Color)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount && HasHairStrandsColor())
	{
		InternalSetHairStrandsAttributes32(PointIndex, HAIR_POINT_ATTRIBUTE_OFFSET_COLOR({DataInterfaceName}_PointAttributeOffsets), HAIR_POINT_ATTRIBUTE_STRIDE_COLOR, {DataInterfaceName}_OutPointAttributeBuffer, PackHairColor(Color), {DataInterfaceName}_PointAttributeIndexToChunkDivAsShift, {DataInterfaceName}_PointAttributeChunkElementCount, {DataInterfaceName}_PointAttributeChunkStrideInBytes);
	}
}

void WritePointRoughness_{DataInterfaceName}(uint PointIndex, float Roughness)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount && HasHairStrandsRoughness())
	{
		InternalSetHairStrandsAttributes(PointIndex, HAIR_POINT_ATTRIBUTE_OFFSET_ROUGHNESS({DataInterfaceName}_PointAttributeOffsets), HAIR_POINT_ATTRIBUTE_STRIDE_ROUGHNESS, {DataInterfaceName}_OutPointAttributeBuffer, PackHairRoughness(Roughness), {DataInterfaceName}_PointAttributeIndexToChunkDivAsShift, {DataInterfaceName}_PointAttributeChunkElementCount, {DataInterfaceName}_PointAttributeChunkStrideInBytes);
	}
}

void WritePointAO_{DataInterfaceName}(uint PointIndex, float Ao)
{
	PointIndex -= {DataInterfaceName}_BasePointIndex;
	
	const uint WriteIndex = PointIndex;
	if (WriteIndex < {DataInterfaceName}_PointCount && HasHairStrandsAO())
	{
		InternalSetHairStrandsAttributes(PointIndex, HAIR_POINT_ATTRIBUTE_OFFSET_AO({DataInterfaceName}_PointAttributeOffsets), HAIR_POINT_ATTRIBUTE_STRIDE_AO, {DataInterfaceName}_OutPointAttributeBuffer, PackHairAO(Ao), {DataInterfaceName}_PointAttributeIndexToChunkDivAsShift, {DataInterfaceName}_PointAttributeChunkElementCount, {DataInterfaceName}_PointAttributeChunkStrideInBytes);
	}
}