// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* -----------------------------------------------------------------
 * Compute Bishop frame
 * -----------------------------------------------------------------
 */

void ComputeMaterialFrame(in int StrandsSize)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex == 0 )
	{
		float4 NodeQuaternion = SharedNodeOrientation[GGroupThreadId.x];
		float3 TangentPrev = normalize(SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x]);
		float3 TangentNext = TangentPrev;

		for (int EdgeIndex = 1, EdgeEnd = StrandsSize-1; EdgeIndex < EdgeEnd; ++EdgeIndex)
		{
			TangentPrev = TangentNext;
			TangentNext = normalize(SharedNodePosition[GGroupThreadId.x+EdgeIndex+1] - SharedNodePosition[GGroupThreadId.x+EdgeIndex]);

			NodeQuaternion = NormalizeQuat( MultiplyQuat( FindQuatBetweenNormals(TangentPrev,TangentNext), NodeQuaternion ) );
			SharedNodeOrientation[GGroupThreadId.x+EdgeIndex] = NodeQuaternion;
			GroupMemoryBarrier();
		}
		GroupMemoryBarrier();
	}
	GroupMemoryBarrier();
}

void UpdateMaterialFrame(in int StrandsSize)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( (LocalIndex > 0) && (LocalIndex < StrandsSize-1) )
	{
		float4 NodeQuaternion = SharedNodeOrientation[GGroupThreadId.x];
		float3 TangentPrev = RotateVectorByQuat( float3(0,0,1), NodeQuaternion);
		float3 TangentNext = normalize(SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x]);

		SharedNodeOrientation[GGroupThreadId.x] = NormalizeQuat( MultiplyQuat( FindQuatBetweenNormals(TangentPrev,TangentNext), NodeQuaternion) );
	}
	GroupMemoryBarrier();
}

/* -----------------------------------------------------------------
 * Alternative version
 * -----------------------------------------------------------------
 */

/*void UpdateMaterialFrame(in int StrandsSize)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( (LocalIndex > 0) && (LocalIndex < StrandsSize-1) )
	{
		float4 NodeQuaternion = SharedPreviousOrientation[GGroupThreadId.x];
		//float3 TangentPrev = RotateVectorByQuat( float3(0,0,1), NodeQuaternion);
		//float3 TangentNext = normalize(SharedPreviousPosition[GGroupThreadId.x+1] - SharedPreviousPosition[GGroupThreadId.x]);

		float3 TangentPrev = normalize(SharedPreviousPosition[GGroupThreadId.x+1] - SharedPreviousPosition[GGroupThreadId.x]);
		float3 TangentNext = normalize(SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x]);

		SharedNodeOrientation[GGroupThreadId.x] = NormalizeQuat( MultiplyQuat( FindQuatBetweenNormals(TangentPrev,TangentNext), NodeQuaternion) );
	}
	GroupMemoryBarrier();
}*/

/* -----------------------------------------------------------------
 * Angular spring material
 * -----------------------------------------------------------------
 */

 void SetupAngularSpringMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	OutMaterialMultiplier = float3(0,0,0);

	float MaterialMultiplier = 0;
	SetupStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,ProjectConstraint,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,MaterialMultiplier);
}

void UpdateAngularSpringMultiplier( in float RestLength, in float3 RestDirection, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float InMaterialCompliance, in float InMaterialWeight, inout float3 OutMaterialMultiplier)
{
	const float3 EdgeDirection = SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1];
	const float3 DeltaVelocity = (EdgeDirection - ( SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousPosition[GGroupThreadId.x-1] ) ) / DeltaTime; 

	const float4 ParentFrame = SlerpQuat(SharedNodeOrientation[GGroupThreadId.x-2],SharedNodeOrientation[GGroupThreadId.x-2],0.5);
	const float3 TargetDirection = RotateVectorByQuat(RestDirection,ParentFrame);

	// XPBD lagrang : dL = -(C+compliance*L) / (dC * invM * dCt + alpha)
	const float3 DeltaLambda = -((EdgeDirection - TargetDirection)/RestLength + 
					OutMaterialMultiplier * InMaterialCompliance  + MaterialDamping * DeltaVelocity /RestLength) * InMaterialWeight;

	// L += dL
	OutMaterialMultiplier += DeltaLambda;

	// XPBD position update : dX = dL * dCt * invM
	const float3 PositionDelta  = DeltaLambda/RestLength;

	// dX += dX
	SharedNodePosition[GGroupThreadId.x] += PositionDelta * SharedInverseMass[GGroupThreadId.x];
	if(!ProjectConstraint)
	{
		SharedNodePosition[GGroupThreadId.x-1] -= PositionDelta * SharedInverseMass[GGroupThreadId.x-1];
	}
}

void SolveAngularSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float3 RestDirection, in float DeltaTime, in float MaterialDamping,
			in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;

	if(EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 1)
		{
			const int IsRed = (GGroupThreadId.x % 2) == 0;
			// Process all the red rods
			if (!IsRed)
			{	
				UpdateAngularSpringMultiplier(RestLength,RestDirection,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			GroupMemoryBarrier(); 

			if (IsRed)
			{
				UpdateAngularSpringMultiplier(RestLength,RestDirection,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			GroupMemoryBarrier(); 
		}
	}
}

void ProjectAngularSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float3 RestDirection, in float DeltaTime, out float3 OutNodePosition)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float3 MaterialMultiplier = float3(0,0,0);

		SetupAngularSpringMaterial(StrandsSize, YoungModulus, RodThickness, RestLength, DeltaTime, true, 0.0, MaterialCompliance, MaterialWeight, MaterialMultiplier);

		{
			for(int i = 2; i < StrandsSize; ++i)
			{
				//ComputeMaterialFrame(StrandsSize);
				if( LocalIndex == i )
				{
					UpdateAngularSpringMultiplier(RestLength,RestDirection,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier);
				}
				GroupMemoryBarrier(); 
			}
		}
	}

	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x];
}
