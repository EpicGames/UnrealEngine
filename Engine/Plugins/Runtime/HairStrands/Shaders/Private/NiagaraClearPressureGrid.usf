// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

static const int FLUID_PRESSURE_OFFSET = 6;
static const int FLUID_DISTANCE_OFFSET = 14;
static const int NUM_ATTRIBUTES = 18;

static const float MAX_DISTANCE = 1e+8;

uint3 GridSize;
int CopyPressure;

Buffer<int>		GridCurrentBuffer;
RWBuffer<int>	GridDestinationBuffer;

[numthreads(THREAD_COUNT, THREAD_COUNT, THREAD_COUNT)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId < GridSize))
	{
		const int LinearIndexYZ = DispatchThreadId.y * GridSize.z + DispatchThreadId.z;
		const int GridSizeYZ = GridSize.y * GridSize.z;
		
		[unroll]
		for(int i = 0; i < NUM_ATTRIBUTES; ++i)
		{
			const int AttrIndex = DispatchThreadId.x+i*GridSize.x;
			const int LinearIndex = AttrIndex.x * GridSizeYZ + LinearIndexYZ;
			GridDestinationBuffer[LinearIndex] = 0;
		}
		if(CopyPressure)
		{
			const int PressureIndex = DispatchThreadId.x+FLUID_PRESSURE_OFFSET*GridSize.x;
			const int LinearIndex = PressureIndex.x * GridSizeYZ + LinearIndexYZ;
			GridDestinationBuffer[LinearIndex] = GridCurrentBuffer[LinearIndex];
		}

		const int DistanceIndex = DispatchThreadId.x+FLUID_DISTANCE_OFFSET*GridSize.x;
		const int LinearIndex = DistanceIndex.x * GridSizeYZ + LinearIndexYZ;
		GridDestinationBuffer[LinearIndex] = MAX_DISTANCE;
	}
}