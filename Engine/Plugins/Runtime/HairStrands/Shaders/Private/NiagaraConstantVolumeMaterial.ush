// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* -----------------------------------------------------------------
 * Constant volume material
 * -----------------------------------------------------------------
 */

 void SetupVolumeSpringMaterial(in int StrandsSize, in float YoungModulus,
	in float RestVolume, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float OutMaterialMultiplier)
{
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	OutMaterialMultiplier = 0.0;

	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 2 )
	{
		// Compliance = 1.0 / (k * dt * dt)
		// with k = (Y / V) 
		// V is the constraint inital volume and Y is the young modulus
		OutMaterialCompliance = RestVolume/(YoungModulus*DeltaTime*DeltaTime);
	}
}
 
void UpdateVolumeSpringMultiplier(in float RestVolume, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float OutMaterialMultiplier)
{
	const int NodeIndexA = GGroupThreadId.x;
	const int NodeIndexB = GGroupThreadId.x-1;
	const int NodeIndexC = GGroupThreadId.x-2;
	const int NodeIndexD = GGroupThreadId.x-3;

	const float3 EdgeVectorA = SharedNodePosition[NodeIndexB] - SharedNodePosition[NodeIndexA];
	const float3 EdgeVectorB = SharedNodePosition[NodeIndexC] - SharedNodePosition[NodeIndexA];
	const float3 EdgeVectorC = SharedNodePosition[NodeIndexD] - SharedNodePosition[NodeIndexA];

	const float NodesVolume = dot(cross(EdgeVectorB,EdgeVectorC),EdgeVectorA);

	const float3 ConstraintGradB = cross(EdgeVectorB,EdgeVectorC);
	const float3 ConstraintGradC = cross(EdgeVectorC,EdgeVectorA);
	const float3 ConstraintGradD = cross(EdgeVectorA,EdgeVectorB);
	const float3 ConstraintGradA = -ConstraintGradB-ConstraintGradC-ConstraintGradD;

	const float3 NodeVelocityA = ( SharedNodePosition[NodeIndexA] - SharedPreviousPosition[NodeIndexA] ) / DeltaTime;
	const float3 NodeVelocityB = ( SharedNodePosition[NodeIndexB] - SharedPreviousPosition[NodeIndexB] ) / DeltaTime;
	const float3 NodeVelocityC = ( SharedNodePosition[NodeIndexC] - SharedPreviousPosition[NodeIndexC] ) / DeltaTime;
	const float3 NodeVelocityD = ( SharedNodePosition[NodeIndexD] - SharedPreviousPosition[NodeIndexD] ) / DeltaTime;

	const float SumInverseMass = !ProjectConstraint ? ( dot(ConstraintGradA,ConstraintGradA) * SharedInverseMass[NodeIndexA] +
														dot(ConstraintGradB,ConstraintGradB) * SharedInverseMass[NodeIndexB] +
													    dot(ConstraintGradC,ConstraintGradC) * SharedInverseMass[NodeIndexC] +
														dot(ConstraintGradD,ConstraintGradD) * SharedInverseMass[NodeIndexD] ) :
														dot(ConstraintGradA,ConstraintGradA) * SharedInverseMass[NodeIndexA] ;

	const float SchurDiagonal = (1.0 + MaterialDamping) * SumInverseMass + MaterialCompliance; 
	const float ConstraintWeight = ( SchurDiagonal != 0.0 ) ? 1.0 / SchurDiagonal : 0.0;

	// XPBD lagrange multiplier update : dL = -(C+compliance*L) / (dC * invM * dCt + alpha)
	const float DeltaLambda = -((NodesVolume - RestVolume) + OutMaterialMultiplier * MaterialCompliance + MaterialDamping * 
		(dot(ConstraintGradA,NodeVelocityA)+dot(ConstraintGradB,NodeVelocityB)+dot(ConstraintGradC,NodeVelocityC)+dot(ConstraintGradD,NodeVelocityD) ) ) * ConstraintWeight;

	// L += dL
	//OutMaterialMultiplier += DeltaLambda;
	OutMaterialMultiplier = NodesVolume;

	// XPBD position update : dX += dL * dCt * invM
	SharedNodePosition[NodeIndexA] += ConstraintGradA * DeltaLambda * SharedInverseMass[NodeIndexA];
	if(!ProjectConstraint)
	{
		SharedNodePosition[NodeIndexB] += ConstraintGradB * DeltaLambda * SharedInverseMass[NodeIndexB]; 
		SharedNodePosition[NodeIndexC] += ConstraintGradC * DeltaLambda * SharedInverseMass[NodeIndexC];
		SharedNodePosition[NodeIndexD] += ConstraintGradD * DeltaLambda * SharedInverseMass[NodeIndexD];
	}
}

void SolveVolumeSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float RestVolume, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float MaterialMultiplier, out float OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	if(EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 2)
		{
			for(int i = 3; i < StrandsSize; ++i)
			{
				if( LocalIndex == i )
				{
					//UpdateVolumeSpringMultiplier(RestVolume,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
					UpdateVolumeSpringMultiplier(RestVolume,DeltaTime,false,0.0,0.0,MaterialWeight,OutMaterialMultiplier);
				}
				GroupMemoryBarrier(); 
			}
		}
	}
	GroupMemoryBarrier();
} 