// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

uint NumElements;
RWBuffer<uint> BoundingBoxBufferA;
RWBuffer<uint> OutBoundingBoxBufferB;

uint FloatToUint( in float FloatValue)
{
	uint3 UnsignedValue = asuint(FloatValue);
	UnsignedValue ^= (1+~(UnsignedValue >> 31) | 0x80000000);

	return UnsignedValue;
}

[numthreads(THREAD_COUNT, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)  
{
	const uint UINT_MAX = FloatToUint(1e+8);
	const uint UINT_MIN = FloatToUint(-1e+8);

	const uint ElementIndex = DispatchThreadId.x;
	if (ElementIndex < NumElements)
	{
		OutBoundingBoxBufferB[0] = BoundingBoxBufferA[0];
		OutBoundingBoxBufferB[1] = BoundingBoxBufferA[1];
		OutBoundingBoxBufferB[2] = BoundingBoxBufferA[2];

		OutBoundingBoxBufferB[3] = BoundingBoxBufferA[3];
		OutBoundingBoxBufferB[4] = BoundingBoxBufferA[4];
		OutBoundingBoxBufferB[5] = BoundingBoxBufferA[5];

		BoundingBoxBufferA[0] = UINT_MAX;
		BoundingBoxBufferA[1] = UINT_MAX;
		BoundingBoxBufferA[2] = UINT_MAX;

		BoundingBoxBufferA[3] = UINT_MIN;
		BoundingBoxBufferA[4] = UINT_MIN;
		BoundingBoxBufferA[5] = UINT_MIN;
	}
}