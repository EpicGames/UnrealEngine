// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

uint3 GridSize;
Buffer<int>		GridCurrentBuffer;
RWBuffer<int>	GridDestinationBuffer;

[numthreads(THREAD_COUNT, THREAD_COUNT, THREAD_COUNT)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId < GridSize))
	{
		const int GridIndex = DispatchThreadId.x * GridSize.y * GridSize.z + DispatchThreadId.y * GridSize.z + DispatchThreadId.z;
		GridDestinationBuffer[GridIndex] = GridCurrentBuffer[GridIndex];
	}
}