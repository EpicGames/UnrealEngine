// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* -----------------------------------------------------------------
 * Align rod material
 * -----------------------------------------------------------------
 */

 void SetupAlignRodMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	
	OutMaterialMultiplier = float3(0, 0, 0);
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	
	if( LocalIndex > 1 )
	{
		// Compliance = 1.0 / (k * dt * dt)
		// with k = L * L * (Y * A / L) (the L*L term before is coming from the fact that our constraint is dL/L and not dL)
		// A is the cross section area (Pi * R * R), L is the rest length and Y is the young modulus
		OutMaterialCompliance = 4.0/(YoungModulus*PI*RodThickness*RestLength*RodThickness*DeltaTime*DeltaTime);

		const float SumInverseMass = SharedInverseInertia[GGroupThreadId.x-1]*4.0;

		const float SchurDiagonal = ( (1.0  + MaterialDamping / DeltaTime ) * SumInverseMass + OutMaterialCompliance );
		OutMaterialWeight = ( SchurDiagonal != 0.0 ) ? 1.0 / SchurDiagonal : 0.0;
	}
}

void UpdateAlignRodMultiplier( in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float3 OutMaterialMultiplier)
{
	float3 TargetDirection;
	float4 q0 = SharedNodeOrientation[GGroupThreadId.x-1];
	TargetDirection[0] = 2.0 * (q0.x * q0.z + q0.w * q0.y);
	TargetDirection[1] = 2.0 * (q0.y * q0.z - q0.w * q0.x);
	TargetDirection[2] = q0.w * q0.w - q0.x * q0.x - q0.y * q0.y + q0.z * q0.z;

	const float4 qebar = float4(-q0.y, q0.x, -q0.w, q0.z);	
	const float3 EdgeDirection = SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1];

	const float3 DeltaVelocity = (EdgeDirection - ( SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousPosition[GGroupThreadId.x-1] ) ) / DeltaTime; 
	const float4 DeltaQuat = (SharedNodeOrientation[GGroupThreadId.x-1] - SharedPreviousOrientation[GGroupThreadId.x-1]) / DeltaTime; 

	const float4 eqbar = InverseQuat(qebar);
	const float3 QuatDamping = -2.0 * ( eqbar.xyz * DeltaQuat.w + eqbar.w * DeltaQuat.xyz - cross(eqbar.xyz,DeltaQuat.xyz) );

	// XPBD lagrange multiplier update : dL = -(C+compliance*L) / (dC * invM * dCt + alpha)
	const float3 DeltaLambda = -( EdgeDirection / RestLength - TargetDirection + 
					OutMaterialMultiplier * MaterialCompliance + MaterialDamping *  QuatDamping ) * MaterialWeight;

	// L += dL
	OutMaterialMultiplier += DeltaLambda;

	// XPBD position update : dX = dL * dCt * invM
	//const float QuaternionDelta = 0.5 * MultiplyQuat(float4(cross( TargetDirection, DeltaLambda),0), q0 );
	const float4 QuaternionDelta = -2.0 * float4( DeltaLambda.xyz * qebar.w + cross(DeltaLambda.xyz, qebar.xyz),
					- dot(DeltaLambda.xyz, qebar.xyz) );

	// dX += dX
	q0 += QuaternionDelta * SharedInverseInertia[GGroupThreadId.x-1];
	SharedNodeOrientation[GGroupThreadId.x-1] = NormalizeQuat(q0);
}

void SolveAlignRodMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 1)
		{
			const int IsRed = (GGroupThreadId.x % 2) == 0;
			// Process all the red rods
			if (IsRed)
			{	
				UpdateAlignRodMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			// Process all the black rods
			GroupMemoryBarrier(); 
			if (!IsRed)
			{
				UpdateAlignRodMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			GroupMemoryBarrier(); 
		}
	}
} 

void ProjectAlignRodMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float4 OutNodeOrientation)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float3 MaterialMultiplier = float3(0,0,0);
		
		SetupAlignRodMaterial(StrandsSize, YoungModulus, RodThickness, RestLength, DeltaTime, true, 0.0, MaterialCompliance, MaterialWeight, MaterialMultiplier);

		if( LocalIndex > 1 )
		{
			for(int i = 2; i < StrandsSize; ++i)
			{
				if( LocalIndex == i)
				{
					UpdateAlignRodMultiplier(RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier);
				}
				GroupMemoryBarrier(); 
			}
		}
	}
	GroupMemoryBarrier();
	OutNodeOrientation = SharedNodeOrientation[GGroupThreadId.x];
}

/* -----------------------------------------------------------------
 * Stretch rod material
 * -----------------------------------------------------------------
 */

void SetupStretchRodMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	
	OutMaterialMultiplier = float3(0, 0, 0);
	OutMaterialWeight = 0.0;
	OutMaterialCompliance = 0.0;
	
	if( LocalIndex > 1 )
	{
		// Compliance = 1.0 / (k * dt * dt)
		// with k = L * L * (Y * A / L) (the L*L term before is coming from the fact that our constraint is dL/L and not dL)
		// A is the cross section area (Pi * R * R), L is the rest length and Y is the young modulus
		OutMaterialCompliance = 4.0/(YoungModulus*PI*RodThickness*RestLength*RodThickness*DeltaTime*DeltaTime);

		const float SumInverseMass = !ProjectConstraint ? ( SharedInverseMass[GGroupThreadId.x] + SharedInverseMass[GGroupThreadId.x-1] ) / (RestLength*RestLength) 
			: SharedInverseMass[GGroupThreadId.x] / (RestLength*RestLength) ;

		const float SchurDiagonal = ( (1.0  + MaterialDamping / DeltaTime ) * SumInverseMass + OutMaterialCompliance );
		OutMaterialWeight = ( SchurDiagonal != 0.0 ) ? 1.0 / SchurDiagonal : 0.0;
	}
}

void UpdateStretchRodMultiplier( in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float3 OutMaterialMultiplier)
{
	float3 TargetDirection;
	float4 q0 = SharedNodeOrientation[GGroupThreadId.x-1];
	TargetDirection[0] = 2.0 * (q0.x * q0.z + q0.w * q0.y);
	TargetDirection[1] = 2.0 * (q0.y * q0.z - q0.w * q0.x);
	TargetDirection[2] = q0.w * q0.w - q0.x * q0.x - q0.y * q0.y + q0.z * q0.z;

	const float4 qebar = float4(-q0.y, q0.x, -q0.w, q0.z);	
	const float3 EdgeDirection = SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1];

	const float3 DeltaVelocity = (EdgeDirection - ( SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousPosition[GGroupThreadId.x-1] ) ) / DeltaTime; 

	// XPBD lagrange multiplier update : dL = -(C+compliance*L) / (dC * invM * dCt + alpha)
	const float3 DeltaLambda = -( EdgeDirection / RestLength - TargetDirection + 
					OutMaterialMultiplier * MaterialCompliance + MaterialDamping * DeltaVelocity/RestLength ) * MaterialWeight;

	// L += dL
	OutMaterialMultiplier += DeltaLambda;

	// XPBD position update : dX = dL * dCt * invM
	const float3 PositionDelta  = DeltaLambda/RestLength;

	// dX += dX
	SharedNodePosition[GGroupThreadId.x] += PositionDelta * SharedInverseMass[GGroupThreadId.x];
	if(!ProjectConstraint)
	{
		SharedNodePosition[GGroupThreadId.x-1] -= PositionDelta * SharedInverseMass[GGroupThreadId.x-1];
	}
}

void SolveStretchRodMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 1)
		{
			const int IsRed = (GGroupThreadId.x % 2) == 0;
			// Process all the red rods
			if (IsRed)
			{	
				UpdateStretchRodMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			// Process all the black rods
			GroupMemoryBarrier(); 
			if (!IsRed)
			{
				UpdateStretchRodMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			GroupMemoryBarrier(); 
		}
	}
} 

void ProjectStretchRodMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float3 OutNodePosition)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float3 MaterialMultiplier = float3(0,0,0);

		SetupStretchRodMaterial(StrandsSize, YoungModulus, RodThickness, RestLength, DeltaTime, true, 0.0, MaterialCompliance, MaterialWeight, MaterialMultiplier);

		if( LocalIndex > 1 )
		{
			for(int i = 2; i < StrandsSize; ++i)
			{
				if( LocalIndex == i)
				{
					UpdateStretchRodMultiplier(RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier);
				}
				GroupMemoryBarrier(); 
			}
		}
	}
	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x];
}

/* -----------------------------------------------------------------
 * Bend rod material
 * -----------------------------------------------------------------
 */

 void SetupBendRodMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	
	OutMaterialMultiplier = float3(0, 0, 0);
	OutMaterialWeight = 0.0;
	OutMaterialCompliance = 0.0;
	
	if( LocalIndex > 0 )
	{
		// Compliance = 1.0 / (k * dt * dt)
		// with k = (Y * I * L) 
		// I is the polar moment of inertia (Pi * R * R * R * R / 4), L is the rest length and Y is the young modulus
		const float InverseMoment = SharedInverseInertia[GGroupThreadId.x];
		OutMaterialCompliance = InverseMoment / (YoungModulus*DeltaTime*DeltaTime);

		//OutMaterialCompliance = 64.0 /(YoungModulus*PI*RestLength*RodThickness*RodThickness*RodThickness*RodThickness*DeltaTime*DeltaTime);

		const float SumInverseMass = !ProjectConstraint ? ( SharedInverseInertia[GGroupThreadId.x] + SharedInverseInertia[GGroupThreadId.x-1] ) * 4.0 / (RestLength*RestLength) 
							: SharedInverseInertia[GGroupThreadId.x]  * 4.0 / (RestLength*RestLength);

		const float SchurDiagonal = ( (1.0  + MaterialDamping / DeltaTime ) * SumInverseMass + OutMaterialCompliance );
		OutMaterialWeight = ( SchurDiagonal != 0.0 ) ? 1.0 / SchurDiagonal : 0.0;
	}
}

void UpdateBendRodMultiplier(in float RestLength, in float4 RestDarboux, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float3 OutMaterialMultiplier)
{
	float4 q0 = SharedNodeOrientation[GGroupThreadId.x-1];
	float4 q1 = SharedNodeOrientation[GGroupThreadId.x];
	const float4 CurrentDarboux = float4(
			q1.xyz * q0.w - q0.xyz * q1.w + cross(-q0.xyz, q1.xyz),
			q0.w * q1.w - dot(-q0.xyz, q1.xyz));
	const float InverseRestLength = 2.0 / RestLength;

	const float4 OmegaPlus = (CurrentDarboux + RestDarboux) * InverseRestLength;
	const float4 OmegaMinus = (CurrentDarboux - RestDarboux) * InverseRestLength;

	const float4 DeltaQuat0 = (SharedNodeOrientation[GGroupThreadId.x-1] - SharedPreviousOrientation[GGroupThreadId.x-1]) / DeltaTime; 
	const float4 DeltaQuat1 = (SharedNodeOrientation[GGroupThreadId.x] - SharedPreviousOrientation[GGroupThreadId.x]) / DeltaTime;

	const float3 QuatDamping = - q0.xyz * DeltaQuat1.w + q0.w * DeltaQuat1.xyz - cross(q0.xyz,DeltaQuat1.xyz) 
							   + q1.xyz * DeltaQuat0.w - q1.w * DeltaQuat0.xyz + cross(q1.xyz,DeltaQuat0.xyz);

	float4 DeltaOmega = (dot(OmegaPlus,OmegaPlus) > dot(OmegaMinus,OmegaMinus) ) ? OmegaMinus : OmegaPlus;
	DeltaOmega.xyz += OutMaterialMultiplier * MaterialCompliance + MaterialDamping * QuatDamping * InverseRestLength;  
	DeltaOmega.w = 0;
	DeltaOmega *= -MaterialWeight;

	OutMaterialMultiplier += DeltaOmega.xyz;

	SharedNodeOrientation[GGroupThreadId.x] += SharedInverseInertia[GGroupThreadId.x] * float4(
			DeltaOmega.xyz * q0.w + cross(q0.xyz, DeltaOmega.xyz), - dot(q0.xyz, DeltaOmega.xyz)) * InverseRestLength;
	SharedNodeOrientation[GGroupThreadId.x] = NormalizeQuat(SharedNodeOrientation[GGroupThreadId.x]);

	if(!ProjectConstraint)
	{
		SharedNodeOrientation[GGroupThreadId.x-1] -= SharedInverseInertia[GGroupThreadId.x-1] *  float4(
			DeltaOmega.xyz * q1.w + cross(q1.xyz, DeltaOmega.xyz), - dot(q1.xyz, DeltaOmega.xyz)) * InverseRestLength;
		SharedNodeOrientation[GGroupThreadId.x-1] = NormalizeQuat(SharedNodeOrientation[GGroupThreadId.x-1]);
	}
}

void SolveBendRodMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float4 RestDarboux, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;

	UpdateMaterialFrame(StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 0)
		{
			const int IsRed = (GGroupThreadId.x % 2) == 0;
			// Process all the red rods
			if (IsRed)
			{	
				UpdateBendRodMultiplier(RestLength,RestDarboux,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			// Process all the black rods
			GroupMemoryBarrier(); 
			if (!IsRed)
			{
				UpdateBendRodMultiplier(RestLength,RestDarboux,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			GroupMemoryBarrier(); 
		}
	}
} 

void ProjectBendRodMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float4 RestDarboux, in float DeltaTime, out float4 OutNodeOrientation)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);

	UpdateMaterialFrame(StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float3 MaterialMultiplier = float3(0,0,0);

		SetupBendRodMaterial(StrandsSize, YoungModulus, RodThickness, RestLength, DeltaTime, true, 0.0, MaterialCompliance, MaterialWeight, MaterialMultiplier);

		if( LocalIndex > 0 )
		{
			for(int i = 1; i < StrandsSize; ++i)
			{
				if( LocalIndex == i)
				{
					UpdateBendRodMultiplier(RestLength,RestDarboux,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier);
				}
				GroupMemoryBarrier(); 
			}
		}
	}
	GroupMemoryBarrier();
	OutNodeOrientation = SharedNodeOrientation[GGroupThreadId.x];
}

/* -----------------------------------------------------------------
 * Alternative of the Bend rod material
 * -----------------------------------------------------------------
 */

/*void UpdateBendTwistMultiplier( in float RestLength, in float4 RestDarboux, in float DeltaTime, 
				in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float3 OutMaterialMultiplier)
{
	float4 q0 = SharedNodeOrientation[GGroupThreadId.x-1];
	float4 q1 = SharedNodeOrientation[GGroupThreadId.x];
	const float4 CurrentDarboux = float4(
			q1.xyz * q0.w - q0.xyz * q1.w + cross(-q0.xyz, q1.xyz),
			q0.w * q1.w - dot(-q0.xyz, q1.xyz));
	const float InverseRestLength = 2.0 / RestLength;

	float3 DeltaOmega = (CurrentDarboux.xyz / CurrentDarboux.w - RestDarboux.xyz / RestDarboux.w) * InverseRestLength;
	DeltaOmega.xyz += OutMaterialMultiplier * MaterialCompliance;  
	DeltaOmega *= MaterialWeight;

	OutMaterialMultiplier -= DeltaOmega.xyz;

	DeltaOmega = DeltaOmega - CurrentDarboux.xyz * dot(CurrentDarboux.xyz,DeltaOmega); 

	const float qtu = dot(q0,q1);
	const float dto = dot(CurrentDarboux.xyz, DeltaOmega.xyz);

	const float4 q1tmp = qtu * q1; 
	const float4 q0tmp = qtu * q0; 

	const float afac = dot(CurrentDarboux.xyz,DeltaOmega);

	SharedNodeOrientation[GGroupThreadId.x-1] += SharedInverseInertia[GGroupThreadId.x-1] * ( float4(
			DeltaOmega.xyz * q1tmp.w + cross(q1tmp.xyz, DeltaOmega.xyz), - dot(q1tmp.xyz, DeltaOmega.xyz)) + dto * q1) * InverseRestLength;
	SharedNodeOrientation[GGroupThreadId.x] -= SharedInverseInertia[GGroupThreadId.x] *  ( float4(
			DeltaOmega.xyz * q0tmp.w + cross(q0tmp.xyz, DeltaOmega.xyz), - dot(q0tmp.xyz, DeltaOmega.xyz)) - dto * q0) * InverseRestLength;

	SharedNodeOrientation[GGroupThreadId.x-1] = NormalizeQuat(SharedNodeOrientation[GGroupThreadId.x-1]);
	SharedNodeOrientation[GGroupThreadId.x] = NormalizeQuat(SharedNodeOrientation[GGroupThreadId.x]);
}*/

