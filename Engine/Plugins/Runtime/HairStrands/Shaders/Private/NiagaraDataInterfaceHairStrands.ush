// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfaceHairStrands.ush 
=============================================================================*/

#pragma once

static const float M_PI = 3.14159265;
static const float3 MAX_POS = float3(1e+8,1e+8,1e+8);

float3 Uint3ToFloat3( in uint3 Uint3Vector)
{
	uint3 SignedVector = Uint3Vector;
	SignedVector.x ^= (((SignedVector.x >> 31)-1) | 0x80000000);
	SignedVector.y ^= (((SignedVector.y >> 31)-1) | 0x80000000);
	SignedVector.z ^= (((SignedVector.z >> 31)-1) | 0x80000000);

	return float3(asfloat(SignedVector.x),asfloat(SignedVector.y),asfloat(SignedVector.z));
 }

 uint3 Float3ToUint3( in float3 Float3Vector)
 {
	uint3 UnsignedVector = uint3(asuint(Float3Vector.x),asuint(Float3Vector.y),asuint(Float3Vector.z));
	UnsignedVector.x ^= (1+~(UnsignedVector.x >> 31) | 0x80000000);
	UnsignedVector.y ^= (1+~(UnsignedVector.y >> 31) | 0x80000000);
	UnsignedVector.z ^= (1+~(UnsignedVector.z >> 31) | 0x80000000);

	return UnsignedVector;
 }

 uint FloatToUint( in float FloatValue)
{
	uint UnsignedValue = asuint(FloatValue);
	UnsignedValue ^= (1+~(UnsignedValue >> 31) | 0x80000000);

	return UnsignedValue;
}

