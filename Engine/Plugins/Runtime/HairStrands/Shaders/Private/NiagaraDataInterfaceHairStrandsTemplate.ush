// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfaceHairStrands.ush 
=============================================================================*/
#include "/Engine/Private/HairStrands/HairStrandsBindingCommon.ush"

RWByteAddressBuffer	{ParameterName}_DeformedPositionBuffer;
ByteAddressBuffer	{ParameterName}_RestPositionBuffer;
ByteAddressBuffer	{ParameterName}_CurvesOffsetsBuffer;
Buffer<float4>		{ParameterName}_RestTrianglePositionBuffer;
Buffer<float4>		{ParameterName}_DeformedTrianglePositionBuffer;
Buffer<uint>		{ParameterName}_RootBarycentricCoordinatesBuffer;
Buffer<uint>		{ParameterName}_RootToUniqueTriangleIndexBuffer;
float4x4			{ParameterName}_WorldTransform;
float4x4			{ParameterName}_WorldInverse;
float4				{ParameterName}_WorldRotation;
float4x4			{ParameterName}_BoneTransform;
float4x4			{ParameterName}_BoneInverse;
float4				{ParameterName}_BoneRotation;
int					{ParameterName}_NumStrands;
int					{ParameterName}_StrandSize;
int					{ParameterName}_InterpolationMode;
float3				{ParameterName}_RestRootOffset;
float3				{ParameterName}_DeformedRootOffset;
float3				{ParameterName}_RestPositionOffset;
StructuredBuffer<float4> {ParameterName}_DeformedPositionOffset;
RWBuffer<uint>	    {ParameterName}_BoundingBoxBuffer;
uint				{ParameterName}_ResetSimulation;
uint				{ParameterName}_RestUpdate;
uint				{ParameterName}_LocalSimulation;
int					{ParameterName}_SampleCount;
uint				{ParameterName}_RBFLocalSpace;
int4				{ParameterName}_BoundingBoxOffsets;
StructuredBuffer<float4> {ParameterName}_RestSamplePositionsBuffer;
StructuredBuffer<float4> {ParameterName}_MeshSampleWeightsBuffer;
StructuredBuffer<float4> {ParameterName}_DeformedSamplePositionsBuffer;
Buffer<float>		{ParameterName}_ParamsScaleBuffer;
float3				{ParameterName}_BoneLinearVelocity;
float3				{ParameterName}_BoneAngularVelocity;
float3				{ParameterName}_BoneLinearAcceleration;
float3				{ParameterName}_BoneAngularAcceleration;

/* -----------------------------------------------------------------
 * Box utilities
 * -----------------------------------------------------------------
 */

// Get the boundingh box
void GetBoundingBox_{ParameterName}(in int BoxIndex, out float3 OutBoxCenter, out float3 OutBoxExtent)
{
	const int BufferOffset = {ParameterName}_BoundingBoxOffsets[BoxIndex] * 6;

	const float3 BoxMin = Uint3ToFloat3(uint3({ParameterName}_BoundingBoxBuffer[0+BufferOffset],
											  {ParameterName}_BoundingBoxBuffer[1+BufferOffset],
											  {ParameterName}_BoundingBoxBuffer[2+BufferOffset]));

	const float3 BoxMax = Uint3ToFloat3(uint3({ParameterName}_BoundingBoxBuffer[3+BufferOffset],
											  {ParameterName}_BoundingBoxBuffer[4+BufferOffset],
											  {ParameterName}_BoundingBoxBuffer[5+BufferOffset]));

	OutBoxExtent = (BoxMax-BoxMin);
	OutBoxCenter = 0.5 * (BoxMin+BoxMax);
}

// Reset the bounding box
void ResetBoundingBox_{ParameterName}(out bool FunctionStatus)
{
	FunctionStatus = false;

	if(GLinearThreadId == 0)
	{
		FunctionStatus = true;

		const int BufferOffset = {ParameterName}_BoundingBoxOffsets[3] * 6;

		const uint UINT_MAX = FloatToUint(1e+8);
		const uint UINT_MIN = FloatToUint(-1e+8);

		{ParameterName}_BoundingBoxBuffer[0+BufferOffset] = UINT_MAX;
		{ParameterName}_BoundingBoxBuffer[1+BufferOffset] = UINT_MAX;
		{ParameterName}_BoundingBoxBuffer[2+BufferOffset] = UINT_MAX;

		{ParameterName}_BoundingBoxBuffer[3+BufferOffset] = UINT_MIN;
		{ParameterName}_BoundingBoxBuffer[4+BufferOffset] = UINT_MIN;
		{ParameterName}_BoundingBoxBuffer[5+BufferOffset] = UINT_MIN;
	}
	DeviceMemoryBarrier();
}

void BuildBoundingBox_{ParameterName}(in float3 NodePosition, out bool FunctionStatus)
{
	FunctionStatus = false;

	const uint3 LocalPosition = Float3ToUint3(NodePosition);

	if(isfinite(NodePosition.x) && isfinite(NodePosition.y) && isfinite(NodePosition.z))
	{
		FunctionStatus = true;

		const int BufferOffset = {ParameterName}_BoundingBoxOffsets[2] * 6;

		InterlockedMin({ParameterName}_BoundingBoxBuffer[0+BufferOffset],LocalPosition.x);
		InterlockedMin({ParameterName}_BoundingBoxBuffer[1+BufferOffset],LocalPosition.y);
		InterlockedMin({ParameterName}_BoundingBoxBuffer[2+BufferOffset],LocalPosition.z);

		InterlockedMax({ParameterName}_BoundingBoxBuffer[3+BufferOffset],LocalPosition.x);
		InterlockedMax({ParameterName}_BoundingBoxBuffer[4+BufferOffset],LocalPosition.y);
		InterlockedMax({ParameterName}_BoundingBoxBuffer[5+BufferOffset],LocalPosition.z);
	}
	DeviceMemoryBarrier();
}

/* -----------------------------------------------------------------
 * Utilities regarding strands curve
 * -----------------------------------------------------------------
 */
FHairCurve UnpackCurve_{ParameterName}(in uint InCurveIndex)
{
	return ReadHairCurve({ParameterName}_CurvesOffsetsBuffer, InCurveIndex);
}

/* -----------------------------------------------------------------
 * Utilities regarding strands indexing
 * -----------------------------------------------------------------
 */

// Given a node index return the strand index and the local index within the strand
int GetNumNodes_{ParameterName}()
{
	return {ParameterName}_NumStrands * {ParameterName}_StrandSize;
}

/* -----------------------------------------------------------------
 * Strands properties
 * -----------------------------------------------------------------
 */

// Given a strand index return the corresponding strand length
float ComputeStrandLength_{ParameterName}(in int StrandIndex)
{
	float StrandLength = 0.0;
	if( StrandIndex < {ParameterName}_NumStrands )
	{
		FHairCurve Curve = UnpackCurve_{ParameterName}(StrandIndex);
		int PointOffset = Curve.PointIndex;
		const int EdgeCount = Curve.PointCount-1;

		float3 PointNext = ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset++);
		float3 PointPrev = PointNext;

		for (int EdgeIndex = 0; EdgeIndex < EdgeCount; ++EdgeIndex)
		{
			PointPrev = PointNext;
			PointNext = ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset++);
			StrandLength += length(PointNext-PointPrev);
		}
	}
	return StrandLength;
}

/* -----------------------------------------------------------------
 * Node Mass/Inertia/Position/Orientation computation
 * -----------------------------------------------------------------
 */

 // Compute the node mass
void ComputeNodeMass_{ParameterName}(in float StrandsDensity, in float NodeThickness, out float OutNodeMass)
{
	const int ReducedSize =  {ParameterName}_StrandSize-1;
	const float CoordScale = 1.0 / (ReducedSize-1.0);

	const int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;

	const float StrandLength = ComputeStrandLength_{ParameterName}(StrandIndex);

	int  LocalIndex = GGroupThreadId.x % {ParameterName}_StrandSize;
	const float EdgeLength = StrandLength * CoordScale;
	const float MeanRadius = 0.5 * NodeThickness;

	LocalIndex = max(0,LocalIndex - 1);
	const float EdgeScale = (LocalIndex == 0 || LocalIndex == (ReducedSize-1)) ? 0.5 : 1.0;

	OutNodeMass = StrandsDensity * EdgeLength * EdgeScale * M_PI * MeanRadius * MeanRadius;
}

// Compute the node inertia
void ComputeNodeInertia_{ParameterName}(in float StrandsDensity, in float NodeThickness, out float3 OutNodeInertia)
{
	const int ReducedSize =  {ParameterName}_StrandSize-1;
	const float CoordScale = 1.0 / (ReducedSize-1.0);

	const int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;

	const float StrandLength = ComputeStrandLength_{ParameterName}(StrandIndex);

	const float EdgeLength = StrandLength * CoordScale;
	const float MeanRadius = 0.5 * NodeThickness;

	const float EdgeMass = StrandsDensity * EdgeLength * M_PI * MeanRadius * MeanRadius;
	const float RadialInertia = EdgeMass * ( 3.0 * MeanRadius * MeanRadius + EdgeLength * EdgeLength) / 12.0;
	OutNodeInertia = float3( RadialInertia, RadialInertia, EdgeMass * MeanRadius * MeanRadius * 0.5 );
}

// Set the node position by linear interpolation over the points
void SetNodePosition_{ParameterName}(in bool RootNode, const float NodePoint, const float PointCount, 
		const float CoordScale, const int PointPrev, const int PointNext, const int PointOffset, in int StrandIndex, out float3 OutNodePosition)
{
	if(RootNode)
	{
		const float FirstPoint = CoordScale * PointCount;
		const int FirstPrev = floor(FirstPoint);
		const int FirstNext = PointPrev+1;

		const float FirstAlpha = FirstPoint - (float)FirstPrev;
		const float3 FirstPosition =  ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset+FirstPrev) * (1.0-FirstAlpha) + 
						 			  ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset+FirstNext) * FirstAlpha;
		const float3 DiffPosition = FirstPosition - ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset);
		const float3 EdgeDirection =  normalize(DiffPosition);

		const float EdgeLength = ComputeStrandLength_{ParameterName}(StrandIndex) * CoordScale;
		OutNodePosition = 	ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset+PointPrev) - EdgeLength * normalize(EdgeDirection);
	}
	else
	{
		const float NodeAlpha = NodePoint - (float)PointPrev;
		OutNodePosition =  ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset+PointPrev) * (1.0-NodeAlpha) + 
						   ReadHairControlPointPosition({ParameterName}_RestPositionBuffer, PointOffset+PointNext) * NodeAlpha;
	}
	OutNodePosition = mul(float4( OutNodePosition+{ParameterName}_RestPositionOffset, 1.0), {ParameterName}_WorldTransform).xyz;
}

// Compute the node position by linear interpolation over the points
void ComputeNodePosition_{ParameterName}(out float3 OutNodePosition)
{
	const int ReducedSize =  {ParameterName}_StrandSize-1;
	const float CoordScale = 1.0 / (ReducedSize-1.0);

	int LocalIndex = GGroupThreadId.x % {ParameterName}_StrandSize;
	int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;

	const bool RootNode = (LocalIndex == 0);
	LocalIndex = max(0,LocalIndex - 1);

	const FHairCurve Curve = UnpackCurve_{ParameterName}(StrandIndex);

	const int PointOffset = Curve.PointIndex;

	const float NodeCoord = (float)(LocalIndex) * CoordScale;
	const float PointCount = Curve.PointCount-1;

	const float NodePoint = NodeCoord * PointCount;
	const int PointPrev = (LocalIndex==0) ? 0 : (LocalIndex==(ReducedSize-1)) ? PointCount-1 : floor(NodePoint);
	const int PointNext = PointPrev+1;

	SetNodePosition_{ParameterName}(RootNode, NodePoint, PointCount, CoordScale, PointPrev, PointNext, PointOffset, StrandIndex, OutNodePosition);
}

// Compute the node position by linear interpolation over the points
void SmoothNodePosition_{ParameterName}(in float SmoothingFilter, inout float3 OutNodePosition)
{
	SharedNodePosition[GGroupThreadId.x] = OutNodePosition;
	GroupMemoryBarrier();

	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);
	if(LocalIndex == 0)
	{
		const float Beta = SmoothingFilter;
		float3 DirM1 = SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x];
		float3 DirM2 = DirM1;

		const float Gamma1 = 2.0 * (1.0-Beta);
		const float Gamma2 = - (1.0-Beta)*(1.0-Beta);
		const float Gamma3 = Beta*Beta;

		float3 NodePosition = SharedNodePosition[GGroupThreadId.x];
		SharedPreviousPosition[GGroupThreadId.x] = NodePosition;
		
		for( int i = GGroupThreadId.x, end = GGroupThreadId.x+{ParameterName}_StrandSize-1; i < end; ++i)
		{
			const float3 DirM3 = SharedNodePosition[i+1] - SharedNodePosition[i];
			const float3 DirMi = Gamma1 * DirM1 + Gamma2 * DirM2 + Gamma3 * DirM3;

			SharedPreviousPosition[i+1]  = SharedPreviousPosition[i] + DirMi;
			GroupMemoryBarrier();

			DirM2 = DirM1;
			DirM1 = DirMi;
		}
	}
	GroupMemoryBarrier();
	OutNodePosition = SharedPreviousPosition[GGroupThreadId.x];
}

// Compute the root orientation
void ComputeRootOrientation_{ParameterName}()
{
	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);
	if(LocalIndex == 0)
	{
		const float3 EdgeDirection = normalize(SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x]);
		const float4 RootQuaternion = FindQuatBetweenNormals(float3(0,0,1),EdgeDirection);

		//const float3 TangentPrev = normalize(RotateVectorByQuat( float3(1,0,0), RootQuaternion));
		//const float3 TangentNext = normalize(cross( normalize(cross(EdgeDirection,float3(0,0,1))), EdgeDirection));

		SharedNodeOrientation[GGroupThreadId.x] = RootQuaternion;

		//SharedNodeOrientation[GGroupThreadId.x] = NormalizeQuat( MultiplyQuat( RootQuaternion, FindQuatBetweenNormals(TangentPrev,TangentNext) ) );
	}
	GroupMemoryBarrier();
}

// Update the root orientation
void UpdateRootOrientation_{ParameterName}()
{
	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);
	if(LocalIndex == 0)
	{
		float4 NodeQuaternion = SharedNodeOrientation[GGroupThreadId.x];
		float3 TangentPrev = RotateVectorByQuat( float3(0,0,1), NodeQuaternion);
		float3 TangentNext = normalize(SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x]);

		SharedNodeOrientation[GGroupThreadId.x] = NormalizeQuat( MultiplyQuat( FindQuatBetweenNormals(TangentPrev,TangentNext), NodeQuaternion) );
	}
	GroupMemoryBarrier();
}

// Compute the node orientation
void ComputeNodeOrientation_{ParameterName}(in float3 NodePosition, out float4 OutNodeOrientation)
{
	SharedNodePosition[GGroupThreadId.x] = NodePosition;
	SharedNodeOrientation[GGroupThreadId.x] = QUATERNION_IDENTITY;
	GroupMemoryBarrier();

	ComputeRootOrientation_{ParameterName}();

	ComputeMaterialFrame({ParameterName}_StrandSize);
	OutNodeOrientation = 	SharedNodeOrientation[GGroupThreadId.x];
}

/* -----------------------------------------------------------------
 * Edge volume, length, rotation, direction
 * -----------------------------------------------------------------
 */

// Init the samples along the strands that will be used to transfer informations to the grid
void InitGridSamples_{ParameterName}(in float3 NodePosition, in float3 NodeVelocity, 
	in float NodeMass, in float GridLength, out int OutNumSamples,
						out float3 OutDeltaPosition, out float3 OutDeltaVelocity, out float OutSampleMass)
{
	SharedNodePosition[GGroupThreadId.x] = NodePosition;
	SharedPreviousPosition[GGroupThreadId.x] = NodeVelocity;
	SharedInverseMass[GGroupThreadId.x] = NodeMass;
	GroupMemoryBarrier();

	const int LocalIndex = GGroupThreadId.x % {ParameterName}_StrandSize;

	if( LocalIndex > 0 )
	{
		OutDeltaVelocity = SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousPosition[GGroupThreadId.x-1];
		OutDeltaPosition = SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1];

		const float2 SegmentWeight = (LocalIndex == 1) ? float2(1.0,0.5) : (LocalIndex == ({ParameterName}_StrandSize-1) ) ? float2(0.5,1.0) : float2(0.5,0.5);
		const float SegmentMass =  (SharedInverseMass[GGroupThreadId.x-1] * SegmentWeight.x + SharedInverseMass[GGroupThreadId.x] * SegmentWeight.y);
		const float SegmentLength = length(OutDeltaPosition);

		OutNumSamples = isfinite(SegmentLength) ? min(ceil(SegmentLength / GridLength), 32) : 1;
		OutSampleMass = SegmentMass / OutNumSamples;
	}
	else
	{
		OutNumSamples = 0;
		OutDeltaPosition = float3(0,0,0);
		OutDeltaVelocity = float3(0,0,0);
		OutSampleMass = 0.0;
	}
}

// Get the sample state given an index and a delta position/velocity
void GetSampleState_{ParameterName}(in float3 NodePosition, in float3 NodeVelocity, in float3 DeltaPosition, in float3 DeltaVelocity, 
			in int NumSamples, in int SampleIndex, out float3 OutSamplePosition, out float3 OutSampleVelocity)
{
	const int LocalIndex = GGroupThreadId.x % {ParameterName}_StrandSize;

	if( LocalIndex > 0 )
	{
		const float SampleCoord = (0.5+SampleIndex) / NumSamples;
		OutSamplePosition = NodePosition + SampleCoord * DeltaPosition;
		OutSampleVelocity = NodeVelocity + SampleCoord * DeltaVelocity;
	}
	else
	{
		OutSamplePosition = float3(0,0,0);
		OutSampleVelocity = float3(0,0,0);
	}
}

/* -----------------------------------------------------------------
 * Edge volume, length, rotation, direction
 * -----------------------------------------------------------------
 */

// Compute the edge volume value
void ComputeEdgeVolume_{ParameterName}(in float3 NodePosition, out float OutEdgeVolume)
{
	SharedNodePosition[GGroupThreadId.x] = NodePosition;
	GroupMemoryBarrier(); 

	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);

	// L = ||P1-P0||
	OutEdgeVolume = 0.0;
	if(LocalIndex > 2)
	{
		const int NodeIndexA = GGroupThreadId.x;
		const int NodeIndexB = GGroupThreadId.x-1;
		const int NodeIndexC = GGroupThreadId.x-2;
		const int NodeIndexD = GGroupThreadId.x-3;

		const float3 EdgeVectorA = SharedNodePosition[NodeIndexB] - SharedNodePosition[NodeIndexA];
		const float3 EdgeVectorB = SharedNodePosition[NodeIndexC] - SharedNodePosition[NodeIndexA];
		const float3 EdgeVectorC = SharedNodePosition[NodeIndexD] - SharedNodePosition[NodeIndexA];

		OutEdgeVolume = dot(cross(EdgeVectorB,EdgeVectorC),EdgeVectorA);
	}
}

// Compute the edge length value
void ComputeEdgeLengthInternal_{ParameterName}(in float3 NodePosition, in int NodeOffset, out float OutEdgeLength)
{
	SharedNodePosition[GGroupThreadId.x] = NodePosition;
	GroupMemoryBarrier(); 

	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);

	// L = ||P1-P0||
	OutEdgeLength = (LocalIndex>NodeOffset) ? length(SharedNodePosition[GGroupThreadId.x]- SharedNodePosition[GGroupThreadId.x-1-NodeOffset]) : 0.0;
}

// Compute the edge darboux vector (diff between consecutive edge orientations)
void ComputeEdgeRotation_{ParameterName}(in float4 NodeOrientation, out float4 OutEdgeRotation)
{
	SharedNodeOrientation[GGroupThreadId.x] = NodeOrientation;
	GroupMemoryBarrier(); 

	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);

	// D = Q0^-1 * Q1
	if(LocalIndex>0)
	{
		float4 q0 = SharedNodeOrientation[GGroupThreadId.x-1];
		float4 q1 = SharedNodeOrientation[GGroupThreadId.x];
		OutEdgeRotation = float4(
				q1.xyz * q0.w - q0.xyz * q1.w + cross(-q0.xyz, q1.xyz),
				q0.w * q1.w - dot(-q0.xyz, q1.xyz));
		float4 OmegaPlus = OutEdgeRotation + float4(0,0,0,1);
		float4 OmegaMinus = OutEdgeRotation - float4(0,0,0,1);
		if( dot(OmegaMinus,OmegaMinus) > dot(OmegaPlus,OmegaPlus) ) OutEdgeRotation = -OutEdgeRotation;
		//OutEdgeRotation = MultiplyQuat(InverseQuat(SharedNodeOrientation[GGroupThreadId.x-1]),SharedNodeOrientation[GGroupThreadId.x]);
	}
	else
	{
		OutEdgeRotation = QUATERNION_IDENTITY;
	}
}

void ComputeEdgeDirection_{ParameterName}(in float3 NodePosition, in float4 NodeOrientation, 
	in float NodeMass, in float3 GravityVector, in float GravityPreloading, in float BendStiffness, in float StrandThickness, in float RestLength, out float3 OutRestDirection)
{
	const int LocalIndex = (GGroupThreadId.x % {ParameterName}_StrandSize);

	SharedNodePosition[GGroupThreadId.x] = NodePosition;
	SharedNodeOrientation[GGroupThreadId.x] = NodeOrientation;
	GroupMemoryBarrier();

	OutRestDirection = float3(0,0,0);
	
	if(GravityPreloading == 0.0)
	{
		if( LocalIndex > 1 )
		{
			const float3 EdgeDirection = SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1];
			OutRestDirection = UnrotateVectorByQuat(EdgeDirection,SharedNodeOrientation[GGroupThreadId.x-2]);
		}
		GroupMemoryBarrier();
	}
	else
	{
		float3 EdgeDirection = (LocalIndex > 1) ? SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1] : float3(0,0,0);

		SharedInverseInertia[GGroupThreadId.x] = (isfinite(RestLength) && (abs(RestLength) > 1e-8f)) ? BendStiffness*PI*StrandThickness*StrandThickness / (4.0*RestLength) : 0.0;
		SharedPreviousPosition[GGroupThreadId.x] = EdgeDirection;
		SharedPreviousOrientation[GGroupThreadId.x] = float4(EdgeDirection,0);

		GroupMemoryBarrier(); 

		{
			const bool IsNotLast = LocalIndex < ({ParameterName}_StrandSize-1);
			const float GradientNext = IsNotLast ?  SharedInverseInertia[GGroupThreadId.x+1]  : 0.0;
			const float GradientPrev = SharedInverseInertia[GGroupThreadId.x];
			const float SchurComplement =  (GradientNext * GradientNext + GradientPrev * GradientPrev);
			const bool bIsNodeValid = ( LocalIndex > 1 ) && (abs(SchurComplement) > 1e-8f);
			
			for(int Index = 0; Index < 100; ++Index)
			{
				const int IsRed = (GGroupThreadId.x % 2) == 0;
				float3 DeltaConstraint = float3(0,0,0);
				if (!IsRed)
				{
					DeltaConstraint = IsNotLast ? ((SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousOrientation[GGroupThreadId.x].xyz) * GradientPrev - 
													(SharedPreviousPosition[GGroupThreadId.x+1] - SharedPreviousOrientation[GGroupThreadId.x+1].xyz) * GradientNext) - NodeMass * GravityVector :
													((SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousOrientation[GGroupThreadId.x].xyz) * GradientPrev) - NodeMass * GravityVector;

					if( bIsNodeValid) SharedPreviousOrientation[GGroupThreadId.x].xyz += DeltaConstraint * GradientPrev / SchurComplement;
				}
				GroupMemoryBarrier(); 
				if(!IsRed && IsNotLast)
				{
					if( bIsNodeValid) SharedPreviousOrientation[GGroupThreadId.x+1].xyz -= DeltaConstraint * GradientNext / SchurComplement;
				}
				GroupMemoryBarrier(); 

				if (IsRed)
				{
					DeltaConstraint = IsNotLast ? ((SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousOrientation[GGroupThreadId.x].xyz) * GradientPrev - 
													(SharedPreviousPosition[GGroupThreadId.x+1] - SharedPreviousOrientation[GGroupThreadId.x+1].xyz) * GradientNext) - NodeMass * GravityVector :
													((SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousOrientation[GGroupThreadId.x].xyz) * GradientPrev) - NodeMass * GravityVector;


					if( bIsNodeValid) SharedPreviousOrientation[GGroupThreadId.x].xyz += DeltaConstraint * GradientPrev / SchurComplement;
				}
				GroupMemoryBarrier();
				if(IsRed && IsNotLast)
				{
					if( bIsNodeValid) SharedPreviousOrientation[GGroupThreadId.x+1].xyz -= DeltaConstraint * GradientNext / SchurComplement;
				}
				GroupMemoryBarrier(); 
			}
			EdgeDirection = SharedPreviousOrientation[GGroupThreadId.x].xyz * GravityPreloading + (1.0-GravityPreloading) * EdgeDirection;
			GroupMemoryBarrier(); 
			OutRestDirection = UnrotateVectorByQuat(EdgeDirection,SharedNodeOrientation[GGroupThreadId.x-2]);
		}

		GroupMemoryBarrier();
	}
}


/* -----------------------------------------------------------------
 * Points position update
 * -----------------------------------------------------------------
 */

// Reset the deformed points position to the rest ones
void ResetPointPosition_{ParameterName}(out bool ReportStatus) 
{
	int LocalIndex = GGroupThreadId.x % {ParameterName}_StrandSize;
	int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;

	ReportStatus = false;

	if(LocalIndex == 0)
	{
		const FHairCurve Curve = UnpackCurve_{ParameterName}(StrandIndex);
		const int PointBegin = Curve.PointIndex;
		const int PointCount = Curve.PointCount;

		const float3 PositionOffsetDelta = {ParameterName}_RestPositionOffset.xyz - {ParameterName}_DeformedPositionOffset[0].xyz;

		ReportStatus = true;
		for (int PointIndex = 0; PointIndex < PointCount; ++PointIndex)
		{
			const FPackedHairPosition Packed = ReadPackedHairPosition({ParameterName}_RestPositionBuffer, PointBegin+PointIndex);
			const float3 CPPosition = UnpackHairControlPointPosition(Packed);
			WritePackedHairControlPointPosition({ParameterName}_DeformedPositionBuffer, PointBegin + PointIndex, Packed, CPPosition + PositionOffsetDelta/*NewPosition*/);
		}
	}
}

// Compute the projection triangle
void ComputeProjectionTriangle_{ParameterName}(in float2 ProjectionUV, in float3 PA, in float3 PB, in float3 PC, in float3 Offset, 
			out float3 OutTrianglePosition, out float4 OutTriangleRotation)
{
	OutTrianglePosition = PA * ProjectionUV.x + PB * ProjectionUV.y + PC * (1.0 - ProjectionUV.x - ProjectionUV.y) + Offset;

	float3 TangentU = PB - PA;
	float3 TangentV = PC - PA;

	const float3 Normal = normalize(cross(TangentU, TangentV));
	TangentV = normalize(TangentV);
	TangentU = normalize(cross(TangentV, Normal));
	
	const float3 RotationMatrix[3] = { TangentU, TangentV, Normal };
	OutTriangleRotation = QuatFromMatrix(RotationMatrix);
}

// Get the deformed triangle
void BuildDeformedTriangle_{ParameterName}(in float2 ProjectionUV, in int TriangleIndex, out float3 OutTrianglePosition, out float4 OutTriangleRotation)
{
	ComputeProjectionTriangle_{ParameterName}(ProjectionUV, 
		{ParameterName}_DeformedTrianglePositionBuffer[TriangleIndex * 3 + 0].xyz,
		{ParameterName}_DeformedTrianglePositionBuffer[TriangleIndex * 3 + 1].xyz,
		{ParameterName}_DeformedTrianglePositionBuffer[TriangleIndex * 3 + 2].xyz,
	    {ParameterName}_DeformedRootOffset, OutTrianglePosition, OutTriangleRotation);
}

// Get the rest triangle
void BuildRestTriangle_{ParameterName}(in float2 ProjectionUV, in int TriangleIndex, out float3 OutTrianglePosition, out float4 OutTriangleRotation)
{
	ComputeProjectionTriangle_{ParameterName}(ProjectionUV, 
		{ParameterName}_RestTrianglePositionBuffer[TriangleIndex * 3 + 0].xyz,
		{ParameterName}_RestTrianglePositionBuffer[TriangleIndex * 3 + 1].xyz,
		{ParameterName}_RestTrianglePositionBuffer[TriangleIndex * 3 + 2].xyz,
	    {ParameterName}_RestRootOffset, OutTrianglePosition, OutTriangleRotation);
}

// Eval the triangle local position
float3 TriangleLocalPosition_{ParameterName}(in float3 TrianglePosition, in float4 TriangleRotation, in float3 WorldPosition)
{
	return RotateVectorByQuat(WorldPosition - TrianglePosition, InverseQuat(TriangleRotation));
}

// Eval the triangle local orientation
float4 TriangleLocalOrientation_{ParameterName}(in float3 TrianglePosition, in float4 TriangleRotation, in float4 WorldOrientation)
{
	return NormalizeQuat(MultiplyQuat(InverseQuat(TriangleRotation), WorldOrientation));
}

// Eval the triangle world position
float3 TriangleWorldPosition_{ParameterName}(in float3 TrianglePosition, in float4 TriangleRotation, in float3 LocalPosition)
{
	return RotateVectorByQuat(LocalPosition, TriangleRotation) + TrianglePosition;
}

// Eval the triangle local orientation
float4 TriangleWorldOrientation_{ParameterName}(in float3 TrianglePosition, in float4 TriangleRotation, in float4 LocalOrientation)
{
	return NormalizeQuat(MultiplyQuat(TriangleRotation, LocalOrientation));
}

// Report interpolated nodes displacements onto the points positions 
void UpdateTriangleDisplace_{ParameterName}(in int StrandIndex, in float3 NodePosition, in float3 RestPosition, 
		out float3 RestTrianglePosition, out float4 RestTriangleOrientation, out float3 DeformedTrianglePosition, out float4 DeformedTriangleOrientation)
{
	if( {ParameterName}_InterpolationMode >= 1)
	{
		const float2 ProjectionUV = UnpackBarycentrics({ParameterName}_RootBarycentricCoordinatesBuffer[StrandIndex]).xy;
		const uint TriangleIndex = {ParameterName}_RootToUniqueTriangleIndexBuffer[StrandIndex];

		BuildRestTriangle_{ParameterName}(ProjectionUV, TriangleIndex, RestTrianglePosition, RestTriangleOrientation);
		BuildDeformedTriangle_{ParameterName}(ProjectionUV, TriangleIndex, DeformedTrianglePosition, DeformedTriangleOrientation);
	
		const float3 LocalRestPosition = TriangleLocalPosition_{ParameterName}(RestTrianglePosition, RestTriangleOrientation, RestPosition);
		const float3 LocalDeformedPosition = TriangleLocalPosition_{ParameterName}(DeformedTrianglePosition, DeformedTriangleOrientation, NodePosition);
	
		SharedNodePosition[GGroupThreadId.x] = LocalDeformedPosition - LocalRestPosition;
		GroupMemoryBarrier();
	}
	else
	{
		SharedNodePosition[GGroupThreadId.x] = NodePosition - RestPosition;
		GroupMemoryBarrier();
	}
}

// Report interpolated nodes displacements onto the points positions 
void UpdateDeformedPositions_{ParameterName}(in float3 LocalDisplace, in int PositionIndex, 
		in float3 RestTrianglePosition, in float4 RestTriangleOrientation, in float3 DeformedTrianglePosition, in float4 DeformedTriangleOrientation)
{
	if( {ParameterName}_InterpolationMode >= 1)
	{
		const FPackedHairPosition Packed = ReadPackedHairPosition({ParameterName}_RestPositionBuffer, PositionIndex);
		const FHairControlPoint CP = UnpackHairControlPoint(Packed);

		const float3 LocalTrianglePosition = TriangleLocalPosition_{ParameterName}(RestTrianglePosition, RestTriangleOrientation, 
			CP.Position.xyz + {ParameterName}_RestPositionOffset) + LocalDisplace;
			
		const float3 LocalComponentPosition = TriangleWorldPosition_{ParameterName}(DeformedTrianglePosition, DeformedTriangleOrientation, LocalTrianglePosition) - {ParameterName}_DeformedPositionOffset[0].xyz;
		
		WritePackedHairControlPointPosition({ParameterName}_DeformedPositionBuffer, PositionIndex, Packed, LocalComponentPosition);
	}
	else
	{
		const FPackedHairPosition Packed = ReadPackedHairPosition({ParameterName}_RestPositionBuffer, PositionIndex);
		const FHairControlPoint CP = UnpackHairControlPoint(Packed);

		const float3 LocalComponentPosition = CP.Position.xyz + LocalDisplace + {ParameterName}_RestPositionOffset - {ParameterName}_DeformedPositionOffset[0].xyz;

		WritePackedHairControlPointPosition({ParameterName}_DeformedPositionBuffer, PositionIndex, Packed, LocalComponentPosition);
	}
}

// Report interpolated nodes displacements onto the points positions 
void UpdatePointPosition_{ParameterName}(in float3 NodePosition, in float3 RestPosition, out bool OutReportStatus)
{
	OutReportStatus = false;
	const int ReducedSize = {ParameterName}_StrandSize - 1;
	
	int LocalIndex = GGroupThreadId.x % {ParameterName}_StrandSize - 1;
	int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;

	float3 RestTrianglePosition = float3(0, 0, 0), DeformedTrianglePosition = float3(0, 0, 0);
	float4 RestTriangleOrientation = QUATERNION_IDENTITY, DeformedTriangleOrientation = QUATERNION_IDENTITY;

	UpdateTriangleDisplace_{ParameterName}(StrandIndex, NodePosition, RestPosition, 
			RestTrianglePosition, RestTriangleOrientation, DeformedTrianglePosition, DeformedTriangleOrientation);
	
	const FHairCurve Curve = UnpackCurve_{ParameterName}(StrandIndex);
	const int PointBegin = Curve.PointIndex;
	const int PointCount = Curve.PointCount;


	for (int PointIndex = 0; PointIndex < PointCount; ++PointIndex)
	{
		const float PointCoord = (float) (PointIndex) / (PointCount - 1.0);
		const float PointNode = PointCoord * (ReducedSize - 1.0);

		const int NodePrev = (PointIndex == 0) ? 0 : (PointIndex == (PointCount - 1.0)) ? ReducedSize - 2 : floor(PointNode);
		const int NodeNext = NodePrev + 1;

		if (NodePrev == LocalIndex)
		{
			OutReportStatus = true;
			const float PointAlpha = PointNode - (float) NodePrev;
			const float3 LocalDisplace = SharedNodePosition[GGroupThreadId.x] * (1.0 - PointAlpha) + SharedNodePosition[GGroupThreadId.x + 1] * PointAlpha;

			UpdateDeformedPositions_{ParameterName}(LocalDisplace, PointBegin + PointIndex, 
				RestTrianglePosition, RestTriangleOrientation, DeformedTrianglePosition, DeformedTriangleOrientation);
		}
	}
}

// Get the point position
void GetPointPosition_{ParameterName}(in int PointIndex, out float3 OutPointPosition)
{
	const float3 Position = ReadHairControlPointPosition({ParameterName}_DeformedPositionBuffer, PointIndex, {ParameterName}_DeformedPositionOffset[0].xyz);
	OutPointPosition = mul(float4(Position, 1.0), {ParameterName}_WorldTransform).xyz;
}

/* -----------------------------------------------------------------
 * Nodes time integration
 * -----------------------------------------------------------------
 */

// Add external force to the linear velocity and advect node position
void AdvectNodePosition_{ParameterName}(in float NodeMass, in bool IsPositionMobile, in float3 ExternalForce, in float3 ForceGradient, in float DeltaTime, inout float3 OutLinearVelocity, inout float3 OutNodePosition)
{	
	if(IsPositionMobile && NodeMass != 0.0)
	{
		const float3 ImplicitGradient =  float3(NodeMass,NodeMass,NodeMass) / DeltaTime - ForceGradient;
		const float3 InverseGradient = float3(1.0/ImplicitGradient.x, 1.0/ImplicitGradient.y, 1.0/ImplicitGradient.z);
		
		if({ParameterName}_LocalSimulation != 0)
		{
			OutLinearVelocity += InverseGradient * ExternalForce - {ParameterName}_BoneLinearAcceleration * DeltaTime - ( cross({ParameterName}_BoneAngularAcceleration, OutNodePosition) +
				2.0 * cross({ParameterName}_BoneAngularVelocity, OutLinearVelocity) + cross({ParameterName}_BoneAngularVelocity, cross({ParameterName}_BoneAngularVelocity,OutNodePosition))) * DeltaTime;
		}
		else
		{
			OutLinearVelocity += InverseGradient * ExternalForce;
		}
		OutNodePosition += OutLinearVelocity * DeltaTime;
	}
}

// Add external torque to the angular velocity and advect node orientation
void AdvectNodeOrientation_{ParameterName}(in float3 NodeInertia, in bool IsOrientationMobile, in float3 ExternalTorque, in float3 TorqueGradient, in float DeltaTime, inout float3 OutAngularVelocity, inout float4 OutNodeOrientation)
{	
	if(IsOrientationMobile && NodeInertia.x != 0.0 && NodeInertia.y != 0.0 && NodeInertia.z != 0.0)
	{
		const float3 ImplicitGradient = NodeInertia / DeltaTime - TorqueGradient;
		const float3 InverseGradient = float3(1.0/ImplicitGradient.x, 1.0/ImplicitGradient.y, 1.0/ImplicitGradient.z);

		OutAngularVelocity += InverseGradient * (ExternalTorque - cross(OutAngularVelocity, NodeInertia * OutAngularVelocity));

		OutNodeOrientation = OutNodeOrientation + 0.5 * DeltaTime * float4(
			OutAngularVelocity.xyz * OutNodeOrientation.w + cross(OutAngularVelocity.xyz, OutNodeOrientation.xyz),
			- dot(OutAngularVelocity.xyz, OutNodeOrientation.xyz));

		OutNodeOrientation = NormalizeQuat(OutNodeOrientation);
	}
}

// Update the node linear velocity based on the node position difference
void UpdateLinearVelocity_{ParameterName}(in float3 PreviousPosition, in float3 NodePosition, in float DeltaTime, out float3 OutLinearVelocity)
{	
	OutLinearVelocity = (NodePosition-PreviousPosition) / DeltaTime;
}


// Update the node angular velocity based on the node orientation difference
void UpdateAngularVelocity_{ParameterName}(in float4 PreviousOrientation, in float4 NodeOrientation, in float DeltaTime, out float3 OutAngularVelocity)
{
	const float4 DeltaQuat = MultiplyQuat(NodeOrientation,InverseQuat(PreviousOrientation));
	const float AxisLength = length( DeltaQuat.xyz );

	if (AxisLength<SMALL_NUMBER)
	{
		 OutAngularVelocity =  DeltaQuat.xyz * 2.0 / DeltaTime;
	}
	else
	{
		const float QuatAngle = 2.0 * atan2(AxisLength,DeltaQuat.w );
		OutAngularVelocity = DeltaQuat.xyz * QuatAngle / (AxisLength*DeltaTime);
	}
}

/* -----------------------------------------------------------------
 * Update node position orientation
 * -----------------------------------------------------------------
 */

// Compute rest position
void ComputeRestPosition_{ParameterName}(in float3 NodePosition, out float3 OutRestPosition)
{	
	OutRestPosition = mul(float4( NodePosition, 1.0), {ParameterName}_WorldInverse).xyz;
}

// Compute rest orientation
void ComputeRestOrientation_{ParameterName}(in float4 NodeOrientation, out float4 OutRestOrientation)
{	
	OutRestOrientation = NormalizeQuat( MultiplyQuat(InverseQuat({ParameterName}_WorldRotation),NodeOrientation) );
}

// Update Node Position
void AttachNodePosition_{ParameterName}(in float3 RestPosition, out float3 OutNodePosition)
{
	OutNodePosition = mul(float4( RestPosition, 1.0), {ParameterName}_WorldTransform).xyz;
}

// Update Node Orientation
void AttachNodeOrientation_{ParameterName}(in float4 RestOrientation, out float4 OutNodeOrientation)
{
	OutNodeOrientation = NormalizeQuat( MultiplyQuat({ParameterName}_WorldRotation,RestOrientation) );
}

// Get World Vector
void GetWorldVector_{ParameterName}(in float3 LocalVector, in bool IsPosition, out float3 OutWorldVector)
{	
	OutWorldVector = {ParameterName}_LocalSimulation != 0 ? mul(float4( LocalVector, (float)IsPosition), {ParameterName}_BoneTransform).xyz : LocalVector;
}

// Get Local Vector
void GetLocalVector_{ParameterName}(in float3 WorldVector, in bool IsPosition, out float3 OutLocalVector)
{
	OutLocalVector = {ParameterName}_LocalSimulation != 0 ? mul(float4( WorldVector, (float)IsPosition), {ParameterName}_BoneInverse).xyz : WorldVector;
}


/* -----------------------------------------------------------------
 * Attached the root to the skinned cache
 * -----------------------------------------------------------------
 */

float3 ApplyRBF_{ParameterName}(in float3 RestControlPoint)
{
	return ApplyRBF(
		RestControlPoint, 
		{ParameterName}_SampleCount, 
		{ParameterName}_RestSamplePositionsBuffer,
		{ParameterName}_MeshSampleWeightsBuffer,
		{ParameterName}_DeformedSamplePositionsBuffer,
		{ParameterName}_RBFLocalSpace);
}

 // Compute the global position
void EvalGlobalPosition_{ParameterName}(in float3 RestPosition, out float3 OutGlobalPosition)
{
	const float3 RestSkinnedPosition = RestPosition;
	OutGlobalPosition = ApplyRBF_{ParameterName}(RestPosition);
}

// Compute the global position
void EvalGlobalOrientation_{ParameterName}(in float3 GlobalPosition, inout float4 OutGlobalOrientation)
{
	SharedNodePosition[GGroupThreadId.x] = GlobalPosition;
	SharedNodeOrientation[GGroupThreadId.x] = OutGlobalOrientation;
	GroupMemoryBarrier(); 

	UpdateRootOrientation_{ParameterName}();

	ComputeMaterialFrame({ParameterName}_StrandSize);
	OutGlobalOrientation = 	SharedNodeOrientation[GGroupThreadId.x];
}

// Compute rest position and orientation
void ComputeLocalState_{ParameterName}(in float3 RestPosition, in float4 RestOrientation, 
												out float3 OutLocalPosition, out float4 OutLocalOrientation)
{
	OutLocalPosition = RestPosition;
	OutLocalOrientation = RestOrientation;

	if( {ParameterName}_InterpolationMode >= 1)
	{
		const int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;
		const float2 ProjectionUV = UnpackBarycentrics({ParameterName}_RootBarycentricCoordinatesBuffer[StrandIndex]).xy;
		const uint TriangleIndex = {ParameterName}_RootToUniqueTriangleIndexBuffer[StrandIndex];

		float3 RestTrianglePosition = float3(0, 0, 0);
		float4 RestTriangleOrientation = QUATERNION_IDENTITY;
		BuildRestTriangle_{ParameterName}(ProjectionUV, TriangleIndex, RestTrianglePosition, RestTriangleOrientation);
		
		OutLocalPosition = TriangleLocalPosition_{ParameterName}(RestTrianglePosition, RestTriangleOrientation, RestPosition);
		OutLocalOrientation = TriangleLocalOrientation_{ParameterName}(RestTrianglePosition, RestTriangleOrientation, RestOrientation);
	}
}

// Update Node Position and orientation
void AttachNodeState_{ParameterName}(in float3 LocalPosition, in float4 LocalOrientation, out float3 OutNodePosition, out float4 OutNodeOrientation)
{	
	OutNodePosition = LocalPosition;
	OutNodeOrientation = LocalOrientation;

	if( {ParameterName}_InterpolationMode >= 1)
	{
		const int StrandIndex = ExecIndex() / {ParameterName}_StrandSize;
		const float2 ProjectionUV = UnpackBarycentrics({ParameterName}_RootBarycentricCoordinatesBuffer[StrandIndex]).xy;
		const uint TriangleIndex = {ParameterName}_RootToUniqueTriangleIndexBuffer[StrandIndex];

		float3 DeformedTrianglePosition = float3(0, 0, 0);
		float4 DeformedTriangleOrientation = QUATERNION_IDENTITY;
		BuildDeformedTriangle_{ParameterName}(ProjectionUV, TriangleIndex, DeformedTrianglePosition, DeformedTriangleOrientation);
		
		OutNodePosition = TriangleWorldPosition_{ParameterName}(DeformedTrianglePosition, DeformedTriangleOrientation, LocalPosition);
		OutNodeOrientation = TriangleWorldOrientation_{ParameterName}(DeformedTrianglePosition, DeformedTriangleOrientation, LocalOrientation);
	}
}

void UpdateNodeState_{ParameterName}(in float3 RestPosition, in float3 NodePosition, in float4 NodeOrientation, out float3 OutNodePosition, out float4 OutNodeOrientation)
{
	float3 GlobalPosition =  NodePosition;
	float4 GlobalOrientation = NodeOrientation;

	if({ParameterName}_InterpolationMode == 2)
	{
		EvalGlobalPosition_{ParameterName}(RestPosition, GlobalPosition);
		EvalGlobalOrientation_{ParameterName}(GlobalPosition,GlobalOrientation);
	}
	AttachNodePosition_{ParameterName}(GlobalPosition,OutNodePosition);
	AttachNodeOrientation_{ParameterName}(GlobalOrientation,OutNodeOrientation);
}

/* -----------------------------------------------------------------
 * General Functions
 * -----------------------------------------------------------------
 */

void GetNumStrands_{ParameterName}(out int OutNumStrands)
{
	OutNumStrands = {ParameterName}_NumStrands;
}

void GetWorldTransform_{ParameterName}(out float4x4 OutWorldTransform)
{
	OutWorldTransform = {ParameterName}_WorldTransform;
}

void GetWorldInverse_{ParameterName}(out float4x4 OutWorldInverse)
{
	OutWorldInverse = {ParameterName}_WorldInverse;
}

void GetStretchScale_{ParameterName}(out float OutStretchScale)
{
	OutStretchScale = {ParameterName}_ParamsScaleBuffer[GGroupThreadId.x % {ParameterName}_StrandSize];
}

void GetBendScale_{ParameterName}(out float OutBendScale)
{
	OutBendScale = {ParameterName}_ParamsScaleBuffer[32 + GGroupThreadId.x % {ParameterName}_StrandSize];
}

void GetRadiusScale_{ParameterName}(out float OutRadiusScale)
{
	OutRadiusScale = {ParameterName}_ParamsScaleBuffer[64 + GGroupThreadId.x % {ParameterName}_StrandSize];
}

void GetThicknessScale_{ParameterName}(out float OutThicknessScale)
{
	OutThicknessScale = {ParameterName}_ParamsScaleBuffer[96 + GGroupThreadId.x % {ParameterName}_StrandSize];
}

void ComputeNodePosition_{ParameterName}(in float SmoothingFilter, out float3 OutNodePosition)
{
	ComputeNodePosition_{ParameterName}(OutNodePosition);
	SmoothNodePosition_{ParameterName}(SmoothingFilter,OutNodePosition);
}

void ComputeEdgeLength_{ParameterName}(in float3 NodePosition, in int NodeOffset, out float OutEdgeLength)
{
	if(NodeOffset == 2)
	{
		ComputeEdgeVolume_{ParameterName}(NodePosition,OutEdgeLength);
	}
	else
	{
		ComputeEdgeLengthInternal_{ParameterName}(NodePosition,NodeOffset,OutEdgeLength);
	}
}

void AdvectNodePosition_{ParameterName}(in float NodeMass, in bool IsPositionMobile, in float3 ExternalForce, in float3 ForceGradient, in float DeltaTime,
								in float3 LinearVelocity, in float3 NodePosition, out float3 OutLinearVelocity, out float3 OutNodePosition)
{
	OutLinearVelocity = LinearVelocity;
	OutNodePosition = NodePosition;
	AdvectNodePosition_{ParameterName}(NodeMass,IsPositionMobile,ExternalForce,ForceGradient,DeltaTime,OutLinearVelocity,OutNodePosition);
}

void AdvectNodeOrientation_{ParameterName}(in float3 NodeInertia, in bool IsOrientationMobile, in float3 ExternalTorque, in float3 TorqueGradient, in float DeltaTime,
								in float3 AngularVelocity, in float4 NodeOrientation, out float3 OutAngularVelocity, out float4 OutNodeOrientation)
{
	OutAngularVelocity = AngularVelocity;
	OutNodeOrientation = NodeOrientation;
	AdvectNodeOrientation_{ParameterName}(NodeInertia,IsOrientationMobile,ExternalTorque,TorqueGradient,DeltaTime,OutAngularVelocity,OutNodeOrientation);
}


void SetupDistanceSpringMaterial_{ParameterName}(in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, in int NodeOffset, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float OutMaterialMultiplier)
{
	if(NodeOffset == 0)
	{
		SetupStretchSpringMaterial({ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,false,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
	}
	else if( NodeOffset == 1)
	{
		SetupBendSpringMaterial({ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,false,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
	}
	else 
	{
		SetupTwistSpringMaterial({ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,false,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
	}
}

void SolveDistanceSpringMaterial_{ParameterName} (in bool EnableConstraint, in float RestLength, in float DeltaTime, in int NodeOffset, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, in float MaterialMultiplier, out float OutMaterialMultiplier)
{
	if(NodeOffset == 0)
	{
		SolveStretchSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,RestLength,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
	}
	else if(NodeOffset == 1)
	{
		SolveBendSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,RestLength,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
	}
	else 
	{
		SolveTwistSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,RestLength,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
	}
}

void ProjectDistanceSpringMaterial_{ParameterName} (in bool EnableConstraint, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, in int NodeOffset, out float3 OutNodePosition)
{
	if(NodeOffset == 0)
	{
		ProjectStretchSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,OutNodePosition);
	}
	else if(NodeOffset == 1)
	{
		ProjectBendSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,OutNodePosition);
	}
	else
	{
		ProjectTwistSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,OutNodePosition);
	}
}

void SetupAngularSpringMaterial_{ParameterName} (in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	SetupAngularSpringMaterial({ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,false,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
}

void SolveAngularSpringMaterial_{ParameterName} (in bool EnableConstraint, in float RestLength, in float3 RestDirection, in float DeltaTime, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	SolveAngularSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,RestLength, RestDirection,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
}

void ProjectAngularSpringMaterial_{ParameterName} (in bool EnableConstraint, in float YoungModulus, in float RodThickness, in float RestLength, in float3 RestDirection, in float DeltaTime, out float3 OutNodePosition)
{
	ProjectAngularSpringMaterial(EnableConstraint,{ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,RestDirection,DeltaTime,OutNodePosition);
}

void SetupStretchRodMaterial_{ParameterName} (in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	SetupStretchRodMaterial({ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,false,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
}

void SolveStretchRodMaterial_{ParameterName} (in bool EnableConstraint, in float RestLength, in float DeltaTime, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	SolveStretchRodMaterial(EnableConstraint,{ParameterName}_StrandSize,RestLength,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
}

void ProjectStretchRodMaterial_{ParameterName} (in bool EnableConstraint, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float3 OutNodePosition)
{
	ProjectStretchRodMaterial(EnableConstraint,{ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,OutNodePosition);
}

void SetupBendRodMaterial_{ParameterName} (in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	SetupBendRodMaterial({ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,DeltaTime,false,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
}

void SolveBendRodMaterial_{ParameterName} (in bool EnableConstraint, in float RestLength, in float4 RestRotation, in float DeltaTime, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	SolveBendRodMaterial(EnableConstraint,{ParameterName}_StrandSize,RestLength,RestRotation,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
}

void ProjectBendRodMaterial_{ParameterName} (in bool EnableConstraint, in float YoungModulus, in float RodThickness, in float RestLength, in float4 RestRotation, in float DeltaTime, out float4 OutNodeOrientation)
{
	ProjectBendRodMaterial(EnableConstraint,{ParameterName}_StrandSize,YoungModulus,RodThickness,RestLength,RestRotation,DeltaTime,OutNodeOrientation);
}

void SolveHardCollisionConstraint_{ParameterName} (in bool EnableConstraint, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal,  in float StaticFriction, in float KineticFriction, in float DeltaTime, out float3 OutMaterialMultiplier )
{
	OutMaterialMultiplier = float3(0,0,0);
	SolveHardCollisionConstraint(EnableConstraint,{ParameterName}_StrandSize,PenetrationDepth, CollisionPosition,CollisionVelocity,CollisionNormal,StaticFriction,KineticFriction,false,DeltaTime);
}

void ProjectHardCollisionConstraint_{ParameterName}(in bool EnableConstraint, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal, in float StaticFriction, in float KineticFriction, in float DeltaTime, out float3 OutNodePosition)
{
	ProjectHardCollisionConstraint(EnableConstraint,{ParameterName}_StrandSize,PenetrationDepth,CollisionPosition,CollisionVelocity,CollisionNormal,StaticFriction,KineticFriction,DeltaTime,OutNodePosition);
}

void SolveSoftCollisionConstraint_{ParameterName} (in bool EnableConstraint, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal,
	in float StaticFriction, in float KineticFriction, in float DeltaTime, in float MaterialDamping,
	in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier
)
{
	SolveSoftCollisionConstraint(EnableConstraint,{ParameterName}_StrandSize,PenetrationDepth, CollisionPosition,CollisionVelocity,CollisionNormal,StaticFriction,KineticFriction,false,DeltaTime,MaterialDamping,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
}

void ProjectSoftCollisionConstraint_{ParameterName} (in bool EnableConstraint, in float ConstraintStiffness, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal, in float StaticFriction, in float KineticFriction, in float DeltaTime, out float3 OutNodePosition )
{
	ProjectSoftCollisionConstraint(EnableConstraint,{ParameterName}_StrandSize,ConstraintStiffness,PenetrationDepth, CollisionPosition,CollisionVelocity,CollisionNormal,StaticFriction,KineticFriction,DeltaTime,OutNodePosition);
}

void SetupSoftCollisionConstraint_{ParameterName} (in float ConstraintStiffness, in float DeltaTime, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier )
{
	SetupSoftCollisionConstraint({ParameterName}_StrandSize,ConstraintStiffness,DeltaTime,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier);
}

void UpdateMaterialFrame_{ParameterName} (out float4 OutNodeOrientation)
{
	UpdateMaterialFrame({ParameterName}_StrandSize);
	OutNodeOrientation = SharedNodeOrientation[GGroupThreadId.x];
}

void ComputeMaterialFrame_{ParameterName} ( out float4 OutNodeOrientation)
{
	ComputeMaterialFrame({ParameterName}_StrandSize);
	OutNodeOrientation = SharedNodeOrientation[GGroupThreadId.x];
}

void ComputeAirDragForce_{ParameterName}(in float AirDensity, in float AirViscosity, in float AirDrag, in float3 AirVelocity, in float NodeThickness, in float3 NodePosition, in float3 NodeVelocity, out float3 OutAirDrag, out float3 OutDragGradient)
{
	ComputeAirDragForce({ParameterName}_StrandSize,AirDensity,AirViscosity,AirDrag,AirVelocity,NodeThickness,NodePosition,NodeVelocity,OutAirDrag,OutDragGradient);
}

void NeedSimulationReset_{ParameterName} ( out bool ResetSimulation)
{
	ResetSimulation  = {ParameterName}_ResetSimulation;
}

void HasGlobalInterpolation_{ParameterName}(out bool GlobalInterpolation)
{
	GlobalInterpolation  = ({ParameterName}_InterpolationMode == 2);
}

void NeedRestUpdate_{ParameterName}( out bool RestUpdate)
{
	RestUpdate  = {ParameterName}_RestUpdate;
}
