// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfacePressureGrid.ush 
=============================================================================*/

#pragma once

static const int FLUID_PRESSURE_OFFSET = 6;
static const int SOLID_WEIGHT_OFFSET = 7;
static const int SOLID_VELOCITY_OFFSET = 10;
static const int SOLID_DISTANCE_OFFSET = 13;
static const int FLUID_DISTANCE_OFFSET = 14;
static const int FLUID_DENSITY_OFFSET = 15;

