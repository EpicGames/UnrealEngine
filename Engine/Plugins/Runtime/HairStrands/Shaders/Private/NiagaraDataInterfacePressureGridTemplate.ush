// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfacePressureGrid.ush 
=============================================================================*/

/* -----------------------------------------------------------------
 * Scale the velocity, mass and distance buffer from the current to the destination buffer
 * -----------------------------------------------------------------
 */

void ScaleCellDistance_{ParameterName}(in int3 GridIndex, in float GridLength )
{
	float FluidDistance = ConvertGridScalar_{ParameterName}(GridIndex,FLUID_DISTANCE_OFFSET);

	[branch]
	if(FluidDistance < 0.5 * GridLength)
	{
		const float SolidDistance = (GetGridScalar_{ParameterName}(int3(GridIndex.x+0,GridIndex.y+0,GridIndex.z+0),SOLID_DISTANCE_OFFSET) + 
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+0,GridIndex.y+0,GridIndex.z+1),SOLID_DISTANCE_OFFSET) +
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+0,GridIndex.y+1,GridIndex.z+0),SOLID_DISTANCE_OFFSET) +
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+0,GridIndex.y+1,GridIndex.z+1),SOLID_DISTANCE_OFFSET) +
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+1,GridIndex.y+0,GridIndex.z+0),SOLID_DISTANCE_OFFSET) +
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+1,GridIndex.y+0,GridIndex.z+1),SOLID_DISTANCE_OFFSET) +
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+1,GridIndex.y+1,GridIndex.z+0),SOLID_DISTANCE_OFFSET) +
									 GetGridScalar_{ParameterName}(int3(GridIndex.x+1,GridIndex.y+1,GridIndex.z+1),SOLID_DISTANCE_OFFSET) ) / 8.0;

		if(SolidDistance < 0.0) FluidDistance = -0.5 * GridLength;
	}

	SetGridScalar_{ParameterName}(GridIndex,FLUID_DISTANCE_OFFSET,FluidDistance);
}

void ScaleFaceVelocity_{ParameterName}(in int3 GridIndex, in int ComponentIndex)
{
	const float GridMomentum = ConvertGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET+ComponentIndex);
	const float GridMass = ConvertGridScalar_{ParameterName}(GridIndex,FLUID_MASS_OFFSET+ComponentIndex);

	float GridVelocity = (GridMass != 0.0) ? GridMomentum / GridMass : 0.0;

	SetGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET+ComponentIndex,GridVelocity);
	SetGridScalar_{ParameterName}(GridIndex,FLUID_MASS_OFFSET+ComponentIndex,GridMass);
}

void ScaleCellFields_{ParameterName}(in int GridHash, in float GridLength, in float DeltaTime, out bool OutTransferStatus)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);

	OutTransferStatus = false;
	if(IsFaceValid_{ParameterName}(GridIndex,0))
	{
		ScaleFaceVelocity_{ParameterName}(GridIndex,0);
	}
	if(IsFaceValid_{ParameterName}(GridIndex,1))
	{
		ScaleFaceVelocity_{ParameterName}(GridIndex,1);
	}
	if(IsFaceValid_{ParameterName}(GridIndex,2))
	{
		ScaleFaceVelocity_{ParameterName}(GridIndex,2);
	}
	if(IsCellValid_{ParameterName}(GridIndex))
	{
		ScaleCellDistance_{ParameterName}(GridIndex,GridLength);
	}
	if(IsCellValid_{ParameterName}(GridIndex))
	{
		const float SourceDensity = ConvertGridScalar_{ParameterName}(GridIndex,FLUID_DENSITY_OFFSET);

		const float CurrentDensity = max(1.0,SourceDensity);
		//const float DensityRatio = GridLength * (1.0-CurrentDensity) / DeltaTime;
		const float DensityRatio = (CurrentDensity != 0.0) ? GridLength * log(1.0/CurrentDensity) / DeltaTime : 0.0;

		SetGridScalar_{ParameterName}(GridIndex,FLUID_DENSITY_OFFSET,DensityRatio);
	}
}

/* -----------------------------------------------------------------
 * Build Distance/Density Field from particles
 * -----------------------------------------------------------------
 */
 
// Compute the distance from the particle to the grid
void BuildDistanceField_{ParameterName}(in float3 GridOrigin, in float GridLength, in float3 ParticlePosition, out bool OutFunctionStatus )
{
	OutFunctionStatus = true;
	const float3 GridPosition = GetGridPosition_{ParameterName}(ParticlePosition,GridOrigin);

	float3 GridFraction = float3(0,0,0);
	const int3 GridIndex = ComputeGridIndex_{ParameterName}(GridPosition,float3(0.5,0.5,0.5),GridLength,GridFraction);

	const float NodeRadius = GridLength * 0.5 * 1.414213;

	for(int i = -2; i <= 2; ++i)
	for(int j = -2; j <= 2; ++j)
	for(int k = -2; k <= 2; ++k)
	{
		const int3 GridLocation = int3(GridIndex.x+i,GridIndex.y+j,GridIndex.z+k);

		if( IsCellValid_{ParameterName}(GridLocation) )
		{
			const float3 CellPosition = (GridLocation+float3(0.5,0.5,0.5))*GridLength+GridOrigin;
			const float CellDistance = length(CellPosition-ParticlePosition) - NodeRadius; 

			InterlockedMinScalar_{ParameterName}(GridLocation,FLUID_DISTANCE_OFFSET,CellDistance);
		}
	}
}

// Transfer the density from the particle to the grid
void BuildDensityField_{ParameterName}(in float3 GridOrigin, in float GridLength,
				in float3 ParticlePosition, in float ParticleMass, in float ParticleDensity, out bool OutFunctionStatus )
{
	OutFunctionStatus = true;
	const float3 GridPosition = GetGridPosition_{ParameterName}(ParticlePosition,GridOrigin);

	float3 GridFraction = float3(0,0,0);
	const int3 GridIndex = ComputeGridIndex_{ParameterName}(GridPosition,float3(0.5,0.5,0.5),GridLength,GridFraction);

	const float GridScale = 1.0 / (GridLength*GridLength*GridLength);

	for(int i = 0; i < 2; ++i)
	{
		const float GridWeightX = (i == 0) ? 1.0 - GridFraction.x : GridFraction.x;
		for(int j = 0; j < 2; ++j)
		{
			const float GridWeightY = (j == 0) ? 1.0 - GridFraction.y : GridFraction.y;
			for(int k = 0; k < 2; ++k)
			{
				const float GridWeightZ = (k == 0) ? 1.0 - GridFraction.z : GridFraction.z;
				const float GridWeightXYZ = GridWeightX*GridWeightY*GridWeightZ;

				const int3 GridLocation = int3(GridIndex.x+i,GridIndex.y+j,GridIndex.z+k);

				if( IsCellValid_{ParameterName}(GridLocation) )
				{
					InterlockedAddScalar_{ParameterName}(GridLocation,FLUID_DENSITY_OFFSET,GridWeightXYZ*ParticleMass*GridScale);
				}
			}
		}
	}
}

/* -----------------------------------------------------------------
 * Get Density field from grid
 * -----------------------------------------------------------------
 */

// Sample the grid velocity field and gradient at the particle world position 
void GetDensityField_{ParameterName}(in float3 GridOrigin, in float GridLength, 
			in float3 ParticlePosition, out float OutParticleDensity)
{
	OutParticleDensity = 0.0;

	if(isfinite(GridLength))
	{
		const float3 GridPosition = GetGridPosition_{ParameterName}(ParticlePosition,GridOrigin);

		float3 GridFraction = float3(0,0,0);
		const int3 GridIndex = ComputeGridIndex_{ParameterName}(GridPosition,float3(0.5,0.5,0.5),GridLength,GridFraction);

		for(int i = 0; i < 2; ++i)
		{
			const float GridWeightX = (i == 0) ? 1.0 - GridFraction.x : GridFraction.x;
			for(int j = 0; j < 2; ++j)
			{
				const float GridWeightY = (j == 0) ? 1.0 - GridFraction.y : GridFraction.y;
				for(int k = 0; k < 2; ++k)
				{
					const float GridWeightZ = (k == 0) ? 1.0 - GridFraction.z : GridFraction.z;
					const float GridWeightXYZ = GridWeightX*GridWeightY*GridWeightZ;

					const int3 GridLocation = int3(GridIndex.x+i,GridIndex.y+j,GridIndex.z+k);

					if( IsCellValid_{ParameterName}(GridLocation) )
					{
						OutParticleDensity += GetGridScalar_{ParameterName}(GridLocation,FLUID_DENSITY_OFFSET) * GridWeightXYZ;
					}
				}
			}
		}
	}
}

// Sample the grid velocity field and gradient at the particle world position 
void UpdateDeformationGradient_{ParameterName}(in float DeltaTime, in float4x4 VelocityGradient, 
				in float4x4 DeformationGradient, out float4x4 OutDeformationGradient, out float OutGradientDeterminant)
{
	float3x3 PositionGradient;
	PositionGradient[0] = VelocityGradient[0].xyz * DeltaTime;
	PositionGradient[1] = VelocityGradient[1].xyz * DeltaTime;
	PositionGradient[2] = VelocityGradient[2].xyz * DeltaTime;

	float3x3 ElasticGradient;
	ElasticGradient[0] = DeformationGradient[0].xyz;
	ElasticGradient[1] = DeformationGradient[1].xyz;
	ElasticGradient[2] = DeformationGradient[2].xyz;

	if(isfinite(PositionGradient[0][0]) && isfinite(PositionGradient[1][1]) && isfinite(PositionGradient[2][2]) )
	{
		ElasticGradient += mul(PositionGradient,ElasticGradient);
	}
	OutGradientDeterminant = determinant(ElasticGradient);

	OutDeformationGradient[0] = float4(ElasticGradient[0].xyz,0.0);
	OutDeformationGradient[1] = float4(ElasticGradient[1].xyz,0.0);
	OutDeformationGradient[2] = float4(ElasticGradient[2].xyz,0.0);
	OutDeformationGradient[3] = float4(0.0,0.0,0.0,0.0);
}

/* -----------------------------------------------------------------
 * Update boundary weights
 * -----------------------------------------------------------------
 */

 float GetFractionInsideEdge_{ParameterName}(in float SignedDistanceA, in float SignedDistanceB)
 {
	const bool IsInsideA = SignedDistanceA<0.0;
	const bool IsInsideB = SignedDistanceB<0.0;

	return (IsInsideA && IsInsideB) ? 1.0 : (IsInsideA && !IsInsideB) ? SignedDistanceA / (SignedDistanceA - SignedDistanceB) : 
		   (!IsInsideA && IsInsideB) ? SignedDistanceB / (SignedDistanceB - SignedDistanceA)  : 0.0;
 }

 float GetFractionInsideImplA_{ParameterName}(in float SignedDistanceA, in float SignedDistanceB, in float SignedDistanceD)
 {
	const float Side0 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceD);
	const float Side1 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceB);
	return 1.0 - 0.5 * Side0 * Side1;
 }

 float GetFractionInsideImplB_{ParameterName}(in float SignedDistanceA, in float SignedDistanceB, in float SignedDistanceC, in float SignedDistanceD)
 {
	const float Side0 = GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceD);
	const float Side1 = GetFractionInsideEdge_{ParameterName}(SignedDistanceB,SignedDistanceC);
	return 0.5 * ( Side0 + Side1);
 }

 float GetFractionInsideImplC_{ParameterName}(in float SignedDistanceA, in float SignedDistanceB, in float SignedDistanceC, in float SignedDistanceD)
 {
	const float MiddlePoint = 0.25 * (SignedDistanceA+SignedDistanceB+SignedDistanceC+SignedDistanceD);
	if(MiddlePoint < 0.0)
	{
		float FaceArea = 0.0;

		const float Side1 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceD);
		const float Side3 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceC,SignedDistanceD);
		FaceArea += 0.5 * Side3 * Side1;

		const float Side2 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceC,SignedDistanceB);
		const float Side0 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceB);
		FaceArea += 0.5 * Side0 * Side2;

		return 1.0 - FaceArea;
	}
	else
	{
		float FaceArea = 0.0;

		const float Side0 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceB);
		const float Side1 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceD);
		FaceArea += 0.5 * Side0 * Side1;

		const float Side2 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceC,SignedDistanceB);
		const float Side3 = 1.0 - GetFractionInsideEdge_{ParameterName}(SignedDistanceC,SignedDistanceD);
		FaceArea += 0.5 * Side3 * Side2;

		return FaceArea;

	}
 }

 float GetFractionInsideImplD_{ParameterName}(in float SignedDistanceA, in float SignedDistanceB, in float SignedDistanceD)
 {
	const float Side0 = GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceD);
	const float Side1 = GetFractionInsideEdge_{ParameterName}(SignedDistanceA,SignedDistanceB);
	return 0.5 * Side0 * Side1;
 }

 float GetFractionInsideFace_{ParameterName}(in float SignedDistanceA, in float SignedDistanceB, in float SignedDistanceC, in float SignedDistanceD )
 {
	const bool IsInsideA = SignedDistanceA<0.0;
	const bool IsInsideB = SignedDistanceB<0.0;
	const bool IsInsideC = SignedDistanceC<0.0;
	const bool IsInsideD = SignedDistanceD<0.0;

	const int InsideCount = IsInsideA + IsInsideB + IsInsideC + IsInsideD;

	[branch]
	if(InsideCount == 4)
	{
		return 1.0;
	}
	else if(InsideCount == 3)
	{
		[branch]
		if(!IsInsideA)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceA,SignedDistanceB,SignedDistanceD);
		}
		else if(!IsInsideB)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceB,SignedDistanceC,SignedDistanceA);
		}
		else if(!IsInsideC)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceC,SignedDistanceD,SignedDistanceB);
		}
		else if(!IsInsideD)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceD,SignedDistanceA,SignedDistanceC);
		}
	}
	else if(InsideCount == 2)
	{
		[branch]
		if(IsInsideA && IsInsideB)
		{
			return GetFractionInsideImplB_{ParameterName}(SignedDistanceA,SignedDistanceB,SignedDistanceC, SignedDistanceD);
		}
		else if(IsInsideB && IsInsideC)
		{
			return GetFractionInsideImplB_{ParameterName}(SignedDistanceB,SignedDistanceC,SignedDistanceD, SignedDistanceA);
		}
		else if(IsInsideC && IsInsideD)
		{
			return GetFractionInsideImplB_{ParameterName}(SignedDistanceC,SignedDistanceD,SignedDistanceA, SignedDistanceB);
		}
		else if(IsInsideD && IsInsideA)
		{
			return GetFractionInsideImplB_{ParameterName}(SignedDistanceD,SignedDistanceA,SignedDistanceB, SignedDistanceC);
		}
		else if(IsInsideA && IsInsideC)
		{
			return GetFractionInsideImplC_{ParameterName}(SignedDistanceA,SignedDistanceB,SignedDistanceC, SignedDistanceD);
		}
		else if(IsInsideB && IsInsideD)
		{
			return GetFractionInsideImplC_{ParameterName}(SignedDistanceB,SignedDistanceC,SignedDistanceD, SignedDistanceA);
		}
	}
	else if(InsideCount == 1)
	{
		[branch]
		if(IsInsideA)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceA,SignedDistanceB,SignedDistanceD);
		}
		else if(IsInsideB)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceB,SignedDistanceC,SignedDistanceA);
		}
		else if(IsInsideC)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceC,SignedDistanceD,SignedDistanceB);
		}
		else if(IsInsideD)
		{
			return GetFractionInsideImplA_{ParameterName}(SignedDistanceD,SignedDistanceA,SignedDistanceC);
		}
	}
	return 0;
 }

// Update the solid weights and velocities
void SetSolidBoundary_{ParameterName}(in int GridHash, in float SolidDistance, in float3 SolidVelocity, out bool OutBoundaryStatus)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);
	OutBoundaryStatus = false;

	[branch]
	if( IsNodeValid_{ParameterName}(GridIndex))
	{
		OutBoundaryStatus = true;
		SetGridScalar_{ParameterName}(GridIndex,SOLID_DISTANCE_OFFSET,SolidDistance);
		SetGridVector_{ParameterName}(GridIndex,SOLID_VELOCITY_OFFSET,SolidVelocity);
	}
}

void ComputeFaceWeights_{ParameterName}(in int3 GridLocation, in int ComponentIndex, out float SignedDistance, inout float ParticleVelocity )
{
	SignedDistance = GetGridScalar_{ParameterName}(GridLocation,SOLID_DISTANCE_OFFSET);
	ParticleVelocity += GetGridScalar_{ParameterName}(GridLocation,SOLID_VELOCITY_OFFSET+ComponentIndex);
}

void ComputeComponentWeights_{ParameterName}(in int3 GridIndex, in int ComponentIndex)
{
	[branch]
	if( IsFaceInside_{ParameterName}(GridIndex,ComponentIndex))
	{
		float SignedDistance[4];
		float ParticleVelocity = 0.0;

		[branch]
		if(ComponentIndex == 0)
		{
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x,GridIndex.y+0,GridIndex.z+0),ComponentIndex,SignedDistance[0],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x,GridIndex.y+0,GridIndex.z+1),ComponentIndex,SignedDistance[1],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x,GridIndex.y+1,GridIndex.z+0),ComponentIndex,SignedDistance[2],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x,GridIndex.y+1,GridIndex.z+1),ComponentIndex,SignedDistance[3],ParticleVelocity);
		}
		else if(ComponentIndex == 1)
		{
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+0,GridIndex.y,GridIndex.z+0),ComponentIndex,SignedDistance[0],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+0,GridIndex.y,GridIndex.z+1),ComponentIndex,SignedDistance[1],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+1,GridIndex.y,GridIndex.z+0),ComponentIndex,SignedDistance[2],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+1,GridIndex.y,GridIndex.z+1),ComponentIndex,SignedDistance[3],ParticleVelocity);
		}
		else
		{
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+0,GridIndex.y+0,GridIndex.z),ComponentIndex,SignedDistance[0],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+0,GridIndex.y+1,GridIndex.z),ComponentIndex,SignedDistance[1],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+1,GridIndex.y+0,GridIndex.z),ComponentIndex,SignedDistance[2],ParticleVelocity);
			ComputeFaceWeights_{ParameterName}(int3(GridIndex.x+1,GridIndex.y+1,GridIndex.z),ComponentIndex,SignedDistance[3],ParticleVelocity);
		}

		const float SolidWeight = max(0.0,min(1.0 - GetFractionInsideFace_{ParameterName}(SignedDistance[0],SignedDistance[1],SignedDistance[3],SignedDistance[2]),1.0));
		const float SolidVelocity = 0.25 * ParticleVelocity;

		SetGridScalar_{ParameterName}(GridIndex,SOLID_WEIGHT_OFFSET+ComponentIndex,SolidWeight);
		SetGridScalar_{ParameterName}(GridIndex,SOLID_VELOCITY_OFFSET+ComponentIndex,SolidVelocity);
	}
}

 // Update the solid weights and velocities
void ComputeBoundaryWeights_{ParameterName}(in int GridHash, out bool OutWeightsStatus)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);
	OutWeightsStatus = false;

	ComputeComponentWeights_{ParameterName}(GridIndex,0);
	ComputeComponentWeights_{ParameterName}(GridIndex,1);
	ComputeComponentWeights_{ParameterName}(GridIndex,2);
}

/* -----------------------------------------------------------------
 * Sample Velocity field from grid
 * -----------------------------------------------------------------
 */

// Compute the grid pressure
// Mdv/dt = Jt * P with C = Volume * div(V) = 0.0 = ( vnx * wnx * dx * dx - vpx * wpx * dx * dx ) + ... (divergence theorem)
// -> dv = inv(M) * Jt * P * dt;
// Jnx = dC/dvnx = wnx * dx * dx, Jpx = dC/dvpx = -wpx * dx * dx
// since C = C0 + dC/dv * dv = C0 + J * dv = 0 = C0 + J * inv(M) * Jt * P * dt; and M = wnx * mnx
// the schur complement is J * inv(M) * Jt * P * dt = -C0 -> P * dt = -C0 / (wnx * dx * dx * wnx * dx * dx / ( wnx * mnx) + wpx * ....)
// P * dt = - sum( vnx * wnx * dx * dx - vpx * wpx * dx * dx ) / sum (wnx * dx * dx * wnx * dx * dx / ( wnx * mnx);
//        = - sum( vnx * wnx ) / sum( wnx * dx * dx / mnx ) = -sum( vnx * wnx ) / ( dx * dx * sum( wnx / mnx ) )
// P * dt * dx * dx = Pt = -sum( vnx * wnx ) / ( sum( wnx / mnx ) )
// dv = wnx * dx * dx * P * dt / (wnx * mnx) = Pt / mnx

float ComputeGridPressure_{ParameterName}(in int3 PrevIndex, in float CentreDistance, out float3 PrevFraction, out float3 NextFraction)
{	
	float GridPressure = 0.0;
	float GridDivergence = 0.0;
	float GridFraction = 0.0;

	for(int ComponentIndex = 0; ComponentIndex < 3; ++ComponentIndex)
	{
		int3 NextIndex = PrevIndex; NextIndex[ComponentIndex] += 1;

		const float PrevDistance = GetGridScalar_{ParameterName}(PrevIndex,FLUID_DISTANCE_OFFSET);
		PrevFraction[ComponentIndex] = (PrevDistance < 0.0) ? 1.0 : max(GetFractionInsideEdge_{ParameterName}(CentreDistance,PrevDistance),0.01);
		const float PrevWeight = GetGridScalar_{ParameterName}(PrevIndex,SOLID_WEIGHT_OFFSET+ComponentIndex);
			
		GridDivergence -= GetGridScalar_{ParameterName}(PrevIndex,SOLID_VELOCITY_OFFSET+ComponentIndex) * (1.0-PrevWeight);
		GridDivergence -= GetGridScalar_{ParameterName}(PrevIndex,FLUID_VELOCITY_OFFSET+ComponentIndex) * PrevWeight;
		GridFraction += PrevWeight / PrevFraction[ComponentIndex];

		const float NextDistance = GetGridScalar_{ParameterName}(NextIndex,FLUID_DISTANCE_OFFSET);
		NextFraction[ComponentIndex] = (NextDistance < 0.0) ? 1.0 : max(GetFractionInsideEdge_{ParameterName}(CentreDistance,NextDistance),0.01);
		const float NextWeight = GetGridScalar_{ParameterName}(NextIndex,SOLID_WEIGHT_OFFSET+ComponentIndex);
			
		GridDivergence += GetGridScalar_{ParameterName}(NextIndex,SOLID_VELOCITY_OFFSET+ComponentIndex) * (1.0-NextWeight);
		GridDivergence += GetGridScalar_{ParameterName}(NextIndex,FLUID_VELOCITY_OFFSET+ComponentIndex) * NextWeight;
		GridFraction += NextWeight / NextFraction[ComponentIndex];
	}
	if( GridFraction > 0.0)
	{
		//GridDivergence += GetGridScalar_{ParameterName}(PrevIndex,FLUID_DENSITY_OFFSET);
		GridPressure = -GridDivergence / (GridFraction);
	}
	return GridPressure;
}

// Project the pressure grid to be divergence free
void SolveGridPressure_{ParameterName}(in int GridHash, in int InitStage, out bool OutProjectStatus)
{	
	OutProjectStatus = false;

	const int3 PrevIndex = GetGridIndex_{ParameterName}(GridHash);
	const bool IsInitStage = (SimulationStageIndex == InitStage) || (SimulationStageIndex == (InitStage+1));

	if(IsCellValid_{ParameterName}(PrevIndex))
	{
		if( (PrevIndex.x+PrevIndex.y+PrevIndex.z+SimulationStageIndex) % 2 == 0)
		{
			OutProjectStatus = true;
			const float CentreDistance = GetGridScalar_{ParameterName}(PrevIndex,FLUID_DISTANCE_OFFSET);
			if( CentreDistance < 0.0)
			{
				float3 PrevFraction = float3(0,0,0);
				float3 NextFraction = float3(0,0,0);

				float GridPressure = ComputeGridPressure_{ParameterName}(PrevIndex,CentreDistance,PrevFraction,NextFraction);
				if(!IsInitStage)
				{
					AddGridScalar_{ParameterName}(PrevIndex,FLUID_PRESSURE_OFFSET,GridPressure);
				}
				else 
				{
					GridPressure =  GetGridScalar_{ParameterName}(PrevIndex,FLUID_PRESSURE_OFFSET);
				}

				for( int ComponentIndex = 0; ComponentIndex < 3; ++ComponentIndex)
				{
					int3 NextIndex = PrevIndex; NextIndex[ComponentIndex] += 1;

					const float PrevWeight = GetGridScalar_{ParameterName}(PrevIndex,SOLID_WEIGHT_OFFSET+ComponentIndex);
					if( PrevWeight > 0.0 && PrevFraction[ComponentIndex] > 0.0)
					{
						AddGridScalar_{ParameterName}(PrevIndex,FLUID_VELOCITY_OFFSET+ComponentIndex,-GridPressure/PrevFraction[ComponentIndex]);
					}
					else
					{
						const float SolidVelocity = GetGridScalar_{ParameterName}(PrevIndex,SOLID_VELOCITY_OFFSET+ComponentIndex);
						SetGridScalar_{ParameterName}(PrevIndex,FLUID_VELOCITY_OFFSET+ComponentIndex,SolidVelocity);
					}

					const float NextWeight = GetGridScalar_{ParameterName}(NextIndex,SOLID_WEIGHT_OFFSET+ComponentIndex);
					if( NextWeight > 0.0 && NextFraction[ComponentIndex] > 0.0)
					{
						AddGridScalar_{ParameterName}(NextIndex,FLUID_VELOCITY_OFFSET+ComponentIndex,GridPressure/NextFraction[ComponentIndex]);
					}
					else
					{
						const float SolidVelocity = GetGridScalar_{ParameterName}(NextIndex,SOLID_VELOCITY_OFFSET+ComponentIndex);
						SetGridScalar_{ParameterName}(NextIndex,FLUID_VELOCITY_OFFSET+ComponentIndex,SolidVelocity);
					}
				}
			}
			else
			{
				SetGridScalar_{ParameterName}(PrevIndex,FLUID_PRESSURE_OFFSET,0.0);
			}
		}
	}
}
