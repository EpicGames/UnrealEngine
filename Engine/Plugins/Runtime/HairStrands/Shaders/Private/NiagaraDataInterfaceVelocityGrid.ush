// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfaceVelocityGrid.ush 
=============================================================================*/

#pragma once

#define FAST_MODE 1

static const float FLOAT_PRECISION = 1e+4;

static const int FLUID_MASS_OFFSET = 0;
static const int FLUID_VELOCITY_OFFSET = 3;
