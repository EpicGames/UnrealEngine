// Copyright Epic Games, Inc. All Rights Reserved.

float4x4		{ParameterName}_WorldTransform;
float4x4		{ParameterName}_WorldInverse;
int3			{ParameterName}_GridSize;
Buffer<int>		{ParameterName}_GridCurrentBuffer;
RWBuffer<int>	{ParameterName}_GridDestinationBuffer;

/* -----------------------------------------------------------------
 * Grid topology conversion
 * -----------------------------------------------------------------
 */

// Compute the grid origin and grid length
void ComputeGridSize_{ParameterName}(in float3 GridCenter,  in float3 GridExtent, out float3 OutGridOrigin, out float OutGridLength)
{
	const float3 GridLengths = float3(2.0*GridExtent.x/({ParameterName}_GridSize.x-2),
									  2.0*GridExtent.y/({ParameterName}_GridSize.y-2),
									  2.0*GridExtent.z/({ParameterName}_GridSize.z-2));

	const float MaxLength = max(GridLengths.x,max(GridLengths.y,GridLengths.z));

	const float3 RegularSize = float3({ParameterName}_GridSize.x*MaxLength,
									  {ParameterName}_GridSize.y*MaxLength,
									  {ParameterName}_GridSize.z*MaxLength);

	OutGridOrigin = GridCenter - 0.5 * RegularSize;
	OutGridLength = MaxLength;
}

// Transform the node world position in grid space
float3 GetGridPosition_{ParameterName}(in float3 ParticlePosition, in float3 GridOrigin)
{
	return ParticlePosition - GridOrigin;
}

/* -----------------------------------------------------------------
 * Grid cell/node/face validity
 * -----------------------------------------------------------------
 */

// Return true if the grid cell index is in any voxel center (0,N-1)
bool IsCellValid_{ParameterName}(in int3 CellIndex)
{
	return (CellIndex.x >= 0 && CellIndex.x < {ParameterName}_GridSize.x) && 
		   (CellIndex.y >= 0 && CellIndex.y < {ParameterName}_GridSize.y) &&
		   (CellIndex.z >= 0 && CellIndex.z < {ParameterName}_GridSize.z);
}

// Return true if the grid face index is for the component one (0,N) and in the voxel center for the others (0,N-1)
bool IsFaceValid_{ParameterName}(in int3 FaceIndex, in int ComponentIndex)
{
	switch(ComponentIndex)
	{
		case 0 :
			return (FaceIndex.x >= 0 && FaceIndex.x <= {ParameterName}_GridSize.x) && 
			   (FaceIndex.y >= 0 && FaceIndex.y < {ParameterName}_GridSize.y) &&
			   (FaceIndex.z >= 0 && FaceIndex.z < {ParameterName}_GridSize.z);
		case 1 : 
			return (FaceIndex.x >= 0 && FaceIndex.x < {ParameterName}_GridSize.x) && 
			   (FaceIndex.y >= 0 && FaceIndex.y <= {ParameterName}_GridSize.y) &&
			   (FaceIndex.z >= 0 && FaceIndex.z < {ParameterName}_GridSize.z);
		default : 
			return (FaceIndex.x >= 0 && FaceIndex.x < {ParameterName}_GridSize.x) && 
			   (FaceIndex.y >= 0 && FaceIndex.y < {ParameterName}_GridSize.y) &&
			   (FaceIndex.z >= 0 && FaceIndex.z <= {ParameterName}_GridSize.z);
	}
}

// Return true if the grid node index is valid (0,N)
bool IsNodeValid_{ParameterName}(in int3 NodeIndex)
{
	return (NodeIndex.x >= 0 && NodeIndex.x <= {ParameterName}_GridSize.x) && 
		   (NodeIndex.y >= 0 && NodeIndex.y <= {ParameterName}_GridSize.y) &&
		   (NodeIndex.z >= 0 && NodeIndex.z <= {ParameterName}_GridSize.z);
}

// Return true if the grid face index is inside the grid
bool IsFaceInside_{ParameterName}(in int3 FaceIndex, in int ComponentIndex)
{
	switch(ComponentIndex)
	{
		case 0 :
			return (FaceIndex.x >  0 && FaceIndex.x < {ParameterName}_GridSize.x) && 
			   (FaceIndex.y >= 0 && FaceIndex.y < {ParameterName}_GridSize.y) &&
			   (FaceIndex.z >= 0 && FaceIndex.z < {ParameterName}_GridSize.z);
		case 1 :
			return (FaceIndex.x >= 0 && FaceIndex.x < {ParameterName}_GridSize.x) && 
			   (FaceIndex.y >  0 && FaceIndex.y < {ParameterName}_GridSize.y) &&
			   (FaceIndex.z >= 0 && FaceIndex.z < {ParameterName}_GridSize.z);
		default : 
			return (FaceIndex.x >= 0 && FaceIndex.x < {ParameterName}_GridSize.x) && 
			   (FaceIndex.y >= 0 && FaceIndex.y < {ParameterName}_GridSize.y) &&
			   (FaceIndex.z >  0 && FaceIndex.z < {ParameterName}_GridSize.z);
	}
}

bool IsCellInside_{ParameterName}(in int3 CellIndex)
{
	return (CellIndex.x > 0 && CellIndex.x < ({ParameterName}_GridSize.x-1)) && 
		   (CellIndex.y > 0 && CellIndex.y < ({ParameterName}_GridSize.y-1)) &&
		   (CellIndex.z > 0 && CellIndex.z < ({ParameterName}_GridSize.z-1));
}

bool IsNodeInside_{ParameterName}(in int3 NodeIndex)
{
	return (NodeIndex.x > 0 && NodeIndex.x < {ParameterName}_GridSize.x) && 
		   (NodeIndex.y > 0 && NodeIndex.y < {ParameterName}_GridSize.y) &&
		   (NodeIndex.z > 0 && NodeIndex.z < {ParameterName}_GridSize.z);
}

/* -----------------------------------------------------------------
 * Grid indexing utilities
 * -----------------------------------------------------------------
 */

// Compute the grid index and fraction given a grid position and a grid offset
int3 ComputeGridIndex_{ParameterName}(in float3 GridPosition, in float3 GridOffset, in float GridLength, out float3 OutGridFraction)
{
	const float3 GridCoordinate = GridPosition / GridLength - GridOffset;

	const int3 GridIndex = floor(GridCoordinate);
	OutGridFraction = GridCoordinate - GridIndex;
	return GridIndex;
}

// Compute the grid index given a grid hash
int3 GetGridIndex_{ParameterName}(in int GridHash)
{
	const int GridSizeYZ = ({ParameterName}_GridSize.y+1) * ({ParameterName}_GridSize.z+1);
	const int GridHashYZ = GridHash % GridSizeYZ;

	return int3(GridHash / GridSizeYZ, GridHashYZ / ({ParameterName}_GridSize.z+1), GridHashYZ % ({ParameterName}_GridSize.z+1));
}

/* -----------------------------------------------------------------
 * Get Cell/Node/Face world positions
 * -----------------------------------------------------------------
 */
 
 // Compute the world face position
void GetFacePosition_{ParameterName}(in int GridHash, in float3 GridOrigin, in float GridLength, in int ComponentIndex, out float3 OutFacePosition)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);

	if( IsFaceValid_{ParameterName}(GridIndex,ComponentIndex) )
	{
		float3 ComponentOffset = float3(0.5,0.5,0.5); ComponentOffset[ComponentIndex] = 0.0;
		OutFacePosition = (GridIndex+ComponentOffset)*GridLength+GridOrigin;
	}
	else
	{
		OutFacePosition = float3(0,0,0);
	}
}

// Compute the world cell position
void GetCellPosition_{ParameterName}(in int GridHash, in float3 GridOrigin, in float GridLength, out float3 OutCellPosition)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);

	if( IsCellValid_{ParameterName}(GridIndex) )
	{
		OutCellPosition = (GridIndex+float3(0.5,0.5,0.5))*GridLength+GridOrigin;
	}
	else
	{
		OutCellPosition = float3(0,0,0);
	}
}

// Compute the world node position
void GetNodePosition_{ParameterName}(in int GridHash, in float3 GridOrigin, in float GridLength, out float3 OutNodePosition)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);

	if( IsNodeValid_{ParameterName}(GridIndex) )
	{
		OutNodePosition = GridIndex*GridLength+GridOrigin;
	}
	else
	{
		OutNodePosition = float3(0,0,0);
	}
}

/* -----------------------------------------------------------------
 * Interlocked Add/Min Float (slow but accurate version) and Int (fast but approximate version)
 * -----------------------------------------------------------------
 */

// Compute the data index from agrid index and a data offset
int GetDataIndex_{ParameterName}(in int3 GridIndex, in int DataOffset)
{
	const int GridIndexX = GridIndex.x + DataOffset * ({ParameterName}_GridSize.x+1);
	return GridIndexX * ({ParameterName}_GridSize.y+1) * ({ParameterName}_GridSize.z+1) + GridIndex.y * ({ParameterName}_GridSize.z+1) + GridIndex.z;
}

#if FAST_MODE == 1

void InterlockedAddInt_{ParameterName}(in int3 GridIndex, in int DataOffset, in int IntValue)
{
	InterlockedAdd({ParameterName}_GridDestinationBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)], IntValue);
}

void InterlockedMinInt_{ParameterName}(in int3 GridIndex, in int DataOffset, in int IntValue)
{
	InterlockedMin({ParameterName}_GridDestinationBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)], IntValue);
}

#else

void InterlockedAddFloat_{ParameterName}(in int3 GridIndex, in int DataOffset, in float FloatValue)
{
	int IntValue = asint(FloatValue);
	int CompareValue = 0;
	int StoredValue = 0;

	[allow_uav_condition] while(true)
	{
		InterlockedCompareExchange({ParameterName}_GridDestinationBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)],CompareValue,IntValue,StoredValue);
		if(StoredValue == CompareValue)
		{
			break;
		}
		CompareValue = StoredValue;
		IntValue = asint(FloatValue+asfloat(StoredValue));
	}
}

void InterlockedMinFloat_{ParameterName}(in int3 GridIndex, in int DataOffset, in float FloatValue)
{
	int IntValue = asint(FloatValue);
	int CompareValue = 0;
	int StoredValue = 0;

	[allow_uav_condition] while(true)
	{
		InterlockedCompareExchange({ParameterName}_GridDestinationBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)],CompareValue,IntValue,StoredValue);
		if(StoredValue == CompareValue)
		{
			break;
		}
		CompareValue = StoredValue;
		IntValue = asint(min(FloatValue,asfloat(StoredValue)));
	}
}

#endif

/* -----------------------------------------------------------------
 * Datas accessors
 * -----------------------------------------------------------------
 */

 void InterlockedMinScalar_{ParameterName}(in int3 GridIndex, in int DataOffset, in float FloatValue)
{
#if FAST_MODE == 1
	const int IntValue = FloatValue * FLOAT_PRECISION;
	InterlockedMinInt_{ParameterName}(GridIndex,DataOffset,IntValue);
#else
	InterlockedMinFloat_{ParameterName}(GridIndex,DataOffset,FloatValue);
#endif
}

void InterlockedAddScalar_{ParameterName}(in int3 GridIndex, in int DataOffset, in float FloatValue)
{
#if FAST_MODE == 1
	const int IntValue = FloatValue * FLOAT_PRECISION;
	InterlockedAddInt_{ParameterName}(GridIndex,DataOffset,IntValue);
#else
	InterlockedAddFloat_{ParameterName}(GridIndex,DataOffset,FloatValue);
#endif
}

float ConvertGridScalar_{ParameterName}(in int3 GridIndex, in int DataOffset)
{
	const int IntValue = {ParameterName}_GridCurrentBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)];

#if FAST_MODE == 1
	return float(IntValue) / FLOAT_PRECISION;
#else
	return asfloat(IntValue);
#endif
}

void AddGridScalar_{ParameterName}(in int3 GridIndex, in int DataOffset, in float FloatValue)
{
	const float SumValue = asfloat({ParameterName}_GridCurrentBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)])+FloatValue;
	{ParameterName}_GridDestinationBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)] = asint(SumValue);
}

float GetGridScalar_{ParameterName}(in int3 GridIndex, in int DataOffset)
{
	return asfloat({ParameterName}_GridCurrentBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)]);
}

void SetGridScalar_{ParameterName}(in int3 GridIndex, in int DataOffset, in float FloatValue)
{
	{ParameterName}_GridDestinationBuffer[GetDataIndex_{ParameterName}(GridIndex,DataOffset)] = asint(FloatValue);
}

void SetGridVector_{ParameterName}(in int3 GridIndex, in int DataOffset, in float3 NodeVector)
{
	SetGridScalar_{ParameterName}(GridIndex,DataOffset,NodeVector.x);
	SetGridScalar_{ParameterName}(GridIndex,DataOffset+1,NodeVector.y);
	SetGridScalar_{ParameterName}(GridIndex,DataOffset+2,NodeVector.z);
}

void AddGridVector_{ParameterName}(in int3 GridIndex, in int DataOffset, in float3 NodeVector)
{
	AddGridScalar_{ParameterName}(GridIndex,DataOffset,NodeVector.x);
	AddGridScalar_{ParameterName}(GridIndex,DataOffset+1,NodeVector.y);
	AddGridScalar_{ParameterName}(GridIndex,DataOffset+2,NodeVector.z);
}

float3 GetGridVector_{ParameterName}(in int3 GridIndex, in int DataOffset)
{
	return float3(GetGridScalar_{ParameterName}(GridIndex,DataOffset),
				  GetGridScalar_{ParameterName}(GridIndex,DataOffset+1),
				  GetGridScalar_{ParameterName}(GridIndex,DataOffset+2));
}

/* -----------------------------------------------------------------
 * Interpolation function and gradient
 * -----------------------------------------------------------------
 */

 float GetShapeFunction_{ParameterName}( in float3 GridFraction, in int3 ijk)
 {
	const float GridWeightX = (ijk.x == 0) ? 1.0 - GridFraction.x : GridFraction.x;
	const float GridWeightY = (ijk.y == 0) ? 1.0 - GridFraction.y : GridFraction.y;
	const float GridWeightZ = (ijk.z == 0) ? 1.0 - GridFraction.z : GridFraction.z;

	return GridWeightX*GridWeightY*GridWeightZ;
 }

 float3 GetShapeGradient_{ParameterName}( in float3 GridFraction, in int3 ijk)
 {
	const float GridWeightX = (ijk.x == 0) ? 1.0 - GridFraction.x : GridFraction.x;
	const float GridWeightY = (ijk.y == 0) ? 1.0 - GridFraction.y : GridFraction.y;
	const float GridWeightZ = (ijk.z == 0) ? 1.0 - GridFraction.z : GridFraction.z;

	const float GridGradientX = (ijk.x == 0) ? -1.0: 1.0;
	const float GridGradientY = (ijk.y == 0) ? -1.0: 1.0;
	const float GridGradientZ = (ijk.z == 0) ? -1.0: 1.0;

	return  float3(GridGradientX*GridWeightY*GridWeightZ,
				   GridWeightX*GridGradientY*GridWeightZ,
				   GridWeightX*GridWeightY*GridGradientZ);
 }

/* -----------------------------------------------------------------
 * Build Velocity Field from nodes
 * -----------------------------------------------------------------
 */
 
// Transfer the velocity field component on a face from the particle to the grid
void BuildVelocityFace_{ParameterName}(in int3 GridIndex, in float ShapeFunction, in int3 ijk, in float3 GridPosition, in float SampleMass, in float SampleMomentum,
			in float3 MomentumGradient, in float3 ComponentOffset, in int ComponentIndex, in float GridLength )
{
	const int3 GridLocation = int3(GridIndex.x+ijk.x,GridIndex.y+ijk.y,GridIndex.z+ijk.z);

	if( IsFaceValid_{ParameterName}(GridLocation,ComponentIndex) )
	{
		const float3 DeltaPosition = (GridLocation+ComponentOffset)*GridLength - GridPosition;
		const float GridMomentum = SampleMomentum + dot(MomentumGradient, DeltaPosition);
		InterlockedAddScalar_{ParameterName}(GridLocation,FLUID_MASS_OFFSET+ComponentIndex,SampleMass*ShapeFunction);
		InterlockedAddScalar_{ParameterName}(GridLocation,FLUID_VELOCITY_OFFSET+ComponentIndex,GridMomentum*ShapeFunction);
	}
}

// Transfer the velocity field component from the particle to the grid
void BuildVelocityComponent_{ParameterName}(in float3 GridPosition, in float SampleMass, in float SampleMomentum,
			in float3 MomentumGradient, in float3 ComponentOffset, in int ComponentIndex, in float GridLength )
{
	float3 GridFraction = float3(0,0,0);
	const int3 GridIndex = ComputeGridIndex_{ParameterName}(GridPosition,ComponentOffset,GridLength,GridFraction);

	for(int i = 0; i < 2; ++i)
	{
		const float GridWeightX = (i == 0) ? 1.0 - GridFraction.x : GridFraction.x;
		for(int j = 0; j < 2; ++j)
		{
			const float GridWeightY = (j == 0) ? 1.0 - GridFraction.y : GridFraction.y;
			for(int k = 0; k < 2; ++k)
			{
				const float GridWeightZ = (k == 0) ? 1.0 - GridFraction.z : GridFraction.z;
				const float GridWeightXYZ = GridWeightX*GridWeightY*GridWeightZ;

				BuildVelocityFace_{ParameterName}(GridIndex,GridWeightXYZ,int3(i,j,k),GridPosition,SampleMass,
								SampleMomentum,MomentumGradient,ComponentOffset,ComponentIndex,GridLength);
			}
		}
	}
}

// Transfer the velocity field from the particle to the grid
void BuildVelocityField_{ParameterName}(in float3 GridOrigin, in float GridLength, in float3 ParticlePosition, in float ParticleMass, in float3 ParticleVelocity, 
			in float4x4 VelocityGradient, out bool OutFunctionStatus )
{
	if(isfinite(ParticlePosition.x) && isfinite(ParticlePosition.y) && isfinite(ParticlePosition.z) &&
	   isfinite(ParticleVelocity.x) && isfinite(ParticleVelocity.y) && isfinite(ParticleVelocity.z))
	{
		const float3 ParticleMomentum = ParticleVelocity * ParticleMass;

		const float3 MomentumGradientX = VelocityGradient[0].xyz * ParticleMass;
		const float3 MomentumGradientY = VelocityGradient[1].xyz * ParticleMass;
		const float3 MomentumGradientZ = VelocityGradient[2].xyz * ParticleMass;

		const float3 GridPosition = GetGridPosition_{ParameterName}(ParticlePosition,GridOrigin);

		BuildVelocityComponent_{ParameterName}(GridPosition,ParticleMass,ParticleMomentum.x,MomentumGradientX,float3(0.0,0.5,0.5),0,GridLength);
		BuildVelocityComponent_{ParameterName}(GridPosition,ParticleMass,ParticleMomentum.y,MomentumGradientY,float3(0.5,0.0,0.5),1,GridLength);
		BuildVelocityComponent_{ParameterName}(GridPosition,ParticleMass,ParticleMomentum.z,MomentumGradientZ,float3(0.5,0.5,0.0),2,GridLength);
	}
	
	OutFunctionStatus = true;
}

/* -----------------------------------------------------------------
 * Sample Velocity field from grid
 * -----------------------------------------------------------------
 */
 
// Sample the grid velocity field and gradient component on a face at the particle local position 
void SampleVelocityFace_{ParameterName}(in int3 GridIndex, in float ShapeFunction, in float3 ShapeGradient, in int3 ijk, in float3 GridPosition, in float3 ComponentOffset, in int ComponentIndex, in float GridLength,
						in bool ScaledVelocity, inout float OutGridVelocity, inout float OutGridMass, inout float3 OutGridGradient)
{
	const int3 GridLocation = int3(GridIndex.x+ijk.x,GridIndex.y+ijk.y,GridIndex.z+ijk.z);

	if( IsFaceValid_{ParameterName}(GridLocation,ComponentIndex) )
	{
		float GridVelocity = 0.0;
		float GridMass = 0.0;

		if(!ScaledVelocity)
		{
			const float GridMomentum = ConvertGridScalar_{ParameterName}(GridLocation,FLUID_VELOCITY_OFFSET+ComponentIndex);
			
			GridMass = ConvertGridScalar_{ParameterName}(GridLocation,FLUID_MASS_OFFSET+ComponentIndex);
			GridVelocity = (GridMass != 0.0) ? GridMomentum / GridMass : 0.0;
		}
		else
		{
			GridMass = GetGridScalar_{ParameterName}(GridLocation,FLUID_MASS_OFFSET+ComponentIndex);
			GridVelocity = GetGridScalar_{ParameterName}(GridLocation,FLUID_VELOCITY_OFFSET+ComponentIndex);
		}

		OutGridMass += GridMass * ShapeFunction;
		OutGridVelocity += GridVelocity * ShapeFunction;
		OutGridGradient += GridVelocity * ShapeGradient;
	}
}

// Sample the grid velocity field and gradient component at the particle local position 
void SampleVelocityComponent_{ParameterName}(in float3 GridPosition, in float3 ComponentOffset, in int ComponentIndex, in float GridLength,
						in bool ScaledVelocity, out float OutGridVelocity, out float OutGridMass, out float3 OutGridGradient)
{
	float3 GridFraction = float3(0,0,0); 
	const int3 GridIndex = ComputeGridIndex_{ParameterName}(GridPosition,ComponentOffset,GridLength,GridFraction);

	OutGridVelocity = 0.0;
	OutGridGradient = float3(0,0,0); 
	OutGridMass = 0.0;

	if(isfinite(GridLength))
	{
		for(int i = 0; i < 2; ++i)
		{
			const float GridWeightX = (i == 0) ? 1.0 - GridFraction.x : GridFraction.x;
			const float GridGradientX = (i == 0) ? -1.0: 1.0;
			for(int j = 0; j < 2; ++j)
			{
				const float GridWeightY = (j == 0) ? 1.0 - GridFraction.y : GridFraction.y;
				const float GridGradientY = (j == 0) ? -1.0: 1.0;
				for(int k = 0; k < 2; ++k)
				{
					const float GridWeightZ = (k == 0) ? 1.0 - GridFraction.z : GridFraction.z;
					const float GridGradientZ = (k == 0) ? -1.0: 1.0;

					const float GridWeightXYZ = GridWeightX*GridWeightY*GridWeightZ;
					const float3 GridGradientXYZ = float3(GridGradientX*GridWeightY*GridWeightZ,
															GridWeightX*GridGradientY*GridWeightZ,
															GridWeightX*GridWeightY*GridGradientZ) / GridLength;

					SampleVelocityFace_{ParameterName}(GridIndex,GridWeightXYZ,GridGradientXYZ,int3(i,j,k),GridPosition,
						ComponentOffset,ComponentIndex,GridLength,ScaledVelocity,OutGridVelocity,OutGridMass,OutGridGradient);
				}
			}
		}
	}
}

// Sample the grid velocity field and gradient at the particle world position 
void SampleVelocityField_{ParameterName}(in float3 GridOrigin, in float GridLength, in float3 ParticlePosition, in bool ScaledVelocity, out float OutParticleMass, out float3 OutParticleVelocity, 
			out float4x4 OutParticleGradient)
{
	OutParticleVelocity = float3(0,0,0);
	OutParticleGradient = float4x4(0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0);

	float3 ParticleMass = float3(0,0,0);

	const float3 GridPosition = GetGridPosition_{ParameterName}(ParticlePosition,GridOrigin);
	SampleVelocityComponent_{ParameterName}(GridPosition,float3(0.0,0.5,0.5),0,GridLength,ScaledVelocity,OutParticleVelocity.x,ParticleMass.x,OutParticleGradient[0].xyz);
	SampleVelocityComponent_{ParameterName}(GridPosition,float3(0.5,0.0,0.5),1,GridLength,ScaledVelocity,OutParticleVelocity.y,ParticleMass.y,OutParticleGradient[1].xyz);
	SampleVelocityComponent_{ParameterName}(GridPosition,float3(0.5,0.5,0.0),2,GridLength,ScaledVelocity,OutParticleVelocity.z,ParticleMass.z,OutParticleGradient[2].xyz);

	OutParticleMass = (ParticleMass.x+ParticleMass.y+ParticleMass.z)/3.0;
}

/* -----------------------------------------------------------------
 * Velocity Accessors
 * -----------------------------------------------------------------
 */

void AddGridVelocity_{ParameterName}(in int GridHash, in float3 GridVelocity, out bool OutAddStatus)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);
	OutAddStatus = true;

	if( IsFaceValid_{ParameterName}(GridIndex,0) )
	{
		OutAddStatus = true;
		AddGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET,GridVelocity.x);
	}
	if( IsFaceValid_{ParameterName}(GridIndex,1) )
	{
		OutAddStatus = true;
		AddGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET+1,GridVelocity.y);
	}
	if( IsFaceValid_{ParameterName}(GridIndex,2) )
	{
		OutAddStatus = true;
		AddGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET+2,GridVelocity.z);
	}
}

void GetGridVelocity_{ParameterName}(in int GridHash, out float3 OutGridVelocity)
{
	const int3 GridIndex = GetGridIndex_{ParameterName}(GridHash);

	OutGridVelocity = float3(0,0,0);

	if( IsFaceValid_{ParameterName}(GridIndex,0) )
	{
		OutGridVelocity.x = GetGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET);
	}
	if( IsFaceValid_{ParameterName}(GridIndex,1) )
	{
		OutGridVelocity.y = GetGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET+1);
	}
	if( IsFaceValid_{ParameterName}(GridIndex,2) )
	{
		OutGridVelocity.z = GetGridScalar_{ParameterName}(GridIndex,FLUID_VELOCITY_OFFSET+2);
	}
}
