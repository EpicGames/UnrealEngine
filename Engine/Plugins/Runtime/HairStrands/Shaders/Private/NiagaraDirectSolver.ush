// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NiagaraDirectSolver.ush
=============================================================================*/

#if GPU_SIMULATION

/* -----------------------------------------------------------------
 * Shared memory for datas that will be accessed within the direct solver 
 * -----------------------------------------------------------------
 */

groupshared float4 SharedTridiagValues[THREADGROUP_SIZE];
groupshared float4 SharedTridiagResult[THREADGROUP_SIZE];

void SolveTridiagSerial2()
{
	if(GGroupThreadId.x == 0)
	{
		{
			SharedTridiagValues[0].z = SharedTridiagValues[0].z / SharedTridiagValues[0].y;
			GroupMemoryBarrier();
			SharedTridiagValues[0].w = SharedTridiagValues[0].w / SharedTridiagValues[0].y;
			GroupMemoryBarrier();

			for(int i = 1; i < THREADGROUP_SIZE; ++i)
			{
				if(i < (THREADGROUP_SIZE-1))
				{
				SharedTridiagValues[i].z = SharedTridiagValues[i].z / ( SharedTridiagValues[i].y - SharedTridiagValues[i-1].z * SharedTridiagValues[i-1].x);
				GroupMemoryBarrier();
				}

				SharedTridiagValues[i].w = (SharedTridiagValues[i].w -SharedTridiagValues[i-1].w *  SharedTridiagValues[i].x)  / ( SharedTridiagValues[i].y - SharedTridiagValues[i-1].z * SharedTridiagValues[i-1].x);
				GroupMemoryBarrier();
			}
		}
		{
			for(int i = THREADGROUP_SIZE-2; i >= 0; --i)
			{
				SharedTridiagValues[i].w = SharedTridiagValues[i].w - SharedTridiagValues[i].z * SharedTridiagValues[i+1].w;
				GroupMemoryBarrier();
			}
		}
	}
	GroupMemoryBarrier();
}

void SolveTridiagSerial()
{
	if(GGroupThreadId.x == 0)
	{
		{
			for(int i = 1; i < THREADGROUP_SIZE; ++i)
			{
				const float W = SharedTridiagValues[i].x / SharedTridiagValues[i-1].y;
				SharedTridiagValues[i].y -= W * SharedTridiagValues[i-1].z;
				GroupMemoryBarrier();

				SharedTridiagValues[i].w -= W * SharedTridiagValues[i-1].w;
				GroupMemoryBarrier();
			}
		}
		SharedTridiagValues[THREADGROUP_SIZE-1].w = SharedTridiagValues[THREADGROUP_SIZE-1].w / SharedTridiagValues[THREADGROUP_SIZE-1].y;
		GroupMemoryBarrier();
		{
			for(int i = THREADGROUP_SIZE-2; i >= 0; --i)
			{
				SharedTridiagValues[i].w = (SharedTridiagValues[i].w - SharedTridiagValues[i].z * SharedTridiagValues[i+1].w) / SharedTridiagValues[i].y;
				GroupMemoryBarrier();
			}
		}
	}
	GroupMemoryBarrier();
}

void SolveConjugateResidual()
{
	float X = 0.0;

	SharedTridiagResult[GGroupThreadId.x] = float4(0.0,0.0,0.0,0.0);
	GroupMemoryBarrier();

	const int PrevIdx = (GGroupThreadId.x == 0) ? GGroupThreadId.x : GGroupThreadId.x-1;
	const int NextIdx = (GGroupThreadId.x == THREADGROUP_SIZE-1) ? GGroupThreadId.x : GGroupThreadId.x+1;

	SharedTridiagResult[GGroupThreadId.x].y = SharedTridiagValues[GGroupThreadId.x].w;
	GroupMemoryBarrier();

	SharedTridiagResult[GGroupThreadId.x].z = SharedTridiagResult[GGroupThreadId.x].y / SharedTridiagValues[GGroupThreadId.x].y;
	SharedTridiagResult[GGroupThreadId.x].w = SharedTridiagResult[GGroupThreadId.x].z;
	GroupMemoryBarrier();
	{

	for(int ii = 0; ii < 10; ++ii)
	{
		SharedTridiagResult[GGroupThreadId.x].x = SharedTridiagValues[GGroupThreadId.x].x * SharedTridiagResult[PrevIdx].w+
												  SharedTridiagValues[GGroupThreadId.x].y * SharedTridiagResult[GGroupThreadId.x].w+
												  SharedTridiagValues[GGroupThreadId.x].z * SharedTridiagResult[NextIdx].w;
		GroupMemoryBarrier();

		SharedTridiagResult[GGroupThreadId.x].y = SharedTridiagValues[GGroupThreadId.x].x * SharedTridiagResult[PrevIdx].z+
												  SharedTridiagValues[GGroupThreadId.x].y * SharedTridiagResult[GGroupThreadId.x].z+
												  SharedTridiagValues[GGroupThreadId.x].z * SharedTridiagResult[NextIdx].z;
		GroupMemoryBarrier();

		float rkArk = 0.0;
		float ApMAp = 0.0;

		[unroll]
		for(int kk = 0; kk < THREADGROUP_SIZE; ++kk)
		{
			rkArk += SharedTridiagResult[kk].z * SharedTridiagResult[kk].y;
			ApMAp += SharedTridiagResult[kk].x * SharedTridiagResult[kk].x / SharedTridiagValues[kk].y;
		}

		float Alpha = (ApMAp != 0.0) ? rkArk / ApMAp : 0.0;
		X += Alpha * SharedTridiagResult[GGroupThreadId.x].w;

		SharedTridiagResult[GGroupThreadId.x].z -= Alpha * SharedTridiagResult[GGroupThreadId.x].x / SharedTridiagValues[GGroupThreadId.x].y;
		GroupMemoryBarrier();

		SharedTridiagResult[GGroupThreadId.x].y = SharedTridiagValues[GGroupThreadId.x].x * SharedTridiagResult[PrevIdx].z+
												  SharedTridiagValues[GGroupThreadId.x].y * SharedTridiagResult[GGroupThreadId.x].z+
												  SharedTridiagValues[GGroupThreadId.x].z * SharedTridiagResult[NextIdx].z;
		GroupMemoryBarrier();

		float rkArkn = 0.0;

		[unroll]
		for(int ll = 0; ll < THREADGROUP_SIZE; ++ll)
		{
			rkArkn += SharedTridiagResult[ll].z * SharedTridiagResult[ll].y;
		}
		float Beta = (rkArk != 0.0) ? rkArkn / rkArk : 0.0;
		SharedTridiagResult[GGroupThreadId.x].w = SharedTridiagResult[GGroupThreadId.x].z + Beta * SharedTridiagResult[GGroupThreadId.x].w;
		//SharedTridiagResult[GGroupThreadId.x].x = SharedTridiagResult[GGroupThreadId.x].y + Beta * SharedTridiagResult[GGroupThreadId.x].x;
		GroupMemoryBarrier();
	}
	}
	
	SharedTridiagValues[GGroupThreadId.x].w = X;
}


void SolveConjugateGradient()
{
	float X = 0.0;

	SharedTridiagResult[GGroupThreadId.x] = float4(0.0,0.0,0.0,0.0);
	GroupMemoryBarrier();

	const int PrevIdx = (GGroupThreadId.x == 0) ? GGroupThreadId.x : GGroupThreadId.x-1;
	const int NextIdx = (GGroupThreadId.x == THREADGROUP_SIZE-1) ? GGroupThreadId.x : GGroupThreadId.x+1;

	SharedTridiagResult[GGroupThreadId.x].y = SharedTridiagValues[GGroupThreadId.x].w;
	GroupMemoryBarrier();

	SharedTridiagResult[GGroupThreadId.x].z = SharedTridiagResult[GGroupThreadId.x].y / SharedTridiagValues[GGroupThreadId.x].y;
	SharedTridiagResult[GGroupThreadId.x].w = SharedTridiagResult[GGroupThreadId.x].z;
	GroupMemoryBarrier();
	{

	for(int ii = 0; ii < 10; ++ii)
	{
		SharedTridiagResult[GGroupThreadId.x].x = SharedTridiagValues[GGroupThreadId.x].x * SharedTridiagResult[PrevIdx].w+
												  SharedTridiagValues[GGroupThreadId.x].y * SharedTridiagResult[GGroupThreadId.x].w+
												  SharedTridiagValues[GGroupThreadId.x].z * SharedTridiagResult[NextIdx].w;
		GroupMemoryBarrier();

		float rkzko = 0.0;
		float pkApk = 0.0;

		[unroll]
		for(int kk = 0; kk < THREADGROUP_SIZE; ++kk)
		{
			rkzko += SharedTridiagResult[kk].y * SharedTridiagResult[kk].z;
			pkApk += SharedTridiagResult[kk].x * SharedTridiagResult[kk].w;
		}

		float Alpha = (pkApk != 0.0) ? rkzko / pkApk : 0.0;
		X += Alpha * SharedTridiagResult[GGroupThreadId.x].w;

		SharedTridiagResult[GGroupThreadId.x].y -= Alpha * SharedTridiagResult[GGroupThreadId.x].x;
		GroupMemoryBarrier();
		SharedTridiagResult[GGroupThreadId.x].z = SharedTridiagResult[GGroupThreadId.x].y / SharedTridiagValues[GGroupThreadId.x].y;
		GroupMemoryBarrier();

		float rkzkn = 0.0;

		[unroll]
		for(int ll = 0; ll < THREADGROUP_SIZE; ++ll)
		{
			rkzkn += SharedTridiagResult[ll].y * SharedTridiagResult[ll].z;
		}
		float Beta = (rkzko != 0.0) ? rkzkn / rkzko : 0.0;
		SharedTridiagResult[GGroupThreadId.x].w  = SharedTridiagResult[GGroupThreadId.x].z + Beta * SharedTridiagResult[GGroupThreadId.x].w;
		GroupMemoryBarrier();
	}
	}
	
	SharedTridiagValues[GGroupThreadId.x].w = X;
}

void SolveTridiagSystem()
{
	float lTemp = 0.0, uTemp = 0.0, hTemp = 0.0;

	[unroll]
	for(int span = 1; span < THREADGROUP_SIZE; span *= 2)
	{
		if( (GGroupThreadId.x - span) >= 0)
		{
			lTemp = (SharedTridiagValues[GGroupThreadId.x - span].y != 0.0) ? -SharedTridiagValues[GGroupThreadId.x].x / SharedTridiagValues[GGroupThreadId.x - span].y : 0.0;
		}
		else
		{
			lTemp = 0.0;
		}
		if( (GGroupThreadId.x + span) < THREADGROUP_SIZE)
		{
			uTemp = (SharedTridiagValues[GGroupThreadId.x + span].y != 0.0) ? -SharedTridiagValues[GGroupThreadId.x].z / SharedTridiagValues[GGroupThreadId.x + span].y : 0.0;
		}
		else
		{
			uTemp = 0.0;
		}
		GroupMemoryBarrier();
		if( (GGroupThreadId.x - span) >= 0)
		{
			SharedTridiagValues[GGroupThreadId.x].y += lTemp * SharedTridiagValues[GGroupThreadId.x - span].z;
			hTemp += lTemp * SharedTridiagValues[GGroupThreadId.x - span].w;
			lTemp *= SharedTridiagValues[GGroupThreadId.x - span].x;
		}
		if( (GGroupThreadId.x + span) < THREADGROUP_SIZE)
		{
			SharedTridiagValues[GGroupThreadId.x].y += uTemp * SharedTridiagValues[GGroupThreadId.x + span].x;
			hTemp += uTemp * SharedTridiagValues[GGroupThreadId.x + span].w;
			uTemp *= SharedTridiagValues[GGroupThreadId.x + span].z;
		}
		GroupMemoryBarrier();
		SharedTridiagValues[GGroupThreadId.x].x = hTemp;
		SharedTridiagValues[GGroupThreadId.x].z = uTemp;
		SharedTridiagValues[GGroupThreadId.x].w = hTemp;
		GroupMemoryBarrier();
	}
	SharedTridiagValues[GGroupThreadId.x].w /= SharedTridiagValues[GGroupThreadId.x].y;
	GroupMemoryBarrier();
}

float SolveTridiag()
{
	const int NumIteration = log2(THREADGROUP_SIZE/2);

	uint DataStride = 1;

	float4 DataNew = {0,0,0,0};

	[unroll]
	for(int i = 0; i < NumIteration; ++i)
	{
		int RightIdx = GGroupThreadId.x + DataStride;
		RightIdx = RightIdx & (THREADGROUP_SIZE-1);

		int LeftIdx = GGroupThreadId.x - DataStride;
		LeftIdx = LeftIdx & (THREADGROUP_SIZE-1);

		const float DataLeft  = SharedTridiagValues[GGroupThreadId.x].x / SharedTridiagValues[LeftIdx].y;
		const float DataRight = SharedTridiagValues[GGroupThreadId.x].z / SharedTridiagValues[RightIdx].y;

		DataNew.y = SharedTridiagValues[GGroupThreadId.x].y - SharedTridiagValues[LeftIdx].z * DataLeft - 
															  SharedTridiagValues[RightIdx].x * DataRight;
		DataNew.w = SharedTridiagValues[GGroupThreadId.x].w - SharedTridiagValues[LeftIdx].w * DataLeft - 
															  SharedTridiagValues[RightIdx].w * DataRight;
		DataNew.x = -SharedTridiagValues[LeftIdx].x * DataLeft;
		DataNew.z = -SharedTridiagValues[RightIdx].z * DataRight;

		GroupMemoryBarrier();

		SharedTridiagValues[GGroupThreadId.x] = DataNew;
		DataStride *= 2;

		GroupMemoryBarrier();
	}

	const int LeftIdx = GGroupThreadId.x & (DataStride-1);
	const int RightIdx = LeftIdx + DataStride;

	const float DataValue = SharedTridiagValues[RightIdx].y * SharedTridiagValues[LeftIdx].y - 
						    SharedTridiagValues[LeftIdx].z * SharedTridiagValues[RightIdx].x;

	return (GGroupThreadId.x < DataStride) ? 
			(SharedTridiagValues[RightIdx].y * SharedTridiagValues[LeftIdx].w - 
			SharedTridiagValues[LeftIdx].z * SharedTridiagValues[RightIdx].w) / DataValue :
			(SharedTridiagValues[RightIdx].w * SharedTridiagValues[LeftIdx].y - 
			SharedTridiagValues[LeftIdx].w * SharedTridiagValues[RightIdx].x) / DataValue;
}

#endif //GPU_SIMULATION