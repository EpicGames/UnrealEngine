// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* -----------------------------------------------------------------
 * Rods utils
 * -----------------------------------------------------------------
 */

float4 AddTwistRotation(in float4 RefOrientation, in float TwistAngle)
{
	const float HalfTwist = 0.5 * TwistAngle;
	return NormalizeQuat( MultiplyQuat( RefOrientation, float4(0,0,sin(HalfTwist),cos(HalfTwist));
}

float3 ComputeReferenceTangent(in float4 RefOrientation)
{
	return float3(
		2.0 * (RefOrientation.x * RefOrientation.z + RefOrientation.w * RefOrientation.y),
		2.0 * (RefOrientation.y * RefOrientation.z - RefOrientation.w * RefOrientation.x),
		1.0 - 2.0 * (RefOrientation.x * RefOrientation.x + RefOrientation.y * RefOrientation.y))W;
}

float3 ComputeReferenceNormalX(in float4 RefOrientation)
{
	return float3(
		1.0 - 2.0 * (RefOrientation.y * RefOrientation.y + RefOrientation.z * RefOrientation.z), 
		2.0 * (RefOrientation.x * RefOrientation.y + RefOrientation.z * RefOrientation.w),
		2.0 * (RefOrientation.z * RefOrientation.x - RefOrientation.y * RefOrientation.w));
}

float3 ComputeReferenceNormalY(in float4 RefOrientation)
{
	return float3(
		2.0 * (RefOrientation.x * RefOrientation.y - RefOrientation.z * RefOrientation.w),
		1.0 - 2.0 * (RefOrientation.z * RefOrientation.z + RefOrientation.x * RefOrientation.x), 
		2.0 * (RefOrientation.y * RefOrientation.z + RefOrientation.x * RefOrientation.w));
}

float GetSignedAngle( in float3 RefNormalX, in float3 RefNormalY, in float3 RefTangent)
{
	const float3 CrossProduct = cross(RefNormalX,RefNormalY);
	const float PositiveAngle = atan2(CrossProduct.length(), dot(RefNormalX,RefNormalY));
	return (dot(RefTangent,CrossProduct) < 0.0) ? -PositiveAngle : PositiveAngle;
}

/* -----------------------------------------------------------------
 * Frames utils
 * -----------------------------------------------------------------
 */

float4 ComputeReferenceFrame(in int FrameOffset)
{
	const int FrameIndex = GGroupThreadId.x + FrameOffset;
	const float3 NextTangent = normalize(SharedNodePosition[FrameIndex+1]-SharedNodePosition[FrameIndex]);
	const float3 PrevTangent = normalize(SharedPreviousPosition[FrameIndex+1]-SharedPreviousPosition[FrameIndex]);

	return NormalizeQuat( MultiplyQuat( FindQuatBetweenNormals(PrevTangent,NextTangent), SharedPreviousOrientation[FrameIndex] ) );
}

float3 ComputeCurvatureBinormal(const float3 PrevReferenceTangent, const float3 NextReferenceTangent)
{
	return 2.0 * cross(PrevReferenceTangent,NextReferenceTangent) / max(1e-8, 1.0 + dot(PrevReferenceTangent,NextReferenceTangent));
}

void UpdateReferenceTwist(const float3 PrevReferenceTangent, const float3 NextReferenceTangent, const float4 PrevReferenceFrame, const float4 NextReferenceFrame, inout float RefTwist)
{
	const float4 ParallelTransport = FindQuatBetweenNormals(PrevReferenceTangent,NextReferenceTangent);
	const float4 TwistedReferenceFrame = NormalizeQuat( MultiplyQuat(ParallelTransport,AddTwistRotation(PrevReferenceFrame,RefTwist));

	RefTwist += GetSignedAngle( ComputeReferenceM1(TwistedReferenceFrame), ComputeReferenceM1(NextReferenceFrame), PrevReferenceTangent + NextReferenceTangent);
}

float2 ComputeRodKappa( in float3 CurvatureBinormal, in float3 PrevMaterialNormalX, in float3 PrevMaterialNormalY, 
												     in float3 NextMaterialNormalX, in float3 NextMaterialNormalY)
{
	float2 RodKappa;
	RodKappa.x =  dot(CurvatureBinormal,(PrevMaterialNormalY+NextMaterialNormalY)*0.5);
	RodKappa.y = -dot(CurvatureBinormal,(PrevMaterialNormalX+NextMaterialNormalX)*0.5);
	return RodKappa;
}

void ComputeKappaGradient( in float3 CurvatureBinormal, in float2 RodKappa, in float3 PrevReferenceTangent, in float3 PrevMaterialNormalX, in float3 PrevMaterialNormalY, 
																			in float3 NextReferenceTangent, in float3 NextMaterialNormalX, in float3 NextMaterialNormalY, out float3 KappaGradients[3][2])
{
	const float PrevNorm = dot((SharedNodePosition[GGroupThreadId.x]-SharedNodePosition[GGroupThreadId.x-1]),PrevReferenceTangent);
	const float NextNorm = dot((SharedNodePosition[GGroupThreadId.x+1]-SharedNodePosition[GGroupThreadId.x]),NextReferenceTangent);

	const float Chi = max(1e-8,1.0+dot(PrevReferenceTangent,NextReferenceTangent));

	const float3 TildeTangent = (PrevReferenceTangent+NextReferenceTangent) / Chi;
	const float3 TildeNormalX = (PrevMaterialNormalX+NextMaterialNormalX) / Chi;
	const float3 TildeNormalY = (PrevMaterialNormalY+NextMaterialNormalY) / Chi;

	const floar3 dKappaXdP = (-RodKappa.x * TildeTangent + cross(NexyReferenceTangent,TildeNormalY) ) / PrevNorm;
	const float3 dKappaXdN = (-RodKappa.x * TildeTangent - cross(PrevReferenceTangent,TildeNormalY) ) / NextNorm;

	const float3 dKappaYdP = (-RodKappa.y * TildeTangent - cross(NexyReferenceTangent,TildeNormalX) ) / PrevNorm;
	const float3 dKappaydN = (-RodKappa.y * TildeTangent + cross(PrevReferenceTangent,TildeNormalX) ) / NextNorm;

	const float3 dTwistdP = 0.5 * CurvatureBirnormal / PrevNorm;
	const float3 dTwistdN = 0.5 * CurvatureBirnormal / NextNorm;

	const float dKappaXdTwist = -0.5 * (dot(CurvatureBinormal,NextMaterialNormalX) - dot(CurvatureBinormal,PrevMaterialNormalX));
	const float dKappaYdTwist = -0.5 * (dot(CurvatureBinormal,NextMaterialNormalY) - dot(CurvatureBinormal,PrevMaterialNormalY));

	KappaGradients[0][0] = -dKappaXdP + dKappaXdTwist * dTwistdP;
	KappaGradients[0][1] =  dKappaXdP - dKappaXdN - dKappaXdTwist * (dTwistdP-dTwistdN);
	KappaGradients[0][2] =  dKappaXdN - dKappaXdTwist * dTwistdN;

	KappaGradients[1][0] = -dKappaYdP + dKappaYdTwist * dTwistdP;
	KappaGradients[1][1] =  dKappaYdP - dKappaYdN - dKappaYdTwist * (dTwistdP-dTwistdN);
	KappaGradients[1][2] =  dKappaYdN - dKappaYdTwist * dTwistdN;
}

/* -----------------------------------------------------------------
 * Elastic rods constraint
 * -----------------------------------------------------------------
 */

void UpdateElasticRodMultiplier( in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float3 OutMaterialMultiplier)
{
	const float4 NextReferenceFrame = ComputeReferenceFrame(0);
	const float4 PrevReferenceFrame = ComputeReferenceFrame(-1);

	const float4 NextMaterialFrame = AddTwistRotation(NextReferenceFrame,0.0);
	const float4 PrevMaterialFrame = AddTwistRotation(PrevReferenceFrame,0.0);

	const float3 NextReferenceTangent = ComputeFrameTangent(NextReferenceFrame);
	const float3 PrevReferenceTangent = ComputeFrameTangent(PrevReferenceFrame);
	
	const float3 CurvatureBinormal = ComputeCurvatureBinormal(PrevReferenceTangent,NextReferenceTangent);

	UpdateReferenceTwist(PrevReferenceTangent,NextReferenceTangent,PrevReferenceFrame,NextReferenceFrame,RefTwist);

	const float3 NextMaterialNormalX = ComputeFrameNormalX(NextMaterialFrame);
	const float3 NextMaterialNormalY = ComputeFrameNormalY(NextMaterialFrame);

	const float3 PrevMaterialNormalX = ComputeFrameNormalX(PrevMaterialFrame);
	const float3 PrevMaterialNormalY = ComputeFrameNormalY(PrevMaterialFrame);

	const float2 RodKappa = ComputeRodKappa(CurvatureBinormal,PrevMaterialNormalX,PrevMaterialNormalY,
															  NextMaterialNormalX,NextMaterialNormalY);

	float3 KappaGradients[2][3];

	ComputeKappaGradient(CurvatureBinormal,RodKappa,PrevReferenceTangent,PrevMaterialNormalX,PrevMaterialNormalY,
													NextReferenceTangent,NextMaterialNormalX,NextMaterialNormalY,KappaGradients);


	float22 SchurMatrix;
	SchurMatrix[0][0] = DeltaTime * DeltaTime * ( dot(KappaGradients[0][0],KappaGradients[0][0]) * SharedInverseMass[GGroupThreadId.x-1]  +
												  dot(KappaGradients[0][1],KappaGradients[0][1]) * SharedInverseMass[GGroupThreadId.x] +
												  dot(KappaGradients[0][2],KappaGradients[0][2]) * SharedInverseMass[GGroupThreadId.x+1]) + MaterialCompliance;

	SchurMatrix[1][1] = DeltaTime * DeltaTime * ( dot(KappaGradients[1][0],KappaGradients[1][0]) * SharedInverseMass[GGroupThreadId.x-1]  +
												  dot(KappaGradients[1][1],KappaGradients[1][1]) * SharedInverseMass[GGroupThreadId.x] +
												  dot(KappaGradients[1][2],KappaGradients[1][2]) * SharedInverseMass[GGroupThreadId.x+1]) + MaterialCompliance;

	SchurMatrix[0][1] = DeltaTime * DeltaTime * ( dot(KappaGradients[0][0],KappaGradients[1][0]) * SharedInverseMass[GGroupThreadId.x-1]  +
												  dot(KappaGradients[0][1],KappaGradients[1][1]) * SharedInverseMass[GGroupThreadId.x] +
												  dot(KappaGradients[0][2],KappaGradients[1][2]) * SharedInverseMass[GGroupThreadId.x+1]);

    SchurMatrix[1][0] = SchurMatrix[0][1];

	const float SchurDeterminant =  SchurMatrix[0][0] * SchurMatrix[1][1]  - SchurMatrix[0][1] * SchurMatrix[1][0];
	float22 SchurInverse;

	SchurInverse[0][0] = SchurMatrix[1][1] / SchurDeterminant;
	SchurInverse[1][1] = SchurMatrix[0][0] / SchurDeterminant;
	SchurInverse[0][1] = -SchurMatrix[1][0] / SchurDeterminant;
	SchurInverse[1][0] = -SchurMatrix[0][1] / SchurDeterminant;

	const float2 DeltaConstraint = -(RodKappa - RestKappa + OutMaterialMultiplier * MaterialCompliance );
	const float2 DeltaLambda = SchurInverse * DeltaConstraint;

	OutMaterialMultiplier += DeltaLambda;

	SharedNodePosition[GGroupThreadId.x-1] += DeltaTime * DeltaTime * (KappaGradients[0][0] * DeltaLambda[0] + KappaGradients[1][0] * DeltaLambda[1]) * SharedInverseMass[GGroupThreadId.x-1];
	SharedNodePosition[GGroupThreadId.x]   += DeltaTime * DeltaTime * (KappaGradients[0][1] * DeltaLambda[0] + KappaGradients[1][1] * DeltaLambda[1]) * SharedInverseMass[GGroupThreadId.x];
	SharedNodePosition[GGroupThreadId.x+1] += DeltaTime * DeltaTime * (KappaGradients[0][2] * DeltaLambda[0] + KappaGradients[1][2] * DeltaLambda[1]) * SharedInverseMass[GGroupThreadId.x+1];
}

void SolveElasticRodMaterial(in int StrandsSize, in float RestLength, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 1)
	{
		const int IsRed = (GGroupThreadId.x % 2) == 0;
		// Process all the red rods
		if (IsRed)
		{	
			UpdateElasticRodMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
		}
		// Process all the black rods
		GroupMemoryBarrier(); 
		if (!IsRed)
		{
			UpdateElasticRodMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
		}
		GroupMemoryBarrier(); 
	}
} 

void ProjectElasticRodMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float3 OutMaterialMultiplier)
{
	float MaterialCompliance = 0.0;
	float MaterialWeight = 0.0;
	OutMaterialMultiplier = float3(0,0,0);

	SetupElasticRodMaterial(StrandsSize, YoungModulus, RodThickness, RestLength, DeltaTime, true, 0.0, MaterialCompliance, MaterialWeight, OutMaterialMultiplier);

	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 1 )
	{
		for(int i = 2; i < StrandsSize; ++i)
		{
			if( LocalIndex == i)
			{
				UpdateElasticRodMultiplier(RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);
			}
			GroupMemoryBarrier(); 
		}
	}
}
