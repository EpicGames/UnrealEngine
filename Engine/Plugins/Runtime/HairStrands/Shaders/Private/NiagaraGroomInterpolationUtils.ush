// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraGroomInterpolationUtils.ush 
=============================================================================*/

#pragma once

#include "/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"

// Compute the projection triangle
void DIHairStrands_ComputeProjectionTriangle(in float2 ProjectionUV, in float3 PA, in float3 PB, in float3 PC, in float3 Offset, 
			out float3 OutTrianglePosition, out float4 OutTriangleRotation)
{
	OutTrianglePosition = PA * ProjectionUV.x + PB * ProjectionUV.y + PC * (1.0 - ProjectionUV.x - ProjectionUV.y) + Offset;

	float3 TangentU = PB - PA;
	float3 TangentV = PC - PA;

	const float3 Normal = normalize(cross(TangentU, TangentV));
	TangentV = normalize(TangentV);
	TangentU = normalize(cross(TangentV, Normal));
	
	const float3 RotationMatrix[3] = { TangentU, TangentV, Normal };
	OutTriangleRotation = QuatFromMatrix(RotationMatrix);
}

// Eval the triangle local position
float3 DIHairStrands_TriangleLocalPosition(in float3 TrianglePosition, in float4 TriangleRotation, in float3 WorldPosition)
{
	return RotateVectorByQuat(WorldPosition - TrianglePosition, InverseQuat(TriangleRotation));
}

// Eval the triangle local orientation
float4 DIHairStrands_TriangleLocalOrientation(in float3 TrianglePosition, in float4 TriangleRotation, in float4 WorldOrientation)
{
	return NormalizeQuat(MultiplyQuat(InverseQuat(TriangleRotation), WorldOrientation));
}

// Eval the triangle world position
float3 DIHairStrands_TriangleWorldPosition(in float3 TrianglePosition, in float4 TriangleRotation, in float3 LocalPosition)
{
	return RotateVectorByQuat(LocalPosition, TriangleRotation) + TrianglePosition;
}

// Eval the triangle local orientation
float4 DIHairStrands_TriangleWorldOrientation(in float3 TrianglePosition, in float4 TriangleRotation, in float4 LocalOrientation)
{
	return NormalizeQuat(MultiplyQuat(TriangleRotation, LocalOrientation));
}