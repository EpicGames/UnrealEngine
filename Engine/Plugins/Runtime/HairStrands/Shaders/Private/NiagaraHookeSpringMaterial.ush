// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* -----------------------------------------------------------------
 * Stretch spring material
 * -----------------------------------------------------------------
 */

 void ResetStretchSpringMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float OutMaterialMultiplier, in int NodeOffset)
{
	// Compliance = 1.0 / (k * dt * dt)
	// with k = L * L * (Y * A / L) (the L*L term before is coming from the fact that our constraint is dL/L and not dL)
	// A is the cross section area (Pi * R * R), L is the rest length and Y is the young modulus
	OutMaterialCompliance = 4.0/(YoungModulus*PI*RodThickness*RestLength*RodThickness*DeltaTime*DeltaTime);

	const float SumInverseMass = !ProjectConstraint ? ( SharedInverseMass[GGroupThreadId.x] + SharedInverseMass[GGroupThreadId.x-1-NodeOffset] ) / (RestLength*RestLength) :
				SharedInverseMass[GGroupThreadId.x] / (RestLength*RestLength);

	const float SchurDiagonal = ( (1.0  + MaterialDamping / DeltaTime ) * SumInverseMass + OutMaterialCompliance );
	OutMaterialWeight = ( SchurDiagonal != 0.0 ) ? 1.0 / SchurDiagonal : 0.0;

	OutMaterialMultiplier = 0;
}


void UpdateStretchSpringMultiplier(in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, in float MaterialCompliance, in float MaterialWeight, inout float OutMaterialMultiplier, in int NodeOffset)
{
	const int PreviousIndex = GGroupThreadId.x-1-NodeOffset;
	float3 EdgeDirection = SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[PreviousIndex];
	const float3 DeltaVelocity = (EdgeDirection - ( SharedPreviousPosition[GGroupThreadId.x] - SharedPreviousPosition[PreviousIndex] ) ) / DeltaTime; 

	const float EdgeLength = length(EdgeDirection);
	EdgeDirection /= EdgeLength;

	// XPBD lagrange multiplier update : dL = -(C+compliance*L) / (dC * invM * dCt + alpha)
	const float DeltaLambda = -((EdgeLength-RestLength)/RestLength + OutMaterialMultiplier * MaterialCompliance + MaterialDamping * dot(EdgeDirection,DeltaVelocity)/RestLength ) * MaterialWeight;

	// L += dL
	OutMaterialMultiplier += DeltaLambda;

	// XPBD position update : dX = dL * dCt * invM
	const float3 PositionDelta = EdgeDirection * DeltaLambda / RestLength;

	// dX += dX
	SharedNodePosition[GGroupThreadId.x] += PositionDelta * SharedInverseMass[GGroupThreadId.x];
	if(!ProjectConstraint)
	{
		SharedNodePosition[PreviousIndex] -= PositionDelta * SharedInverseMass[PreviousIndex];
	}
}

void SetupStretchSpringMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float OutMaterialMultiplier)
{
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	OutMaterialMultiplier = 0.0;
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 1 )
	{
		ResetStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,ProjectConstraint,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier,0);
	}
}

void SolveStretchSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float MaterialMultiplier, out float OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	if(EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 1)
		{
			const int IsRed = (GGroupThreadId.x % 2) == 0;
			// Process all the red rods
			if (IsRed)
			{	
				UpdateStretchSpringMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier,0);
			}
			// Process all the black rods
			GroupMemoryBarrier(); 
			if (!IsRed)
			{
				UpdateStretchSpringMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier,0);
			}
			GroupMemoryBarrier(); 
		}
	}
} 

void ProjectStretchSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float3 OutNodePosition)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float MaterialMultiplier = 0.0;

		if( LocalIndex > 1 )
		{
			ResetStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,0);

			for(int i = 2; i < StrandsSize; ++i)
			{
				if( LocalIndex == i )
				{
					UpdateStretchSpringMultiplier(RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,0);
				}
				GroupMemoryBarrier(); 
			}
		}
		
	}
	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x] ;
}

/* -----------------------------------------------------------------
 * Bend spring material
 * -----------------------------------------------------------------
 */

 void SetupBendSpringMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float OutMaterialMultiplier)
{
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	OutMaterialMultiplier = 0.0;
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 1 )
	{
		ResetStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,ProjectConstraint,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier,1);
	}
}

void SolveBendSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float MaterialMultiplier, out float OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	if(EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 1)
		{
			const int IsRed = ((GGroupThreadId.x/2) % 2) == 0;
			// Process all the red rods
			if (IsRed)
			{	
				UpdateStretchSpringMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier,1);
			}
			// Process all the black rods
			GroupMemoryBarrier(); 
			if (!IsRed)
			{
				UpdateStretchSpringMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier,1);
			}
			GroupMemoryBarrier(); 
		}
	}
} 

void ProjectBendSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float3 OutNodePosition)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float MaterialMultiplier = 0.0;

		if( LocalIndex > 1 )
		{
			ResetStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,1);

			for(int i = 2; i < StrandsSize; ++i)
			{
				if( LocalIndex == i )
				{
					UpdateStretchSpringMultiplier(RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,1);
				}
				GroupMemoryBarrier(); 
			}
		}
	}
	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x];
}

/* -----------------------------------------------------------------
 * Twist spring material
 * -----------------------------------------------------------------
 */

 void SetupTwistSpringMaterial(in int StrandsSize, in float YoungModulus, in float RodThickness, 
	in float RestLength, in float DeltaTime, in bool ProjectConstraint, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float OutMaterialMultiplier)
{
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	OutMaterialMultiplier = 0.0;
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 2 )
	{
		ResetStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,ProjectConstraint,MaterialDamping,OutMaterialCompliance,OutMaterialWeight,OutMaterialMultiplier,2);
	}
}

void SolveTwistSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float RestLength, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float MaterialMultiplier, out float OutMaterialMultiplier)
{
	OutMaterialMultiplier = MaterialMultiplier;
	if(EnableConstraint)
	{
		const int LocalIndex = (GGroupThreadId.x % StrandsSize);
		if( LocalIndex > 2)
		{
			const int IsRed = ((GGroupThreadId.x/3) % 2) == 0;
			// Process all the red rods
			if (IsRed)
			{	
				UpdateStretchSpringMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier,2);
			}
			// Process all the black rods
			GroupMemoryBarrier(); 
			if (!IsRed)
			{
				UpdateStretchSpringMultiplier(RestLength,DeltaTime,false,MaterialDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier,2);
			}
			GroupMemoryBarrier(); 
		}
	}
} 

void ProjectTwistSpringMaterial(in bool EnableConstraint, in int StrandsSize, in float YoungModulus, in float RodThickness, in float RestLength, in float DeltaTime, out float3 OutNodePosition)
{
	const int LocalIndex = (GGroupThreadId.x % StrandsSize);

	if(DeltaTime != 0.0 && EnableConstraint)
	{
		float MaterialCompliance = 0.0;
		float MaterialWeight = 0.0;
		float MaterialMultiplier = 0.0;

		if( LocalIndex > 2 )
		{
			ResetStretchSpringMaterial(StrandsSize,YoungModulus,RodThickness,RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,2);

			for(int i = 3; i < StrandsSize; ++i)
			{
				if( LocalIndex == i )
				{
					UpdateStretchSpringMultiplier(RestLength,DeltaTime,true,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,2);
				}
				GroupMemoryBarrier(); 
			}
		}
	}
	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x];
}
