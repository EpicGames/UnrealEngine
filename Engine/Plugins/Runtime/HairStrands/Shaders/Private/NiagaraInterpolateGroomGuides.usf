// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraInterpolateGroomGuides.usf 
=============================================================================*/

#pragma once

#define LOCAL_INTERPOLATION 0
#define WORLD_INTERPOLATION 1

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/HairStrands/HairStrandsPack.ush"

#if PERMUTATION_INTERPOLATION == LOCAL_INTERPOLATION

#include "NiagaraGroomInterpolationUtils.ush"

Buffer<float4>		RestTrianglePositionBuffer;
Buffer<float4>		DeformedTrianglePositionBuffer;
Buffer<uint>		RootBarycentricCoordinatesBuffer;
Buffer<uint>		RootToUniqueTriangleIndexBuffer;

#endif

ByteAddressBuffer	PointToCurveIndexBuffer;
RWByteAddressBuffer	DeformedPositionBuffer;
ByteAddressBuffer	RestPositionBuffer;
ByteAddressBuffer	CurvesOffsetsBuffer;
StructuredBuffer<float4> DeformedPositionOffset;
float3				RestPositionOffset;
float4x4			WorldToLocal;

int					StrandsSize;
int					NumPoints;

Buffer<float>		NiagaraFloatBuffer;
int					NiagaraFloatStride;

int					NodePositionComponent;
int					RestPositionComponent;

#if PERMUTATION_INTERPOLATION == LOCAL_INTERPOLATION

// Get the deformed triangle
void DIHairStrands_BuildDeformedTriangle(in float2 ProjectionUV, in uint TriangleIndex, in int StrandIndex, out float3 OutTrianglePosition, out float4 OutTriangleRotation)
{
	DIHairStrands_ComputeProjectionTriangle(
		ProjectionUV, 
		DeformedTrianglePositionBuffer[TriangleIndex * 3 + 0].xyz,
		DeformedTrianglePositionBuffer[TriangleIndex * 3 + 1].xyz, 
		DeformedTrianglePositionBuffer[TriangleIndex * 3 + 2].xyz, 
		float3(0,0,0), OutTrianglePosition, OutTriangleRotation);
}

// Get the rest triangle
void DIHairStrands_BuildRestTriangle(in float2 ProjectionUV, in uint TriangleIndex, in int StrandIndex, out float3 OutTrianglePosition, out float4 OutTriangleRotation)
{
	DIHairStrands_ComputeProjectionTriangle(
		ProjectionUV, 
		RestTrianglePositionBuffer[TriangleIndex * 3 + 0].xyz,
		RestTrianglePositionBuffer[TriangleIndex * 3 + 1].xyz,
		RestTrianglePositionBuffer[TriangleIndex * 3 + 2].xyz,
		float3(0,0,0), OutTrianglePosition, OutTriangleRotation);
}

#endif

[numthreads(THREAD_COUNT, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x < uint(NumPoints))
	{
		const int StrandIndex = ReadHairPointToCurveIndex(PointToCurveIndexBuffer, DispatchThreadId.x);

		const FHairCurve Curve = ReadHairCurve(CurvesOffsetsBuffer, StrandIndex);
		const int PointBegin = Curve.PointIndex;
		const int PointCount = Curve.PointCount;
		const int PointIndex = DispatchThreadId.x - PointBegin;
		
		
		const float PointCoord = (float) (PointIndex) / (PointCount - 1.0);
		const float PointNode = PointCoord * (StrandsSize - 2);

		int NodePrev = (PointIndex == 0) ? 0 : (PointIndex == (PointCount - 1.0)) ? StrandsSize - 3 : floor(PointNode);
		const int NodeNext = NodePrev + 1;

		const float PointAlpha = PointNode - (float) NodePrev;
		
		const int NodePositionXOffset = NodePositionComponent * NiagaraFloatStride + StrandIndex * StrandsSize;
		const int NodePositionYOffset = NodePositionXOffset + NiagaraFloatStride;
		const int NodePositionZOffset = NodePositionYOffset + NiagaraFloatStride;
		
		const float3 LocalPrevNodePosition = mul(float4(NiagaraFloatBuffer[NodePositionXOffset+NodePrev],
		                                   NiagaraFloatBuffer[NodePositionYOffset+NodePrev],
		                                   NiagaraFloatBuffer[NodePositionZOffset+NodePrev],1), WorldToLocal).xyz;

		const float3 LocalNextNodePosition = mul(float4(NiagaraFloatBuffer[NodePositionXOffset+NodeNext],
										   NiagaraFloatBuffer[NodePositionYOffset+NodeNext],
										   NiagaraFloatBuffer[NodePositionZOffset+NodeNext],1), WorldToLocal).xyz;

		const int RestPositionXOffset = RestPositionComponent * NiagaraFloatStride + StrandIndex * StrandsSize;
		const int RestPositionYOffset = RestPositionXOffset + NiagaraFloatStride;
		const int RestPositionZOffset = RestPositionYOffset + NiagaraFloatStride;
		
		const float3 LocalPrevRestPosition = float3(NiagaraFloatBuffer[RestPositionXOffset+NodePrev],
										   NiagaraFloatBuffer[RestPositionYOffset+NodePrev],
										   NiagaraFloatBuffer[RestPositionZOffset+NodePrev]);

		const float3 LocalNextRestPosition = float3(NiagaraFloatBuffer[RestPositionXOffset+NodeNext],
										   NiagaraFloatBuffer[RestPositionYOffset+NodeNext],
										   NiagaraFloatBuffer[RestPositionZOffset+NodeNext]);

#if PERMUTATION_INTERPOLATION == LOCAL_INTERPOLATION
		
		const float2 ProjectionUV = UnpackBarycentrics(RootBarycentricCoordinatesBuffer[StrandIndex]).xy;

		float3 RestTrianglePosition = float3(0, 0, 0), DeformedTrianglePosition = float3(0, 0, 0);
		float4 RestTriangleOrientation = QUATERNION_IDENTITY, DeformedTriangleOrientation = QUATERNION_IDENTITY;
		
		const uint TriangleIndex = RootToUniqueTriangleIndexBuffer[StrandIndex];
		DIHairStrands_BuildRestTriangle(ProjectionUV, TriangleIndex, StrandIndex, RestTrianglePosition, RestTriangleOrientation);
		DIHairStrands_BuildDeformedTriangle(ProjectionUV, TriangleIndex, StrandIndex, DeformedTrianglePosition, DeformedTriangleOrientation);
		
		const float3 TrianglePrevRestPosition = DIHairStrands_TriangleLocalPosition(RestTrianglePosition, RestTriangleOrientation, LocalPrevRestPosition);
		const float3 TrianglePrevNodePosition = DIHairStrands_TriangleLocalPosition(DeformedTrianglePosition, DeformedTriangleOrientation, LocalPrevNodePosition);
			
		const float3 TriangleNextRestPosition = DIHairStrands_TriangleLocalPosition(RestTrianglePosition, RestTriangleOrientation, LocalNextRestPosition);
		const float3 TriangleNextNodePosition = DIHairStrands_TriangleLocalPosition(DeformedTrianglePosition, DeformedTriangleOrientation, LocalNextNodePosition);

		const float3 LocalPointDisplace = (TrianglePrevNodePosition - TrianglePrevRestPosition)  * (1.0 - PointAlpha) + (TriangleNextNodePosition - TriangleNextRestPosition) * PointAlpha;
		
		const FPackedHairPosition Packed = ReadPackedHairPosition(RestPositionBuffer, DispatchThreadId.x);
		const float3 LocalTrianglePosition = DIHairStrands_TriangleLocalPosition(RestTrianglePosition, RestTriangleOrientation, 
					UnpackHairControlPointPosition(Packed, RestPositionOffset)) + LocalPointDisplace;
		
		const float3 LocalComponentPosition = DIHairStrands_TriangleWorldPosition(DeformedTrianglePosition, DeformedTriangleOrientation, LocalTrianglePosition) - DeformedPositionOffset[0].xyz;
		WritePackedHairControlPointPosition(DeformedPositionBuffer, DispatchThreadId.x, Packed, LocalComponentPosition);

#endif
		
#if PERMUTATION_INTERPOLATION == WORLD_INTERPOLATION
		
		const float3 LocalPointDisplace = (LocalPrevNodePosition - LocalPrevRestPosition)  * (1.0 - PointAlpha) + (LocalNextNodePosition - LocalNextRestPosition) * PointAlpha;

		const FPackedHairPosition Packed = ReadPackedHairPosition(RestPositionBuffer, DispatchThreadId.x);
		const float3 LocalComponentPosition = ReadHairControlPointPosition(RestPositionBuffer, DispatchThreadId.x, RestPositionOffset) + LocalPointDisplace;
		WritePackedHairControlPointPosition(DeformedPositionBuffer, DispatchThreadId.x, Packed, LocalComponentPosition, DeformedPositionOffset[0].xyz);
		
#endif
	}
}
