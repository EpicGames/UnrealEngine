// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/* -----------------------------------------------------------------
 * Collision Utilities
 * -----------------------------------------------------------------
 */

  bool ComputeCollisionDatas( in float3 DeltaPosition, in float3 DeltaVelocity, in float3 CollisionNormal, in float PenetrationDepth,
							  in float StaticFriction, in float KineticFriction, in bool IsProjection, inout float DeltaFNormal, 
							  inout float AlphaTangentA, inout float DeltaVTangentA, inout float3 CollisionTangentA, 
							  inout float AlphaTangentB, inout float DeltaVTangentB, inout float3 CollisionTangentB)
 {
	const float DeltaVNormal = dot(DeltaVelocity,CollisionNormal);
	const float DeltaPNormal = dot(DeltaPosition,CollisionNormal)-PenetrationDepth;

	if (DeltaPNormal < 0.0)
	{
		DeltaFNormal = IsProjection ? DeltaPNormal : DeltaVNormal;
		const float NormDeltaFN = abs(DeltaFNormal);

		const float3 CollisionTangent = DeltaVelocity - DeltaVNormal * CollisionNormal;
		const float TangentLength = length(CollisionTangent);

		CollisionTangentA = (TangentLength > 0.0) ? CollisionTangent/TangentLength : (abs(CollisionNormal.x-1.0f) > 0.0) ? float3(1,0,0) : 
																					        (abs(CollisionNormal.y-1.0f) > 0.0) ? float3(0,1,0) :
																						     float3(0,0,1);
		CollisionTangentB = cross(CollisionNormal,CollisionTangentA);
		CollisionTangentA = cross(CollisionTangentB,CollisionNormal);

		const float TangentLengthA = length(CollisionTangentA);
		const float TangentLengthB = length(CollisionTangentB);

		CollisionTangentA = (TangentLengthA > 0.0) ? CollisionTangentA/TangentLengthA : float3(0,0,0);
		CollisionTangentB = (TangentLengthB > 0.0) ? CollisionTangentB/TangentLengthB : float3(0,0,0);

		DeltaVTangentA = dot( DeltaVelocity, CollisionTangentA);
		DeltaVTangentB = dot( DeltaVelocity, CollisionTangentB);

		const float NormDeltaVTA = abs(DeltaVTangentA);
		const float NormDeltaVTB = abs(DeltaVTangentB);

		AlphaTangentA = ( NormDeltaVTA < StaticFriction * NormDeltaFN ) ? 1.0 : (KineticFriction > 0.0) ? min(KineticFriction*NormDeltaFN/NormDeltaVTA, 1.0) : 0.0;
		AlphaTangentB = ( NormDeltaVTB < StaticFriction * NormDeltaFN ) ? 1.0 : (KineticFriction > 0.0) ? min(KineticFriction*NormDeltaFN/NormDeltaVTB, 1.0) : 0.0;
		return true;
	}
	return false;
 }

/* -----------------------------------------------------------------
 * Static hard collision constraint
 * -----------------------------------------------------------------
 */

 void UpdateHardCollisionConstraint( in float3 DeltaPosition, in float3 DeltaVelocity, in float3 CollisionNormal, in float PenetrationDepth,
							  in float StaticFriction, in float KineticFriction, in bool IsProjection)
 {
	float DeltaFNormal = 0.0; 

	float AlphaTangentA = 0.0;
	float DeltaVTangentA = 0.0;
	float3 CollisionTangentA = float3(0,0,0);
	
	float AlphaTangentB = 0.0;
	float DeltaVTangentB = 0.0;
	float3 CollisionTangentB = float3(0,0,0);

	if( ComputeCollisionDatas(DeltaPosition,DeltaVelocity,CollisionNormal,PenetrationDepth,StaticFriction,KineticFriction,IsProjection,
			DeltaFNormal,AlphaTangentA,DeltaVTangentA,CollisionTangentA,AlphaTangentB,DeltaVTangentB,CollisionTangentB))
	{
		SharedNodePosition[GGroupThreadId.x] -= DeltaFNormal * CollisionNormal + AlphaTangentA * CollisionTangentA * DeltaVTangentA + AlphaTangentB * CollisionTangentB * DeltaVTangentB;
	}
 }

 void SolveHardCollisionConstraint(in bool EnableConstraint, in int StrandsSize, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal, 
				in float StaticFriction, in float KineticFriction, in bool IsProjection, in float DeltaTime )
{
	if( GGroupThreadId.x % StrandsSize > 1 && EnableConstraint)
	{
		const float3 NodeVelocity = (SharedNodePosition[GGroupThreadId.x] - SharedPreviousPosition[GGroupThreadId.x])/DeltaTime;
		const float3 DeltaPosition = SharedNodePosition[GGroupThreadId.x] - CollisionPosition;
		const float3 DeltaVelocity = (NodeVelocity - CollisionVelocity) * DeltaTime;

		UpdateHardCollisionConstraint(DeltaPosition,DeltaVelocity,CollisionNormal,PenetrationDepth,StaticFriction,KineticFriction,IsProjection);
		GroupMemoryBarrier();
	}
}

void ProjectHardCollisionConstraint(in bool EnableConstraint, in int StrandsSize, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal, 
				in float StaticFriction, in float KineticFriction, in float DeltaTime, out float3 OutNodePosition  )
{
	if(DeltaTime != 0.0)
	{
		SolveHardCollisionConstraint(EnableConstraint,StrandsSize,PenetrationDepth,CollisionPosition,CollisionVelocity,
					CollisionNormal,StaticFriction,KineticFriction,true,DeltaTime);
	}

	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x];
}


/* -----------------------------------------------------------------
 * Static soft collision constraint
 * -----------------------------------------------------------------
 */

 void UpdateSoftCollisionConstraint(in float3 DeltaPosition, in float3 DeltaVelocity, in float3 CollisionNormal, in float PenetrationDepth,
							  in float StaticFriction, in float KineticFriction, in bool IsProjection, in float DeltaTime, in float MaterialDamping, 
							  in float MaterialCompliance, in float MaterialWeight, inout float3 OutMaterialMultiplier )
{

	float DeltaFNormal = 0.0; 

	float AlphaTangentA = 0.0;
	float DeltaVTangentA = 0.0;
	float3 CollisionTangentA = float3(0,0,0);
	
	float AlphaTangentB = 0.0;
	float DeltaVTangentB = 0.0;
	float3 CollisionTangentB = float3(0,0,0);

	if( ComputeCollisionDatas(DeltaPosition,DeltaVelocity,CollisionNormal,PenetrationDepth,StaticFriction,KineticFriction,IsProjection,
			DeltaFNormal,AlphaTangentA,DeltaVTangentA,CollisionTangentA,AlphaTangentB,DeltaVTangentB,CollisionTangentB))
	{
		const float LocalCompliance = ((PenetrationDepth <= 0.0) || ( PenetrationDepth > 0.0 && dot(DeltaPosition,CollisionNormal) < 0.0)) ? 0.0 : 
							MaterialCompliance;// * max(0.0,min(1.0,(1.0 + DeltaPNormal / PenetrationDepth)));

		const float ScaledInverseMass = ( (1.0  + MaterialDamping / DeltaTime ) * SharedInverseMass[GGroupThreadId.x] );

		const float3 SchurDiagonal = float3(ScaledInverseMass + LocalCompliance, ScaledInverseMass + LocalCompliance, ScaledInverseMass + LocalCompliance );
		const float MaterialWeightN = ( SchurDiagonal.x != 0.0 ) ? 1.0 / SchurDiagonal.x : 0.0;
		const float MaterialWeightA = ( SchurDiagonal.y != 0.0 ) ? 1.0 / SchurDiagonal.y : 0.0;
		const float MaterialWeightB = ( SchurDiagonal.z != 0.0 ) ? 1.0 / SchurDiagonal.z : 0.0;

		const float3 DeltaLambda = float3(
		    -( DeltaFNormal + OutMaterialMultiplier.x * LocalCompliance + MaterialDamping * dot(DeltaVelocity,CollisionNormal) / DeltaTime ) * MaterialWeightN,
			-( DeltaVTangentA + OutMaterialMultiplier.y * LocalCompliance + MaterialDamping * DeltaVTangentA / DeltaTime ) * MaterialWeightA * AlphaTangentA,
			-( DeltaVTangentB + OutMaterialMultiplier.z * LocalCompliance + MaterialDamping * DeltaVTangentB / DeltaTime ) * MaterialWeightB * AlphaTangentB);

		OutMaterialMultiplier += DeltaLambda;

		SharedNodePosition[GGroupThreadId.x] += SharedInverseMass[GGroupThreadId.x] * (DeltaLambda.x * CollisionNormal + DeltaLambda.y * CollisionTangentA + DeltaLambda.z * CollisionTangentB );
	}
	else
	{
		OutMaterialMultiplier = float3(0,0,0);
	}
}

 void SetupSoftCollisionConstraint(in int StrandsSize, in float ConstraintStiffness,
	in float DeltaTime, in float MaterialDamping, out float OutMaterialCompliance, out float OutMaterialWeight, out float3 OutMaterialMultiplier)
{
	OutMaterialCompliance = 0.0;
	OutMaterialWeight = 0.0;
	OutMaterialMultiplier = float3(0.0,0.0,0.0);

	const int LocalIndex = (GGroupThreadId.x % StrandsSize);
	if( LocalIndex > 1 )
	{
		OutMaterialCompliance = (ConstraintStiffness > 0.0) ? 1.0/(ConstraintStiffness*DeltaTime*DeltaTime) : 0.0;
	}
}

void SolveSoftCollisionConstraint(in bool EnableConstraint, in int StrandsSize, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal, 
				in float StaticFriction, in float KineticFriction, in bool IsProjection, in float DeltaTime, in float MaterialDamping, 
			in float MaterialCompliance, in float MaterialWeight, in float3 MaterialMultiplier, out float3 OutMaterialMultiplier )
{
	OutMaterialMultiplier = MaterialMultiplier;
	if( GGroupThreadId.x % StrandsSize > 1 && EnableConstraint)
	{
		const float3 NodeVelocity = (SharedNodePosition[GGroupThreadId.x] - SharedPreviousPosition[GGroupThreadId.x])/DeltaTime;
		const float3 DeltaPosition = SharedNodePosition[GGroupThreadId.x] - CollisionPosition;
		const float3 DeltaVelocity = (NodeVelocity - CollisionVelocity) * DeltaTime;

		const float LocalDamping = IsProjection ? 0.0 : MaterialDamping;
		UpdateSoftCollisionConstraint(DeltaPosition,DeltaVelocity,CollisionNormal,PenetrationDepth,
				StaticFriction,KineticFriction,IsProjection,DeltaTime,LocalDamping,MaterialCompliance,MaterialWeight,OutMaterialMultiplier);

		GroupMemoryBarrier();
	}
}

void ProjectSoftCollisionConstraint(in bool EnableConstraint, in int StrandsSize, in float ConstraintStiffness, in float PenetrationDepth, in float3 CollisionPosition, in float3 CollisionVelocity, in float3 CollisionNormal, 
				in float StaticFriction, in float KineticFriction, in float DeltaTime, out float3 OutNodePosition  )
{
	float MaterialCompliance = 0.0;
	float MaterialWeight = 0.0;
	float3 MaterialMultiplier = float3(0.0,0.0,0.0);
	float3 OutMaterialMultiplier = float3(0.0,0.0,0.0);

	if(DeltaTime != 0.0)
	{
		SetupSoftCollisionConstraint(StrandsSize,ConstraintStiffness,DeltaTime,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier);

		SolveSoftCollisionConstraint(EnableConstraint,StrandsSize,PenetrationDepth,CollisionPosition,CollisionVelocity,
			CollisionNormal,StaticFriction,KineticFriction,true,DeltaTime,0.0,MaterialCompliance,MaterialWeight,MaterialMultiplier,OutMaterialMultiplier);
	}

	GroupMemoryBarrier();
	OutNodePosition = SharedNodePosition[GGroupThreadId.x];
}


