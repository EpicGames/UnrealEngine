// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraStrandsExternalForce.ush 
=============================================================================*/

#pragma once

/* -----------------------------------------------------------------
 * Compute Air Drag Force
 * -----------------------------------------------------------------
 */

// Air Density = 1 kg/m3 = 1e-3 g/cm3
// Dynamic Viscosity = 1e-5 Pa.s = 1e-6 g/(cm.s)
// Compute air drag force
void ComputeAirDragForce(in int StrandSize, in float AirDensity, in float AirViscosity, in float AirDrag, 
	in float3 AirVelocity, in float NodeThickness, in float3 NodePosition, in float3 NodeVelocity, out float3 OutAirDrag, out float3 OutForceGradient)
{	
	const float3 DeltaVelocity = NodeVelocity - AirVelocity;
	const float DeltaVNorm = length(DeltaVelocity);

	SharedNodePosition[GGroupThreadId.x] = NodePosition;
	GroupMemoryBarrier();
	
	float AlignedTangent = 0.0;
	float TangentLength = 0.0;
	if( DeltaVNorm > 1e-6)
	{
		const int LocalIndex = GGroupThreadId.x % StrandSize;
		float3 NodeTangent = (LocalIndex == 0) ? SharedNodePosition[GGroupThreadId.x+1] - SharedNodePosition[GGroupThreadId.x] : 
												 SharedNodePosition[GGroupThreadId.x] - SharedNodePosition[GGroupThreadId.x-1]; 
		TangentLength = length(NodeTangent);
		NodeTangent = normalize(NodeTangent);

		AlignedTangent = abs(dot(NodeTangent,DeltaVelocity)/DeltaVNorm);
	}
	const float NodeDiameter = NodeThickness * (1.0 - AlignedTangent) + TangentLength * AlignedTangent;
	const float NodeArea = NodeDiameter * NodeThickness;

	// V = 4 * PI * R * R * R / 3 = PI * D * D * D / 6 = M / rho 
	//const float NodeDiameter = pow(6.0 * (NodeMass / StrandsDensity ) / PI, 1.0 / 3.0);
	//const float NodeArea = PI * NodeDiameter * NodeDiameter / 4.0;

	const float LinearDrag = 3.0 * PI * NodeDiameter * AirViscosity * AirDrag;
	const float QuadraticDrag = 0.5 * AirDensity * NodeArea * AirDrag;

	const float DragCoeff = LinearDrag + QuadraticDrag * DeltaVNorm;

	OutAirDrag = - DragCoeff * DeltaVelocity;

	OutForceGradient = - float3(DragCoeff,DragCoeff,DragCoeff);
	if( DeltaVNorm > 1e-6)
	{
		OutForceGradient -= QuadraticDrag * float3(DeltaVelocity.x*DeltaVelocity.x,
												   DeltaVelocity.y*DeltaVelocity.y,
												   DeltaVelocity.z*DeltaVelocity.z) / DeltaVNorm;
	}
}









