// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h" // for IMPLEMENT_MODULE(), FDefaultModuleImpl

IMPLEMENT_MODULE(FDefaultModuleImpl, HairCardGeneratorFramework);