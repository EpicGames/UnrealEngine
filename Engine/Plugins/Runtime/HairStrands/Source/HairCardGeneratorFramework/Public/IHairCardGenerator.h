// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h" // for RegisterModularFeature()/UnregisterModularFeature()

#define UE_API HAIRCARDGENERATORFRAMEWORK_API

class  UGroomAsset;
struct FHairGroupsCardsSourceDescription;
class  UHairCardGenerationSettings;

class IHairCardGenerator : public IModularFeature
{
public:
	static UE_API const FName ModularFeatureName; // "HairCardGenerator"

	virtual bool GenerateHairCardsForLOD(UGroomAsset* Groom, FHairGroupsCardsSourceDescription& CardsDesc) = 0;

	virtual bool IsCompatibleSettings(UHairCardGenerationSettings* OldSettings) = 0;
};

namespace HairCardGenerator_Utils
{
	inline void RegisterModularHairCardGenerator(IHairCardGenerator* Generator)
	{
		IModularFeatures::Get().RegisterModularFeature(IHairCardGenerator::ModularFeatureName, Generator);
	}

	inline void UnregisterModularHairCardGenerator(IHairCardGenerator* Generator)
	{
		IModularFeatures::Get().UnregisterModularFeature(IHairCardGenerator::ModularFeatureName, Generator);
	}
}

#undef UE_API
