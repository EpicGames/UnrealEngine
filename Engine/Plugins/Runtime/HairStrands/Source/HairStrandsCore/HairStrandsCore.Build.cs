// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairStrandsCore : ModuleRules
	{
		public HairStrandsCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"GeometryCache",
					"Projects",
					"MeshDescription",
					"MovieScene",
					"NiagaraCore",
					"NiagaraShader",
					"DataflowCore",
					"DataflowEngine",
					"RenderCore",
					"Renderer",
					"VectorVM",
					"RHI",
					"TraceLog",
					"StaticMeshDescription",
					"Eigen",
					"ChaosCore",
					"Chaos",
					"GeometryCore"
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Niagara",
					"ChaosCaching"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Shaders",
				});

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DerivedDataCache",
						"MeshBuilder",
						"NaniteBuilder"
					});

				PrivateIncludePathModuleNames.AddRange(
					new string[]
					{
						"DerivedDataCache",
					});
			}
		}
	}
}
