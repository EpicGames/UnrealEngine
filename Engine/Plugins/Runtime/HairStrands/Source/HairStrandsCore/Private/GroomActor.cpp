// Copyright Epic Games, Inc. All Rights Reserved. 

#include "GroomActor.h"
#include "GroomComponent.h"
#include "Components/BillboardComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "Internationalization/Internationalization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomActor)

#define LOCTEXT_NAMESPACE "HairStrands"

AGroomActor::AGroomActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GroomComponent = CreateDefaultSubobject<UGroomComponent>(TEXT("GroomComponent0"));
	RootComponent = GroomComponent;

// Disable in editor icon as this bias the actor bounds size, and prevent to have 
// accurate focusing on the groom component.
#if WITH_EDITOR
#if 0
	struct FConstructorStatics
	{
		FName ID_Sprite;
		FText NAME_Sprite;
		FColor SceneBaseColor;
		FVector SceneBaseSize;
		FConstructorStatics()
			: ID_Sprite(TEXT("GroomActor"))
			, NAME_Sprite(LOCTEXT("RootSpriteInfo", "Groom Actor"))
			, SceneBaseColor(100, 255, 255, 255)
			, SceneBaseSize(600.0f, 600.0f, 400.0f)
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> RootTextureObject = TEXT("/HairStrands/Icons/S_GroomActor");
		
		SpriteComponent->Sprite = RootTextureObject.Get();
		SpriteComponent->SetRelativeScale3D_Direct(FVector(1.0f, 1.0f, 1.0f));
		SpriteComponent->bHiddenInGame = false;
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Sprite;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Sprite;
		SpriteComponent->bIsScreenSizeScaled = true;
		SpriteComponent->SetupAttachment(RootComponent);
	}
#endif
#endif
}

#if WITH_EDITOR
bool AGroomActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (GroomComponent->GroomAsset)
	{
		Objects.Add(GroomComponent->GroomAsset);
	}
	return true;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE





