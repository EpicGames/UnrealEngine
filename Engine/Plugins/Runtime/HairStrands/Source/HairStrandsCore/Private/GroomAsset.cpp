// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAsset.h"

#include "Async/Async.h"
#include "Async/UniqueLock.h"
#include "Engine/Texture2D.h"
#include "EngineUtils.h"
#include "GroomAssetImportData.h"
#include "GroomBuilder.h"
#include "HairCardsBuilder.h"
#include "GroomImportOptions.h"
#include "GroomSettings.h"
#include "Materials/MaterialInterface.h"
#include "RenderingThread.h"
#include "Engine/AssetUserData.h"
#include "HairStrandsVertexFactory.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/App.h"
#include "AssetCompilingManager.h"
#include "Misc/Paths.h"
#include "RHIStaticStates.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/AnimObjectVersion.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "NiagaraSystem.h"
#include "GroomComponent.h"
#include "Math/Box.h"
#include "Engine/StaticMesh.h"
#include "Logging/LogMacros.h"
#include "HairStrandsCore.h"
#include "UObject/ObjectSaveContext.h"
#include "GroomBindingAsset.h"
#include "GroomDeformerBuilder.h"
#include "HairCardsVertexFactory.h"
#include "PSOPrecache.h"
#include "UObject/ICookInfo.h"
#include "UObject/UObjectIterator.h"
#include "RenderGraphBuilder.h"
#include "UObject/DevObjectVersion.h"
#include "GroomVisualizationData.h"
#include "NaniteGroomBuilder.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#endif
#include "Misc/CoreMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAsset)

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "PlatformInfo.h"
#endif

#if WITH_EDITORONLY_DATA
#include "DerivedDataCacheInterface.h"
#include "EditorFramework/AssetImportData.h"
#endif

LLM_DEFINE_TAG(Groom);

#define LOCTEXT_NAMESPACE "GroomAsset"

static int32 GHairStrandsLoadAsset = 1;
static FAutoConsoleVariableRef CVarHairStrandsLoadAsset(TEXT("r.HairStrands.LoadAsset"), GHairStrandsLoadAsset, TEXT("Allow groom asset to be loaded"));

// Editor async groom load can be useful in a workflow that consists mostly of loading grooms from a hot DDC
static int32 GEnableGroomAsyncLoad = 0;
static FAutoConsoleVariableRef CVarGroomAsyncLoad(TEXT("r.HairStrands.AsyncLoad"), GEnableGroomAsyncLoad, TEXT("Allow groom asset to be loaded asynchronously in the editor"));

static int32 GUpdateGroomGroupsNames = 0;
static FAutoConsoleVariableRef CVarUpdateGroomGroupsNames(TEXT("r.HairStrands.UpdateGroupNames"), GUpdateGroomGroupsNames, TEXT("Update groom asset groups' names if not already serialized"));

void UpdateHairStrandsVerbosity(IConsoleVariable* InCVarVerbosity);
static TAutoConsoleVariable<int32> GHairStrandsWarningLogVerbosity(
	TEXT("r.HairStrands.Log"),
	-1,
	TEXT("Enable warning log report for groom related asset (0: no logging, 1: error only, 2: error & warning only, other: all logs). By default all logging are enabled (-1). Value needs to be set at startup time."),
	FConsoleVariableDelegate::CreateStatic(UpdateHairStrandsVerbosity));

static int32 GHairStrandsDDCLogEnable = 0;
static FAutoConsoleVariableRef CVarHairStrandsDDCLogEnable(TEXT("r.HairStrands.DDCLog"), GHairStrandsDDCLogEnable, TEXT("Enable DDC logging for groom assets and groom binding assets"));

static int32 GHairStrandsReleaseResourceBeforeBuildingData = 1;
static FAutoConsoleVariableRef CVarHairStrandsReleaseResourceBeforeBuildingData(TEXT("r.HairStrands.ReleaseResourceBeforeBuildingData"), GHairStrandsReleaseResourceBeforeBuildingData, TEXT("Experimental settings for releasing resources prior to building data. This avoids issue when reloading groom."));

static EGroomLODMode InternalGetHairStrandsLODMode()
{
	return GetHairStrandsLODMode() > 0 ? EGroomLODMode::Auto : EGroomLODMode::Manual;
}

static int32 GHairStrandsSupportCompressedPosition = 0;
static FAutoConsoleVariableRef CVarHairStrandsSupportCompressedPosition(TEXT("r.HairStrands.CompressedPosition"), GHairStrandsSupportCompressedPosition, TEXT("Optional compessed position"), ECVF_ReadOnly);

bool DoesHairStrandsSupportCompressedPosition()
{
	return GHairStrandsSupportCompressedPosition > 0;
}

bool IsHairStrandsDDCLogEnable()
{
	return GHairStrandsDDCLogEnable > 0;
}

uint32 GetAssetNameHash(const FString& In)
{
	return CityHash32((const char*)*In, In.Len() * sizeof(FString::ElementType));
}

static int32 GHairMaxSimulatedLOD = -1;
static FAutoConsoleVariableRef CVarHairMaxSimulatedLOD(TEXT("r.HairStrands.MaxSimulatedLOD"), GHairMaxSimulatedLOD, TEXT("Maximum hair LOD to be simulated"));
static bool IsHairLODSimulationEnabled(const int32 LODIndex) { return (LODIndex >= 0 && (GHairMaxSimulatedLOD < 0 || (GHairMaxSimulatedLOD >= 0 && LODIndex <= GHairMaxSimulatedLOD))); }

static int32 GHairStrandsUsesNaniteGroom = 0;
static FAutoConsoleVariableRef CVarHairStrandsUsesNaniteGroom(TEXT("r.HairStrands.Nanite"), GHairStrandsUsesNaniteGroom, TEXT("Experimental: Use Nanite to render groom geometry. Works only with non-simulated grooms and is incompatible with groom binding."), ECVF_ReadOnly);
static bool UsesNaniteGroom()
{
	return NaniteCurvesSupported() && GHairStrandsUsesNaniteGroom > 0;
}

/////////////////////////////////////////////////////////////////////////////////////////
// Nanite
bool UGroomAsset::UsesNanite() const
{
	return GetRenderData() != nullptr;
}

FGroomAssetRenderData* UGroomAsset::GetRenderData()
{
	return RenderData.Get();
}

const FGroomAssetRenderData* UGroomAsset::GetRenderData() const
{
	return RenderData.Get();
}

void UGroomAsset::SetRenderData(TUniquePtr<class FGroomAssetRenderData>&& InRenderData)
{
	RenderData = MoveTemp(InRenderData);
}

bool UGroomAsset::HasValidNaniteData() const
{
	if (const FGroomAssetRenderData* SMRenderData = GetRenderData())
	{
		return SMRenderData->HasValidNaniteData();
	}

	return false;
}

bool UGroomAsset::HasNaniteFallbackMesh(EShaderPlatform ShaderPlatform) const
{	
	return false;
}

#if WITH_EDITOR
void UGroomAsset::ClearAllCachedCookedPlatformData()
{
	if (UsesNaniteGroom())
	{
		if (FGroomAssetRenderData* SMRenderData = GetRenderData())
		{
			SMRenderData->GetResources()->DropBulkData();
		}
	}

	Super::ClearAllCachedCookedPlatformData();
}
#endif

UMaterialInterface* UGroomAsset::GetNaniteAuditMaterial(int32 MaterialIndex) const
{
	UMaterialInterface* Out = nullptr;
	if (GetHairGroupsMaterials().IsValidIndex(MaterialIndex))
	{
		Out = GetHairGroupsMaterials()[MaterialIndex].Material;
	}
	return Out;
}

const Nanite::FMaterialAudit* UGroomAsset::GetNaniteMaterials() const
{
	if (const FGroomAssetRenderData* LocalRenderData = GetRenderData())
	{
		return LocalRenderData->GetMaterialAudit();
	}
	return nullptr;
}

/////////////////////////////////////////////////////////////////////////////////////////
// Asset dump function

void DumpLoadedGroomData(IConsoleVariable* InCVarPakTesterEnabled);
void DumpLoadedGroomAssets(IConsoleVariable* InCVarPakTesterEnabled);
void DumpLoadedGroomComponent(IConsoleVariable* InCVarPakTesterEnabled);
void DumpLoadedGroomBindingAssets(IConsoleVariable* InCVarPakTesterEnabled);

int32 GHairStrandsDump_GroomAsset = 0;
static FAutoConsoleVariableRef CVarHairStrandsDump_GroomAsset(
	TEXT("r.HairStrands.Dump.GroomAsset"),
	GHairStrandsDump_GroomAsset,
	TEXT("Dump information of all the loaded groom assets."),
	FConsoleVariableDelegate::CreateStatic(DumpLoadedGroomAssets));

int32 GHairStrandsDump_GroomBindingAsset = 0;
static FAutoConsoleVariableRef CVarHairStrandsDump_GroomBindingAsset(
	TEXT("r.HairStrands.Dump.GroomBindingAsset"),
	GHairStrandsDump_GroomBindingAsset,
	TEXT("Dump information of all the loaded groom binding assets."),
	FConsoleVariableDelegate::CreateStatic(DumpLoadedGroomBindingAssets));

int32 GHairStrandsDump_All = 0;
static FAutoConsoleVariableRef CVarHairStrandsDump_All(
	TEXT("r.HairStrands.Dump"),
	GHairStrandsDump_All,
	TEXT("Dump all the loaded groom assets, groom binding assets, and instanciated groom components."),
	FConsoleVariableDelegate::CreateStatic(DumpLoadedGroomData));

uint32 FHairGroupPlatformData::FStrands::GetDataSize() const
{
	uint32 ATotal = 0;
	uint32 BTotal = 0;
	uint32 CTotal = 0;
	ATotal += ::GetDataSize(BulkData);
	ATotal += ::GetDataSize(InterpolationBulkData);
	BTotal += ClusterBulkData.Data.CurveToClusterIds.IsBulkDataLoaded() ? ClusterBulkData.Data.CurveToClusterIds.GetBulkDataSize() : 0;
	CTotal += ClusterBulkData.Data.PackedClusterInfos.IsBulkDataLoaded()? ClusterBulkData.Data.PackedClusterInfos.GetBulkDataSize() : 0;
	CTotal += ClusterBulkData.Data.PointLODs.IsBulkDataLoaded()	? ClusterBulkData.Data.PointLODs.GetBulkDataSize() : 0;


	return ATotal + BTotal + CTotal;
}

FGroomAssetMemoryStats FGroomAssetMemoryStats::Get(const FHairGroupPlatformData& InData, const FHairGroupResources& In)
{
	FGroomAssetMemoryStats Out;
	Out.CPU.Guides  = InData.Guides.GetDataSize();
	Out.CPU.Strands = InData.Strands.GetDataSize();
	Out.CPU.Cards   = InData.Cards.GetDataSize();
	Out.CPU.Meshes  = InData.Meshes.GetDataSize();

	Out.GPU.Guides  = In.Guides.GetResourcesSize();
	Out.GPU.Strands = In.Strands.GetResourcesSize();
	Out.GPU.Cards   = In.Cards.GetResourcesSize();
	Out.GPU.Meshes  = In.Meshes.GetResourcesSize();

	
	// Strands only
	if (In.Strands.RestResource) 			Out.Memory.Rest 			= In.Strands.RestResource->GetResourcesSize();
	if (In.Strands.InterpolationResource) 	Out.Memory.Interpolation 	= In.Strands.InterpolationResource->GetResourcesSize();
	if (In.Strands.ClusterResource)			Out.Memory.Cluster 			= In.Strands.ClusterResource->GetResourcesSize();
#if RHI_RAYTRACING
	if (In.Strands.RaytracingResource) 		Out.Memory.Raytracing 		= In.Strands.RaytracingResource->GetResourcesSize();
#endif
	if (In.Strands.RestResource) 			Out.Curves.Rest 			= In.Strands.RestResource->MaxAvailableCurveCount;
	if (In.Strands.InterpolationResource) 	Out.Curves.Interpolation 	= In.Strands.InterpolationResource->MaxAvailableCurveCount;
	if (In.Strands.ClusterResource)			Out.Curves.Cluster 			= In.Strands.ClusterResource->MaxAvailableCurveCount;
#if RHI_RAYTRACING
	if (In.Strands.RaytracingResource)		Out.Curves.Raytracing 		= In.Strands.RaytracingResource->MaxAvailableCurveCount;
#endif
	return Out;
}

void FGroomAssetMemoryStats::Accumulate(const FGroomAssetMemoryStats& In)
{
	CPU.Guides += In.CPU.Guides ;
	CPU.Strands+= In.CPU.Strands;
	CPU.Cards  += In.CPU.Cards  ;
	CPU.Meshes += In.CPU.Meshes ;

	GPU.Guides += In.GPU.Guides ;
	GPU.Strands+= In.GPU.Strands;
	GPU.Cards  += In.GPU.Cards  ;
	GPU.Meshes += In.GPU.Meshes ;
}
uint32 FGroomAssetMemoryStats::GetTotalCPUSize() const
{
	uint32 Out = 0;
	Out += CPU.Guides ;
	Out += CPU.Strands;
	Out += CPU.Cards  ;
	Out += CPU.Meshes ;
	return Out;
}

uint32 FGroomAssetMemoryStats::GetTotalGPUSize() const
{
	uint32 Out = 0;
	Out += GPU.Guides ;
	Out += GPU.Strands;
	Out += GPU.Cards  ;
	Out += GPU.Meshes ;
	return Out;
}

void DumpLoadedGroomAssets(IConsoleVariable* InCVarPakTesterEnabled)
{
	const bool bDetails = true;
	const float ToMb = 1.f / 1000000.f;

	uint32 Total_Asset = 0;
	uint32 Total_Group = 0;

	FGroomAssetMemoryStats Total;

	UE_LOGF(LogHairStrands, Log, "[Groom] ##### UGroomAssets #####");
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogHairStrands, Log, "--  No.  - LOD -    CPU Total (     Guides|    Strands|      Cards|     Meshes) -    GPU Total (     Guides|    Strands|      Cards|     Meshes) - Asset Name ");
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	FString OutputData;
	for (TObjectIterator<UGroomAsset> AssetIt; AssetIt; ++AssetIt)
	{
		if (AssetIt)
		{
			const uint32 GroupCount = AssetIt->GetHairGroupsPlatformData().Num();
			for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
			{
				const FGroomAssetMemoryStats Group = FGroomAssetMemoryStats::Get(AssetIt->GetHairGroupsPlatformData()[GroupIt], AssetIt->GetHairGroupsResources()[GroupIt]);
				Total.Accumulate(Group);
				const uint32 LODCount = AssetIt->GetHairGroupsLOD()[GroupIt].LODs.Num();
				if (bDetails)
				{
//					UE_LOGF(LogHairStrands, Log, "--  No.  - LOD -    CPU Total (     Guides|    Strands|      Cards|     Meshes) -    GPU Total (     Guides|    Strands|      Cards|     Meshes) - Asset Name ");
//					UE_LOGF(LogHairStrands, Log, "-- 00/00 -  00 -  0000.0Mb (0000.0Mb |0000.0Mb|0000.0Mb |0000.0Mb) -  0000.0Mb (0000.0Mb|0000.0Mb|0000.0Mb|0000.0Mb) - AssetName");
					UE_LOGF(LogHairStrands, Log, "-- %2d/%2d -  %2d -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb) -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb) - %ls", 
						GroupIt,
						GroupCount,
						LODCount, 
						Group.GetTotalCPUSize() * ToMb,
						Group.CPU.Guides * ToMb,
						Group.CPU.Strands* ToMb,
						Group.CPU.Cards  * ToMb,
						Group.CPU.Meshes * ToMb,
                        Group.GetTotalGPUSize()* ToMb,
						Group.GPU.Guides * ToMb,
						Group.GPU.Strands* ToMb,
						Group.GPU.Cards  * ToMb,
						Group.GPU.Meshes * ToMb,
						GroupIt == 0 ? *AssetIt->GetPathName() : TEXT("."));
				}
			}

			Total_Asset++;
			Total_Group += GroupCount;
		}
	}

	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogHairStrands, Log, "-- A:%3d|G:%3d -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb) -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb)",
		Total_Asset,
		Total_Group,
		Total.GetTotalCPUSize() * ToMb,
		Total.CPU.Guides * ToMb,
		Total.CPU.Strands * ToMb,
		Total.CPU.Cards * ToMb,
		Total.CPU.Meshes * ToMb,
        Total.GetTotalGPUSize() * ToMb,
		Total.GPU.Guides * ToMb,
		Total.GPU.Strands * ToMb,
		Total.GPU.Cards * ToMb,
		Total.GPU.Meshes * ToMb);
}

FGroomBindingAssetMemoryStats FGroomBindingAssetMemoryStats::Get(const UGroomBindingAsset::FHairGroupPlatformData& InCPU, const UGroomBindingAsset::FHairGroupResource& InGPU)
{
	FGroomBindingAssetMemoryStats Out;
	for (const FHairStrandsRootBulkData& In : InCPU.SimRootBulkDatas) { Out.CPU.Guides += In.GetDataSize(); }
	for (const FHairStrandsRootBulkData& In : InCPU.RenRootBulkDatas) { Out.CPU.Strands+= In.GetDataSize(); }

	if (const FHairStrandsRestRootResource* R = InGPU.SimRootResources)
	{
		Out.GPU.Guides += R->GetResourcesSize();
	}
	if (const FHairStrandsRestRootResource* R = InGPU.RenRootResources)
	{
		Out.GPU.Strands += R->GetResourcesSize();
	}

	const uint32 CardCount = InCPU.CardsRootBulkDatas.Num();
	for (uint32 CardIt = 0; CardIt < CardCount; ++CardIt)
	{
		for (const FHairStrandsRootBulkData& In : InCPU.CardsRootBulkDatas[CardIt]) { Out.CPU.Cards += In.GetDataSize(); }
		if (const FHairStrandsRestRootResource* R = InGPU.CardsRootResources[CardIt])
		{
			Out.GPU.Cards += R->GetResourcesSize();
		}
	}

	return Out;
}

void FGroomBindingAssetMemoryStats::Accumulate(const FGroomBindingAssetMemoryStats& In)
{
	CPU.Guides += In.CPU.Guides ;
	CPU.Strands+= In.CPU.Strands;
	CPU.Cards  += In.CPU.Cards  ;

	GPU.Guides += In.GPU.Guides ;
	GPU.Strands+= In.GPU.Strands;
	GPU.Cards  += In.GPU.Cards  ;
}
uint32 FGroomBindingAssetMemoryStats::GetTotalCPUSize() const
{
	uint32 Out = 0;
	Out += CPU.Guides ;
	Out += CPU.Strands;
	Out += CPU.Cards  ;
	return Out;
}

uint32 FGroomBindingAssetMemoryStats::GetTotalGPUSize() const
{
	uint32 Out = 0;
	Out += GPU.Guides ;
	Out += GPU.Strands;
	Out += GPU.Cards  ;
	return Out;
}

void DumpLoadedGroomBindingAssets(IConsoleVariable* InCVarPakTesterEnabled)
{
	const bool bDetails = true;
	const float ToMb = 1.f / 1000000.f;

	uint32 Total_Asset = 0;
	uint32 Total_Group = 0;

	FGroomBindingAssetMemoryStats Total;
	
	UE_LOGF(LogHairStrands, Log, "[Groom] ##### UGroomBindingAssets #####");
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogHairStrands, Log, "--  No.  - LOD -    CPU Total (     Guides|    Strands|      Cards) -    GPU Total (     Guides|    Strands|      Cards) - Asset Name ");
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	FString OutputData;
	for (TObjectIterator<UGroomBindingAsset> AssetIt; AssetIt; ++AssetIt)
	{
		if (AssetIt)
		{			
			const uint32 GroupCount = AssetIt->GetHairGroupsPlatformData().Num();
			for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
			{
				const FGroomBindingAssetMemoryStats Group = FGroomBindingAssetMemoryStats::Get(AssetIt->GetHairGroupsPlatformData()[GroupIt], AssetIt->GetHairGroupResources()[GroupIt]);
				Total.Accumulate(Group);

				const uint32 SkelLODCount = AssetIt->GetHairGroupsPlatformData()[GroupIt].RenRootBulkDatas.Num();
				if (bDetails)
				{
//					UE_LOGF(LogHairStrands, Log, "--  No.  - LOD -    CPU Total (     Guides|    Strands|      Cards) -    GPU Total (     Guides|    Strands|      Cards) - Asset Name ");
//					UE_LOGF(LogHairStrands, Log, "-- 00/00 -  00 -  0000.0Mb (0000.0Mb |0000.0Mb|0000.0Mb) -  0000.0Mb (0000.0Mb|0000.0Mb|0000.0Mb) - AssetName");
					UE_LOGF(LogHairStrands, Log, "-- %2d/%2d -  %2d -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb) -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb) - %ls", 
						GroupIt,
						GroupCount,
						SkelLODCount,
                        Group.GetTotalCPUSize() * ToMb,
						Group.CPU.Guides * ToMb,
						Group.CPU.Strands* ToMb,
						Group.CPU.Cards  * ToMb,
                        Group.GetTotalGPUSize()* ToMb,
						Group.GPU.Guides * ToMb,
						Group.GPU.Strands* ToMb,
						Group.GPU.Cards  * ToMb,
						GroupIt == 0 ? *AssetIt->GetPathName() : TEXT("."));
				}
			}

			Total_Asset++;
			Total_Group += GroupCount;
		}
	}
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogHairStrands, Log, "-- A:%3d|G:%3d -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb) -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb)",
		Total_Asset,
		Total_Group,
		Total.GetTotalCPUSize() * ToMb,
		Total.CPU.Guides * ToMb,
		Total.CPU.Strands * ToMb,
		Total.CPU.Cards * ToMb,
		Total.GetTotalGPUSize() * ToMb,
		Total.GPU.Guides * ToMb,
		Total.GPU.Strands * ToMb,
		Total.GPU.Cards * ToMb);
}

void DumpLoadedGroomData(IConsoleVariable* InCVarPakTesterEnabled)
{
	DumpLoadedGroomAssets(InCVarPakTesterEnabled);
	DumpLoadedGroomBindingAssets(InCVarPakTesterEnabled);
	DumpLoadedGroomComponent(InCVarPakTesterEnabled);
}

/////////////////////////////////////////////////////////////////////////////////////////

void UpdateHairStrandsVerbosity(IConsoleVariable* InCVarVerbosity)
{
	const int32 Verbosity = InCVarVerbosity->GetInt();
	switch (Verbosity)
	{
	case 0:  UE_SET_LOG_VERBOSITY(LogHairStrands, NoLogging); break;
	case 1:  UE_SET_LOG_VERBOSITY(LogHairStrands, Error); break;
	case 2:  UE_SET_LOG_VERBOSITY(LogHairStrands, Warning); break;
	default: UE_SET_LOG_VERBOSITY(LogHairStrands, Log); break;
	};
}

/////////////////////////////////////////////////////////////////////////////////////////

template<typename ResourceType>
static void InitAtlasTexture(ResourceType* InResource, const FHairGroupCardsTextures& In, const bool bInvertUV)
{
	if (InResource == nullptr)
	{
		return;
	}
	InResource->bInvertUV = bInvertUV;

	const TArray<UTexture2D*>& InTextures = In.Textures;
	const uint32 InLayoutIndex = uint32(In.Layout);

	bool bHasValidTextures = false;
	for (UTexture2D* T : InTextures)
	{
		if (T)
		{
			T->ConditionalPostLoad();
			bHasValidTextures = true;
		}
	}

	if (bHasValidTextures)
	{
		ENQUEUE_RENDER_COMMAND(HairStrandsCardsTextureCommand)(/*UE::RenderCommandPipe::Groom,*/
			[InResource, InTextures, InLayoutIndex](FRHICommandListImmediate& RHICmdList)
		{
			FSamplerStateRHIRef DefaultSampler = TStaticSamplerState<SF_AnisotropicLinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 8/*MaxAnisotropy*/>::GetRHI();

			const uint32 TextureCount = InTextures.Num();
			InResource->Textures.SetNum(TextureCount);
			InResource->Samplers.SetNum(TextureCount);
			InResource->LayoutIndex = InLayoutIndex;

			for (uint32 TextureIt=0;TextureIt<TextureCount;++TextureIt)
			{
				if (UTexture2D* Texture = InTextures[TextureIt])
				{
					InResource->Textures[TextureIt] = Texture->TextureReference.TextureReferenceRHI;
				}
				InResource->Samplers[TextureIt] = DefaultSampler;
			}
		});
	}
}

template<typename T>
static const T* GetSourceDescription(const TArray<T>& InHairGroups, uint32 GroupIndex, uint32 LODIndex, int32* SourceIndex=nullptr)
{
	int32 LocalSourceIndex = 0;
	for (const T& SourceDesc : InHairGroups)
	{
		if (SourceDesc.GroupIndex == GroupIndex && SourceDesc.LODIndex == LODIndex)
		{
			if (SourceIndex) { *SourceIndex = LocalSourceIndex; }
			return &SourceDesc;
		}
		++LocalSourceIndex;
	}
	if (SourceIndex) { *SourceIndex = -1; }
	return nullptr;
}

template<typename T>
static T* GetSourceDescription(TArray<T>& InHairGroups, uint32 GroupIndex, uint32 LODIndex, int32* SourceIndex=nullptr)
{
	int32 LocalSourceIndex = 0;
	for (T& SourceDesc : InHairGroups)
	{
		if (SourceDesc.GroupIndex == GroupIndex && SourceDesc.LODIndex == LODIndex)
		{
			if (SourceIndex) { *SourceIndex = LocalSourceIndex; }
			return &SourceDesc;
		}
		++LocalSourceIndex;
	}
	if (SourceIndex) { *SourceIndex = -1; }
	return nullptr;
}

bool IsHairStrandsAssetLoadingEnable()
{
	return GHairStrandsLoadAsset > 0;
}

// Build: 
// * Guides bulk data
// * Strands bulk data
// * Interpolation bulk data
// * Clusters (bulk) data
//
// Guides, strands, and interpolation data are all transient
static bool BuildHairGroup_Strands(
	const int32 GroupIndex,
	const bool bNeedInterpolationData,
	const FHairDescriptionGroups& HairDescriptionGroups, 
	const TArray<FHairGroupsInterpolation>& InHairGroupsInterpolation,
	const TArray<FHairGroupsLOD>& InHairGroupsLOD,
	TArray<FHairGroupInfoWithVisibility>& OutHairGroupsInfo,
	TArray<FHairGroupPlatformData>& OutHairGroupsData, 
	const bool bBuildSourceMapping)
{
	const bool bIsValid = HairDescriptionGroups.IsValid();
	if (bIsValid)
	{
		const FHairDescriptionGroup& HairGroup = HairDescriptionGroups.HairGroups[GroupIndex];
		check(GroupIndex <= HairDescriptionGroups.HairGroups.Num());
		check(GroupIndex == HairGroup.Info.GroupIndex);

		FHairStrandsDatas StrandsData;
		FHairStrandsDatas GuidesData;
		FGroomBuilder::BuildData(HairGroup, InHairGroupsInterpolation[GroupIndex], OutHairGroupsInfo[GroupIndex], StrandsData, GuidesData, true/*bAllowCurveReordering*/, true/*bApplyDecimation*/, bBuildSourceMapping);

		FGroomBuilder::BuildBulkData(HairGroup.Info, GuidesData, OutHairGroupsData[GroupIndex].Guides.BulkData, false /*bAllowCompression*/);
		FGroomBuilder::BuildBulkData(HairGroup.Info, StrandsData, OutHairGroupsData[GroupIndex].Strands.BulkData, true /*bAllowCompression*/);

		// If there is no simulation or no global interpolation on that group there is no need for builder the interpolation data
		if (bNeedInterpolationData)
		{
			FHairStrandsInterpolationDatas InterpolationData;
			FGroomBuilder::BuildInterplationData(HairGroup.Info, StrandsData, GuidesData, InHairGroupsInterpolation[GroupIndex].InterpolationSettings, InterpolationData);
			FGroomBuilder::BuildInterplationBulkData(GuidesData, InterpolationData, OutHairGroupsData[GroupIndex].Strands.InterpolationBulkData);
		}
		else
		{
			OutHairGroupsData[GroupIndex].Strands.InterpolationBulkData.Reset();
		}

		FGroomBuilder::BuildClusterBulkData(StrandsData, HairDescriptionGroups.Bounds.SphereRadius, InHairGroupsLOD[GroupIndex], OutHairGroupsData[GroupIndex].Strands.ClusterBulkData);

		// Curve prototype
		if (IsRenderCurveEnabled())
		{
			FHairStrandsDatas StrandsDataNoShuffled;
			FHairStrandsDatas GuidesDataNoShuffled;
			FGroomBuilder::BuildData(HairGroup, InHairGroupsInterpolation[GroupIndex], OutHairGroupsInfo[GroupIndex], StrandsDataNoShuffled, GuidesDataNoShuffled, false /*bAllowCurveReordering*/);
			FGroomBuilder::BuildRenderCurveResourceBulkData(StrandsDataNoShuffled, OutHairGroupsData[GroupIndex].Strands.CurveResourceData);
		}
	}
	return bIsValid;
}

/////////////////////////////////////////////////////////////////////////////////////////

uint8 UGroomAsset::GenerateClassStripFlags(FArchive& Ar)
{
#if WITH_EDITOR
	const bool bIsCook = Ar.IsCooking();
	const ITargetPlatform* CookTarget = Ar.CookingTarget();

	bool bIsStrandsSupportedOnTargetPlatform = true;
	bool bIsLODStripped = false;
	bool bIsStrandsStripped = false;
	bool bIsCardsStripped = false;
	bool bIsMeshesStripped = false;
	if (bIsCook)
	{
		// Determine if strands are supported on any shader formats used by the target cook platform
		TArray<FName> ShaderFormats;
		CookTarget->GetAllTargetedShaderFormats(ShaderFormats);
		bIsStrandsSupportedOnTargetPlatform = false;
		for (int32 FormatIndex = 0; FormatIndex < ShaderFormats.Num(); ++FormatIndex)
		{
			const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormats[FormatIndex]);
			if (IsHairStrandsSupported(EHairStrandsShaderType::Strands, ShaderPlatform))
			{
				bIsStrandsSupportedOnTargetPlatform = true;
				break;
			}
		}

		// Determine the platform min LOD if the stripping hasn't been disabled
		const bool bDisableMinLODStrip = GetDisableBelowMinLodStripping().GetValueForPlatform(*CookTarget->IniPlatformName());
		int32 PlatformMinLOD = 0;
		if (!bDisableMinLODStrip)
		{
			PlatformMinLOD = GetMinLOD().GetValueForPlatform(*CookTarget->IniPlatformName());
		}

		bIsLODStripped = (PlatformMinLOD > 0) && !bDisableMinLODStrip;
		if (bIsLODStripped)
		{
			for (const FHairGroupsLOD& GroupsLOD : GetHairGroupsLOD())
			{
				for (int32 LODIndex = 0; LODIndex < PlatformMinLOD && LODIndex < GroupsLOD.LODs.Num(); ++LODIndex)
				{
					const FHairLODSettings& LODSetting = GroupsLOD.LODs[LODIndex];
					switch (LODSetting.GeometryType)
					{
					case EGroomGeometryType::Strands:
						bIsStrandsStripped = true;
						break;
					case EGroomGeometryType::Cards:
						bIsCardsStripped = true;
						break;
					case EGroomGeometryType::Meshes:
						bIsMeshesStripped = true;
						break;
					}
				}
			}
		}
	}

	uint8 ClassDataStripFlags = 0;
	ClassDataStripFlags |= HasImportedStrandsData() ? CDSF_ImportedStrands : 0;

	// Strands are cooked out if the platform doesn't support them even if they were not marked for stripping
	ClassDataStripFlags |= !bIsStrandsSupportedOnTargetPlatform || bIsStrandsStripped ? CDSF_StrandsStripped : 0;
	ClassDataStripFlags |= bIsLODStripped ? CDSF_MinLodData : 0;
	ClassDataStripFlags |= bIsCardsStripped ? CDSF_CardsStripped : 0;
	ClassDataStripFlags |= bIsMeshesStripped ? CDSF_MeshesStripped : 0;

	return ClassDataStripFlags;
#else
	return 0;
#endif
}

void UGroomAsset::ApplyStripFlags(uint8 StripFlags, const ITargetPlatform* CookTarget)
{
#if WITH_EDITOR
	if (!CookTarget)
	{
		return;
	}

	const bool bDisableMinLodStrip = GetDisableBelowMinLodStripping().GetValueForPlatform(*CookTarget->IniPlatformName()); 
	int32 PlatformMinLOD = 0;
	if (!bDisableMinLodStrip)
	{
		PlatformMinLOD = GetMinLOD().GetValueForPlatform(*CookTarget->IniPlatformName());
	}

	const bool bIsStrandsStrippedForCook = !!(StripFlags & CDSF_StrandsStripped);

	// Set the CookedOut flags as appropriate and compute the value for EffectiveLODBias
	for (int32 GroupIndex = 0; GroupIndex < GetHairGroupsPlatformData().Num(); ++GroupIndex)
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		GroupData.Strands.bIsCookedOut = bIsStrandsStrippedForCook;

		// Determine the max LOD for strands since they are cooked out if the platform doesn't support them
		// They are cooked out as a whole
		int32 StrandsMaxLOD = -1;
		if (bIsStrandsStrippedForCook)
		{
			const FHairGroupsLOD& GroupsLOD = GetHairGroupsLOD()[GroupIndex];
			for (int32 LODIndex = 0; LODIndex < GroupsLOD.LODs.Num(); ++LODIndex)
			{
				const FHairLODSettings& LODSetting = GroupsLOD.LODs[LODIndex];
				if (LODSetting.GeometryType == EGroomGeometryType::Strands)
				{
					StrandsMaxLOD = LODIndex;
				}
			}
		}
		
		// The MinLOD for stripping this group, taking into account the possibility of strands being cooked out
		const int32 MinStrippedLOD = FMath::Max(StrandsMaxLOD + 1, PlatformMinLOD);
		GetEffectiveLODBias()[GroupIndex] = MinStrippedLOD;

		for (int32 Index = 0; Index < GroupData.Cards.LODs.Num(); ++Index)
		{
			GroupData.Cards.LODs[Index].bIsCookedOut = !!(StripFlags & CDSF_CardsStripped) && (Index < MinStrippedLOD);
		}

		for (int32 Index = 0; Index < GroupData.Meshes.LODs.Num(); ++Index)
		{
			GroupData.Meshes.LODs[Index].bIsCookedOut = !!(StripFlags & CDSF_MeshesStripped) && (Index < MinStrippedLOD);
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA
bool UGroomAsset::HasImportedStrandsData() const
{
	bool bHasImportedStrandsData = false;
	for (const FString& Key : StrandsDerivedDataKey)
	{
		bHasImportedStrandsData |= !Key.IsEmpty();
	}
	return bHasImportedStrandsData;
}
#endif

// Serialization for *array* of hair elements
static void InternalSerializeCards(FArchive& Ar, UObject* Owner, TArray<FHairGroupPlatformData::FCards::FLOD>& CardLODData);
static void InternalSerializeMeshes(FArchive& Ar, UObject* Owner, TArray<FHairGroupPlatformData::FMeshes::FLOD>& MeshLODData);
static void InternalSerializePlatformDatas(FArchive& Ar, UObject* Owner, TArray<FHairGroupPlatformData>& GroupData);

// Serialization for hair elements
static void InternalSerializeCard(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FCards::FLOD& CardLODData);
static void InternalSerializeMesh(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FMeshes::FLOD& MeshLODData);
static void InternalSerializeGuide(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FGuides& GuideData);
static void InternalSerializeStrand(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FStrands& StrandData, bool bHeader, bool bData, bool* bOutHasDataInCache=nullptr);
static void InternalSerializePlatformData(FArchive& Ar, UObject* Owner, FHairGroupPlatformData& GroupData);

// This dummy serialization function is only intended to support *loading* of legacy content 
static void DummySerizalizationForLegacyPurpose(FArchive& Ar)
{
	// Only support loading of legacy content 
	check(Ar.IsLoading());
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	uint32 GroupCount = 0;					Ar << GroupCount;
	for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
	{
		// Strands
		{
			uint32 CurveCount = 0;			Ar << CurveCount;
			uint32 PointCount = 0;			Ar << PointCount;
			float MaxLength = 0;			Ar << MaxLength;
			float MaxRadius = 0;			Ar << MaxRadius;
			FBox BoundingBox;				Ar << BoundingBox;
			uint32 Flags = 0;				Ar << Flags;
			uint32 AttributeOffsets[7] = { 0 };
			for (uint8 It = 0; It < 7; ++It)
			{
				Ar << AttributeOffsets[It];
			}
		}

		// Guides
		{
			uint32 CurveCount = 0;			Ar << CurveCount;
			uint32 PointCount = 0;			Ar << PointCount;
			float MaxLength = 0;			Ar << MaxLength;
			float MaxRadius = 0;			Ar << MaxRadius;
			FBox BoundingBox;				Ar << BoundingBox;
			uint32 Flags = 0;				Ar << Flags;
			uint32 AttributeOffsets[7] = { 0 };
			for (uint8 It = 0; It < 7; ++It)
			{
				Ar << AttributeOffsets[It];
			}
		}

		// Interpolation
		{
			uint32 Flags = 0;				Ar << Flags;
			uint32 PointCount = 0;			Ar << PointCount;
			uint32 SimPointCount = 0;		Ar << SimPointCount;
		}

		// Cluster
		{
			uint32 ClusterCount = 0;		Ar << ClusterCount;
			uint32 ClusterLODCount = 0;		Ar << ClusterLODCount;
			uint32 VertexCount = 0;			Ar << VertexCount;
			uint32 VertexLODCount = 0;		Ar << VertexLODCount;
			TArray<bool> LODVisibility;		Ar << LODVisibility;
			TArray<float>CPULODScreenSize;	Ar << CPULODScreenSize;
		}

		// Misc
		{
			bool bIsCooked; 				Ar << bIsCooked;
		}
	}
}

void UGroomAsset::Serialize(FArchive& Ar)
{
	uint8 ClassDataStripFlags = GenerateClassStripFlags(Ar);
	ApplyStripFlags(ClassDataStripFlags, Ar.CookingTarget());

	TObjectPtr<UDataflow> StrippedDataflowAsset = nullptr;
	bool bStrippedDataflowData = false;
		
	bool bIsCookedOrCooking = Ar.IsCooking();
	if ((bIsCookedOrCooking && Ar.IsSaving()) || (Ar.IsCountingMemory() && Ar.IsFilterEditorOnly()))
	{
#if WITH_EDITOR
		// The dataflow asset is only needed for the editor, so we just remove it when cooking 
		StrippedDataflowAsset = DataflowSettings.GetDataflowAsset();
		DataflowSettings.SetDataflowAsset(nullptr);
		bStrippedDataflowData = true;
#endif
	}

	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);			// Needed to support MeshDescription AttributesSet serialization
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);	// Needed to support MeshDescription AttributesSet serialization
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);    // Needed to support Cards and Cluster culling serialization

	// For older assets, force width override to preserve prior to fixing the imported groom's width.
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::GroomAssetWidthOverride && Ar.IsLoading())
	{
		for (FHairGroupsRendering& Group : GetHairGroupsRendering())
		{
			Group.GeometrySettings.HairWidth_Override = true;
		}
	}

	if (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::GroomWithDescription)
	{
		FStripDataFlags StripFlags(Ar, ClassDataStripFlags);

		bool bHasImportedStrands = true; // true because prior to this version, we assume that a groom is created from imported strands
		if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::GroomLODStripping)
		{
			// StripFlags keeps the flags only when cooking, but the class flags are needed when not cooking
			Ar << ClassDataStripFlags;
			bHasImportedStrands = ClassDataStripFlags & CDSF_ImportedStrands;
		}


		if (StripFlags.IsEditorDataStripped())
		{
			// When cooking data or loading cooked data
			InternalSerializePlatformDatas(Ar, this, GetHairGroupsPlatformData());
		}
		else if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::GroomAssetRemoveInAssetSerialization && Ar.IsLoading() && !bHasImportedStrands)
		{
			// Legacy support for groom asset not having strands data
			DummySerizalizationForLegacyPurpose(Ar);
		}
#if WITH_EDITORONLY_DATA
		else
		{
			auto SerializeHairDescription = [this, &Ar](const EHairDescriptionType DescriptionType)
			{
				const uint8 DescriptionIndex = FMath::Clamp(static_cast<uint8>(DescriptionType), 0, 1);
					
				// When serializing data for editor, serialize the HairDescription as bulk data
				// The computed groom data is fetched from the Derived Data Cache
				if (!HairDescriptionBulkData[DescriptionIndex])
				{
					// When loading, bulk data can be null so instantiate a new one to serialize into
					HairDescriptionBulkData[DescriptionIndex] = MakeUnique<FHairDescriptionBulkData>();
				}

				// We don' serialize the hair description if it is a transactional object (i.e., for undo/redo)
				if (!Ar.IsTransacting())
				{
					HairDescriptionBulkData[DescriptionIndex]->Serialize(Ar, this);
				}
			};

			SerializeHairDescription(EHairDescriptionType::Source);
			
			if (GetHairDescriptionType() == EHairDescriptionType::Edit)
			{
				SerializeHairDescription(EHairDescriptionType::Edit);
			}
		}
#endif // WITH_EDITORONLY_DATA

		// Important : this needs to remain after the call to Super::Serialize
		if (bStrippedDataflowData)
		{
#if WITH_EDITORONLY_DATA
			DataflowSettings.SetDataflowAsset(StrippedDataflowAsset);
#endif
		}
	}
	else
	{
		// Old format serialized the computed groom data directly, but are no longer supported
		UE_LOGF(LogHairStrands, Error, "[Groom] The groom asset %ls is too old. Please reimported the groom from its original source file.", *GetName());
	}

	// Basic handling of Nanite resources
	if (UsesNaniteGroom())
	{
		#if WITH_EDITOR
		if (bIsCookedOrCooking && DoesTargetPlatformSupportNanite(Ar.CookingTarget()))
		{			
			GetRenderData()->Serialize(Ar, this, true);
		}
		#else
		if (DoesPlatformSupportNanite(GMaxRHIShaderPlatform))
		{
			check(Ar.IsLoading());
			SetRenderData(MakeUnique<FGroomAssetRenderData>());
			GetRenderData()->InitNaniteResources();
			GetRenderData()->Serialize(Ar, this, true);
			GetRenderData()->BuildMaterials(this);
		}
		#endif
	}
}

UGroomAsset::UGroomAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), DataflowSettings(this, nullptr, TEXT("GroomAssetTerminal"))
{
	bIsInitialized = false;

	SetMinLOD(0);
	SetDisableBelowMinLodStripping(false);
}

bool UGroomAsset::HasGeometryType(uint32 GroupIndex, EGroomGeometryType Type) const
{
	check(GroupIndex < uint32(GetHairGroupsLOD().Num()));
	const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
	for (uint32 LODIt=0;LODIt<LODCount;++LODIt)
	{
		if (GetHairGroupsLOD()[GroupIndex].LODs[LODIt].GeometryType == Type)
			return true;
	}
	return false;
}

bool UGroomAsset::HasGeometryType(EGroomGeometryType Type) const
{
	const uint32 GroupCount = GetHairGroupsLOD().Num();
	for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
	{
		if (HasGeometryType(GroupIt, Type))
			return true;
	}
	return false;
}

bool GetSupportHairStrandsLinearSweptSpheres(EShaderPlatform InShaderPlatform);

TArray<FHairVertexFactoryTypesPerMaterialData> UGroomAsset::CollectVertexFactoryTypesPerMaterialData(EShaderPlatform ShaderPlatform)
{
	TArray<FHairVertexFactoryTypesPerMaterialData> VFsPerMaterials;

	const auto AddVFPerMaterial = [&VFsPerMaterials](int32 InMaterialIndex, EHairGeometryType InHairGeometryType, const FVertexFactoryType* InVFType)
	{
		if (InMaterialIndex < 0)
		{
			return;
		}

		FHairVertexFactoryTypesPerMaterialData* VFsPerMaterial = VFsPerMaterials.FindByPredicate([InMaterialIndex, InHairGeometryType](const FHairVertexFactoryTypesPerMaterialData& Other) { return (Other.MaterialIndex == InMaterialIndex && Other.HairGeometryType == InHairGeometryType); });
		if (VFsPerMaterial == nullptr)
		{
			VFsPerMaterial = &VFsPerMaterials.AddDefaulted_GetRef();
			VFsPerMaterial->MaterialIndex = InMaterialIndex;
			VFsPerMaterial->HairGeometryType = InHairGeometryType;
		}
		VFsPerMaterial->VertexFactoryDataList.AddUnique(FPSOPrecacheVertexFactoryData(InVFType));
	};

	const int32 GroupCount = GetNumHairGroups();
	for (int32 GroupIt = 0; GroupIt < GroupCount; GroupIt++)
	{
		const FHairGroupPlatformData& InGroupData = GetHairGroupsPlatformData()[GroupIt];

		// Strands
		{
			const int32 MaterialIndex = GetMaterialIndex(GetHairGroupsRendering()[GroupIt].MaterialSlotName);

			if (GetSupportHairStrandsLinearSweptSpheres(ShaderPlatform) && IsHairStrandsLinearSweptSpheresEnabled())
			{
				AddVFPerMaterial(MaterialIndex, EHairGeometryType::Strands, &FLinearSweptSpheresHairStrandsVertexFactory::StaticType);
			}
			else
			{
				AddVFPerMaterial(MaterialIndex, EHairGeometryType::Strands, &FHairStrandsVertexFactory::StaticType);
			}
		}

		// Cards
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards, ShaderPlatform))
		{
			uint32 CardsLODIndex = 0;
			for (const FHairGroupPlatformData::FCards::FLOD& LOD : InGroupData.Cards.LODs)
			{
				if (LOD.IsValid())
				{
					// Material
					int32 MaterialIndex = INDEX_NONE;
					for (const FHairGroupsCardsSourceDescription& Desc : GetHairGroupsCards())
					{
						if (Desc.GroupIndex == GroupIt && Desc.LODIndex == CardsLODIndex)
						{
							MaterialIndex = GetMaterialIndex(Desc.MaterialSlotName);
							break;
						}
					}
					AddVFPerMaterial(MaterialIndex, EHairGeometryType::Cards, &FHairCardsVertexFactory::StaticType);
				}
				++CardsLODIndex;
			}
		}

		// Meshes
		{
			uint32 MeshesLODIndex = 0;
			for (const FHairGroupPlatformData::FMeshes::FLOD& LOD : InGroupData.Meshes.LODs)
			{
				if (LOD.IsValid())
				{
					// Material
					int32 MaterialIndex = INDEX_NONE;
					for (const FHairGroupsMeshesSourceDescription& Desc : GetHairGroupsMeshes())
					{
						if (Desc.GroupIndex == GroupIt && Desc.LODIndex == MeshesLODIndex)
						{
							MaterialIndex = GetMaterialIndex(Desc.MaterialSlotName);
							break;
						}
					}
					AddVFPerMaterial(MaterialIndex, EHairGeometryType::Meshes, &FHairCardsVertexFactory::StaticType);
				}
				++MeshesLODIndex;
			}
		}
	}

	return VFsPerMaterials;
}

enum EGroomAssetChangeType
{
	GroomChangeType_Interpolation = 1,
	GroomChangeType_Cards = 2,
	GroomChangeType_Meshes = 4,
	GroomChangeType_LOD = 8
};

inline void InternalUpdateResource(FRenderResource* Resource)
{
	if (Resource)
	{
		BeginUpdateResourceRHI(Resource);
	}
}

template<typename T>
inline void InternalReleaseResource(T*& Resource, bool bResetLoadedSize=false)
{
	if (Resource)
	{
		T* InResource = Resource;
		if (bResetLoadedSize)
		{
			Resource->InternalResetLoadedSize();
		}

		ENQUEUE_RENDER_COMMAND(ReleaseHairResourceCommand)(UE::RenderCommandPipe::Groom,
		[InResource](FRHICommandList& RHICmdList)
		{
			InResource->ReleaseResource();
			delete InResource;
		});
		Resource = nullptr;
	}
}

#if WITH_EDITOR
void UGroomAsset::UpdateResource()
{
	LLM_SCOPE_BYTAG(Groom);

	uint32 AllChangeType = 0;
	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		uint32 ChangeType = (CachedHairGroupsLOD[GroupIndex] == GetHairGroupsLOD()[GroupIndex] ? 0 : GroomChangeType_LOD);
		AllChangeType = AllChangeType | ChangeType;

		// No need to reload the resources as this needs only be done when the derived data are update, 
		// which is already called in such a case.
		#if 0
		check(GroupIndex < uint32(GetNumHairGroups()));
		FHairGroupResources& GroupData = GetHairGroupsResources()[GroupIndex];
		if (GroupData.Guides.RestResource)
		{
			InternalUpdateResource(GroupData.Guides.RestResource);
		}

		if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands))
		{
			InternalUpdateResource(GroupData.Strands.RestResource);
			InternalUpdateResource(GroupData.Strands.InterpolationResource);
			InternalUpdateResource(GroupData.Strands.ClusterResource);

			#if RHI_RAYTRACING
			if (IsHairRayTracingEnabled())
			{
				// Release raytracing resources, so that it will be recreate by the component.
				InternalReleaseResource(GroupData.Strands.RaytracingResource);
			}
			#endif
		}
		#endif
	}

	if (BuildCardsData())  { AllChangeType |= GroomChangeType_Cards;  }
	if (BuildMeshesData()) { AllChangeType |= GroomChangeType_Meshes; }

	if (AllChangeType & (GroomChangeType_LOD | GroomChangeType_Cards | GroomChangeType_Meshes))
	{
		OnGroomAssetResourcesChanged.Broadcast();
	}

	UpdateHairGroupsInfo();
	UpdateCachedSettings();
}
#endif // #if WITH_EDITOR

////////////////////////////////////////////////////////////////////////////////////////////////////
// Initialize resources

void UGroomAsset::InitResources()
{
	LLM_SCOPE_BYTAG(Groom);

	InitGuideResources();
	InitStrandsResources();
	InitCardsResources();
	InitMeshesResources();

	if (FGroomAssetRenderData* LocalRenderData = GetRenderData())
	{
		LocalRenderData->InitResources(GetWorld() ? GetWorld()->GetFeatureLevel() : ERHIFeatureLevel::Num, this);
	}

	bIsInitialized = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Release resources

void UGroomAsset::ReleaseResource()
{
	bIsInitialized = false;
	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		ReleaseGuidesResource(GroupIndex);
		ReleaseStrandsResource(GroupIndex);
		ReleaseCardsResource(GroupIndex);
		ReleaseMeshesResource(GroupIndex);
	}

	if (FGroomAssetRenderData* LocalRenderData = GetRenderData())
	{
		LocalRenderData->ReleaseResources();
	}
}

void UGroomAsset::ReleaseGuidesResource(uint32 GroupIndex)
{
	if (GetHairGroupsResources().IsValidIndex(GroupIndex))
	{
		FHairGroupResources& GroupData = GetHairGroupsResources()[GroupIndex];
		InternalReleaseResource(GroupData.Guides.RestResource);
	}
}

void UGroomAsset::ReleaseStrandsResource(uint32 GroupIndex)
{
	if (GetHairGroupsResources().IsValidIndex(GroupIndex))
	{
		FHairGroupResources& GroupData = GetHairGroupsResources()[GroupIndex];
		InternalReleaseResource(GroupData.Strands.RestResource, true);
		InternalReleaseResource(GroupData.Strands.ClusterResource, true);
		InternalReleaseResource(GroupData.Strands.InterpolationResource, true);
		#if RHI_RAYTRACING
		InternalReleaseResource(GroupData.Strands.RaytracingResource);
		#endif
	}
}

void UGroomAsset::ReleaseCardsResource(uint32 GroupIndex)
{
	if (GetHairGroupsResources().IsValidIndex(GroupIndex))
	{
		FHairGroupResources& GroupData = GetHairGroupsResources()[GroupIndex];
		for (FHairGroupResources::FCards::FLOD& LOD : GroupData.Cards.LODs)
		{
			InternalReleaseResource(LOD.RestResource);
			InternalReleaseResource(LOD.InterpolationResource);
			InternalReleaseResource(LOD.GuideRestResource);
			InternalReleaseResource(LOD.GuideInterpolationResource);
			#if RHI_RAYTRACING
			InternalReleaseResource(LOD.RaytracingResource);
			#endif
		}
	}
}

void UGroomAsset::ReleaseMeshesResource(uint32 GroupIndex)
{
	if (GetHairGroupsResources().IsValidIndex(GroupIndex))
	{
		FHairGroupResources& GroupData = GetHairGroupsResources()[GroupIndex];
		for (FHairGroupResources::FMeshes::FLOD& LOD : GroupData.Meshes.LODs)
		{
			InternalReleaseResource(LOD.RestResource);
			#if RHI_RAYTRACING
			InternalReleaseResource(LOD.RaytracingResource);
			#endif
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// 

void UGroomAsset::UpdateHairGroupsInfo()
{
#if WITH_EDITORONLY_DATA
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomAsset::UpdateHairGroupsInfo);
	UE::TUniqueLock LockScope(InternalLock);
#endif

	const uint32 GroupCount = GetNumHairGroups();
	const bool bForceReset = GetHairGroupsInfo().Num() != GroupCount;
	GetHairGroupsInfo().SetNum(GroupCount);

	uint32 GroupIndex = 0;
	for (FHairGroupPlatformData& Data: GetHairGroupsPlatformData())
	{
		FHairGroupInfoWithVisibility& Info = GetHairGroupsInfo()[GroupIndex];
		Info.GroupIndex = GroupIndex;
		Info.NumCurves = Data.Strands.BulkData.GetNumCurves();
		Info.NumGuides = Data.Guides.BulkData.GetNumCurves();
		Info.NumCurveVertices = Data.Strands.BulkData.GetNumPoints();
		Info.NumGuideVertices = Data.Guides.BulkData.GetNumPoints();
		Info.MaxCurveLength = Data.Strands.BulkData.GetMaxLength();
	#if WITH_EDITORONLY_DATA
		// Transfer group name from the hair description group to the UGroomAsset.
		// This pass is used right after an asset is imported.
		if (const FHairDescriptionGroups* LocalHairDescriptionGroups = CachedHairDescriptionGroups[GetHairDescriptionIndex()].Get())
		{
			if (LocalHairDescriptionGroups->HairGroups.IsValidIndex(GroupIndex))
			{
				Info.GroupName = LocalHairDescriptionGroups->HairGroups[GroupIndex].Info.GroupName;
			}
		}
	#endif
		if (bForceReset)
		{
			Info.bIsVisible = true;
		}
		++GroupIndex;
	}
}

static void SyncHairGroupsName(UGroomAsset* In)
{
#if WITH_EDITORONLY_DATA
	check(In);

	uint32 GroupIndex = 0;
	for (FHairGroupInfoWithVisibility& Info : In->GetHairGroupsInfo())
	{
		if (In->GetHairGroupsPhysics().IsValidIndex(GroupIndex))
		{
			In->GetHairGroupsPhysics()[GroupIndex].GroupName = Info.GroupName;
		}

		if (In->GetHairGroupsRendering().IsValidIndex(GroupIndex))
		{
			In->GetHairGroupsRendering()[GroupIndex].GroupName = Info.GroupName;
		}

		if (In->GetHairGroupsInterpolation().IsValidIndex(GroupIndex))
		{
			In->GetHairGroupsInterpolation()[GroupIndex].GroupName = Info.GroupName;
		}

		++GroupIndex;
	}
#endif
}

static bool UpdateHairGroupsName(UGroomAsset* In)
{
	bool bNeedResaved = false;
#if WITH_EDITORONLY_DATA
	check(In);

	uint32 GroupIndex = 0;
	for (FHairGroupInfoWithVisibility& Info : In->GetHairGroupsInfo())
	{
		// If the group name was not serialized, try to retrieve it from the hair description
		if (Info.GroupName == NAME_None && In->CanRebuildFromDescription())
		{
			const FHairDescriptionGroups& LocalHairDescriptionGroups = In->GetHairDescriptionGroups();
			if (LocalHairDescriptionGroups.HairGroups.IsValidIndex(GroupIndex))
			{
				Info.GroupName = LocalHairDescriptionGroups.HairGroups[GroupIndex].Info.GroupName;
				bNeedResaved = true;
			}
		}
		SyncHairGroupsName(In);

		++GroupIndex;
	}
#endif
	return bNeedResaved;
}

template<typename T>
bool ConvertMaterial(TArray<T>& Groups, TArray<FHairGroupsMaterial>& InHairGroupsMaterials)
{
	bool bNeedSaving = false;
	for (T& Group : Groups)
	{
		if (Group.Material)
		{
			FName MaterialSlotName = Group.Material->GetFName();
			bool bFound = false;
			for (FHairGroupsMaterial& Material : InHairGroupsMaterials)
			{
				if (Material.SlotName == MaterialSlotName)
				{
					bFound = true;
					Group.MaterialSlotName = MaterialSlotName;
					break;
				}
			}

			if (!bFound)
			{
				FHairGroupsMaterial& MaterialEntry = InHairGroupsMaterials.AddDefaulted_GetRef();
				MaterialEntry.Material = Group.Material;
				MaterialEntry.SlotName = MaterialSlotName;
				Group.MaterialSlotName = MaterialSlotName;
			}
			Group.Material = nullptr;
			bNeedSaving = true;
		}
	}

	return bNeedSaving;
}

void UGroomAsset::PostLoad()
{
	LLM_SCOPE_BYTAG(Groom);

	Super::PostLoad();

	DataflowSettings.PostLoad();

	// Early out if hair strands are disabled
	const bool bIsHairEnabled = IsGroomEnabled();
	if (!bIsHairEnabled)
	{
		return;
	}

	// Compute a hash of the Groom asset fullname for finding unique groom during LOD selection/streaming
	AssetNameHash = GetAssetNameHash(GetFullName());

	// Legacy asset are missing rendering or interpolation settings
#if WITH_EDITORONLY_DATA
	const bool bIsLegacyAsset = GetHairGroupsInterpolation().Num() == 0;
	if (bIsLegacyAsset)
	{
		if (HairDescriptionBulkData[GetHairDescriptionIndex()])
		{
			const FHairDescriptionGroups& LocalHairDescriptionGroups = GetHairDescriptionGroups();

			const uint32 GroupCount = LocalHairDescriptionGroups.HairGroups.Num();
			SetNumGroup(GroupCount);
		}
		else
		{
			// Old format serialized the computed groom data directly, but are no longer supported
			UE_LOGF(LogHairStrands, Error, "[Groom] The groom asset %ls is too old. Please reimported the groom from its original source file.", *GetName());
			SetNumGroup(0, false);
		}
	}
#endif
	// Convert old rigging/guide override
	for (FHairGroupsInterpolation& Group : GetHairGroupsInterpolation())
	{
		const bool bNeedLegacyConversion = Group.InterpolationSettings.bOverrideGuides_DEPRECATED || Group.RiggingSettings.bEnableRigging_DEPRECATED;
		if (bNeedLegacyConversion)
		{				
			if (Group.RiggingSettings.bEnableRigging_DEPRECATED && Group.InterpolationSettings.bOverrideGuides_DEPRECATED)
			{
				Group.InterpolationSettings.GuideType = EGroomGuideType::Rigged;
			}
			else if (Group.InterpolationSettings.bOverrideGuides_DEPRECATED)
			{
				Group.InterpolationSettings.GuideType = EGroomGuideType::Generated;
			}

			Group.InterpolationSettings.RiggedGuideNumCurves = Group.RiggingSettings.NumCurves_DEPRECATED;
			Group.InterpolationSettings.RiggedGuideNumPoints = Group.RiggingSettings.NumPoints_DEPRECATED;

			// Reset deprecated values, so that if the asset is saved, the legacy conversion will no longer occur
			Group.InterpolationSettings.bOverrideGuides_DEPRECATED = false;
			Group.RiggingSettings.bEnableRigging_DEPRECATED = false;
			Group.RiggingSettings.NumCurves_DEPRECATED = 0;
			Group.RiggingSettings.NumPoints_DEPRECATED = 0;
		}
	}

	// Convert cards
	for (FHairGroupsCardsSourceDescription& Group : GetHairGroupsCards())
	{
		// Convert old procedural cards to import cards
		if (Group.SourceType_DEPRECATED == EHairCardsSourceType::Procedural)
		{
			Group.bInvertUV = true;
			Group.SourceType_DEPRECATED = EHairCardsSourceType::Imported;
			Group.ImportedMesh = Group.ProceduralMesh_DEPRECATED;
			Group.ProceduralMesh_DEPRECATED = nullptr;
			Group.ImportedMeshKey = FString();
		}

		// Convert old textures
		if (Group.Textures.Textures.Num() == 0)
		{
			Group.Textures.Layout = EHairTextureLayout::Layout0;
			Group.Textures.Textures.Reserve(6);
			Group.Textures.Textures.Add(Group.Textures.DepthTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.CoverageTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.TangentTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.AttributeTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.MaterialTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.AuxilaryDataTexture_DEPRECATED);
		}
	}

	// Convert meshes
	for (FHairGroupsMeshesSourceDescription& Group : GetHairGroupsMeshes())
	{
		// Convert old textures
		if (Group.Textures.Textures.Num() == 0)
		{
			Group.Textures.Layout = EHairTextureLayout::Layout1;
			Group.Textures.Textures.Reserve(6);
			Group.Textures.Textures.Add(Group.Textures.DepthTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.CoverageTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.TangentTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.AttributeTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.MaterialTexture_DEPRECATED);
			Group.Textures.Textures.Add(Group.Textures.AuxilaryDataTexture_DEPRECATED);
		}
	}

#if WITH_EDITORONLY_DATA
	bool bSucceed = true;
	{
		// Interpolation settings are used for building the interpolation data, and per se defined the number of groups
		const uint32 GroupCount = GetHairGroupsInterpolation().Num();
		if (uint32(GetNumHairGroups()) != GroupCount)
		{
			SetNumGroup(GroupCount);
		}

		StrandsDerivedDataKey.SetNum(GroupCount);
		CardsDerivedDataKey.SetNum(GroupCount);
		MeshesDerivedDataKey.SetNum(GroupCount);

		// The async load is allowed only if all the groom derived data is already cached since parts of the build path need to run in the game thread
		bool bAsyncLoadEnabled = (GEnableGroomAsyncLoad > 0) && IsFullyCached();

		// Some contexts like cooking and commandlets are not conducive to async load
		bAsyncLoadEnabled = bAsyncLoadEnabled && !GIsCookerLoadingPackage && !IsRunningCommandlet() && !FApp::IsUnattended();
		if (bAsyncLoadEnabled)
		{
			GroomAssetStrongPtr = TStrongObjectPtr<UGroomAsset>(this); // keeps itself alive while completing the async load
			Async(EAsyncExecution::LargeThreadPool,
				[this]()
				{
					UE_LOGF(LogHairStrands, Log, "[Groom] %ls is fully cached. Loading it asynchronously.", *GetName());

					if (CacheDerivedDatas() && !bRetryLoadFromGameThread)
					{
						// Post-load completion code that must be executed in the game thread
						Async(EAsyncExecution::TaskGraphMainThread,
							[this]()
							{
								GroomAssetStrongPtr.Reset();

								InitResources();

								// This will update the GroomComponents that are using groom that was async loaded
								{
									FGroomComponentRecreateRenderStateContext RecreateContext(this);
								}

								OnGroomAsyncLoadFinished.Broadcast();
							});
					}
					else
					{
						UE_LOGF(LogHairStrands, Log, "[Groom] %ls failed to load asynchronously. Trying synchronous load.", *GetName());

						// Load might have failed because if failed to fetch the data from the DDC
						// Retry a sync load from the game thread
						Async(EAsyncExecution::TaskGraphMainThread,
							[this]()
							{
								GroomAssetStrongPtr.Reset();

								CacheDerivedDatas();

								OnGroomAsyncLoadFinished.Broadcast();
							});
					}
				});

			return;
		}
		else
		{
			bSucceed = UGroomAsset::CacheDerivedDatas();
		}
	}
#endif

	check(GetNumHairGroups() > 0);

	// Convert legacy version
	{
		bool bNeedSaving = false;

		// Convert material data to new format
		{
			bNeedSaving = bNeedSaving || ConvertMaterial(GetHairGroupsRendering(),	GetHairGroupsMaterials());
			bNeedSaving = bNeedSaving || ConvertMaterial(GetHairGroupsCards(),		GetHairGroupsMaterials());
			bNeedSaving = bNeedSaving || ConvertMaterial(GetHairGroupsMeshes(),		GetHairGroupsMaterials());
		}

		// Populate group name
		if (GUpdateGroomGroupsNames)
		{
			bNeedSaving = UpdateHairGroupsName(this);
		}

		if (bNeedSaving)
		{
			MarkPackageDirty();
		}
	}

	// * When running with the editor, InitResource is called in CacheDerivedDatas
	// * When running without the editor, InitResource is explicitely called here
#if !WITH_EDITOR
	if (!IsTemplate() && IsHairStrandsAssetLoadingEnable())
	{
		// Resize resources based on loaded data
		const uint32 GroupCount = GetHairGroupsPlatformData().Num();
		GetHairGroupsResources().Init(FHairGroupResources(), GroupCount);

		InitResources();
	}
#endif

	check(AreGroupsValid());

	UpdateHairGroupsInfo();
#if WITH_EDITORONLY_DATA
	UpdateCachedSettings();
	SyncHairGroupsName(this);
#endif // #if WITH_EDITORONLY_DATA

	if (IsResourcePSOPrecachingEnabled())
	{
		ERHIFeatureLevel::Type FeatureLevel = GetWorld() ? GetWorld()->GetFeatureLevel() : GMaxRHIFeatureLevel;
		EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

		TArray<FHairVertexFactoryTypesPerMaterialData> VFsPerMaterials = CollectVertexFactoryTypesPerMaterialData(ShaderPlatform);

		TArray<FMaterialPSOPrecacheRequestID> RequestIDs;
		FPSOPrecacheParams PrecachePSOParams;
		for (FHairVertexFactoryTypesPerMaterialData& VFsPerMaterial : VFsPerMaterials)
		{
			UMaterialInterface* MaterialInterface = GetHairGroupsMaterials()[VFsPerMaterial.MaterialIndex].Material;
			if (MaterialInterface)
			{
				MaterialInterface->PrecachePSOs(VFsPerMaterial.VertexFactoryDataList, PrecachePSOParams, EPSOPrecachePriority::Medium, RequestIDs);
			}
		}
	}
	// Update the physics system based on the enum for picking the right asset while packaging 
	UpdatePhysicsSystems();
}

void UGroomAsset::BeginDestroy()
{
	ReleaseResource();
	Super::BeginDestroy();
}

#if WITH_EDITOR

static bool IsCardsTextureResources(const FName PropertyName)
{
	return
		   PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupCardsTextures, Layout)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupCardsTextures, Textures);
}
static void InitCardsTextureResources(UGroomAsset* GroomAsset);

static bool IsStrandsInterpolationAttributes(const FName PropertyName)
{
	return
		   PropertyName == GET_MEMBER_NAME_CHECKED(FHairDecimationSettings, CurveDecimation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairDecimationSettings, VertexDecimation)


		// Add dependency on simulation, on deformer, and on per LOD-simulation/global-interpolation to strip-out interpolation data if there are not needed
		|| PropertyName == UGroomAsset::GetEnableGlobalInterpolationMemberName()
		|| PropertyName == UGroomAsset::GetEnableSimulationCacheMemberName()
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairSolverSettings, EnableSimulation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairSolverSettings, bEnableDeformation)

		// Decimation control needs to force a rebuild
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, CurveDecimation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, VertexDecimation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, AngularThreshold)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, ScreenSize)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, ThicknessScale)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, bVisible)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, Simulation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, GlobalInterpolation)

		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, GuideType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, HairToGuideDensity)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, RiggedGuideNumCurves)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, RiggedGuideNumPoints)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, InterpolationQuality)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, InterpolationDistance)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, bRandomizeGuide)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, bUseUniqueGuide);
}

static bool IsStrandsLODAttributes(const FName PropertyName)
{
	return
		// LOD count needs for cluster rebuilding
		PropertyName == UGroomAsset::GetHairGroupsLODMemberName()

		// LOD groups needs for cluster rebuilding
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsLOD, LODs)

		// LOD settings needs for cluster rebuilding
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, CurveDecimation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, VertexDecimation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, AngularThreshold)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, ScreenSize)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, ThicknessScale)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, bVisible)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, GeometryType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, BindingType)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, Simulation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, GlobalInterpolation);
}

static bool IsRiggingSettings(const FName PropertyName)
{
	return
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, GuideType) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, RiggedGuideNumCurves) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairInterpolationSettings, RiggedGuideNumPoints);
}

static bool IsCardsAttributes(const FName PropertyName)
{
	return PropertyName == UGroomAsset::GetHairGroupsCardsMemberName()
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, GuideType)
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, ImportedMesh)
//	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, Textures)
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, GroupIndex)
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, LODIndex);
}

static bool IsMeshesAttributes(const FName PropertyName)
{
	return PropertyName == UGroomAsset::GetHairGroupsMeshesMemberName()
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsMeshesSourceDescription, ImportedMesh)
//	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsMeshesSourceDescription, Textures)
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsMeshesSourceDescription, GroupIndex)
	|| PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsMeshesSourceDescription, LODIndex);
}

bool UGroomAsset::Modify(bool bAlwaysMarkDirty)
{
	// We are not really async compilable at the moment, but this will
	// cue the other compilers that depends on this asset (i.e. UGroomBindings)
	// to finish their async compilation before we start any modification.
	FAssetCompilingManager::Get().FinishCompilationForObjects({ this });

	return Super::Modify(bAlwaysMarkDirty);
}

void UGroomAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	const FName PropertyName = PropertyThatChanged ? PropertyThatChanged->GetFName() : NAME_None;
	const bool bGeometryTypeChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, GeometryType);
	if (bGeometryTypeChanged)
	{
		// If the asset didn't have any mesh or cards, we need to create/allocate the cards/mesh groups
		const uint32 GroupCount = GetHairGroupsPlatformData().Num();
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards) && HasGeometryType(EGroomGeometryType::Cards) && GetHairGroupsCards().Num() == 0)
		{
			GetHairGroupsCards().Init(FHairGroupsCardsSourceDescription(), GroupCount);
			FHairGroupsCardsSourceDescription Dirty;
			Dirty.ImportedMesh = nullptr;
			CachedHairGroupsCards.Init(Dirty, GroupCount);
		}

		if (IsHairStrandsEnabled(EHairStrandsShaderType::Meshes) && HasGeometryType(EGroomGeometryType::Meshes) && GetHairGroupsMeshes().Num() == 0)
		{
			GetHairGroupsMeshes().Init(FHairGroupsMeshesSourceDescription(), GroupCount);
			FHairGroupsMeshesSourceDescription Dirty;
			Dirty.ImportedMesh = nullptr;
			CachedHairGroupsMeshes.Init(Dirty, GroupCount);
		}
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FHairSolverSettings, NiagaraSolver))
	{
		UpdatePhysicsSystems();
	}

	// Rebuild the groom cached data if interpolation or LODs have changed
	const bool bNeedRebuildDerivedData = 
		IsStrandsInterpolationAttributes(PropertyName) || 
		IsStrandsLODAttributes(PropertyName) || 
		IsCardsAttributes(PropertyName) || 
		IsMeshesAttributes(PropertyName);
	if (bNeedRebuildDerivedData)
	{
		CacheDerivedDatas();
	}

	// Update cards/meshes texture array according to the layout prior to reload the UI
	const bool bCardsOrMeshesGroupChanged = 
		PropertyName == GetHairGroupsCardsMemberName() ||
		PropertyName == GetHairGroupsMeshesMemberName() ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupCardsTextures, Layout);
	if (bCardsOrMeshesGroupChanged)
	{
		for (auto& Group : GetHairGroupsCards())
		{
			Group.Textures.Textures.SetNum(GetHairTextureLayoutTextureCount(Group.Textures.Layout));
		}
		for (auto& Group : GetHairGroupsMeshes())
		{
			Group.Textures.Textures.SetNum(GetHairTextureLayoutTextureCount(Group.Textures.Layout));
		}
	}

	const bool bHairStrandsRaytracingRadiusChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FHairShadowSettings, HairRaytracingRadiusScale);

	// Fast path to only rebuild the proxies
	if (PropertyName == UGroomAsset::GetAutoLODBiasMemberName())
	{
		FGroomComponentRecreateRenderStateContext Context(this);
		return;
	}

	// By pass update if bStrandsInterpolationChanged has the resources have already been recreated
	if (!bNeedRebuildDerivedData)
	{
		FGroomComponentRecreateRenderStateContext Context(this);
		UpdateResource();
	}

	// Special path for reloading the cards texture if needed as texture are not part of the cards DDC key, so the build is not retrigger 
	// when a user change the cards texture asset from the texture panel
	const bool bCardsResourcesUpdate = IsCardsTextureResources(PropertyName);
	if (bCardsResourcesUpdate)
	{
		FGroomComponentRecreateRenderStateContext Context(this);
		InitCardsTextureResources(this);
	}

	// If rigging options changed, recreate skeletal mesh asset
	const bool bRiggingChanged = IsRiggingSettings(PropertyName);
	if (bRiggingChanged)
	{
		bool bUpdateRiggedMesh = false;
		for (const FHairGroupsInterpolation& Interpolation : GetHairGroupsInterpolation())
		{
			bUpdateRiggedMesh |= Interpolation.InterpolationSettings.GuideType == EGroomGuideType::Rigged;
		}
		if (bUpdateRiggedMesh)
		{
			SetRiggedSkeletalMesh(FGroomDeformerBuilder::CreateSkeletalMesh(this));
		}
	}

	const bool bCardMaterialChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, Material);
	const bool bMeshMaterialChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsMeshesSourceDescription, Material);

	if (bNeedRebuildDerivedData || bGeometryTypeChanged || bCardMaterialChanged || bMeshMaterialChanged)
	{
		// Delegate used for notifying groom data & groom resoures invalidation
		OnGroomAssetResourcesChanged.Broadcast();
	}
	else
	{
		// Delegate used for notifying groom data invalidation
		OnGroomAssetChanged.Broadcast();
	}
}

void UGroomAsset::MarkMaterialsHasChanged()
{
	OnGroomAssetResourcesChanged.Broadcast();
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UGroomAsset::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UGroomAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}

	// Dataflow specific tags
	UE::Dataflow::InstanceUtils::GetAssetRegistryTags(this, Context);

	Super::GetAssetRegistryTags(Context);
}

void UGroomAsset::PostInitProperties()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}

	Super::PostInitProperties();
}
#endif

void UGroomAsset::UpdatePhysicsSystems()
{
	FCookLoadScope UsedInGameScope(ECookLoadType::UsedInGame);
	for (auto& Group : GetHairGroupsPhysics())
	{
		if (Group.SolverSettings.NiagaraSolver == EGroomNiagaraSolvers::AngularSprings)
		{
			Group.SolverSettings.CustomSystem = LoadObject<UNiagaraSystem>(nullptr, TEXT("/HairStrands/Emitters/StableSpringsSystem.StableSpringsSystem"));
		}
		else if (Group.SolverSettings.NiagaraSolver == EGroomNiagaraSolvers::CosseratRods)
		{
			Group.SolverSettings.CustomSystem = LoadObject<UNiagaraSystem>(nullptr, TEXT("/HairStrands/Emitters/StableRodsSystem.StableRodsSystem"));
		}
	}
}

int32 UGroomAsset::GetNumHairGroups() const
{
	return GetHairGroupsPlatformData().Num();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Array serialize
void InternalSerializeMeshes(FArchive& Ar, UObject* Owner, TArray<FHairGroupPlatformData::FMeshes::FLOD>& LODs)
{
	uint32 Count = LODs.Num();
	Ar << Count;
	if (Ar.IsLoading())
	{
		LODs.SetNum(Count);
	}
	for (uint32 MeshIt = 0; MeshIt < Count; ++MeshIt)
	{
		InternalSerializeMesh(Ar, Owner, LODs[MeshIt]);
	}
}

void InternalSerializeCards(FArchive& Ar, UObject* Owner, TArray<FHairGroupPlatformData::FCards::FLOD>& LODs)
{
	uint32 Count = LODs.Num();
	Ar << Count;
	if (Ar.IsLoading())
	{
		LODs.SetNum(Count);
	}
	for (uint32 CardIt = 0; CardIt < Count; ++CardIt)
	{
		InternalSerializeCard(Ar, Owner, LODs[CardIt]);
	}
}

void InternalSerializePlatformDatas(FArchive& Ar, UObject* Owner, TArray<FHairGroupPlatformData>& GroupDatas)
{
	uint32 GroupCount = GroupDatas.Num();
	Ar << GroupCount;
	if (Ar.IsLoading())
	{
		GroupDatas.SetNum(GroupCount);
	}
	for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
	{
		InternalSerializePlatformData(Ar, Owner, GroupDatas[GroupIt]);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Element serialize
static void InternalSerializeCard(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FCards::FLOD& CardLODData)
{
	if (!Ar.IsCooking() || !CardLODData.bIsCookedOut)
	{
		CardLODData.BulkData.Serialize(Ar);
		CardLODData.InterpolationBulkData.Serialize(Ar);
		CardLODData.GuideBulkData.Serialize(Ar, Owner);
		CardLODData.GuideInterpolationBulkData.Serialize(Ar, Owner);
	}
	else
	{
		// LOD has been marked to be cooked out so serialize empty data
		FHairCardsBulkData NoCardsBulkData;
		FHairCardsInterpolationBulkData NoInterpolationBulkData;
		FHairStrandsBulkData NoGuideBulkData;
		FHairStrandsInterpolationBulkData NoGuideInteprolationBulkData;

		NoCardsBulkData.Serialize(Ar);
		NoInterpolationBulkData.Serialize(Ar);
		NoGuideBulkData.Serialize(Ar, Owner);
		NoGuideInteprolationBulkData.Serialize(Ar, Owner);
	}
}

static void InternalSerializeMesh(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FMeshes::FLOD& MeshLODData)
{
	if (!Ar.IsCooking() || !MeshLODData.bIsCookedOut)
	{
		MeshLODData.BulkData.Serialize(Ar);
	}
	else
	{
		// LOD has been marked to be cooked out so serialize empty data
		FHairMeshesBulkData NoMeshesBulkData;
		NoMeshesBulkData.Serialize(Ar);
	}
}

static void InternalSerializeGuide(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FGuides& GuideData)
{
	GuideData.BulkData.Serialize(Ar, Owner);
}

static void InternalSerializeStrand(FArchive& Ar, UObject* Owner, FHairGroupPlatformData::FStrands& StrandData, bool bHeader, bool bData, bool* bOutHasDataInCache)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	// When cooking data, force loading of *all* bulk data prior to saving them
	// Note: bFillBulkdata is true for filling in the bulkdata container prior to serialization. This also forces the resources loading 
	// from the 'start' (i.e., without offset)
	const bool bFillInBulkDataForCooking = Ar.IsCooking() && Ar.IsSaving();
	if (bFillInBulkDataForCooking)
	{
		{ FHairStreamingRequest R; R.Request(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, StrandData.BulkData,               true /*bWait*/, true /*bFillBulkdata*/, true /*bWarmCache*/, Owner->GetFName()); }
		{ FHairStreamingRequest R; R.Request(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, StrandData.InterpolationBulkData,  true /*bWait*/, true /*bFillBulkdata*/, true /*bWarmCache*/, Owner->GetFName()); }
		{ FHairStreamingRequest R; R.Request(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, StrandData.ClusterBulkData,        true /*bWait*/, true /*bFillBulkdata*/, true /*bWarmCache*/, Owner->GetFName()); }
	}

	if (!Ar.IsCooking() || !StrandData.bIsCookedOut)
	{
		if (bHeader)	{ StrandData.BulkData.SerializeHeader(Ar, Owner); }
		if (bData)		{ StrandData.BulkData.SerializeData(Ar, Owner); }
		if (bHeader)	{ StrandData.InterpolationBulkData.SerializeHeader(Ar, Owner); }
		if (bData)		{ StrandData.InterpolationBulkData.SerializeData(Ar, Owner); }
		if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::SerializeHairClusterCullingData)
		{
			if (bHeader){ StrandData.ClusterBulkData.SerializeHeader(Ar, Owner); }
			if (bData)	{ StrandData.ClusterBulkData.SerializeData(Ar, Owner); }
		}

		// Experimental curve rendering
		if (IsRenderCurveEnabled())
		{
			if (bHeader){ StrandData.CurveResourceData.Serialize(Ar, Owner); }
		}

		#if WITH_EDITORONLY_DATA
		// Pre-warm DDC cache
		const bool bPreWarmCache = Ar.IsLoading() && bHeader && !bData;
		if (bPreWarmCache)
		{
			bool bHasDataInCache = true;
			{ FHairStreamingRequest R; bHasDataInCache &= R.WarmCache(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, StrandData.BulkData); }
			{ FHairStreamingRequest R; bHasDataInCache &= R.WarmCache(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, StrandData.InterpolationBulkData); }
			{ FHairStreamingRequest R; bHasDataInCache &= R.WarmCache(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, StrandData.ClusterBulkData); }

			if (bOutHasDataInCache) { *bOutHasDataInCache = bHasDataInCache; }
		}
		#endif
	}
	else
	{
		// Fall back to no data, but still serialize guide data as they an be used for cards simulation
		// Theoritically, we should have something to detect if we are going to use or not guide (for 
		// simulation or RBF deformation) on the target platform
		FHairGroupPlatformData::FStrands NoStrandsData;
		if (bHeader) 	{ NoStrandsData.BulkData.SerializeHeader(Ar, Owner); }
		if (bData)		{ NoStrandsData.BulkData.SerializeData(Ar, Owner); }
		if (bHeader) 	{ NoStrandsData.InterpolationBulkData.SerializeHeader(Ar, Owner); }
		if (bData) 		{ NoStrandsData.InterpolationBulkData.SerializeData(Ar, Owner); }
		if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::SerializeHairClusterCullingData)
		{
			if (bHeader){ NoStrandsData.ClusterBulkData.SerializeHeader(Ar, Owner); }
			if (bData)	{ NoStrandsData.ClusterBulkData.SerializeData(Ar, Owner); }
		}
	}
}

static void InternalSerializePlatformData(FArchive& Ar, UObject* Owner, FHairGroupPlatformData& GroupData)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	InternalSerializeGuide(Ar, Owner, GroupData.Guides);
	InternalSerializeStrand(Ar, Owner, GroupData.Strands, true, true);

	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::SerializeGroomCardsAndMeshes)
	{
		bool bIsCooked = Ar.IsCooking();
		Ar << bIsCooked;

		if (bIsCooked)
		{
			InternalSerializeCards(Ar, Owner, GroupData.Cards.LODs);
			InternalSerializeMeshes(Ar, Owner, GroupData.Meshes.LODs);
		}
	}
}

void UGroomAsset::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UGroomAsset::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void UGroomAsset::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UGroomAsset::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

bool UGroomAsset::CanRebuildFromDescription() const
{
#if WITH_EDITORONLY_DATA
	return HairDescriptionBulkData[GetHairDescriptionIndex()].IsValid() && !HairDescriptionBulkData[GetHairDescriptionIndex()]->IsEmpty();
#else
	return false;
#endif
}

bool UGroomAsset::AreGroupsValid() const
{
	const uint32 GroupCount = GetHairGroupsInterpolation().Num();
	return
		GroupCount > 0 &&
		GetHairGroupsPlatformData().Num() == GroupCount &&
		GetHairGroupsPhysics().Num() == GroupCount &&
		GetHairGroupsRendering().Num() == GroupCount &&
		GetHairGroupsLOD().Num() == GroupCount;
}

void UGroomAsset::SetNumGroup(uint32 InGroupCount, bool bResetGroupData, bool bResetOtherData)
{
	ReleaseResource();
	if (bResetGroupData)
	{
		GetHairGroupsPlatformData().Reset();
	}

	// In order to preserve the existing asset settings, the settings are only reset if the group count has changed.
	if (InGroupCount != GetHairGroupsPlatformData().Num())
	{
		GetHairGroupsPlatformData().Init(FHairGroupPlatformData(), InGroupCount);
		GetHairGroupsResources().Init(FHairGroupResources(), InGroupCount);
	}

	if (InGroupCount != GetHairGroupsPhysics().Num())
	{
		if (bResetOtherData)
		{
			GetHairGroupsPhysics().Init(FHairGroupsPhysics(), InGroupCount);
		}
		else
		{
			GetHairGroupsPhysics().SetNum(InGroupCount);
		}
	}

	if (InGroupCount != GetHairGroupsRendering().Num())
	{
		if (bResetOtherData)
		{
			GetHairGroupsRendering().Init(FHairGroupsRendering(), InGroupCount);
		}
		else
		{
			GetHairGroupsRendering().SetNum(InGroupCount);
		}
	}

	if (InGroupCount != GetHairGroupsInterpolation().Num())
	{
		if (bResetOtherData)
		{
			GetHairGroupsInterpolation().Init(FHairGroupsInterpolation(), InGroupCount);
		}
		else
		{
			GetHairGroupsInterpolation().SetNum(InGroupCount);
		}
	}

	if (InGroupCount != GetHairGroupsLOD().Num())
	{
		if (bResetOtherData)
		{
			GetHairGroupsLOD().Init(FHairGroupsLOD(), InGroupCount);
		}
		else
		{
			GetHairGroupsLOD().SetNum(InGroupCount);
		}

		// Insure that each group has at least one LOD
		for (FHairGroupsLOD& GroupLOD : GetHairGroupsLOD())
		{
			if (GroupLOD.LODs.IsEmpty())
			{
				GroupLOD = FHairGroupsLOD::GetDefault();
			}
		}
	}

	if (InGroupCount != GetHairGroupsInfo().Num())
	{
		if (bResetOtherData)
		{
			GetHairGroupsInfo().Init(FHairGroupInfoWithVisibility(), InGroupCount);
		}
		else
		{
			GetHairGroupsInfo().SetNum(InGroupCount);
		}
	}

	// The following groups remain unchanged:
	// * HairGroupsCards
	// * GetHairGroupsMeshes()
	// * GetHairGroupsMaterials()

	GetEffectiveLODBias().SetNum(InGroupCount);

#if WITH_EDITORONLY_DATA
	StrandsDerivedDataKey.SetNum(InGroupCount);
	CardsDerivedDataKey.SetNum(InGroupCount);
	MeshesDerivedDataKey.SetNum(InGroupCount);
#endif
}

void UGroomAsset::ClearNumGroup(uint32 InGroupCount)
{
	ReleaseResource();

	const FHairGroupsLOD Default = FHairGroupsLOD::GetDefault();

	GetHairGroupsPlatformData().Init(FHairGroupPlatformData(), InGroupCount);
	GetHairGroupsResources().Init(FHairGroupResources(), InGroupCount);
	GetHairGroupsPhysics().Init(FHairGroupsPhysics(), InGroupCount);
	GetHairGroupsRendering().Init(FHairGroupsRendering(), InGroupCount);
	GetHairGroupsInterpolation().Init(FHairGroupsInterpolation(), InGroupCount);	
	GetHairGroupsLOD().Init(Default, InGroupCount);
	GetHairGroupsInfo().Init(FHairGroupInfoWithVisibility(), InGroupCount);
	GetEffectiveLODBias().Init(0, InGroupCount);
	// The following groups remain unchanged:
	// * HairGroupsCards
	// * GetHairGroupsMeshes()
	// * GetHairGroupsMaterials()
	#if WITH_EDITORONLY_DATA
	StrandsDerivedDataKey.Init(FString(), InGroupCount);
	CardsDerivedDataKey.Init(FString(), InGroupCount);
	MeshesDerivedDataKey.Init(FString(), InGroupCount);
	#endif
}

void UGroomAsset::SetStableRasterization(bool bEnable)
{
	for (FHairGroupsRendering& Group : GetHairGroupsRendering())
	{
		Group.AdvancedSettings.bUseStableRasterization = bEnable;
	}
}

void UGroomAsset::SetScatterSceneLighting(bool bEnable)
{
	for (FHairGroupsRendering& Group : GetHairGroupsRendering())
	{
		Group.AdvancedSettings.bScatterSceneLighting = bEnable;
	}
}

void UGroomAsset::SetHairWidth(float Width)
{
	for (FHairGroupsRendering& Group : GetHairGroupsRendering())
	{
		Group.GeometrySettings.HairWidth = Width;
	}
}

// If groom derived data needs to be rebuilt (new format, serialization
// differences, etc.) replace the version GUID below with a new one.
// In case of merge conflicts with DDC versions, you MUST generate a new GUID
// and set this new GUID as the version.
// DDC Guid needs to be bumped in:
// * Main    : Engine\Source\Runtime\Core\Private\UObject\DevObjectVersion.cpp
// * Release : Engine\Source\Runtime\Core\Private\UObject\UE5ReleaseStreamObjectVersion.cpp
// * ...

#if WITH_EDITORONLY_DATA

namespace GroomDerivedDataCacheUtils
{
	const FString& GetGroomDerivedDataVersion()
	{
		static FString CachedVersionString = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().GROOM_DERIVED_DATA_VERSION).ToString();
		return CachedVersionString;
	}

	FString BuildGroomDerivedDataKey(const FString& KeySuffix)
	{
		return FDerivedDataCacheInterface::BuildCacheKey(*(TEXT("GROOM_V") + FGroomBuilder::GetVersion() + TEXT("_")), *GetGroomDerivedDataVersion(), *KeySuffix);
	}

	void SerializeHairSettingsForDDC(FArchive& Ar, uint32 GroupIndex, FHairGroupsInterpolation& InterpolationSettings, FHairGroupsLOD& LODSettings, FHairGroupsPhysics& PhysicsSettings, bool bRequireInterpolationData)
	{
		bool bSupportCompressedPosition = DoesHairStrandsSupportCompressedPosition();
		bool bSupportTriangleStrip = GetHairStrandsUsesTriangleStrips();

		// Note: this serializer is only used to build the groom DDC key, no versioning is required
		Ar << GroupIndex;
		Ar << bRequireInterpolationData;
		Ar << bSupportCompressedPosition;
		Ar << bSupportTriangleStrip;

		InterpolationSettings.BuildDDCKey(Ar);
		LODSettings.BuildDDCKey(Ar);
		PhysicsSettings.BuildDDCKey(Ar);
	}

	// Geneate DDC key for an entire hair group (i.e. with all its LODs) for cards data
	FString BuildCardsDerivedDataKeySuffix(uint32 GroupIndex, const TArray<FHairLODSettings>& LODs, TArray<FHairGroupsCardsSourceDescription>& SourceDescriptions)
	{
		// Serialize the FHairGroupsCardsSourceDescription into a temporary array
		// The archive is flagged as persistent so that machines of different endianness produce identical binary results.
		TArray<uint8> TempBytes;
		TempBytes.Reserve(512);
		FMemoryWriter Ar(TempBytes, true);

		Ar << GroupIndex;
		for (int32 Index = 0; Index < SourceDescriptions.Num(); ++Index)
		{
			FHairGroupsCardsSourceDescription& Desc = SourceDescriptions[Index];
			if (Desc.GroupIndex != GroupIndex || Desc.LODIndex < 0)
			{
				continue;
			}

			// Also need to cross-check the LOD settings; there might not be a LOD settings that matches the LODIndex
			// in which case, no cards data is actually built for the LODIndex
			if (!LODs.IsValidIndex(Desc.LODIndex))
			{
				continue;
			}

			Ar << Desc.LODIndex;
			if (Desc.ImportedMesh)
			{
				FString Key = Desc.GetMeshKey();
				Ar << Key;
			}
			Ar << Desc.GuideType;
			// Material is not included as it doesn't affect the data building
		}

		FSHAHash Hash;
		FSHA1::HashBuffer(TempBytes.GetData(), TempBytes.Num(), Hash.Hash);

		static FString CardPrefixString(TEXT("CARDS_V") + FHairCardsBuilder::GetVersion() + TEXT("_"));
		return CardPrefixString + Hash.ToString();
	}

	FString BuildMeshesDerivedDataKeySuffix(uint32 GroupIndex, const TArray<FHairLODSettings>& LODs, TArray<FHairGroupsMeshesSourceDescription>& SourceDescriptions)
	{
		// Serialize the FHairGroupsMeshesSourceDescription into a temporary array
		// The archive is flagged as persistent so that machines of different endianness produce identical binary results.
		TArray<uint8> TempBytes;
		TempBytes.Reserve(512);
		FMemoryWriter Ar(TempBytes, true);

		Ar << GroupIndex;
		for (int32 Index = 0; Index < SourceDescriptions.Num(); ++Index)
		{
			FHairGroupsMeshesSourceDescription& Desc = SourceDescriptions[Index];
			if (Desc.GroupIndex != GroupIndex || Desc.LODIndex < 0)
			{
				continue;
			}

			// Also need to cross-check the LOD settings; there might not be a LOD settings that matches the LODIndex
			// in which case, no mesh data is actually built for the LODIndex
			if (!LODs.IsValidIndex(Desc.LODIndex) || LODs[Desc.LODIndex].GeometryType != EGroomGeometryType::Meshes)
			{
				continue;
			}

			Ar << Desc.LODIndex;
			if (Desc.ImportedMesh)
			{
				FString Key = Desc.GetMeshKey();
				Ar << Key;
			}
			// Material is not included as it doesn't affect the data building
		}

		FSHAHash Hash;
		FSHA1::HashBuffer(TempBytes.GetData(), TempBytes.Num(), Hash.Hash);

		static FString MeshPrefixString(TEXT("MESHES_V") + FHairMeshesBuilder::GetVersion() + TEXT("_"));
		return MeshPrefixString + Hash.ToString();
	}

	FString BuildStrandsDerivedDataKeySuffix(uint32 GroupIndex, const FHairGroupsInterpolation& InterpolationSettings, const FHairGroupsLOD& LODSettings, const FHairGroupsPhysics& PhysicsSettings, bool bNeedInterpolationData, const FHairDescriptionBulkData* HairBulkDescription)
	{
		// Serialize the build settings into a temporary array
		// The archive is flagged as persistent so that machines of different endianness produce identical binary results.
		TArray<uint8> TempBytes;
		TempBytes.Reserve(64);
		FMemoryWriter Ar(TempBytes, /*bIsPersistent=*/ true);

		GroomDerivedDataCacheUtils::SerializeHairSettingsForDDC(Ar, GroupIndex, const_cast<FHairGroupsInterpolation&>(InterpolationSettings), const_cast<FHairGroupsLOD&>(LODSettings), const_cast<FHairGroupsPhysics&>(PhysicsSettings), bNeedInterpolationData);

		FString KeySuffix;
		if (HairBulkDescription)
		{
			// Reserve twice the size of TempBytes because of ByteToHex below + 3 for "ID" and \0
			KeySuffix.Reserve(HairBulkDescription->GetIdString().Len() + TempBytes.Num() * 2 + 3);
			KeySuffix += TEXT("ID");
			KeySuffix += HairBulkDescription->GetIdString();
		}
		else
		{
			KeySuffix.Reserve(TempBytes.Num() * 2 + 1);
		}

		// Now convert the raw bytes to a string
		const uint8* SettingsAsBytes = TempBytes.GetData();
		for (int32 ByteIndex = 0; ByteIndex < TempBytes.Num(); ++ByteIndex)
		{
			ByteToHex(SettingsAsBytes[ByteIndex], KeySuffix);
		}

		return KeySuffix;
	}
}

bool UGroomAsset::IsFullyCached()
{
	// Check if all the groom derived data for strands, cards and meshes are already stored in the DDC
	TArray<FString> CacheKeys;
	const uint32 GroupCount = GetHairGroupsInterpolation().Num();
	for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
	{
		const FString StrandsDDCKey = UGroomAsset::GetDerivedDataKeyForStrands(GroupIndex);
		CacheKeys.Add(StrandsDDCKey);

		// Some cards and meshes LOD settings may not produce any derived data so those must be excluded
		bool bHasCardsLODs = false;
		bool bHasMeshesLODs = false;
		const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
		for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			// GetSourceDescription will cross-check the LOD settings with the cards/meshes settings to see if they would produce any data
			if (const FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt))
			{
				Desc->HasMeshChanged(); // this query will trigger a load of the mesh dependency, which has to be done in the game thread
				bHasCardsLODs = true;
			}
			if (const FHairGroupsMeshesSourceDescription* Desc = GetSourceDescription(GetHairGroupsMeshes(), GroupIndex, LODIt))
			{
				Desc->HasMeshChanged(); // this query will trigger a load of the mesh dependency, which has to be done in the game thread
				bHasMeshesLODs = true;
			}
		}

		if (bHasCardsLODs)
		{
			const FString CardsKeySuffix = GroomDerivedDataCacheUtils::BuildCardsDerivedDataKeySuffix(GroupIndex, GetHairGroupsLOD()[GroupIndex].LODs, GetHairGroupsCards());
			const FString CardsDDCKey = StrandsDDCKey + CardsKeySuffix;
			CacheKeys.Add(CardsDDCKey);
		}

		if (bHasMeshesLODs)
		{
			const FString MeshesKeySuffix = GroomDerivedDataCacheUtils::BuildMeshesDerivedDataKeySuffix(GroupIndex, GetHairGroupsLOD()[GroupIndex].LODs, GetHairGroupsMeshes());
			const FString MeshesDDCKey = GroomDerivedDataCacheUtils::BuildGroomDerivedDataKey(MeshesKeySuffix);
			CacheKeys.Add(MeshesDDCKey);
		}
	}

	return GetDerivedDataCacheRef().AllCachedDataProbablyExists(CacheKeys);
}

FString UGroomAsset::GetDerivedDataKey(bool bUseStrandCachedKey)
{
	FString OutKey;
	for (uint32 GroupIt = 0, GroupCount = GetNumHairGroups(); GroupIt < GroupCount; ++GroupIt)
	{		
		// Optionnaly use the cached strands key. This is to workaround an issue with legacy HairDescription which uses FEditorBulkData, 
		// and whose StringId is altered when loaded (only for legacy hair description converted to FEditorBulkData). This StringId modification
		// changes the DDC key of a groom asset depending if has been loaded or not, causing DDC miss
		const bool bComputeStrandsKey = !bUseStrandCachedKey || StrandsDerivedDataKey[GroupIt].IsEmpty();
		const FString StrandsKey = bComputeStrandsKey ? GetDerivedDataKeyForStrands(GroupIt) : StrandsDerivedDataKey[GroupIt];

		// Compute group key
		OutKey += StrandsKey;
		OutKey += GetDerivedDataKeyForCards(GroupIt, StrandsKey);
		OutKey += GetDerivedDataKeyForMeshes(GroupIt);
	}
	return OutKey;
}

FString UGroomAsset::GetDerivedDataKeyForCards(uint32 GroupIndex, const FString& StrandsKey)
{
	const FString KeySuffix = GroomDerivedDataCacheUtils::BuildCardsDerivedDataKeySuffix(GroupIndex, GetHairGroupsLOD()[GroupIndex].LODs, GetHairGroupsCards());
	const FString DerivedDataKey = StrandsKey + KeySuffix;
	return DerivedDataKey;
}

FString UGroomAsset::GetDerivedDataKeyForMeshes(uint32 GroupIndex)
{
	const FString KeySuffix = GroomDerivedDataCacheUtils::BuildMeshesDerivedDataKeySuffix(GroupIndex, GetHairGroupsLOD()[GroupIndex].LODs, GetHairGroupsMeshes());
	const FString DerivedDataKey = GroomDerivedDataCacheUtils::BuildGroomDerivedDataKey(KeySuffix);
	return DerivedDataKey;
}

FString UGroomAsset::GetDerivedDataKeyForStrands(uint32 GroupIndex)
{
	const FHairGroupsInterpolation& InterpolationSettings = GetHairGroupsInterpolation()[GroupIndex];
	const FHairGroupsLOD& LODSettings = GetHairGroupsLOD()[GroupIndex];
	const FHairGroupsPhysics& PhysicsSettings = GetHairGroupsPhysics()[GroupIndex];

	// If simulation or global interpolation is enabled, then interpolation data are required. Otherwise they can be skipped
	const bool bNeedInterpolationData = NeedsInterpolationData(GroupIndex);
	const FString KeySuffix = GroomDerivedDataCacheUtils::BuildStrandsDerivedDataKeySuffix(GroupIndex, InterpolationSettings, LODSettings, PhysicsSettings, bNeedInterpolationData, HairDescriptionBulkData[GetHairDescriptionIndex()].Get());
	const FString DerivedDataKey = GroomDerivedDataCacheUtils::BuildGroomDerivedDataKey(KeySuffix);

	return DerivedDataKey;
}

void UGroomAsset::CommitHairDescription(FHairDescription&& InHairDescription, EHairDescriptionType Type)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomAsset::CommitHairDescription);

	SetHairDescriptionType(Type);

	UE::TUniqueLock LockScope(InternalLock);

	CachedHairDescription[GetHairDescriptionIndex()] = MakeUnique<FHairDescription>(InHairDescription);

	if (!HairDescriptionBulkData[GetHairDescriptionIndex()])
	{
		HairDescriptionBulkData[GetHairDescriptionIndex()] = MakeUnique<FHairDescriptionBulkData>();
	}

	// Update the cached hair description groups with the new hair description data
	CachedHairDescriptionGroups[GetHairDescriptionIndex()] = MakeUnique<FHairDescriptionGroups>();
	FGroomBuilder::BuildHairDescriptionGroups(*CachedHairDescription[GetHairDescriptionIndex()], *CachedHairDescriptionGroups[GetHairDescriptionIndex()]);

	HairDescriptionBulkData[GetHairDescriptionIndex()]->SaveHairDescription(*CachedHairDescription[GetHairDescriptionIndex()]);
}

const FHairDescriptionGroups& UGroomAsset::GetHairDescriptionGroups() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomAsset::GetHairDescriptionGroups);

	check(HairDescriptionBulkData[GetHairDescriptionIndex()]);
	
	{
		UE::TUniqueLock LockScope(InternalLock);

		if (!CachedHairDescription[GetHairDescriptionIndex()])
		{
			TUniquePtr<FHairDescription> HairDescription = MakeUnique<FHairDescription>();
			HairDescriptionBulkData[GetHairDescriptionIndex()]->LoadHairDescription(*HairDescription);

			// Since we don't take read-lock, we need to make sure all memory is visible inside newly created object before publishing it.
			FPlatformMisc::MemoryBarrier();

			CachedHairDescription[GetHairDescriptionIndex()] = MoveTemp(HairDescription);
		}

		if (!CachedHairDescriptionGroups[GetHairDescriptionIndex()])
		{
			TUniquePtr<FHairDescriptionGroups> HairDescriptionGroups = MakeUnique<FHairDescriptionGroups>();
			FGroomBuilder::BuildHairDescriptionGroups(*CachedHairDescription[GetHairDescriptionIndex()], *HairDescriptionGroups, true /*bAllowAddEndControlPoint*/);
			
			// Since we don't take read-lock, we need to make sure all memory is visible inside newly created object before publishing it.
			FPlatformMisc::MemoryBarrier();

			CachedHairDescriptionGroups[GetHairDescriptionIndex()] = MoveTemp(HairDescriptionGroups);
		}
	}

	return *CachedHairDescriptionGroups[GetHairDescriptionIndex()];
}

FHairDescription UGroomAsset::LoadHairDescription(const EHairDescriptionType DescriptionType) const
{
	const uint8 DescriptionIndex = FMath::Clamp(static_cast<uint8>(DescriptionType), 0, 1);
	
	FHairDescription OutHairDescription;
	if (HairDescriptionBulkData[DescriptionIndex])
	{
		HairDescriptionBulkData[DescriptionIndex]->LoadHairDescription(OutHairDescription);
	}
	return MoveTemp(OutHairDescription);
}

FHairDescription UGroomAsset::GetHairDescription() const
{
	return LoadHairDescription(GetHairDescriptionType());
}

bool UGroomAsset::CacheDerivedDatas()
{
	bRetryLoadFromGameThread = false;

	// Delete existing resources from GroomComponent and recreate them when the FGroomComponentRecreateRenderStateContext's destructor is called
	// These resources are recreated only when running from the game thread (non-async loading path)
	const bool bIsGameThread = IsInGameThread();
	FGroomComponentRecreateRenderStateContext RecreateContext(bIsGameThread ? this : nullptr);

	// (Temporary) Workaround when reloading groom asset several time, which can cause resources to be still alive while the data are being updated 
	// by CacheDerivedData. The BulkData referenced by the resources becomes no longer valid, causing use-after-free crash.
	const bool bReleaseResourceBeforeCacheDerivedData = bIsGameThread && GHairStrandsReleaseResourceBeforeBuildingData > 0;
	if (bReleaseResourceBeforeCacheDerivedData)
	{
		const bool bFlushCommands = bIsInitialized;
		ReleaseResource();
		if (bFlushCommands)
		{
			FlushRenderingCommands();
		}
	}

	const uint32 GroupCount = GetHairGroupsInterpolation().Num();
	bool bSucceed = true;
	for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
	{
		bSucceed = bSucceed && CacheDerivedData(GroupIndex);
	}

	#if WITH_EDITOR
	if (UsesNaniteGroom())
	{
		// Cache derived data for the running platform.
		ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
		ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
		if (DoesTargetPlatformSupportNanite(RunningPlatform))
		{
			SetRenderData(MakeUnique<FGroomAssetRenderData>());
			GetRenderData()->Cache(RunningPlatform, this/*, LODSettings*/);
		}
	}
	#endif

	if (bSucceed)
	{
		UpdateCachedSettings();
		UpdateHairGroupsInfo();
	
		if (bIsGameThread)
		{
			if (!bReleaseResourceBeforeCacheDerivedData)
			{
				ReleaseResource();
			}
			InitResources();
		}
	}
	return bSucceed;
}

bool UGroomAsset::CacheDerivedData(uint32 GroupIndex)
{
	// Check if the asset correctly initialized prior building
	if (!AreGroupsValid())
	{
		return false;
	}

	const uint32 GroupCount = GetHairGroupsInterpolation().Num();
	check(GroupIndex < GroupCount);
	if (!GetHairGroupsInterpolation().IsValidIndex(GroupIndex))
	{
		return false;
	}

	FString DerivedDataKey;
	bool bSuccess = false;

	// Cache each geometric representation separately
	bSuccess |= CacheStrandsData(GroupIndex, DerivedDataKey);
	bSuccess |= CacheCardsData(GroupIndex, DerivedDataKey);
	bSuccess |= CacheMeshesData(GroupIndex);

	return bSuccess;
}

bool UGroomAsset::CacheStrandsData(uint32 GroupIndex, FString& OutDerivedDataKey)
{
	if (!HairDescriptionBulkData[GetHairDescriptionIndex()] || HairDescriptionBulkData[GetHairDescriptionIndex()]->IsEmpty())
	{
		// Groom having only cards/meshes (i.e., not containing imported strands) don't have hair description
		bool bHasStrands = false;
		for (const FHairLODSettings& LOD : GetHairGroupsLOD()[GroupIndex].LODs)
		{
			if (LOD.GeometryType == EGroomGeometryType::Strands)
			{
				bHasStrands = true;
				break;
			}
		}

		if (bHasStrands)
		{
			UE_LOGF(LogHairStrands, Error, "[Groom] The groom asset %ls is too old. Please reimported the groom from its original source file.", *GetName());
		}
		return false;
	}

	using namespace UE;
	using namespace UE::DerivedData;

	const FString DerivedDataKey= UGroomAsset::GetDerivedDataKeyForStrands(GroupIndex);
	const FCacheKey HeaderKey = ConvertLegacyCacheKey(DerivedDataKey + FString(TEXT("_Header")));
	const FSharedString Name = MakeStringView(GetPathName());
	FSharedBuffer Data;
	{
		FRequestOwner Owner(EPriority::Blocking);
		GetCache().GetValue({ {Name, HeaderKey} }, Owner, [&Data](FCacheGetValueResponse&& Response)
		{
			Data = Response.Value.GetData().Decompress();
		});
		Owner.Wait();
	}

	// Populate key/name for streaming data request
	auto FillDrivedDataKey = [&DerivedDataKey, &Name](FHairGroupPlatformData& In)
	{
		In.Strands.BulkData.DerivedDataKey = DerivedDataKey + FString(TEXT("_RestData"));
		In.Strands.InterpolationBulkData.DerivedDataKey = DerivedDataKey + FString(TEXT("_InterpolationData"));
		In.Strands.ClusterBulkData.DerivedDataKey = DerivedDataKey + FString(TEXT("_ClusterData"));
	};

	FHairGroupPlatformData& PlatformData = GetHairGroupsPlatformData()[GroupIndex];

	bool bSuccess = false;
	if (Data)
	{
		UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[Groom/DDC] Strands - Found (Groom:%ls Group6:%d).", *GetName(), GroupIndex);

		// Reset hair group data to ensure previously loaded bulk data are cleared/cleaned prior to load new data.
		PlatformData = FHairGroupPlatformData();
		FillDrivedDataKey(PlatformData);

		FMemoryReaderView Ar(Data, /*bIsPersistent*/ true);

		InternalSerializeGuide(Ar, this, PlatformData.Guides);
		InternalSerializeStrand(Ar, this, PlatformData.Strands, true/*Header*/, false/*Data*/, &bSuccess);
	}
	if (!bSuccess)
	{
		if (!IsInGameThread())
		{
			// Strands build might actually be thread safe, but retry on the game thread to be safe
			bRetryLoadFromGameThread = true;
			return false;
		}

		UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[Groom/DDC] Strands - Not found (Groom:%ls Group:%d).", *GetName(), GroupIndex);

		const FHairDescriptionGroups& LocalHairDescriptionGroups = GetHairDescriptionGroups();

		if (!LocalHairDescriptionGroups.IsValid())
		{
			UE_LOGF(LogHairStrands, Warning, "[Groom] The groom asset %ls does not have valid hair groups.", *GetName());
			return false;
		}

		// Build groom data with the new build settings
		bSuccess = BuildHairGroup_Strands(
			GroupIndex,
			NeedsInterpolationData(GroupIndex),
			LocalHairDescriptionGroups,
			GetHairGroupsInterpolation(),
			GetHairGroupsLOD(),
			GetHairGroupsInfo(),
			GetHairGroupsPlatformData(), 
			IsDeformationEnable(GroupIndex));
		
		if (bSuccess)
		{
			FillDrivedDataKey(PlatformData);

			// Header
			{
				TArray<uint8> WriteData;
				FMemoryWriter Ar(WriteData, /*bIsPersistent*/ true);
				InternalSerializeGuide(Ar, this, PlatformData.Guides);
				InternalSerializeStrand(Ar, this, PlatformData.Strands, true/*Header*/, false/*Data*/);

				FRequestOwner AsyncOwner(EPriority::Normal);
				GetCache().PutValue({ {Name, HeaderKey, FValue::Compress(MakeSharedBufferFromArray(MoveTemp(WriteData)))} }, AsyncOwner);
				AsyncOwner.KeepAlive();
			}

			// Data (Rest)
			{
				TArray<FCachePutValueRequest> Out;
				PlatformData.Strands.BulkData.Write_DDC(this, Out);

				FRequestOwner AsyncOwner(EPriority::Normal);
				GetCache().PutValue(Out, AsyncOwner);
				AsyncOwner.KeepAlive();
			}

			// Data (Interpolation)
			if (PlatformData.Strands.InterpolationBulkData.GetResourceCount() > 0)
			{
				TArray<FCachePutValueRequest> Out;
				PlatformData.Strands.InterpolationBulkData.Write_DDC(this, Out);

				FRequestOwner AsyncOwner(EPriority::Normal);
				GetCache().PutValue(Out, AsyncOwner);
				AsyncOwner.KeepAlive();
			}

			// Data (cluster Culling)
			{
				TArray<FCachePutValueRequest> Out;
				PlatformData.Strands.ClusterBulkData.Write_DDC(this, Out);

				FRequestOwner AsyncOwner(EPriority::Normal);
				GetCache().PutValue(Out, AsyncOwner);
				AsyncOwner.KeepAlive();
			}
		}
	}

	OutDerivedDataKey = DerivedDataKey;
	StrandsDerivedDataKey[GroupIndex] = DerivedDataKey;

	return bSuccess;
}

bool UGroomAsset::HasValidData_Cards(uint32 GroupIndex) const
{
	const FHairGroupsLOD& GroupsLOD = GetHairGroupsLOD()[GroupIndex];
	for (int32 LODIt = 0; LODIt < GroupsLOD.LODs.Num(); ++LODIt)
	{
		// Note: We don't condition/filter resource allocation based on GeometryType == EGroomGeometryType::Cards, because 
		// geometry type can be switched at runtime (between cards/strands)
		{
			if (const FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt))
			{
				if (Desc->GetMesh())
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool UGroomAsset::HasValidData_Meshes(uint32 GroupIndex) const
{
	const FHairGroupsLOD& GroupsLOD = GetHairGroupsLOD()[GroupIndex];
	for (int32 LODIt = 0; LODIt < GroupsLOD.LODs.Num(); ++LODIt)
	{
		if (GroupsLOD.LODs[LODIt].GeometryType == EGroomGeometryType::Meshes)
		{
			if (const FHairGroupsMeshesSourceDescription* Desc = GetSourceDescription(GetHairGroupsMeshes(), GroupIndex, LODIt))
			{
				if (Desc->ImportedMesh)
				{
					return true;
				}
			}
		}
	}
	return false;
}

// Return true if the meshes data have changed
bool UGroomAsset::CacheCardsData(uint32 GroupIndex, const FString& StrandsKey)
{
	// We should never query the DDC for something that we won't be able to build. So check that first.
	// The goal is to avoid paying the price of a guaranteed failure on high-latency DDC backends every run.
	const bool bHasValidSourceData = HasValidData_Cards(GroupIndex);
	if (!bHasValidSourceData)
	{
		const bool bNeedReset = !CardsDerivedDataKey[GroupIndex].IsEmpty();
		CardsDerivedDataKey[GroupIndex].Empty();

		// Initialized LODs
		const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
		GetHairGroupsPlatformData()[GroupIndex].Cards.LODs.Empty(); // Force reset data
		GetHairGroupsPlatformData()[GroupIndex].Cards.LODs.SetNum(LODCount);

		return bNeedReset;
	}

	const FString KeySuffix = GroomDerivedDataCacheUtils::BuildCardsDerivedDataKeySuffix(GroupIndex, GetHairGroupsLOD()[GroupIndex].LODs, GetHairGroupsCards());
	const FString DerivedDataKey = StrandsKey + KeySuffix;

	// Query DDC
	using namespace UE;
	using namespace UE::DerivedData;
	const FCacheKey Key = ConvertLegacyCacheKey(DerivedDataKey);
	const FSharedString Name = MakeStringView(GetPathName());
	FSharedBuffer Data;
	{
		FRequestOwner Owner(EPriority::Blocking);
		GetCache().GetValue({ {Name, Key} }, Owner, [&Data](FCacheGetValueResponse&& Response)
		{
			Data = Response.Value.GetData().Decompress();
		});
		Owner.Wait();
	}

	// * If data is available, populate LOD data
	// * Otherwise build the LOD data
	FHairGroupPlatformData& HairGroupData = GetHairGroupsPlatformData()[GroupIndex];
	if (Data)
	{
		UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[Groom/DDC] Cards - Found (Groom:%ls Group:%d).", *GetName(), GroupIndex);
		
		FMemoryReaderView Ar(Data, /*bIsPersistent*/ true);
		InternalSerializeCards(Ar, this, HairGroupData.Cards.LODs);
	}
	else
	{
		UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[Groom/DDC] Cards - Not found (Groom:%ls Group:%d).", *GetName(), GroupIndex);

		const bool bSucceed = BuildHairGroup_Cards(GroupIndex);
		if (!bSucceed)
		{
			CardsDerivedDataKey[GroupIndex].Empty();
			return false;
		}

		TArray<uint8> WriteData;
		FMemoryWriter Ar(WriteData, /*bIsPersistent*/ true);
		InternalSerializeCards(Ar, this, HairGroupData.Cards.LODs);
		FRequestOwner AsyncOwner(EPriority::Normal);
		GetCache().PutValue({ {Name, Key, FValue::Compress(MakeSharedBufferFromArray(MoveTemp(WriteData)))} }, AsyncOwner);
		AsyncOwner.KeepAlive();
	}

	// Update the imported mesh DDC keys
	FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
	for (int32 LODIt = 0; LODIt < GroupData.Cards.LODs.Num(); ++LODIt)
	{
		if (FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt))
		{
			Desc->UpdateMeshKey();
		}
	}

	CardsDerivedDataKey[GroupIndex] = DerivedDataKey;

	return true;
}

// Return true if the meshes data have changed
bool UGroomAsset::CacheMeshesData(uint32 GroupIndex)
{
	// We should never query the DDC for something that we won't be able to build. So check that first.
	// The goal is to avoid paying the price of a guaranteed failure on high-latency DDC backends every run.
	const bool bHasValidSourceData = UGroomAsset::HasValidData_Meshes(GroupIndex);
	if (!bHasValidSourceData)
	{
		const bool bNeedReset = !MeshesDerivedDataKey[GroupIndex].IsEmpty();
		MeshesDerivedDataKey[GroupIndex].Empty();

		// Initialized LODs
		const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
		GetHairGroupsPlatformData()[GroupIndex].Meshes.LODs.Empty(); // Force reset data
		GetHairGroupsPlatformData()[GroupIndex].Meshes.LODs.SetNum(LODCount);

		return bNeedReset;
	}

	const FString KeySuffix = GroomDerivedDataCacheUtils::BuildMeshesDerivedDataKeySuffix(GroupIndex, GetHairGroupsLOD()[GroupIndex].LODs, GetHairGroupsMeshes());
	const FString DerivedDataKey = GroomDerivedDataCacheUtils::BuildGroomDerivedDataKey(KeySuffix);

	// Query DDC
	using namespace UE;
	using namespace UE::DerivedData;
	const FCacheKey Key = ConvertLegacyCacheKey(DerivedDataKey);
	const FSharedString Name = MakeStringView(GetPathName());
	FSharedBuffer Data;
	{
		FRequestOwner Owner(EPriority::Blocking);
		GetCache().GetValue({ {Name, Key} }, Owner, [&Data](FCacheGetValueResponse&& Response)
		{
			Data = Response.Value.GetData().Decompress();
		});
		Owner.Wait();
	}

	// * If data is available, populate LOD data
	// * Otherwise build the LOD data
	FHairGroupPlatformData& HairGroupData = GetHairGroupsPlatformData()[GroupIndex];
	if (Data)
	{
		UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[Groom/DDC] Meshes - Found (Groom:%ls Group:%d).", *GetName(), GroupIndex);

		FMemoryReaderView Ar(Data, /*bIsPersistent*/ true);
		InternalSerializeMeshes(Ar, this, HairGroupData.Meshes.LODs);
	}
	else
	{
		UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[Groom/DDC] Meshes - Not found (Groom:%ls Group:%d).", *GetName(), GroupIndex);

		const bool bSucceed = BuildHairGroup_Meshes(GroupIndex);
		if (!bSucceed)
		{
			MeshesDerivedDataKey[GroupIndex].Empty();
			return false;
		}

		TArray<uint8> WriteData;
		FMemoryWriter Ar(WriteData, /*bIsPersistent*/ true);
		InternalSerializeMeshes(Ar, this, HairGroupData.Meshes.LODs);
		FRequestOwner AsyncOwner(EPriority::Normal);
		GetCache().PutValue({ {Name, Key, FValue::Compress(MakeSharedBufferFromArray(MoveTemp(WriteData)))} }, AsyncOwner);
		AsyncOwner.KeepAlive();
	}

	MeshesDerivedDataKey[GroupIndex] = DerivedDataKey;

	return true;
}

inline FString GetLODName(const UGroomAsset* Asset, uint32 LODIndex)
{
	return Asset->GetOutermost()->GetName() + FString::Printf(TEXT("_LOD%d"), LODIndex);
}

static bool InternalImportCardGeometry(
	const FHairGroupsCardsSourceDescription* InDesc, 
	const FHairDescriptionGroups& InHairDescriptionGroups, 
	const TArray<FHairGroupsInterpolation>& InHairInterpolationGroups,
	FHairCardsBulkData& OutBulkData, 
	FHairStrandsDatas& OutGuidesData, 
	FHairCardsInterpolationBulkData& OutInterpolationBulkData)
{
	UStaticMesh* CardsMesh = InDesc->ImportedMesh;
	check(CardsMesh);

	// * Create a transient FHairStrandsData in order to extract RootUV and transfer them to cards data
	// * Voxelize hair strands data (all group), to transfer group index from strands to cards
	//FHairGroupInfo DummyInfo;
	if (InHairDescriptionGroups.IsValid() && InHairDescriptionGroups.HairGroups.IsValidIndex(InDesc->GroupIndex))
	{
		FHairStrandsDatas TempHairGuidesData;
		FHairStrandsDatas TempHairStrandsData;
		FGroomBuilder::BuildData(
			InHairDescriptionGroups.HairGroups[InDesc->GroupIndex], 
			InHairInterpolationGroups[InDesc->GroupIndex],
			TempHairStrandsData,
			TempHairGuidesData);

		FHairStrandsVoxelData TempHairStrandsVoxelData;
		FGroomBuilder::VoxelizeGroupIndex(InHairDescriptionGroups, TempHairStrandsVoxelData);

		// Transfer RootUV from strands to guides
		const bool bNeedRootUVTransfer = InDesc->GuideType == EHairCardsGuideType::GuideBased;
		if (bNeedRootUVTransfer)
		{
			struct FRoot
			{
				FVector3f Position;
				FVector2f RootUV;
			};

			// Extra all roots from strands
			TArray<FRoot> StrandsRoots;
			StrandsRoots.SetNum(TempHairStrandsData.GetNumCurves());
			for (uint32 CurveIt = 0, CurveCount = TempHairStrandsData.GetNumCurves(); CurveIt < CurveCount; ++CurveIt)
			{
				StrandsRoots[CurveIt].Position = TempHairStrandsData.StrandsPoints.PointsPosition[TempHairStrandsData.StrandsCurves.CurvesOffset[CurveIt]];
				StrandsRoots[CurveIt].RootUV   = TempHairStrandsData.StrandsCurves.CurvesRootUV[CurveIt];
			}

			// Find closest strands root for each guide roots, and assign RootUV
			for (uint32 CurveIt = 0, CurveCount = TempHairGuidesData.GetNumCurves(); CurveIt < CurveCount; ++CurveIt)
			{
				float MaxDistance = FLT_MAX;
				const FVector3f Position = TempHairGuidesData.StrandsPoints.PointsPosition[TempHairGuidesData.StrandsCurves.CurvesOffset[CurveIt]];
				for (const FRoot& Root : StrandsRoots)
				{
					const float Dist = (Position - Root.Position).Length();
					if (Dist < MaxDistance)
					{
						TempHairGuidesData.StrandsCurves.CurvesRootUV[CurveIt] = Root.RootUV;
						MaxDistance = Dist;
					}
				}
			}
		}
		return FHairCardsBuilder::ImportGeometry(CardsMesh, TempHairGuidesData, TempHairStrandsData, TempHairStrandsVoxelData, InDesc->GuideType == EHairCardsGuideType::Generated, OutBulkData, OutGuidesData, OutInterpolationBulkData);
	}
	else
	{
		// If no hair description is available, build hair card with dummy hair strands data
		FHairStrandsDatas DummyHairGuidesData;
		FHairStrandsDatas DummyHairStrandsData;
		FHairStrandsVoxelData DummyHairStrandsVoxelData;
		return FHairCardsBuilder::ImportGeometry(CardsMesh, DummyHairGuidesData, DummyHairStrandsData, DummyHairStrandsVoxelData, true /*bGenerateGuidesFromCardGeometry*/, OutBulkData, OutGuidesData, OutInterpolationBulkData);
	}
}

bool UGroomAsset::HasChanged_Cards(uint32 GroupIndex, TArray<bool>& OutIsValid) const 
{
	if (!IsHairStrandsEnabled(EHairStrandsShaderType::Cards) || (GetHairGroupsCards().Num() == 0 && CachedHairGroupsCards.Num() == 0))
	{
		return false;
	}

	bool bHasChanged = GetHairGroupsCards().Num() != CachedHairGroupsCards.Num();

	check(GroupIndex < uint32(GetNumHairGroups()));
	const FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];

	// The settings might have been previously cached without the data having been built
	const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
	OutIsValid.Init(false, LODCount);
	for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		int32 SourceIt = 0;
		if (const FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt, &SourceIt))
		{
			bool bIsLODValid = GroupData.Cards.LODs.IsValidIndex(LODIt) && GroupData.Cards.LODs[LODIt].HasValidData() && !Desc->HasMeshChanged();
			if (const FHairGroupsCardsSourceDescription* CachedDesc = (CachedHairGroupsCards.IsValidIndex(SourceIt) ? &CachedHairGroupsCards[SourceIt] : nullptr))
			{
				bIsLODValid = bIsLODValid && *CachedDesc == *Desc;
			}
			OutIsValid[LODIt] = bIsLODValid;
		}

		bHasChanged |= !OutIsValid[LODIt];
	}

	return bHasChanged;
}

bool UGroomAsset::BuildHairGroup_Cards(uint32 GroupIndex)
{
	LLM_SCOPE_BYTAG(Groom);

	TArray<bool> bIsAlreadyBuilt;
	if (!HasChanged_Cards(GroupIndex, bIsAlreadyBuilt))
	{
		return false;
	}

	const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
	FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];

	bool bDataBuilt = false;
	GroupData.Cards.LODs.Empty(); // Force reset data
	GroupData.Cards.LODs.SetNum(LODCount);
	for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		if (FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt))
		{
			FHairGroupPlatformData::FCards::FLOD& LOD = GroupData.Cards.LODs[LODIt];
			LOD.BulkData.Reset();

			// 1. Load geometry data, if any
			bool bSuccess = false;
			FHairStrandsDatas LODGuidesData;
			if (UStaticMesh* CardsMesh = Desc->ImportedMesh)
			{
				CardsMesh->ConditionalPostLoad();

				const FHairDescriptionGroups& LocalHairDescriptionGroups = GetHairDescriptionGroups();
				const TArray<FHairGroupsInterpolation>& LocalInterpolationGroups= GetHairGroupsInterpolation();
				bSuccess = InternalImportCardGeometry(Desc, LocalHairDescriptionGroups, LocalInterpolationGroups, LOD.BulkData, LODGuidesData, LOD.InterpolationBulkData);
				if (!bSuccess)
				{
					UE_LOGF(LogHairStrands, Warning, "Failed to import cards from %ls for Group %d LOD %d.", *CardsMesh->GetName(), GroupIndex, LODIt);
					LOD.BulkData.Reset();
				}
			}

			bDataBuilt |= bSuccess;

			// Clear the cards stats
			Desc->CardsInfo.NumCardVertices = 0;
			Desc->CardsInfo.NumCards = 0;

			// 2. Init geometry & texture resources, and generate interpolation data/resources
			if (bSuccess)
			{
				// Create own interpolation settings for cards.
				// Force closest guides as this is the most relevant matching metric for cards, due to their coarse geometry
				FHairInterpolationSettings CardsInterpolationSettings = GetHairGroupsInterpolation()[GroupIndex].InterpolationSettings;
				CardsInterpolationSettings.GuideType = EGroomGuideType::Imported;
				CardsInterpolationSettings.bUseUniqueGuide = true;
				CardsInterpolationSettings.bRandomizeGuide = false;
				CardsInterpolationSettings.InterpolationDistance = EHairInterpolationWeight::Parametric;
				CardsInterpolationSettings.InterpolationQuality = EHairInterpolationQuality::High;

				FHairGroupInfo DummyGroupInfo;

				// 1. Build bulk data for the cards guides
				FGroomBuilder::BuildBulkData(DummyGroupInfo, LODGuidesData, LOD.GuideBulkData, false /*bAllowCompression*/);

				// 2. (Re)Build data for the sim guides (since there are transient/no-cached)
				const FHairDescriptionGroups& LocalHairDescriptionGroups = GetHairDescriptionGroups();
				FHairStrandsDatas SimGuidesData;
				if (LocalHairDescriptionGroups.IsValid())
				{
					FHairStrandsDatas RenStrandsData; // Dummy, not used
					FGroomBuilder::BuildData(LocalHairDescriptionGroups.HairGroups[GroupIndex], GetHairGroupsInterpolation()[GroupIndex], RenStrandsData, SimGuidesData);
				}
				else
				{
					// Use cards' guides as simulation guides for building interpolation.
					// but these cards' guides won't be simulated, since they are not written as main guides
					SimGuidesData = LODGuidesData;
				}
				// 3. Compute interpolation data & bulk data between the sim guides and the cards guides
				FHairStrandsInterpolationDatas LODGuideInterpolationData;
				FGroomBuilder::BuildInterplationData(DummyGroupInfo, LODGuidesData, SimGuidesData, CardsInterpolationSettings, LODGuideInterpolationData);
				FGroomBuilder::BuildInterplationBulkData(SimGuidesData, LODGuideInterpolationData, LOD.GuideInterpolationBulkData);

				// Update card stats to display
				Desc->CardsInfo.NumCardVertices = LOD.BulkData.GetNumVertices();
				Desc->CardsInfo.NumCards 		= LOD.GuideBulkData.GetNumCurves();
			}
		}
	}
	return bDataBuilt;
}

bool UGroomAsset::BuildCardsData()
{
	bool bDataChanged = false;
	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		bDataChanged |= CacheCardsData(GroupIndex, StrandsDerivedDataKey[GroupIndex]);
	}

	if (bDataChanged)
	{
		InitCardsResources();
	}

	return bDataChanged;
}

bool UGroomAsset::BuildMeshesData()
{
	bool bDataChanged = false;
	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		bDataChanged |= CacheMeshesData(GroupIndex);
	}

	if (bDataChanged)
	{
		InitMeshesResources();
	}

	return bDataChanged;
}

bool UGroomAsset::HasChanged_Meshes(uint32 GroupIndex, TArray<bool>& OutIsValid) const 
{
	if (!IsHairStrandsEnabled(EHairStrandsShaderType::Meshes) || (GetHairGroupsMeshes().Num() == 0 && CachedHairGroupsMeshes.Num() == 0))
	{
		return false;
	}

	bool bHasChanged = GetHairGroupsMeshes().Num() != CachedHairGroupsMeshes.Num();

	const FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];

	// The settings might have been previously cached without the data having been built
	const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
	OutIsValid.Init(false, LODCount);
	for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		int32 SourceIt = 0;
		if (const FHairGroupsMeshesSourceDescription* Desc = GetSourceDescription(GetHairGroupsMeshes(), GroupIndex, LODIt, &SourceIt))
		{
			bool bIsLODValid = GroupData.Meshes.LODs.IsValidIndex(LODIt) && GroupData.Meshes.LODs[LODIt].HasValidData() && !Desc->HasMeshChanged();

			if (const FHairGroupsMeshesSourceDescription* CachedDesc = CachedHairGroupsMeshes.IsValidIndex(SourceIt) ? &CachedHairGroupsMeshes[SourceIt] : nullptr)
			{
				bIsLODValid = bIsLODValid && *CachedDesc == *Desc;
			}

			OutIsValid[LODIt] = bIsLODValid;
		}
		bHasChanged |= !OutIsValid[LODIt];
	}	
	
	return bHasChanged;
}

bool UGroomAsset::BuildHairGroup_Meshes(uint32 GroupIndex)
{
	TArray<bool> bIsAlreadyBuilt;
	if (!HasChanged_Meshes(GroupIndex, bIsAlreadyBuilt))
	{
		return false;
	}

	FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
	const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();

	bool bDataBuilt = false;
	GroupData.Meshes.LODs.Empty(); // Force reset data
	GroupData.Meshes.LODs.SetNum(LODCount);
	for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		int32 SourceIt = 0;
		if (const FHairGroupsMeshesSourceDescription* Desc = GetSourceDescription(GetHairGroupsMeshes(), GroupIndex, LODIt, &SourceIt))
		{
			if (!IsInGameThread())
			{
				// Build needs to execute from the game thread
				bRetryLoadFromGameThread = true;
				return false;
			}

			FHairGroupPlatformData::FMeshes::FLOD& LOD = GroupData.Meshes.LODs[LODIt];

			if (Desc->ImportedMesh)
			{
				Desc->ImportedMesh->ConditionalPostLoad();
				FHairMeshesBuilder::ImportGeometry(
					Desc->ImportedMesh,
					LOD.BulkData);
			}
			else
			{
				// Build a default box
				FHairMeshesBuilder::BuildGeometry(
					GroupData.Strands.BulkData.Header.BoundingBox,
					LOD.BulkData);
			}

			bDataBuilt |= true;
		}
	}

	// Update the imported mesh DDC keys
	for (int32 LODIt = 0; LODIt < GroupData.Meshes.LODs.Num(); ++LODIt)
	{
		if (FHairGroupsMeshesSourceDescription* Desc = GetSourceDescription(GetHairGroupsMeshes(), GroupIndex, LODIt))
		{
			Desc->UpdateMeshKey();
		}
	}

	return bDataBuilt;
}

void UGroomAsset::UpdateCachedSettings()
{
#if WITH_EDITORONLY_DATA
	CachedHairGroupsRendering		= GetHairGroupsRendering();
	CachedHairGroupsPhysics			= GetHairGroupsPhysics();
	CachedHairGroupsInterpolation	= GetHairGroupsInterpolation();
	CachedHairGroupsLOD				= GetHairGroupsLOD();
	CachedHairGroupsCards			= GetHairGroupsCards();
	CachedHairGroupsMeshes			= GetHairGroupsMeshes();
#endif
}
#endif // WITH_EDITORONLY_DATA


void UGroomAsset::InitGuideResources()
{
	// Guide resources are lazy allocated, as these resources are only used when RBF/simulation is enabled by a groom component
	// for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	//{
	//	AllocateGuidesResources(GroupIndex);
	//}
}

FHairStrandsRestResource* UGroomAsset::AllocateGuidesResources(uint32 GroupIndex)
{
	if (GetHairGroupsPlatformData().IsValidIndex(GroupIndex))
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResources = GetHairGroupsResources()[GroupIndex];
		if (GroupData.Guides.HasValidData())
		{
			if (GroupResources.Guides.RestResource == nullptr)
			{
				GroupResources.Guides.RestResource = new FHairStrandsRestResource(GroupData.Guides.BulkData, EHairStrandsResourcesType::Guides, FHairResourceName(GetFName(), GroupIndex), GetAssetPathName());
			}
			return GroupResources.Guides.RestResource;
		}
	}
	return nullptr;
}

FHairStrandsInterpolationResource* UGroomAsset::AllocateInterpolationResources(uint32 GroupIndex)
{
	if (GetHairGroupsPlatformData().IsValidIndex(GroupIndex))
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResources = GetHairGroupsResources()[GroupIndex];
		check(GroupData.Guides.HasValidData());
		if (GroupResources.Strands.InterpolationResource == nullptr)
		{
			GroupResources.Strands.InterpolationResource = new FHairStrandsInterpolationResource(GroupData.Strands.InterpolationBulkData, FHairResourceName(GetFName(), GroupIndex), GetAssetPathName());
		}
		return GroupResources.Strands.InterpolationResource;
	}
	return nullptr;
}

#if RHI_RAYTRACING
FHairStrandsRaytracingResource* UGroomAsset::AllocateCardsRaytracingResources(uint32 GroupIndex, uint32 LODIndex)
{
	if (IsRayTracingAllowed() && GetHairGroupsLOD().IsValidIndex(GroupIndex) && GetHairGroupsLOD()[GroupIndex].LODs.IsValidIndex(LODIndex))
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];
		FHairGroupPlatformData::FCards::FLOD& LODData = GroupData.Cards.LODs[LODIndex];
		FHairGroupResources::FCards::FLOD& LOD = GroupResource.Cards.LODs[LODIndex];
		check(LODData.HasValidData());

		if (LOD.RaytracingResource == nullptr)
		{
			LOD.RaytracingResource = new FHairStrandsRaytracingResource(LODData.BulkData, FHairResourceName(GetFName(), GroupIndex, LODIndex), GetAssetPathName(LODIndex));
		}
		return LOD.RaytracingResource;
	}
	return nullptr;
}

FHairStrandsRaytracingResource* UGroomAsset::AllocateMeshesRaytracingResources(uint32 GroupIndex, uint32 LODIndex)
{
	if (IsRayTracingAllowed() && GetHairGroupsLOD().IsValidIndex(GroupIndex) && GetHairGroupsLOD()[GroupIndex].LODs.IsValidIndex(LODIndex))
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];

		FHairGroupPlatformData::FMeshes::FLOD& LODData = GroupData.Meshes.LODs[LODIndex];
		FHairGroupResources::FMeshes::FLOD& LOD = GroupResource.Meshes.LODs[LODIndex];
		check(LODData.HasValidData());
		if (LOD.RaytracingResource == nullptr)
		{
			LOD.RaytracingResource = new FHairStrandsRaytracingResource(LODData.BulkData, FHairResourceName(GetFName(), GroupIndex, LODIndex), GetAssetPathName(LODIndex));
		}
		return LOD.RaytracingResource;
	}
	return nullptr;
}

FHairStrandsRaytracingResource** UGroomAsset::GetStrandsRaytracingResourcesPtr(uint32 GroupIndex)
{
	if (GetHairGroupsPlatformData().IsValidIndex(GroupIndex))
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];
		check(GroupData.Strands.HasValidData());
		return &GroupResource.Strands.RaytracingResource;
	}
	return nullptr;
}

FHairStrandsRaytracingResource* UGroomAsset::AllocateStrandsRaytracingResources(uint32 GroupIndex)
{
	if (IsRayTracingAllowed() && GetHairGroupsPlatformData().IsValidIndex(GroupIndex))
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];
		check(GroupData.Strands.HasValidData());

		if (GroupResource.Strands.RaytracingResource == nullptr)
		{
			GroupResource.Strands.RaytracingResource = new FHairStrandsRaytracingResource(GroupData.Strands.BulkData, FHairResourceName(GetFName(), GroupIndex), GetAssetPathName());
		}
		return GroupResource.Strands.RaytracingResource;
	}
	return nullptr;
}
#endif // RHI_RAYTRACING

void UGroomAsset::InitStrandsResources()
{
	// Even though we shouldn't build the strands resources if the platforms does 
	// not support it, we can't test it as this is dependant on the EShaderPlatform 
	// enum, which is only available on the scene, and not available as the time the 
	// groom asset is loaded and initialized.
	// We should be testing: IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform)
	//
	// To handle this we assume that thecooking as strip out the strands data
	if (!IsHairStrandsEnabled(EHairStrandsShaderType::Strands))
	{
		return;
	}

	const FName OwnerName = GetAssetPathName();

	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];

		if (GroupData.Strands.HasValidData())
		{
			FHairResourceName ResourceName(GetFName(), GroupIndex);
			GroupResource.Strands.RestResource = new FHairStrandsRestResource(GroupData.Strands.BulkData, EHairStrandsResourcesType::Strands, ResourceName, OwnerName);
			GroupResource.Strands.ClusterResource = new FHairStrandsClusterResource(GroupData.Strands.ClusterBulkData, ResourceName, OwnerName);

			// Interpolation are lazy allocated, as these resources are only used when RBF/simulation is enabled by a groom component
			// AllocateInterpolationResources(GroupIndex);
		}
	}
}

static void InitCardsTextureResources(UGroomAsset* GroomAsset)
{
	if (!GroomAsset || !IsHairStrandsEnabled(EHairStrandsShaderType::Cards) || GroomAsset->GetHairGroupsCards().Num() == 0)
	{
		return;
	}

	for (uint32 GroupIndex = 0, GroupCount = GroomAsset->GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		FHairGroupPlatformData& GroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GroomAsset->GetHairGroupsResources()[GroupIndex];

		const uint32 LODCount = GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num();
		GroupData.Cards.LODs.SetNum(LODCount);

		for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			if (const FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GroomAsset->GetHairGroupsCards(), GroupIndex, LODIt))
			{
				FHairGroupPlatformData::FCards::FLOD& LODData = GroupData.Cards.LODs[LODIt];
				FHairGroupResources::FCards::FLOD& LOD = GroupResource.Cards.LODs[LODIt];
				if (LODData.HasValidData())
				{
					InitAtlasTexture(LOD.RestResource, Desc->Textures, Desc->bInvertUV);
				}
			}
		}
	}
}

void UGroomAsset::InitCardsResources()
{
	LLM_SCOPE_BYTAG(Groom);

	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		check(GetHairGroupsPlatformData().IsValidIndex(GroupIndex));

		// * Release resource if any
		// * Resize the LODs to match the group LOD count
		// Even if the groom does not have cards/meshes data, LOD datas/resources count needs to match
		ReleaseCardsResource(GroupIndex);

		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = HairGroupsResources[GroupIndex];

		const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
		check(uint32(GroupData.Cards.LODs.Num()) == LODCount);
		GroupResource.Cards.LODs.SetNum(LODCount);

		// If the group does not have cards, skip resource creation
		if (!IsHairStrandsEnabled(EHairStrandsShaderType::Cards) || GetHairGroupsCards().Num() == 0)
		{
			continue;
		}

		for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			FHairGroupPlatformData::FCards::FLOD& LODData = GroupData.Cards.LODs[LODIt];
			FHairGroupResources::FCards::FLOD& LOD = GroupResource.Cards.LODs[LODIt];

			if (const FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt))
			{
				const bool bNeedInitialization = LOD.RestResource == nullptr && LODData.HasValidData();
				if (bNeedInitialization)
				{
					FHairResourceName ResourceName(GetFName(), GroupIndex, LODIt);
					const FName OwnerName = GetAssetPathName(LODIt);
	
					LOD.RestResource 				= new FHairCardsRestResource(LODData.BulkData, ResourceName, OwnerName);
					LOD.InterpolationResource		= new FHairCardsInterpolationResource(LODData.InterpolationBulkData, ResourceName, OwnerName);
					LOD.GuideRestResource			= new FHairStrandsRestResource(LODData.GuideBulkData, EHairStrandsResourcesType::Cards, ResourceName, OwnerName);
					LOD.GuideInterpolationResource	= new FHairStrandsInterpolationResource(LODData.GuideInterpolationBulkData, ResourceName, OwnerName);

					// Resource allocation
					// Immediate allocation, as needed for the vertex factory, input stream building
					BeginInitResource(LOD.RestResource); 
					BeginInitResource(LOD.InterpolationResource);
					BeginInitResource(LOD.GuideRestResource);
					BeginInitResource(LOD.GuideInterpolationResource);
	
					InitAtlasTexture(LOD.RestResource, Desc->Textures, Desc->bInvertUV);
				}
			}

			// Update groom cards stats (visible in groom editor)
			#if WITH_EDITOR
			if (FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIt))
			{	
				// Update card stats to display
				Desc->CardsInfo.NumCardVertices = LODData.BulkData.GetNumVertices();
				Desc->CardsInfo.NumCards 		= LODData.GuideBulkData.GetNumCurves();
			}
			#endif
		}
	}
}

void UGroomAsset::InitMeshesResources()
{
	LLM_SCOPE_BYTAG(Groom);

	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		check(GetHairGroupsPlatformData().IsValidIndex(GroupIndex));

		// * Release resource if any
		// * Resize the LODs to match the group LOD count
		// Even if the groom does not have cards/meshes data, LOD datas/resources count needs to match
		ReleaseMeshesResource(GroupIndex);

		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];

		const uint32 LODCount = GetHairGroupsLOD()[GroupIndex].LODs.Num();
		check(uint32(GroupData.Meshes.LODs.Num()) == LODCount);
		GroupResource.Meshes.LODs.SetNum(LODCount);

		if (!IsHairStrandsEnabled(EHairStrandsShaderType::Meshes) || GetHairGroupsMeshes().Num() == 0)
		{
			continue;
		}

		for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			if (const FHairGroupsMeshesSourceDescription* Desc = GetSourceDescription(GetHairGroupsMeshes(), GroupIndex, LODIt))
			{
				FHairGroupPlatformData::FMeshes::FLOD& LODData = GroupData.Meshes.LODs[LODIt];
				FHairGroupResources::FMeshes::FLOD& LOD = GroupResource.Meshes.LODs[LODIt];
	
				if (LODData.HasValidData())
				{
					LOD.RestResource = new FHairMeshesRestResource(LODData.BulkData, FHairResourceName(GetFName(), GroupIndex, LODIt), GetAssetPathName(LODIt));
					BeginInitResource(LOD.RestResource); // Immediate allocation, as needed for the vertex factory, input stream building
					InitAtlasTexture(LOD.RestResource, Desc->Textures, false /*bInvertUV*/);
				}
			}
		}
	}
}

EGroomGeometryType UGroomAsset::GetGeometryType(int32 GroupIndex, int32 LODIndex) const
{
	if (GetHairGroupsLOD().IsValidIndex(GroupIndex) && GetHairGroupsLOD()[GroupIndex].LODs.IsValidIndex(LODIndex))
	{
		return GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].GeometryType;
	}
	return EGroomGeometryType::Strands;
}

EGroomBindingType UGroomAsset::GetBindingType(int32 GroupIndex, int32 LODIndex) const
{
	if (GetHairGroupsLOD().IsValidIndex(GroupIndex) && GetHairGroupsLOD()[GroupIndex].LODs.IsValidIndex(LODIndex) && IsHairStrandsBindingEnable())
	{
		return GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].BindingType;
	}
	return EGroomBindingType::Rigid; // Fallback to rigid by default
}

bool UGroomAsset::IsVisible(int32 GroupIndex, int32 LODIndex) const
{
	if (GetHairGroupsLOD().IsValidIndex(GroupIndex) && GetHairGroupsLOD()[GroupIndex].LODs.IsValidIndex(LODIndex))
	{
		return GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].bVisible;
	}
	return false;
}

bool UGroomAsset::IsDeformationEnable(int32 GroupIndex) const
{
	return 
		(GetGuideType(GroupIndex) == EGroomGuideType::Rigged) ||
		(GetHairGroupsPhysics().IsValidIndex(GroupIndex) &&	GetHairGroupsPhysics()[GroupIndex].SolverSettings.bEnableDeformation);
}

EGroomGuideType UGroomAsset::GetGuideType(int32 GroupIndex) const
{
	return GetHairGroupsInterpolation().IsValidIndex(GroupIndex) ? GetHairGroupsInterpolation()[GroupIndex].InterpolationSettings.GuideType : EGroomGuideType::Generated;
}

bool UGroomAsset::IsSimulationEnable(int32 GroupIndex, int32 LODIndex) const 
{
	if (!GetHairGroupsLOD().IsValidIndex(GroupIndex) || LODIndex >= GetHairGroupsLOD()[GroupIndex].LODs.Num() || !IsHairStrandsSimulationEnable())
	{
		return false;
	}

	check(GetHairGroupsPhysics().Num() == GetHairGroupsLOD().Num());

	// If the LOD index is not forced, then its value could be -1. In this case we return the 'global' asset value
	if (LODIndex < 0)
	{
		return GetHairGroupsPhysics()[GroupIndex].SolverSettings.EnableSimulation;
	}

	if (!IsHairLODSimulationEnabled(LODIndex))
	{
		return false;
	}

	return GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].bVisible && (
		GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].Simulation == EGroomOverrideType::Enable ||
		(GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].Simulation == EGroomOverrideType::Auto && GetHairGroupsPhysics()[GroupIndex].SolverSettings.EnableSimulation));
}

bool UGroomAsset::IsGlobalInterpolationEnable(int32 GroupIndex, int32 LODIndex) const
{
	if (GetHairGroupsLOD().IsValidIndex(GroupIndex))
	{
		if (GetHairGroupsLOD()[GroupIndex].LODs.IsValidIndex(LODIndex))
		{
			return
				GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].GlobalInterpolation == EGroomOverrideType::Enable ||
				(GetHairGroupsLOD()[GroupIndex].LODs[LODIndex].GlobalInterpolation == EGroomOverrideType::Auto && GetEnableGlobalInterpolation());
		}
	}
	return GetEnableGlobalInterpolation();
}

bool UGroomAsset::IsSimulationEnable() const
{
	for (int32 GroupIndex = 0; GroupIndex < GetHairGroupsPlatformData().Num(); ++GroupIndex)
	{
		for (int32 LODIt = 0; LODIt < GetHairGroupsLOD()[GroupIndex].LODs.Num(); ++LODIt)
		{
			if (IsSimulationEnable(GroupIndex, LODIt))
			{
				return true;
			}
		}
	}

	return false;
}

bool UGroomAsset::NeedsInterpolationData(int32 GroupIndex) const
{
	for (int32 LODIt = 0; LODIt < GetHairGroupsLOD()[GroupIndex].LODs.Num(); ++LODIt)
	{
		if (IsSimulationEnable(GroupIndex, LODIt) || IsGlobalInterpolationEnable(GroupIndex, LODIt) || GetEnableSimulationCache())
		{
			return true;
		}
	}

	return IsDeformationEnable(GroupIndex);
}

bool UGroomAsset::NeedsInterpolationData() const
{
	for (int32 GroupIndex = 0; GroupIndex < GetHairGroupsPlatformData().Num(); ++GroupIndex)
	{
		if (NeedsInterpolationData(GroupIndex))
		{
			return true;
		}
	}

	return false;
}

bool FHairDescriptionGroups::IsValid() const
{
	for (const FHairDescriptionGroup& HairGroup : HairGroups)
	{
		if (HairGroup.Info.NumCurves == 0)
		{
			return false;
		}
	}

	return 	HairGroups.Num() > 0;
}

int32 UGroomAsset::GetLODCount() const
{
	int32 MaxLODCount = -1;
	for (const FHairGroupsLOD& S : GetHairGroupsLOD())
	{
		MaxLODCount = FMath::Max(MaxLODCount, S.LODs.Num());
	}
	return MaxLODCount;
}

#if WITH_EDITORONLY_DATA
void UGroomAsset::StripLODs(const TArray<int32>& LODsToKeep, bool bRebuildResources)
{
	// Assume that the LOD are ordered from 0 ... Max
	// Export all LODs if the list is empty or has the same number of LODs
	if (LODsToKeep.Num() == GetLODCount() || LODsToKeep.Num() == 0)
	{
		return;
	}

	const int32 GroupCount = GetHairGroupsLOD().Num();
	int32 LODsTOKeepIndex = LODsToKeep.Num()-1;

	// Remove the LOD settings prior to rebuild the LOD data
	const int32 LODCount = GetLODCount();
	for (int32 LODIt = LODCount-1; LODIt > 0; --LODIt)
	{
		if (LODIt == LODsToKeep[LODsTOKeepIndex])
		{
			continue;
		}

		for (int32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
		{
			GetHairGroupsLOD()[GroupIt].LODs.RemoveAt(LODIt);
		}

		--LODsTOKeepIndex;
	}

	CacheDerivedDatas();
}
#endif // WITH_EDITORONLY_DATA

bool UGroomAsset::HasDebugData() const
{
	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		if (!GetHairGroupsPlatformData()[GroupIndex].Debug.Data.IsValid())
		{
			return false;
		}
	}

	return true;
}

void UGroomAsset::CreateDebugData()
{
#if WITH_EDITORONLY_DATA
	if (!IsHairStrandsEnabled(EHairStrandsShaderType::Strands) || !CanRebuildFromDescription())
		return;

	for (uint32 GroupIndex = 0, GroupCount = GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		// 2.1 Generate transient strands data
		const FHairDescriptionGroups& LocalHairDescriptionGroups = GetHairDescriptionGroups();
		FHairStrandsDatas StrandsData;
		FHairStrandsDatas GuidesData;
		FGroomBuilder::BuildData(LocalHairDescriptionGroups.HairGroups[GroupIndex], GetHairGroupsInterpolation()[GroupIndex], StrandsData, GuidesData);

		FHairGroupPlatformData& GroupData = GetHairGroupsPlatformData()[GroupIndex];
		CreateHairStrandsDebugDatas(StrandsData, GroupData.Debug.Data);

		if (GroupData.Debug.Data.IsValid())
		{
			FHairGroupResources& GroupResource = GetHairGroupsResources()[GroupIndex];
			GroupResource.Debug.Resource = new FHairStrandsDebugResources();

			FHairStrandsDebugDatas* InData = &GroupData.Debug.Data;
			FHairStrandsDebugResources* InResource = GroupResource.Debug.Resource;
			ENQUEUE_RENDER_COMMAND(HairStrandsDebugResourceCommand)(/*UE::RenderCommandPipe::Groom,*/
				[InData, InResource](FRHICommandListImmediate& RHICmdList)
				{
					FRDGBuilder GraphBuilder(RHICmdList);
					CreateHairStrandsDebugResources(GraphBuilder, InData, InResource);
					GraphBuilder.Execute();
				});
		}
	}
#endif
}

int32 UGroomAsset::GetMaterialIndex(FName MaterialSlotName) const
{
	const int32 SlotCount = GetHairGroupsMaterials().Num();
	for (int32 SlotIt = 0; SlotIt < SlotCount; ++SlotIt)
	{
		if (GetHairGroupsMaterials()[SlotIt].SlotName == MaterialSlotName)
		{
			return SlotIt;
		}
	}

	return INDEX_NONE;
}

bool UGroomAsset::IsMaterialSlotNameValid(FName MaterialSlotName) const
{
	return GetMaterialIndex(MaterialSlotName) != INDEX_NONE;
}

TArray<FName> UGroomAsset::GetMaterialSlotNames() const
{
	TArray<FName> MaterialNames;
	for (const FHairGroupsMaterial& Material : GetHairGroupsMaterials())
	{
		MaterialNames.Add(Material.SlotName);
	}

	return MaterialNames;
}

template<typename T>
bool InternalIsMaterialUsed(const TArray<T>& Groups, const FName& MaterialSlotName)
{
	bool bNeedSaving = false;
	for (const T& Group : Groups)
	{
		if (Group.MaterialSlotName == MaterialSlotName)
		{
			return true;
		}
	}

	return false;
}

bool UGroomAsset::IsMaterialUsed(int32 MaterialIndex) const
{
	if (MaterialIndex < 0 || MaterialIndex >= GetHairGroupsMaterials().Num())
		return false;

	const FName MaterialSlotName = GetHairGroupsMaterials()[MaterialIndex].SlotName;
	return 
		InternalIsMaterialUsed(GetHairGroupsRendering(), MaterialSlotName) ||
		InternalIsMaterialUsed(GetHairGroupsCards(), MaterialSlotName) ||
		InternalIsMaterialUsed(GetHairGroupsMeshes(), MaterialSlotName);
}

#if WITH_EDITORONLY_DATA
bool UGroomAsset::GetHairStrandsDatas(
	const int32 GroupIndex,
	FHairStrandsDatas& OutStrandsData,
	FHairStrandsDatas& OutGuidesData) const
{
	if (!CanRebuildFromDescription())
	{
		return false;
	}
	const FHairDescriptionGroups& DescriptionGroups = GetHairDescriptionGroups();
	const bool bIsValid = DescriptionGroups.IsValid() && GroupIndex < DescriptionGroups.HairGroups.Num();
	if (bIsValid)
	{
		const FHairDescriptionGroup& HairGroup = DescriptionGroups.HairGroups[GroupIndex];
		check(GroupIndex == HairGroup.Info.GroupIndex);

		FHairGroupInfo DummyInfo;
		FGroomBuilder::BuildData(HairGroup, GetHairGroupsInterpolation()[GroupIndex], OutStrandsData, OutGuidesData);
	}
	return bIsValid;
}

bool UGroomAsset::GetHairCardsGuidesDatas(
	const int32 GroupIndex,
	const int32 LODIndex,
	FHairStrandsDatas& OutCardsGuidesData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomAsset::GetHairCardsGuidesDatas);
	FHairStrandsDatas StrandsData;
	FHairStrandsDatas GuidesData;
	const bool bIsValid = GetHairStrandsDatas(GroupIndex, StrandsData, GuidesData);
	if (!bIsValid)
	{
		return false;
	}

	if (FHairGroupsCardsSourceDescription* Desc = GetSourceDescription(GetHairGroupsCards(), GroupIndex, LODIndex))
	{
		// 1. Load geometry data, if any
		if (UStaticMesh* CardsMesh = Desc->ImportedMesh)
		{
			CardsMesh->ConditionalPostLoad();

			FHairCardsInterpolationBulkData	OutDummyInterpolationBulkData;
			FHairCardsBulkData OutDummyLODBulkData;
			const FHairDescriptionGroups& LocalHairDescriptionGroups = GetHairDescriptionGroups();
			const TArray<FHairGroupsInterpolation>& LocalInterpolationGroups = GetHairGroupsInterpolation();
			InternalImportCardGeometry(Desc, LocalHairDescriptionGroups, LocalInterpolationGroups, OutDummyLODBulkData, OutCardsGuidesData, OutDummyInterpolationBulkData);
			return true;
		}
	}
	return false;
}
#endif // WITH_EDITORONLY_DATA

void UGroomAsset::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsRendering().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsPhysics().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsInterpolation().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsLOD().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsCards().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsMeshes().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsMaterials().GetAllocatedSize());
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsPlatformData().GetAllocatedSize());
	
	// Datas
	for (const FHairGroupPlatformData & GroupData : GetHairGroupsPlatformData())
	{		
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GroupData.Guides.GetDataSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GroupData.Strands.GetDataSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GroupData.Cards.GetDataSize());
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GroupData.Meshes.GetDataSize());
	}

	// Resources
	for (const FHairGroupResources & GroupResource : GetHairGroupsResources())
	{		
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(GroupResource.Guides.GetResourcesSize());
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(GroupResource.Strands.GetResourcesSize());
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(GroupResource.Cards.GetResourcesSize());
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(GroupResource.Meshes.GetResourcesSize());
	}
}

FName UGroomAsset::GetAssetPathName(int32 LODIndex)
{
#if RHI_ENABLE_RESOURCE_INFO
	if (LODIndex > -1)
	{
		return FName(FString::Printf(TEXT("%s [LOD%d]"), *GetPathName(), LODIndex));
	}
	else
	{
		return FName(GetPathName());
	}
#else
	return NAME_None;
#endif
}

#undef LOCTEXT_NAMESPACE

#define GROOMASSET_DEFINE_MEMBER_NAME(Name) \
	FName UGroomAsset::Get##Name##MemberName()\
	{\
		PRAGMA_DISABLE_DEPRECATION_WARNINGS\
		return FName(TEXT(#Name));\
		PRAGMA_ENABLE_DEPRECATION_WARNINGS\
	}

#define GROOMASSET_DEFINE_GET(Type, Name, Access, Const0, Const1) \
	Const0 Type Access UGroomAsset::Get##Name() Const1\
	{\
		PRAGMA_DISABLE_DEPRECATION_WARNINGS\
		return Name;\
		PRAGMA_ENABLE_DEPRECATION_WARNINGS\
	}

#define GROOMASSET_DEFINE_SET(Type, Name, Access, Const) \
	void UGroomAsset::Set##Name(Const Type Access In##Name)\
	{\
		PRAGMA_DISABLE_DEPRECATION_WARNINGS\
		Name = In##Name;\
		PRAGMA_ENABLE_DEPRECATION_WARNINGS\
	}

#define DEFINE_GROOM_MEMBER_ACCESSOR_BP(Type, Name)\
	GROOMASSET_DEFINE_MEMBER_NAME(Name)\
	GROOMASSET_DEFINE_GET(Type, Name, , , const)\
	GROOMASSET_DEFINE_SET(Type, Name, , )

#define DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(Type, Name)\
	GROOMASSET_DEFINE_MEMBER_NAME(Name)\
	GROOMASSET_DEFINE_GET(Type, Name, &, , )\
	GROOMASSET_DEFINE_SET(Type, Name, &, const)\
	GROOMASSET_DEFINE_GET(Type, Name, &, const, const)

#define DEFINE_GROOM_MEMBER_ACCESSOR(Type, Name)\
	GROOMASSET_DEFINE_MEMBER_NAME(Name)\
	GROOMASSET_DEFINE_GET(Type, Name, , , const)\
	GROOMASSET_DEFINE_SET(Type, Name, , )

#define DEFINE_GROOM_MEMBER_ACCESSOR_ARRAY(Type, Name)\
	GROOMASSET_DEFINE_MEMBER_NAME(Name)\
	GROOMASSET_DEFINE_GET(Type, Name, &, , )\
	GROOMASSET_DEFINE_SET(Type, Name, &, const)\
	GROOMASSET_DEFINE_GET(Type, Name, &, const, const)

// Define most of the groom member accessor
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsRendering>, HairGroupsRendering);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsPhysics>, HairGroupsPhysics);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsInterpolation>, HairGroupsInterpolation);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsLOD>, HairGroupsLOD);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsCardsSourceDescription>, HairGroupsCards);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsMeshesSourceDescription>, HairGroupsMeshes);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<FHairGroupsMaterial>, HairGroupsMaterials);
DEFINE_GROOM_MEMBER_ACCESSOR_BP_ARRAY(TArray<int32>, DeformedGroupSections);

DEFINE_GROOM_MEMBER_ACCESSOR_BP(bool, EnableGlobalInterpolation);
DEFINE_GROOM_MEMBER_ACCESSOR_BP(bool, EnableSimulationCache);
DEFINE_GROOM_MEMBER_ACCESSOR_BP(EGroomInterpolationType, HairInterpolationType);
DEFINE_GROOM_MEMBER_ACCESSOR_BP(USkeletalMesh*, RiggedSkeletalMesh);

DEFINE_GROOM_MEMBER_ACCESSOR(FPerPlatformInt, MinLOD);
DEFINE_GROOM_MEMBER_ACCESSOR(FPerPlatformBool, DisableBelowMinLodStripping);
DEFINE_GROOM_MEMBER_ACCESSOR_ARRAY(TArray<float>, EffectiveLODBias);
DEFINE_GROOM_MEMBER_ACCESSOR_ARRAY(TArray<FHairGroupPlatformData>, HairGroupsPlatformData);
DEFINE_GROOM_MEMBER_ACCESSOR_ARRAY(TArray<FHairGroupInfoWithVisibility>, HairGroupsInfo);

GROOMASSET_DEFINE_MEMBER_NAME(DataflowSettings)

const TArray<FHairGroupResources>& UGroomAsset::GetHairGroupsResources() const
{
	return HairGroupsResources;
}

TArray<FHairGroupResources>& UGroomAsset::GetHairGroupsResources()
{
	return HairGroupsResources;
}

#if WITH_EDITOR
void UGroomAsset::RecreateResources()
{
	FGroomComponentRecreateRenderStateContext RecreateContext(this);
	ReleaseResource();
	InitResources();
}

void UGroomAsset::ChangePlatformLevel(ERHIFeatureLevel::Type In)
{
	ChangeShaderPlatform(GShaderPlatformForFeatureLevel[In]);
}

void UGroomAsset::ChangeFeatureLevel(ERHIFeatureLevel::Type In)
{
	ChangeShaderPlatform(GShaderPlatformForFeatureLevel[In]);
}

void UGroomAsset::ChangeShaderPlatform(EShaderPlatform InShaderPlatform)
{
	// When changing platform preview, recreate resources to the correct platform settings (e.g., r.hairstrands.strands=0/1)
	if (CachedResourceShaderPlatform != InShaderPlatform)
	{
		RecreateResources();
		CachedResourceShaderPlatform = InShaderPlatform;
	}
}
#endif


GROOMASSET_DEFINE_MEMBER_NAME(LODMode)
EGroomLODMode UGroomAsset::GetLODMode() const
{
	return LODMode == EGroomLODMode::Default ? InternalGetHairStrandsLODMode() : LODMode;
}

GROOMASSET_DEFINE_MEMBER_NAME(AutoLODBias)
float UGroomAsset::GetAutoLODBias() const
{
	return AutoLODBias;
}

float UGroomAsset::GetAutoLODBias(int32 InGroupIndex) const
{
	float GroupAutoLODBias = 0;
	if (GetHairGroupsLOD().IsValidIndex(InGroupIndex))
	{
		GroupAutoLODBias = GetHairGroupsLOD()[InGroupIndex].AutoLODBias;
	}
	return FMath::Clamp(AutoLODBias + GroupAutoLODBias, -1.f, 1.f);
}

TObjectPtr<UDataflowBaseContent> UGroomAsset::CreateDataflowContent()
{
	TObjectPtr<UDataflowGroomContent> GroomContent = NewObject<UDataflowGroomContent>(this, UDataflowGroomContent::StaticClass());
	GroomContent->SetIsSaved(false);

	GroomContent->SetDataflowOwner(this);
	GroomContent->SetTerminalAsset(this);

	WriteDataflowContent(GroomContent);
	
	return GroomContent;
}

void UGroomAsset::WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const
{
	if(const TObjectPtr<UDataflowGroomContent> GroomContent = Cast<UDataflowGroomContent>(DataflowContent))
	{
		GroomContent->SetDataflowAsset(DataflowSettings.GetDataflowAsset());
		GroomContent->SetDataflowTerminal(DataflowSettings.GetDataflowTerminal().ToString());

#if WITH_EDITORONLY_DATA
		GroomContent->SetAnimationAsset(DataflowSettings.GetPreviewAnimationAsset());
		GroomContent->SetBindingAsset(DataflowSettings.GetPreviewBindingAsset());
#endif
	}
}

void UGroomAsset::ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent)
{
	if(const TObjectPtr<UDataflowGroomContent> GroomContent = Cast<UDataflowGroomContent>(DataflowContent))
	{
#if WITH_EDITORONLY_DATA
		DataflowSettings.SetPreviewAnimationAsset( GroomContent->GetAnimationAsset());
		DataflowSettings.SetPreviewBindingAsset(GroomContent->GetBindingAsset());
#endif
	}
}

void UGroomAsset::ValidateSimulationCache()
{
	Modify();
	SetEnableSimulationCache(true);

#if WITH_EDITORONLY_DATA
	CacheDerivedDatas();
#endif
}

