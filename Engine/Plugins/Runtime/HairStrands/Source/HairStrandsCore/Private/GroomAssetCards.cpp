// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetCards.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "GroomAsset.h" // for EHairAtlasTextureType

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetCards)

FHairGroupsCardsSourceDescription::FHairGroupsCardsSourceDescription()
{
	MaterialSlotName = NAME_None;
	SourceType_DEPRECATED = EHairCardsSourceType::Imported;
	GuideType = EHairCardsGuideType::Generated;
	GroupIndex = 0;
	LODIndex = -1;
}

void FHairGroupCardsTextures::SetLayout(EHairTextureLayout InLayout)
{
	Layout = InLayout;

	uint32 TextureCount = GetHairTextureLayoutTextureCount(Layout);
	Textures.SetNum(TextureCount, EAllowShrinking::Yes);
}

void FHairGroupCardsTextures::SetTexture(int32 SlotIdx, UTexture2D* Texture)
{
	check(Textures.IsValidIndex(SlotIdx));
	Textures[SlotIdx] = Texture;
}

bool FHairGroupsCardsSourceDescription::operator==(const FHairGroupsCardsSourceDescription& A) const
{
	return
		MaterialSlotName == A.MaterialSlotName &&
		GroupIndex == A.GroupIndex &&
		LODIndex == A.LODIndex &&
		ImportedMesh == A.ImportedMesh;
}

FString FHairGroupsCardsSourceDescription::GetMeshKey() const
{
#if WITH_EDITORONLY_DATA
	if (UStaticMesh* Mesh = GetMesh())
	{
		Mesh->ConditionalPostLoad();
		FStaticMeshSourceModel& SourceModel = Mesh->GetSourceModel(0);
		if (SourceModel.GetMeshDescriptionBulkData())
		{
			return SourceModel.GetMeshDescriptionBulkData()->GetUnversionedIdString();
		}
	}
#endif
	return TEXT("INVALID_MESH");
}

bool FHairGroupsCardsSourceDescription::HasMeshChanged() const
{
#if WITH_EDITORONLY_DATA
	if (ImportedMesh)
	{
		return ImportedMeshKey != GetMeshKey();
	}
#endif
	return false;
}

void FHairGroupsCardsSourceDescription::UpdateMeshKey()
{
#if WITH_EDITORONLY_DATA
	if (ImportedMesh)
	{
		ImportedMeshKey = GetMeshKey();
	}
#endif
}

UStaticMesh* FHairGroupsCardsSourceDescription::GetMesh() const
{
#if WITH_EDITORONLY_DATA
	return ImportedMesh;
#else
	return nullptr;
#endif
}

uint32 GetHairTextureLayoutTextureCount(EHairTextureLayout In)
{
	uint32 OutCount = 0;
	switch (In)
	{
		case EHairTextureLayout::Layout0: OutCount = 6u; break;
		case EHairTextureLayout::Layout1: OutCount = 6u; break;
		case EHairTextureLayout::Layout2: OutCount = 3u; break;
		case EHairTextureLayout::Layout3: OutCount = 3u; break;
	}
	check(OutCount <= HAIR_CARDS_MAX_TEXTURE_COUNT);
	return OutCount;
}

const TCHAR* GetHairTextureLayoutTextureName(EHairTextureLayout InLayout, uint32 InIndex, bool bDetail)
{
	switch (InLayout)
	{
		case EHairTextureLayout::Layout0:
		{
			check(6 == GetHairTextureLayoutTextureCount(EHairTextureLayout::Layout0));
			switch (InIndex)
			{
				case 0: return bDetail ? TEXT("Depth\n R8")									: TEXT("Depth");
				case 1: return bDetail ? TEXT("Coverage\n R8")								: TEXT("Coverage");
				case 2: return bDetail ? TEXT("Tangent\n RGB8")								: TEXT("Tangent");
				case 3: return bDetail ? TEXT("Attributes\n RootUV | CoordU | Seed\n RGB8")	: TEXT("Attributes");
				case 4: return bDetail ? TEXT("Material\n Color | Roughess\n RBGA8 ")		: TEXT("Material");
				case 5: return bDetail ? TEXT("Auxiliary\n RGBA8")							: TEXT("Auxiliary");
			}
		} break;
		case EHairTextureLayout::Layout1:
		{
			check(6 == GetHairTextureLayoutTextureCount(EHairTextureLayout::Layout1));
			switch (InIndex)
			{
				case 0: return bDetail ? TEXT("Depth\n R8") 								: TEXT("Depth");
				case 1: return bDetail ? TEXT("Coverage\n R8") 								: TEXT("Coverage");
				case 2: return bDetail ? TEXT("Tangent\n RGB8") 							: TEXT("Tangent");
				case 3: return bDetail ? TEXT("Attributes\n RootUV | CoordU | Seed\n RGB8") : TEXT("Attributes");
				case 4: return bDetail ? TEXT("Material\n Color | GroupID\n RBGA8") 		: TEXT("Material");
				case 5: return bDetail ? TEXT("Auxiliary\n RGBA8") 							: TEXT("Auxiliary");
			}
		} break;
		case EHairTextureLayout::Layout2:
		{
			check(3 == GetHairTextureLayoutTextureCount(EHairTextureLayout::Layout2));
			switch (InIndex)
			{
				case 0: return bDetail ? TEXT("Tangent | CoordU\n RGBA8")					: TEXT("TangentCoordU");
				case 1: return bDetail ? TEXT("Coverage | Depth | Seed\n RGB8")				: TEXT("CoverageDepthSeed");
				case 2: return bDetail ? TEXT("Color | Roughness\n RGBA8")					: TEXT("ColorRoughness");
			}
		} break;
		case EHairTextureLayout::Layout3:
		{
			check(3 == GetHairTextureLayoutTextureCount(EHairTextureLayout::Layout3));
			switch (InIndex)
			{
				case 0: return bDetail ? TEXT("Tangent | CoordU\n RGB8")					: TEXT("TangentCoordU");
				case 1: return bDetail ? TEXT("ColorXY | Depth | GroupID\n RGBA8")			: TEXT("ColorXYDepthGroupID");
				case 2: return bDetail ? TEXT("RooUV | Seed | Coverage \n RGBA8")			: TEXT("RooUVSeedCoverage");
			}
		} break;
	}

	check(false);
	return TEXT("Unknown");
}