// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetDataflow.h"
#include "GroomBindingAsset.h"
#include "Animation/AnimationAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetDataflow)


UDataflowGroomContent::UDataflowGroomContent() : Super()
{
	bHideSkeletalMesh = true;
	bHideAnimationAsset = false;
}

void UDataflowGroomContent::SetBindingAsset(const TObjectPtr<UGroomBindingAsset>& InBindingAsset)
{
	BindingAsset = InBindingAsset;
	if(BindingAsset)
	{
		if(SkeletalMesh != BindingAsset->GetTargetSkeletalMesh())
		{
			SetSkeletalMesh(BindingAsset->GetTargetSkeletalMesh(), true);
		}
	}
	SetConstructionDirty(true);
	SetSimulationDirty(true);
}

#if WITH_EDITOR

void UDataflowGroomContent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	const FName PropertyName = PropertyChangedEvent.Property->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDataflowGroomContent, BindingAsset))
	{
		SetBindingAsset(BindingAsset);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif //if WITH_EDITOR

void UDataflowGroomContent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	UDataflowGroomContent* This = CastChecked<UDataflowGroomContent>(InThis);
	Collector.AddReferencedObject(This->BindingAsset);
	Super::AddReferencedObjects(InThis, Collector);
}

void UDataflowGroomContent::SetActorProperties(TObjectPtr<AActor>& PreviewActor) const
{
	Super::SetActorProperties(PreviewActor);
	OverrideActorProperty(PreviewActor, BindingAsset, TEXT("BindingAsset"));
}

#if WITH_EDITORONLY_DATA

void FGroomDataflowSettings::SetPreviewBindingAsset(UGroomBindingAsset* BindingAsset)
{
	PreviewBindingAsset = BindingAsset;
}

UGroomBindingAsset* FGroomDataflowSettings::GetPreviewBindingAsset() const
{
	// Load the binding asset if it's not already loaded
	return PreviewBindingAsset.LoadSynchronous();
}

void FGroomDataflowSettings::SetPreviewAnimationAsset(UAnimationAsset* AnimationAsset)
{
	PreviewAnimationAsset = AnimationAsset;
}

UAnimationAsset* FGroomDataflowSettings::GetPreviewAnimationAsset() const
{
	// Load the animation asset if it's not already loaded
	return PreviewAnimationAsset.LoadSynchronous();
}

#endif

FGroomDataflowSettings::FGroomDataflowSettings(UObject* InOwner, UDataflow* InDataflowAsset, FName InTerminalNodeName) : FDataflowInstance(InOwner, InDataflowAsset, InTerminalNodeName)
{
	RestCollection = MakeShared<FManagedArrayCollection>();
}

FName FGroomDataflowSettings::GetDataflowAssetMemberName()
{
	return FName("DataflowAsset");
}

FName FGroomDataflowSettings::GetDataflowTerminalMemberName()
{
	return FName("DataflowTerminal");
}

bool FGroomDataflowSettings::Serialize(FArchive& Ar)
{
	// Serialize normal tagged property data
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		UScriptStruct* const Struct = FGroomDataflowSettings::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), Struct, nullptr);
	}
	if(RestCollection.IsValid())
	{
		Chaos::FChaosArchive ChaosAr(Ar);
		RestCollection->Serialize(ChaosAr);
	}
	
	return true;
}
