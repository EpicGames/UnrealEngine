// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetInterpolation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetInterpolation)

FHairDecimationSettings::FHairDecimationSettings()
{
	CurveDecimation = 1;
	VertexDecimation = 1;
}

FHairInterpolationSettings::FHairInterpolationSettings()
{
	bOverrideGuides_DEPRECATED = false;
	HairToGuideDensity = 0.1f;
	InterpolationQuality = EHairInterpolationQuality::High;
	InterpolationDistance = EHairInterpolationWeight::Parametric;
	bRandomizeGuide = false;
	bUseUniqueGuide = false;
	GuideType = EGroomGuideType::Imported;
	RiggedGuideNumCurves = 10;
	RiggedGuideNumPoints = 4;
}

FHairDeformationSettings::FHairDeformationSettings()
{
	bEnableRigging_DEPRECATED = false;
	NumCurves_DEPRECATED = 10;
	NumPoints_DEPRECATED = 4;
}

FHairGroupsInterpolation::FHairGroupsInterpolation()
{
	DecimationSettings = FHairDecimationSettings();
	InterpolationSettings = FHairInterpolationSettings();
}

bool FHairLODSettings::operator==(const FHairLODSettings& A) const
{
	return
		VertexDecimation == A.VertexDecimation &&
		AngularThreshold == A.AngularThreshold &&
		CurveDecimation == A.CurveDecimation &&
		ThicknessScale == A.ThicknessScale &&
		bVisible == A.bVisible &&
		ScreenSize == A.ScreenSize &&
		GeometryType == A.GeometryType &&
		BindingType == A.BindingType &&
		Simulation == A.Simulation &&
		GlobalInterpolation == A.GlobalInterpolation;
}

bool FHairGroupsLOD::operator==(const FHairGroupsLOD& A) const
{
	if (LODs.Num() != A.LODs.Num())
	{
		return false;
	}

	for (uint32 LODIt = 0, LODCount = LODs.Num(); LODIt < LODCount; ++LODIt)
	{
		if (!(LODs[LODIt] == A.LODs[LODIt]))
		{
			return false;
		}
	}

	return true;
}


bool FHairDecimationSettings::operator==(const FHairDecimationSettings& A) const
{
	return
		CurveDecimation == A.CurveDecimation &&
		VertexDecimation == A.VertexDecimation;
}

bool FHairInterpolationSettings::operator==(const FHairInterpolationSettings& A) const
{
	return
		GuideType == A.GuideType &&
		HairToGuideDensity == A.HairToGuideDensity &&
		InterpolationQuality == A.InterpolationQuality &&
		InterpolationDistance == A.InterpolationDistance &&
		bRandomizeGuide == A.bRandomizeGuide &&
		bUseUniqueGuide == A.bUseUniqueGuide &&
		RiggedGuideNumCurves == A.RiggedGuideNumCurves &&
		RiggedGuideNumPoints == A.RiggedGuideNumPoints;
}

bool FHairGroupsInterpolation::operator==(const FHairGroupsInterpolation& A) const
{
	return
		DecimationSettings == A.DecimationSettings &&
		InterpolationSettings == A.InterpolationSettings;
}

void FHairGroupsInterpolation::BuildDDCKey(FArchive& Ar)
{
	Ar << DecimationSettings.CurveDecimation;
	Ar << DecimationSettings.VertexDecimation;
	Ar << InterpolationSettings.GuideType;
	Ar << InterpolationSettings.HairToGuideDensity;
	Ar << InterpolationSettings.InterpolationQuality;
	Ar << InterpolationSettings.InterpolationDistance;
	Ar << InterpolationSettings.bRandomizeGuide;
	Ar << InterpolationSettings.bUseUniqueGuide;
	Ar << InterpolationSettings.RiggedGuideNumCurves;
	Ar << InterpolationSettings.RiggedGuideNumPoints;
}

void FHairGroupsLOD::BuildDDCKey(FArchive& Ar)
{
	for (FHairLODSettings& LOD : LODs)
	{
		Ar << LOD.CurveDecimation;
		Ar << LOD.VertexDecimation;
		Ar << LOD.AngularThreshold;
		Ar << LOD.ScreenSize;
		Ar << LOD.ThicknessScale;
		Ar << LOD.bVisible;
		Ar << LOD.BindingType;
		Ar << LOD.GeometryType;
		Ar << LOD.Simulation;
		Ar << LOD.GlobalInterpolation;
	}
}

FHairGroupsLOD FHairGroupsLOD::GetDefault()
{
	FHairGroupsLOD Out;
	FHairLODSettings& S = Out.LODs.AddDefaulted_GetRef();
	S.ScreenSize = 1;
	S.CurveDecimation = 1;
	return Out;
}