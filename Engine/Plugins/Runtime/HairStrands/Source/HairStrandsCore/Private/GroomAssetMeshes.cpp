// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetMeshes.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetMeshes)

FHairGroupsMeshesSourceDescription::FHairGroupsMeshesSourceDescription()
{
	 ImportedMesh = nullptr;
	 MaterialSlotName = NAME_None;
	 GroupIndex = 0;
	 LODIndex = -1;
}

bool FHairGroupsMeshesSourceDescription::operator==(const FHairGroupsMeshesSourceDescription& A) const
{
	return
		GroupIndex == A.GroupIndex && 
		LODIndex == A.LODIndex &&
		MaterialSlotName == A.MaterialSlotName &&
		ImportedMesh == A.ImportedMesh;
}

FString FHairGroupsMeshesSourceDescription::GetMeshKey() const
{
#if WITH_EDITORONLY_DATA
	if (ImportedMesh)
	{
		ImportedMesh->ConditionalPostLoad();
		FStaticMeshSourceModel& SourceModel = ImportedMesh->GetSourceModel(0);
		if (SourceModel.GetMeshDescriptionBulkData())
		{
			return SourceModel.GetMeshDescriptionBulkData()->GetUnversionedIdString();
		}
	}
#endif
	return TEXT("INVALID_MESH");
}

bool FHairGroupsMeshesSourceDescription::HasMeshChanged() const
{
#if WITH_EDITORONLY_DATA
	if (ImportedMesh)
	{
		return ImportedMeshKey != GetMeshKey();
	}
#endif
	return false;
}

void FHairGroupsMeshesSourceDescription::UpdateMeshKey()
{
#if WITH_EDITORONLY_DATA
	if (ImportedMesh)
	{
		ImportedMeshKey = GetMeshKey();
	}
#endif
}
