// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetPhysics.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetPhysics)

FHairSolverSettings::FHairSolverSettings()
{
	EnableSimulation = false;
	bEnableDeformation = false;
	NiagaraSolver = EGroomNiagaraSolvers::AngularSprings;
	SubSteps = 5;
	IterationCount = 5;
	GravityPreloading = 0.0f;
	bForceVisible = false;
}

FHairExternalForces::FHairExternalForces()
{
	GravityVector = FVector(0.0, 0.0, -981.0);
	AirDrag = 0.1f;
	AirVelocity = FVector(0, 0, 0);
}

FHairBendConstraint::FHairBendConstraint()
{
	SolveBend = true;
	ProjectBend = false;
	BendDamping = 0.005f;
	BendStiffness = 0.01f;

	BendScale.GetRichCurve()->SetKeyInterpMode(BendScale.GetRichCurve()->AddKey(0.f, 1.f), ERichCurveInterpMode::RCIM_Cubic);
	BendScale.GetRichCurve()->SetKeyInterpMode(BendScale.GetRichCurve()->AddKey(1.f, 1.f), ERichCurveInterpMode::RCIM_Cubic);
}

FHairStretchConstraint::FHairStretchConstraint()
{
	SolveStretch = true;
	ProjectStretch = false;
	StretchDamping = 0.005f;
	StretchStiffness = 1.0f;

	StretchScale.GetRichCurve()->SetKeyInterpMode(StretchScale.GetRichCurve()->AddKey(0.f, 1.f), ERichCurveInterpMode::RCIM_Cubic);
	StretchScale.GetRichCurve()->SetKeyInterpMode(StretchScale.GetRichCurve()->AddKey(1.f, 1.f), ERichCurveInterpMode::RCIM_Cubic);
}

FHairCollisionConstraint::FHairCollisionConstraint()
{
	SolveCollision = true;
	ProjectCollision = true;
	KineticFriction = 0.1f;
	StaticFriction = 0.1f;
	StrandsViscosity = 1.0f;
	CollisionRadius = 0.1f;
	GridDimension = FIntVector(30,30,30);

	RadiusScale.GetRichCurve()->SetKeyInterpMode(RadiusScale.GetRichCurve()->AddKey(0.f, 1.0f), ERichCurveInterpMode::RCIM_Cubic);
	RadiusScale.GetRichCurve()->SetKeyInterpMode(RadiusScale.GetRichCurve()->AddKey(1.f, 0.1f), ERichCurveInterpMode::RCIM_Cubic);
}

FHairMaterialConstraints::FHairMaterialConstraints()
{
	BendConstraint = FHairBendConstraint();
	StretchConstraint = FHairStretchConstraint();
	CollisionConstraint = FHairCollisionConstraint();
}

FHairStrandsParameters::FHairStrandsParameters()
{
	StrandsSize = EGroomStrandsSize::Size8;
	StrandsDensity = 1.0f;
	StrandsSmoothing = 0.1f;
	StrandsThickness = 0.01f;

	ThicknessScale.GetRichCurve()->SetKeyInterpMode(ThicknessScale.GetRichCurve()->AddKey(0.f, 1.0f), ERichCurveInterpMode::RCIM_Cubic);
	ThicknessScale.GetRichCurve()->SetKeyInterpMode(ThicknessScale.GetRichCurve()->AddKey(1.f, 1.0f), ERichCurveInterpMode::RCIM_Cubic);
}

FHairGroupsPhysics::FHairGroupsPhysics()
{
	SolverSettings = FHairSolverSettings();
	ExternalForces = FHairExternalForces();
	MaterialConstraints = FHairMaterialConstraints();
	StrandsParameters = FHairStrandsParameters();
}

void FHairGroupsPhysics::BuildDDCKey(FArchive& Ar)
{
	// This is used to know if curve/point mapping needs to be built for a particular groom/group
	Ar << SolverSettings.bEnableDeformation;
}

bool FHairSolverSettings::operator==(const FHairSolverSettings& A) const
{
	return 
		EnableSimulation == A.EnableSimulation &&
		bEnableDeformation == A.bEnableDeformation &&
		NiagaraSolver == A.NiagaraSolver &&
		SubSteps == A.SubSteps &&
		IterationCount == A.IterationCount &&
		bForceVisible == A.bForceVisible;
}

bool FHairExternalForces::operator==(const FHairExternalForces& A) const
{
	return
		GravityVector == A.GravityVector &&
		AirDrag == A.AirDrag &&
		AirVelocity == A.AirVelocity;
}
bool FHairBendConstraint::operator==(const FHairBendConstraint& A) const
{
	return
		SolveBend == A.SolveBend &&
		ProjectBend == A.ProjectBend &&
		BendDamping == A.BendDamping &&
		BendStiffness == A.BendStiffness;
		//&& BendScale == A.BendScale;
}
bool FHairStretchConstraint::operator==(const FHairStretchConstraint& A) const
{
	return
		SolveStretch == A.SolveStretch &&
		ProjectStretch == A.ProjectStretch &&
		StretchDamping == A.StretchDamping &&
		StretchStiffness == A.StretchStiffness;
		//  &&StretchScale == A.StretchScale;
}
bool FHairCollisionConstraint::operator==(const FHairCollisionConstraint& A) const
{
	return
		SolveCollision == A.SolveCollision &&
		ProjectCollision == A.ProjectCollision &&
		KineticFriction == A.KineticFriction &&
		StaticFriction == A.StaticFriction &&
		StrandsViscosity == A.StrandsViscosity &&
		CollisionRadius == A.CollisionRadius &&
		GridDimension == A.GridDimension;
		//&& RadiusScale == A.RadiusScale;
}

bool FHairMaterialConstraints::operator==(const FHairMaterialConstraints& A) const
{
	return
		BendConstraint == A.BendConstraint &&
		StretchConstraint == A.StretchConstraint &&
		CollisionConstraint == A.CollisionConstraint;
}
bool FHairStrandsParameters::operator==(const FHairStrandsParameters& A) const
{
	return
		StrandsSize == A.StrandsSize &&
		StrandsDensity == A.StrandsDensity &&
		StrandsSmoothing == A.StrandsSmoothing &&
		StrandsThickness == A.StrandsThickness;
	// && ThicknessScale == A.ThicknessScale;
}
bool FHairGroupsPhysics::operator==(const FHairGroupsPhysics& A) const
{
	return
		SolverSettings == A.SolverSettings &&
		ExternalForces == A.ExternalForces &&
		MaterialConstraints == A.MaterialConstraints &&
		StrandsParameters == A.StrandsParameters;
}
