// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetRendering.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetRendering)

FHairGeometrySettings::FHairGeometrySettings()
{
	HairWidth = 0.01f;
	HairRootScale = 1;
	HairTipScale = 1;
}

FHairShadowSettings::FHairShadowSettings()
{
	HairShadowDensity = 1;
	HairRaytracingRadiusScale = 1;
	bVoxelize = true;
	bUseHairRaytracingGeometry = false;
};

FHairAdvancedRenderingSettings::FHairAdvancedRenderingSettings()
{
	bUseStableRasterization = false;
	bScatterSceneLighting = false;
};

FHairGroupsRendering::FHairGroupsRendering()
{
	 GeometrySettings	= FHairGeometrySettings();
	 ShadowSettings		= FHairShadowSettings();
	 AdvancedSettings	= FHairAdvancedRenderingSettings();
}

bool FHairGeometrySettings::operator==(const FHairGeometrySettings& A) const
{
	return
		HairWidth == A.HairWidth &&
		HairRootScale == A.HairRootScale &&
		HairTipScale == A.HairTipScale;
}

bool FHairShadowSettings::operator==(const FHairShadowSettings& A) const
{
	return
		HairShadowDensity == A.HairShadowDensity &&
		HairRaytracingRadiusScale == A.HairRaytracingRadiusScale &&
		bUseHairRaytracingGeometry == A.bUseHairRaytracingGeometry &&
		bVoxelize == A.bVoxelize;
}
bool FHairAdvancedRenderingSettings::operator==(const FHairAdvancedRenderingSettings& A) const
{
	return
		bUseStableRasterization == A.bUseStableRasterization &&
		bScatterSceneLighting == A.bScatterSceneLighting;
}
bool FHairGroupsRendering::operator==(const FHairGroupsRendering& A) const
{
	return
		GeometrySettings == A.GeometrySettings &&
		ShadowSettings == A.ShadowSettings &&
		AdvancedSettings == A.AdvancedSettings;
}
