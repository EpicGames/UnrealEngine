// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBindingAsset.h"

#include "EngineUtils.h"
#include "GeometryCache.h"
#include "GeometryCacheMeshData.h"
#include "GroomAsset.h"
#include "GroomBindingCompiler.h"
#include "GroomBindingBuilder.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/StructuredLog.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Serialization/MemoryReader.h"
#include "UObject/UObjectIterator.h"
#include "UObject/AnimObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/DevObjectVersion.h"
#include "Misc/CoreMisc.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "GroomComponent.h"
#include "GroomVisualizationData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomBindingAsset)

#if WITH_EDITORONLY_DATA
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataCacheInterface.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#endif

LLM_DECLARE_TAG(Groom);

static int32 GHairStrandsDelayBindingDataRelease = 1;
static FAutoConsoleVariableRef CVarHairStrandsDelayBindingDataRelease(TEXT("r.HairStrands.Binding.DelayBindingDataRelease"), GHairStrandsDelayBindingDataRelease, TEXT("Delay groom binding data release (Experimental)"));

static TAutoConsoleVariable<int32> CVarMemoryForGroomBindingAssetCompile(
	TEXT("Memory.MemoryForGroomBindingAssetCompile"),
	512,
	TEXT("Memory in MiB set aside for a GroomBindingAsset compile job\n"),
	ECVF_Default);

/////////////////////////////////////////////////////////////////////////////////////////

bool IsHairStrandsDDCLogEnable();
uint32 GetAssetNameHash(const FString& In);

////////////////////////////////////////////////////////////////////////////////////////////////////////////

static void InternalSerializeGuides(FArchive& Ar, UObject* Owner, TArray<FHairStrandsRootBulkData>& Datas)
{
	uint32 MeshLODCount = Datas.Num();
	Ar << MeshLODCount;
	if (Ar.IsLoading())
	{
		Datas.SetNum(MeshLODCount);
	}
	for (FHairStrandsRootBulkData& Data : Datas)
	{
		Data.SerializeHeader(Ar, Owner);
		Data.SerializeData(Ar, Owner);
	}
}

static void InternalSerializeStrands(FArchive& Ar, UObject* Owner, TArray<FHairStrandsRootBulkData>& Datas, uint32 Flags, bool bHeader, bool bData)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	const bool bStripped = (Flags & UGroomAsset::CDSF_StrandsStripped);
	if (!bStripped)
	{
		uint32 MeshLODCount = Datas.Num();
		Ar << MeshLODCount;
		if (Ar.IsLoading())
		{
			Datas.SetNum(MeshLODCount);
		}
		for (FHairStrandsRootBulkData& Data : Datas)
		{
			if (bHeader){ Data.SerializeHeader(Ar, Owner); }
			if (bData)	{ Data.SerializeData(Ar, Owner); }
		}
	}
}

static void InternalSerializeCards(FArchive& Ar, UObject* Owner, TArray<TArray<FHairStrandsRootBulkData>>& Datass)
{
	uint32 CardLODCount = Datass.Num();
	Ar << CardLODCount;
	if (Ar.IsLoading())
	{
		Datass.SetNum(CardLODCount);
	}
	for (TArray<FHairStrandsRootBulkData>& Datas : Datass)
	{
		uint32 MeshLODCount = Datas.Num();
		Ar << MeshLODCount;
		if (Ar.IsLoading())
		{
			Datas.SetNum(MeshLODCount);
		}
		for (FHairStrandsRootBulkData& Data : Datas)
		{	
			Data.SerializeHeader(Ar, Owner);
			Data.SerializeData(Ar, Owner);
		}
	}
}

static void InternalSerializePlatformData(FArchive& Ar, UObject* Owner, UGroomBindingAsset::FHairGroupPlatformData& GroupData, uint32 Flags, bool bHeader, bool bData)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	// Guides
	InternalSerializeGuides(Ar, Owner, GroupData.SimRootBulkDatas);

	// Strands
	InternalSerializeStrands(Ar, Owner, GroupData.RenRootBulkDatas, Flags, bHeader, bData);

	// Cards
	InternalSerializeCards(Ar, Owner, GroupData.CardsRootBulkDatas);

	Ar << GroupData.TargetMeshMinLOD;
}

static void InternalSerializePlatformDatas(FArchive& Ar, UObject* Owner, TArray<UGroomBindingAsset::FHairGroupPlatformData>& GroupDatas, uint32 Flags)
{
	uint32 GroupCount = GroupDatas.Num();
	Ar << Flags;
	Ar << GroupCount;
	if (Ar.IsLoading())
	{
		GroupDatas.SetNum(GroupCount);
	}
	for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
	{
		InternalSerializePlatformData(Ar, Owner, GroupDatas[GroupIt], Flags, true /*bHeader*/, true /*bData*/);
	}
}

/*-----------------------------------------------------------------------------
	FGroomBindingAsyncBuildScope
-----------------------------------------------------------------------------*/
thread_local const UGroomBindingAsset* FGroomBindingAsyncBuildScope::Asset = nullptr;

FGroomBindingAsyncBuildScope::FGroomBindingAsyncBuildScope(const UGroomBindingAsset* InAsset)
{
	PreviousScope = Asset;
	Asset = InAsset;
}

FGroomBindingAsyncBuildScope::~FGroomBindingAsyncBuildScope()
{
	check(Asset);
	Asset = PreviousScope;
}

bool FGroomBindingAsyncBuildScope::ShouldWaitOnLockedProperties(const UGroomBindingAsset* InAsset)
{
	return Asset != InAsset;
}

/*-----------------------------------------------------------------------------
	FGroomBindingAsyncBuildWorker
-----------------------------------------------------------------------------*/
void FGroomBindingAsyncBuildWorker::DoWork()
{
	if (BuildContext.IsSet())
	{
		GroomBinding->ExecuteCacheDerivedDatas(*BuildContext);
	}
}

void UGroomBindingAsset::WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties AsyncProperties, EGroomBindingAsyncPropertyLockType LockType) const
{
#if WITH_EDITOR
	// We need to protect internal asset data from race conditions during async build
	if (IsCompiling())
	{
		if (FGroomBindingAsyncBuildScope::ShouldWaitOnLockedProperties(this))
		{
			bool bIsLocked = true;
			// We can remove the lock if we're accessing in read-only and there is no write-lock
			if ((LockType & EGroomBindingAsyncPropertyLockType::ReadOnly) == EGroomBindingAsyncPropertyLockType::ReadOnly)
			{
				// Maintain the lock if the write-lock bit is non-zero
				bIsLocked &= (ModifiedProperties & (uint64)AsyncProperties) != 0;
			}

			if (bIsLocked)
			{
				FString PropertyName = StaticEnum<EGroomBindingAsyncProperties>()->GetNameByValue((int64)AsyncProperties).ToString();
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("GroomBindingCompilationStall %s"), *PropertyName));

				if (IsInGameThread())
				{
					UE_LOGF(
						LogHairStrands,
						Verbose,
						"Accessing property %ls of the GroomBindingAsset while it is still being built asynchronously will force it to be compiled before continuing. "
							"For better performance, consider making the caller async aware so it can wait until the groom binding is ready to access this property."
							"To better understand where those calls are coming from, you can use Editor.AsyncAssetDumpStallStacks on the console.",
						*PropertyName
					);

					FGroomBindingCompilingManager::Get().FinishCompilation({ const_cast<UGroomBindingAsset*>(this) });
				}
				else
				{
					// Trying to access a property from another thread that cannot force finish the compilation is invalid
					ensureMsgf(
						false,
						TEXT("Accessing property %s of the GroomBindingAsset while it is still being built asynchronously is only supported on the game-thread. "
							"To avoid any race-condition, consider finishing the compilation before pushing tasks to other threads or making higher-level game-thread code async aware so it "
							"schedules the task only when the groom binding's compilation is finished. If this is a blocker, you can disable async groom binding compilation from the editor experimental settings."),
						*PropertyName
					);
				}
			}
		}
		// If we're accessing this property from the async build thread, make sure the property is still protected from access from other threads.
		else
		{
			bool bIsLocked = true;
			if ((LockType & EGroomBindingAsyncPropertyLockType::ReadOnly) == EGroomBindingAsyncPropertyLockType::ReadOnly)
			{
				bIsLocked &= (AccessedProperties & (uint64)AsyncProperties) != 0;
			}

			if ((LockType & EGroomBindingAsyncPropertyLockType::WriteOnly) == EGroomBindingAsyncPropertyLockType::WriteOnly)
			{
				bIsLocked &= (ModifiedProperties & (uint64)AsyncProperties) != 0;
			}
			ensureMsgf(bIsLocked, TEXT("Property %s has not been locked properly for use by async build"), *StaticEnum<EGroomBindingAsyncProperties>()->GetNameByValue((int64)AsyncProperties).ToString());
		}
	}
#endif
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UGroomBindingAsset::Serialize(FArchive& Ar)
{
	uint8 Flags = 0;
#if WITH_EDITOR
	if (GetGroom())
	{
		Flags = GetGroom()->GenerateClassStripFlags(Ar);
	}

	if (IsCompiling())
	{
		// Skip serialization during compilation if told to do so.
		if (Ar.ShouldSkipCompilingAssets())
		{
			return;
		}

		// Since UPROPERTY are accessed directly by offset during serialization instead of using accessors,
		// the protection put in place to automatically finish compilation if a locked property is accessed will not work.
		// We have no choice but to force finish the compilation here to avoid potential race conditions between 
		// async compilation and the serialization.
		FGroomBindingCompilingManager::Get().FinishCompilation({ this });
	}
#endif

	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	// Early out if hair strands are disabled
	const bool bIsHairEnabled = IsGroomEnabled();
	if (!bIsHairEnabled)
	{
		return;
	}

#if WITH_EDITOR
	// When using editor:
	// * The header are loaded in CacheDerivedData(), and the data are streamed from DDC
	// * When cooking, we write out data from the cached cooked platform data
	if (Ar.IsCooking())
	{
		if (TArray<UGroomBindingAsset::FHairGroupPlatformData>* CookedDatas = GetCachedCookedPlatformData(Ar.CookingTarget()))
		{
			InternalSerializePlatformDatas(Ar, this /*Owner*/, *CookedDatas, Flags);
			bIsValid = true;
		}
		else
		{
			UE_LOGF(LogHairStrands, Error, "[Groom] The binding asset (%ls) has missing cooked platform data.", *GetName());
		}
	}
#else
	// Always loaded dara from the archive when not using the editor
	{
		InternalSerializePlatformDatas(Ar, this, GetHairGroupsPlatformData(), Flags);
		bIsValid = true;
	}
#endif
}

void UGroomBindingAsset::InitResource()
{
	LLM_SCOPE_BYTAG(Groom);

	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomBindingAsset::InitResource);

	// Ensure we are releasing binding resources before reallocating them
	ReleaseResource(true/*bResetLoadedSize*/);

	for (UGroomBindingAsset::FHairGroupPlatformData& BulkData : GetHairGroupsPlatformData())
	{
		const int32 GroupIndex = GetHairGroupResources().Num();
		FHairGroupResource& Resource = GetHairGroupResources().AddDefaulted_GetRef();

		FHairResourceName ResourceName(GetFName(), GroupIndex);
		const FName OwnerName = GetAssetPathName();

		// Guides
		Resource.SimRootResources = nullptr;
		if (BulkData.SimRootBulkDatas.Num() > 0)
		{
			Resource.SimRootResources = new FHairStrandsRestRootResource(BulkData.SimRootBulkDatas, EHairStrandsResourcesType::Guides, ResourceName, OwnerName);
			Resource.SimRootResources->BeginInitResource();
		}

		// Strands
		Resource.RenRootResources = nullptr;
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands) && BulkData.RenRootBulkDatas.Num() > 0)
		{
			Resource.RenRootResources = new FHairStrandsRestRootResource(BulkData.RenRootBulkDatas, EHairStrandsResourcesType::Strands, ResourceName, OwnerName);
			Resource.RenRootResources->BeginInitResource();
		}

		// Cards
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards))
		{
			const uint32 CardsLODCount = BulkData.CardsRootBulkDatas.Num();
			Resource.CardsRootResources.SetNum(CardsLODCount);
			for (uint32 CardsLODIt=0; CardsLODIt<CardsLODCount; ++CardsLODIt)
			{
				Resource.CardsRootResources[CardsLODIt] = nullptr;
				if (BulkData.CardsRootBulkDatas[CardsLODIt].Num() > 0)
				{
					Resource.CardsRootResources[CardsLODIt] = new FHairStrandsRestRootResource(BulkData.CardsRootBulkDatas[CardsLODIt], EHairStrandsResourcesType::Cards, FHairResourceName(GetFName(), GroupIndex, CardsLODIt), GetAssetPathName(CardsLODIt));
					Resource.CardsRootResources[CardsLODIt]->BeginInitResource();
				}
			}
		}
	}
}

void UGroomBindingAsset::UpdateResource()
{
	for (FHairGroupResource& Resource : GetHairGroupResources())
	{
		if (Resource.SimRootResources)
		{
			Resource.SimRootResources->BeginUpdateResourceRHI();
		}

		if (Resource.RenRootResources)
		{
			Resource.RenRootResources->BeginUpdateResourceRHI();
		}

		for (FHairStrandsRestRootResource* CardsRootResource : Resource.CardsRootResources)
		{
			if (CardsRootResource)
			{
				CardsRootResource->BeginUpdateResourceRHI();
			}
		}
	}
}

void UGroomBindingAsset::ReleaseResource(bool bResetLoadedSize)
{
	// Delay destruction to insure that the rendering thread is done with all resources usage
	if (GetHairGroupResources().Num() > 0)
	{
		for (FHairGroupResource& Resource : GetHairGroupResources())
		{
			FHairStrandsRestRootResource* InSimRootResources = Resource.SimRootResources;
			FHairStrandsRestRootResource* InRenRootResources = Resource.RenRootResources;
			ENQUEUE_RENDER_COMMAND(ReleaseHairStrandsResourceCommand)(UE::RenderCommandPipe::Groom,
				[InSimRootResources, InRenRootResources, bResetLoadedSize](FRHICommandList& RHICmdList)
			{
				if (InSimRootResources)
				{
					InSimRootResources->ReleaseResource();
					delete InSimRootResources;
				}
				if (InRenRootResources)
				{
					if (bResetLoadedSize)
					{
						InRenRootResources->InternalResetLoadedSize();
					}
					InRenRootResources->ReleaseResource();
					delete InRenRootResources;
				}
			});
			Resource.SimRootResources = nullptr;
			Resource.RenRootResources = nullptr;

			for (FHairStrandsRestRootResource*& InCardsRootResources : Resource.CardsRootResources)
			{
				ENQUEUE_RENDER_COMMAND(ReleaseHairStrandsResourceCommand)(UE::RenderCommandPipe::Groom,
					[InCardsRootResources](FRHICommandList& RHICmdList)
					{
						if (InCardsRootResources)
						{
							InCardsRootResources->ReleaseResource();
							delete InCardsRootResources;
						}
					});
				InCardsRootResources = nullptr;
			}
		}
		GetHairGroupResources().Empty();
	}

	// Process resources to be deleted (should happen only in editor)
	FHairGroupResource ResourceToDelete;
	while (RemoveHairGroupResourcesToDelete(ResourceToDelete))
	{
		FHairStrandsRestRootResource* InSimRootResources = ResourceToDelete.SimRootResources;
		FHairStrandsRestRootResource* InRenRootResources = ResourceToDelete.RenRootResources;
		TArray<FHairStrandsRestRootResource*> InCardsRootResources = ResourceToDelete.CardsRootResources;
		ENQUEUE_RENDER_COMMAND(ReleaseHairStrandsResourceCommand)(UE::RenderCommandPipe::Groom,
			[InSimRootResources, InRenRootResources, InCardsRootResources](FRHICommandList& RHICmdList)
		{
			if (InSimRootResources)
			{
				InSimRootResources->ReleaseResource();
				delete InSimRootResources;
			}
			if (InRenRootResources)
			{
				InRenRootResources->ReleaseResource();
				delete InRenRootResources;
			}
			for (FHairStrandsRestRootResource* InCardRootResource : InCardsRootResources)
			{
				if (InCardRootResource)
				{
					InCardRootResource->ReleaseResource();
					delete InCardRootResource;
				}
			}
		});
	}

	// Process data to be deleted (should happen only in editor)
	FHairGroupPlatformDataArray* DataToDelete = nullptr;
	while (HairGroupPlatformDataToDelete.Dequeue(DataToDelete))
	{
		ENQUEUE_RENDER_COMMAND(ReleaseHairStrandsPlatformDataCommand)(UE::RenderCommandPipe::Groom,
		[DataToDelete](FRHICommandList& RHICmdList)
		{
			DataToDelete->Data.Empty();
			delete DataToDelete;
		});
	}
}

void UGroomBindingAsset::FlushRenderingCommandIfUsed(const UGroomBindingAsset* In)
{
	bool bNeedFlushRenderingCommand = false;
	for (TObjectIterator<UGroomComponent> HairStrandsComponentIt; HairStrandsComponentIt; ++HairStrandsComponentIt)
	{
		if (HairStrandsComponentIt->BindingAsset == In
			#if WITH_EDITORONLY_DATA
			|| HairStrandsComponentIt->BindingAssetBeingLoaded == In
			#endif
			)
		{
			if (HairStrandsComponentIt->IsRenderStateCreated())
			{
				HairStrandsComponentIt->DestroyRenderState_Concurrent();
				bNeedFlushRenderingCommand = true;
			}
		}
	}
	if (bNeedFlushRenderingCommand)
	{
		FlushRenderingCommands();
	}
}

void UGroomBindingAsset::StreamInForCPUAccess(bool bWait)
{
	check(IsValid());

	for (UGroomBindingAsset::FHairGroupPlatformData& Data : GetHairGroupsPlatformData())
	{
		for (FHairStrandsRootBulkData& BulkData : Data.RenRootBulkDatas)
		{
			FHairStreamingRequest R;
			R.Request(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, BulkData, bWait, true /*bFillBulkdata*/, false /*bWarmCache*/, GetFName());
		}
	}
}

void UGroomBindingAsset::Reset()
{
	FlushRenderingCommandIfUsed(this);

	// No need for resetting LoadedSize as the bulk datas are removed (e.g., when reloading/reimporting a groom asset)
	ReleaseResource(false/*bResetLoadedSize*/);
	for (UGroomBindingAsset::FHairGroupPlatformData& Data : GetHairGroupsPlatformData())
	{
		Data.SimRootBulkDatas.Empty();
		Data.RenRootBulkDatas.Empty();

		for (TArray<FHairStrandsRootBulkData>& CardsRootBulkData : Data.CardsRootBulkDatas)
		{
			CardsRootBulkData.Empty();
		}
	}

	bIsValid = false;
}

bool UGroomBindingAsset::HasAnyDependenciesCompiling() const
{
#if WITH_EDITORONLY_DATA
	return
		(GetTargetSkeletalMesh() && GetTargetSkeletalMesh()->IsCompiling()) ||
		(GetSourceSkeletalMesh() && GetSourceSkeletalMesh()->IsCompiling());
#else
	return false;
#endif
}

#if WITH_EDITORONLY_DATA
void UGroomBindingAsset::InvalidateBinding(class USkeletalMesh*)
{
	InvalidateBinding();
}

void UGroomBindingAsset::InvalidateBinding()
{
	// Nothing else to do as we're going to be rebuilt when our dependencies finish caching anyway
	if (HasAnyDependenciesCompiling())
	{
		return;
	}

	BeginCacheDerivedDatas();
}

#endif

void UGroomBindingAsset::PostLoad()
{
	LLM_SCOPE_BYTAG(Groom);

	Super::PostLoad();

	// Early out if hair strands are disabled
	const bool bIsHairEnabled = IsGroomEnabled();
	if (!bIsHairEnabled)
	{
		return;
	}

	// Compute a hash of the Groom asset fullname for finding unique groom during LOD selection/streaming
	AssetNameHash = GetAssetNameHash(GetFullName());

	if (UGroomAsset* LocalGroom = GetGroom())
	{
		// Make sure that the asset initialized its resources first since the component needs them to initialize its own resources
		LocalGroom->ConditionalPostLoad();

	#if WITH_EDITOR
		if (GetTargetSkeletalMesh())
		{
			GetTargetSkeletalMesh()->ConditionalPostLoad();
		}

		if (GetSourceSkeletalMesh())
		{
			GetSourceSkeletalMesh()->ConditionalPostLoad();
		}

		if (GetTargetGeometryCache())
		{
			GetTargetGeometryCache()->ConditionalPostLoad();
		}
		
		if (GetSourceGeometryCache())
		{
			GetSourceGeometryCache()->ConditionalPostLoad();
		}

		RegisterSkeletalMeshDelegates();
		RegisterGroomDelegates();

		InvalidateBinding();
	#endif
	}

	// * When running with the editor, InitResource is called in CacheDerivedDatas
	// * When running without the editor, InitResource is explicitely called here
#if !WITH_EDITOR
	if (!IsTemplate() && IsValid())
	{
		InitResource();
	}
#endif
}

void UGroomBindingAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
#if WITH_EDITOR
	OnGroomBindingAssetChanged.Broadcast();
#endif
}

void UGroomBindingAsset::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);
#if WITH_EDITOR
	OnGroomBindingAssetChanged.Broadcast();
#endif
}

void UGroomBindingAsset::BeginDestroy()
{
	ReleaseResource(false/*bResetLoadedSize*/);
	Super::BeginDestroy();

#if WITH_EDITOR
	UnregisterSkeletalMeshDelegates();

	UnregisterGroomDelegates();
#endif
}

bool UGroomBindingAsset::IsCompatible(const USkeletalMesh* InSkeletalMesh, const UGroomBindingAsset* InBinding, bool bIssueWarning)
{
	if (InBinding && InSkeletalMesh && IsHairStrandsBindingEnable())
	{
		if (InBinding->GetGroomBindingType() != EGroomBindingMeshType::SkeletalMesh)
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The binding asset (%ls) cannot be bound to a SkeletalMesh because it is not the correct binding type.", *InBinding->GetName());
			}
			return false;
		}

		if (!InBinding->GetTargetSkeletalMesh())
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The binding asset (%ls) does not have a target skeletal mesh. Falling back onto non-binding version.", *InBinding->GetName());
			}
			return false;
		}
		
		// Relax the mismatch of skeletal mesh as this is not necesarely a good metric: something the same skeletal mesh can be imported with/without animation, and all projection data 
		// matches in this case and it is useful to be able to reuse the binding asset in this case
		#if 0
		// TODO: need something better to assess that skeletal meshes match. In the mean time, string comparison. 
		// Since they can be several instances of a skeletalMesh asset (??), a numerical suffix is sometime added to the name (e.g., SkeletalName_0).
		// This is why we are using substring comparison.
		//if (InSkeletalMesh->GetPrimaryAssetId() != InBinding->TargetSkeletalMesh->GetPrimaryAssetId())
		if (!InSkeletalMesh->GetName().Contains(InBinding->TargetSkeletalMesh->GetName()))
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The Groom binding (%ls) does not reference the same skeletal asset (BindingAsset's skeletal:%ls vs. Attached skeletal:%ls). The binding asset will not be used.",
					*InBinding->GetName(),
					*InBinding->TargetSkeletalMesh->GetName(),
					*InSkeletalMesh->GetName());
			}
			return false;
		}
		#endif

		for (const FHairGroupResource& Resource : InBinding->GetHairGroupResources())
		{
			if (Resource.SimRootResources && InSkeletalMesh->GetLODNum() != Resource.SimRootResources->GetLODCount())
			{
				if (bIssueWarning)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] The Groom binding (%ls) does not have the same have LOD count (LOD sim:%d) than the skeletal mesh (%ls, LOD:%d). The binding asset will not be used.",
						*InBinding->GetName(),
						Resource.SimRootResources->GetLODCount(),
						*InSkeletalMesh->GetName(),
						InSkeletalMesh->GetLODNum());
				}
				return false;
			}

			if (Resource.RenRootResources && InSkeletalMesh->GetLODNum() != Resource.RenRootResources->GetLODCount() && IsHairStrandsEnabled(EHairStrandsShaderType::Strands))
			{
				if (bIssueWarning)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] The Groom binding (%ls) does not have the same have LOD count (LOD render:%d) than the skeletal mesh (%ls, LOD:%d). The binding asset will not be used.",
						*InBinding->GetName(),
						Resource.RenRootResources->GetLODCount(),
						*InSkeletalMesh->GetName(),
						InSkeletalMesh->GetLODNum());
				}
				return false;
			}
		}
	}

	return true;
}

bool UGroomBindingAsset::IsCompatible(const UGeometryCache* InGeometryCache, const UGroomBindingAsset* InBinding, bool bIssueWarning)
{
	if (InBinding && InGeometryCache && IsHairStrandsBindingEnable())
	{
		if (InBinding->GetGroomBindingType() != EGroomBindingMeshType::GeometryCache)
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The binding asset (%ls) cannot be bound to a GeometryCache because it is not the correct binding type.", *InBinding->GetName());
			}
			return false;
		}

		if (!InBinding->GetTargetGeometryCache())
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The binding asset (%ls) does not have a target GeometryCache.", *InBinding->GetName());
			}
			return false;
		}

		TArray<FGeometryCacheMeshData> MeshesData;
		InGeometryCache->GetMeshDataAtTime(0.0f, MeshesData);
		if (MeshesData.Num() > 1)
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] Cannot be bound to a non-flattened GeometryCache. Re-import %ls with 'Flatten Tracks' enabled.", *InGeometryCache->GetName());
			}
			return false;
		}
		else if (MeshesData.Num() == 0)
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] %ls is not a valid GeometryCache to bind to.", *InGeometryCache->GetName());
			}
			return false;
		}
	}

	return true;
}

static bool DoesGroomNeedStrandsBinding(const UGroomAsset* InGroom, uint32 InGroupIndex)
{
	if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands))
	{
		for (const FHairLODSettings& LODSettings : InGroom->GetHairGroupsLOD()[InGroupIndex].LODs)
		{
			if (LODSettings.bVisible && LODSettings.GeometryType == EGroomGeometryType::Strands)
			{
				return true;
			}
		}
	}
	return false;
}

bool UGroomBindingAsset::IsCompatible(const UGroomAsset* InGroom, const UGroomBindingAsset* InBinding, bool bIssueWarning)
{
	if (InBinding && InGroom && IsHairStrandsBindingEnable())
	{
		if (InBinding->GetGroom() && !InBinding->GetGroom()->IsValid())
		{
			// The groom could be invalid if it's still being loaded asynchronously
			return false;
		}

		if (!InBinding->GetGroom())
		{
			UE_CLOGF(bIssueWarning, LogHairStrands, Warning, "[Groom] The binding asset (%ls) does not reference a groom. Falling back onto non-binding version.", *InBinding->GetName());
			return false;
		}

		if (InGroom->GetPrimaryAssetId() != InBinding->GetGroom()->GetPrimaryAssetId())
		{
			UE_LOGF(LogHairStrands, Warning, "[Groom] The Groom binding (%ls) does not reference the same groom asset (BindingAsset's groom:%ls vs. Groom:%ls). The binding asset will not be used.", 
				*InBinding->GetName(),
				*InBinding->GetGroom()->GetName(),
				*InGroom->GetName());
			return false;
		}

		const uint32 GroupCount = InGroom->GetNumHairGroups();
		if (GroupCount != InBinding->GetGroupInfos().Num())
		{
			UE_CLOGF(bIssueWarning, LogHairStrands, Warning, "[Groom] The GroomBinding asset (%ls) does not contains the same number of groups (%d vs. %d) than the groom (%ls). The binding asset will not be used.",
				*InBinding->GetName(),
				GroupCount,
				InBinding->GetGroupInfos().Num(),
				*InGroom->GetName());
			return false;
		}

		for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
		{
			// Guides
			{
				const uint32 GroomCount = InGroom->GetHairGroupsPlatformData()[GroupIt].Guides.BulkData.GetNumCurves();
				const uint32 BindingCount = InBinding->GetGroupInfos()[GroupIt].SimRootCount;

				if (GroomCount != 0 && GroomCount != BindingCount)
				{
					UE_CLOGF(bIssueWarning, LogHairStrands, Warning, "[Groom] The GroomBinding asset (%ls) does not contains the same guides in group %d (%d vs. %d) than the groom (%ls). The binding asset will not be used.",
						*InBinding->GetName(),
						GroupIt,
						GroomCount,
						BindingCount,
						*InGroom->GetName());
					return false;
				}
			}

			// Strands
			const bool bNeedStrandsRoot = DoesGroomNeedStrandsBinding(InGroom, GroupIt);
			if (bNeedStrandsRoot)
			{
				const uint32 GroomCount = InGroom->GetHairGroupsPlatformData()[GroupIt].Strands.BulkData.GetNumCurves();
				const uint32 BindingCount = InBinding->GetGroupInfos()[GroupIt].RenRootCount;

				// Groom may have stripped strands data so GroomCount would be 0
				if (GroomCount != 0 && GroomCount != BindingCount)
				{
					UE_CLOGF(bIssueWarning, LogHairStrands, Warning, "[Groom] The GroomBinding asset (%ls) does not contains the same curves in group %d (%d vs. %d) than the groom (%ls). The binding asset will not be used.",
						*InBinding->GetName(),
						GroupIt,
						GroomCount,
						BindingCount,
						*InGroom->GetName());
					return false;
				}
			}
		}
	}
	return true;
}

bool UGroomBindingAsset::IsBindingAssetValid(const UGroomBindingAsset* InBinding, bool bIsBindingReloading, bool bIssueWarning)
{
	if (InBinding && IsHairStrandsBindingEnable())
	{
		if (!InBinding->IsValid())
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The binding asset (%ls) is invalid. It failed to load or build. Falling back onto non-binding version.", *InBinding->GetName());
			}
			return false;
		}

		if (!InBinding->GetGroom())
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The binding asset (%ls) does not reference a groom. Falling back onto non-binding version.", *InBinding->GetName());
			}
			return false;
		}
		if (InBinding->GetGroupInfos().Num() == 0)
		{
			if (bIssueWarning)
			{
				UE_LOGF(LogHairStrands, Warning, "[Groom] The GroomBinding asset (Groom:%ls) does not contain any groups. It is invalid and can't be assigned. The binding asset will not be used.", *InBinding->GetGroom()->GetName());
			}
			return false;
		}

		uint32 GroupIt = 0;
		for (const FGoomBindingGroupInfo& Info : InBinding->GetGroupInfos())
		{
			if (Info.SimRootCount == 0)
			{
				if (bIssueWarning)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] The GroomBinding asset (Groom:%ls) has group with 0 guides. It is invalid and can't be assigned. The binding asset will not be used.", *InBinding->GetGroom()->GetName());
				}
				return false;
			}

			const bool bNeedStrandsRoot = DoesGroomNeedStrandsBinding(InBinding->GetGroom(), GroupIt);
			if (bNeedStrandsRoot && Info.RenRootCount == 0 && IsHairStrandsEnabled(EHairStrandsShaderType::Strands))
			{
				if (bIssueWarning)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] The GroomBinding asset (Groom:%ls) has group with 0 curves. It is invalid and can't be assigned. The binding asset will not be used.", *InBinding->GetGroom()->GetName());
				}
				return false;
			}

			++GroupIt;
		}
	}
	return true;
}

bool UGroomBindingAsset::HasValidTarget() const
{
	return (GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh && GetTargetSkeletalMesh()) ||
		   (GetGroomBindingType() == EGroomBindingMeshType::GeometryCache && GetTargetGeometryCache());
}

void UGroomBindingAsset::Build(FOnGroomBindingAssetBuildComplete CompletionDelegate)
{
	Build(CompletionDelegate, FOnGroomBindingAssetBuildCompleteNative());
}

void UGroomBindingAsset::Build(const FOnGroomBindingAssetBuildCompleteNative& CompletionDelegate)
{
	Build(FOnGroomBindingAssetBuildComplete(), CompletionDelegate);
}

void UGroomBindingAsset::Build(const FOnGroomBindingAssetBuildComplete& DynamicCompletionDelegate, const FOnGroomBindingAssetBuildCompleteNative& NativeCompletionDelegate)
{
	if (GetGroom() && HasValidTarget())
	{
#if WITH_EDITOR
		OnGroomBindingAssetChanged.Broadcast();
#endif
		Reset();
		BeginCacheDerivedDatas(DynamicCompletionDelegate, NativeCompletionDelegate);
	}
	else
	{
		DynamicCompletionDelegate.ExecuteIfBound(this, EGroomBindingAssetBuildResult::Failed);
		NativeCompletionDelegate.ExecuteIfBound(this, EGroomBindingAssetBuildResult::Failed);
	}
}


#if WITH_EDITOR
void UGroomBindingAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	BeginCacheDerivedDatas();
	OnGroomBindingAssetChanged.Broadcast();
}
#endif // WITH_EDITOR

void UpdateGroomBindingAssetInfos(UGroomBindingAsset* In)
{
	if (In)
	{
		const uint32 GroupCount = In->GetHairGroupsPlatformData().Num();
		In->GetGroupInfos().SetNum(GroupCount);
		for (uint32 GroupIt=0; GroupIt< GroupCount; ++GroupIt)
		{
			FGoomBindingGroupInfo& Info = In->GetGroupInfos()[GroupIt];
			const UGroomBindingAsset::FHairGroupPlatformData& BulkData = In->GetHairGroupsPlatformData()[GroupIt];
			Info.SimRootCount = BulkData.SimRootBulkDatas.Num() > 0 ? BulkData.SimRootBulkDatas[0].GetRootCount() : 0u;
			Info.SimLODCount  = BulkData.SimRootBulkDatas.Num();

			Info.RenRootCount = BulkData.RenRootBulkDatas.Num() > 0 ? BulkData.RenRootBulkDatas[0].GetRootCount() : 0u;
			Info.RenLODCount  = BulkData.RenRootBulkDatas.Num();
		}
	}
}

#if WITH_EDITORONLY_DATA

// If groom binding derived data needs to be rebuilt (new format, serialization
// differences, etc.) replace the version GUID below with a new one.
// In case of merge conflicts with DDC versions, you MUST generate a new GUID
// and set this new GUID as the version.
// DDC Guid needs to be bumped in:
// * Main    : Engine\Source\Runtime\Core\Private\UObject\DevObjectVersion.cpp
// * Release : Engine\Source\Runtime\Core\Private\UObject\UE5ReleaseStreamObjectVersion.cpp
// * ...

namespace GroomBindingDerivedDataCacheUtils
{
	const FString& GetGroomBindingDerivedDataVersion()
	{
		static FString CachedVersionString = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().GROOM_BINDING_DERIVED_DATA_VERSION).ToString();
		return CachedVersionString;
	}

	FString BuildGroomBindingDerivedDataKey(const FString& KeySuffix)
	{
		return FDerivedDataCacheInterface::BuildCacheKey(*(TEXT("GROOMBINDING_V") + FGroomBindingBuilder::GetVersion() + TEXT("_")), *GetGroomBindingDerivedDataVersion(), *KeySuffix);
	}
}

static FString BuildDerivedDataKeySuffix(const UGroomBindingAsset& BindingAsset, const ITargetPlatform* TargetPlatform, int32 SourceMeshLOD, int32 TargetMeshMinLOD)
{
	FString BindingType;
	FString SourceKey;
	FString TargetKey;
	FString SourceLODKey;
	FString TargetLODKey;

	if (BindingAsset.GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		// Binding type is implicitly SkeletalMesh so keep BindingType empty to prevent triggering rebuild of old binding for nothing
		if (USkeletalMesh* SourceMesh = BindingAsset.GetSourceSkeletalMesh())
		{
			SourceKey = SourceMesh->BuildDerivedDataKey(TargetPlatform);
			SourceLODKey = FString::FromInt(SourceMeshLOD);
		}

		if (USkeletalMesh* TargetMesh = BindingAsset.GetTargetSkeletalMesh())
		{
			TargetKey = TargetMesh->BuildDerivedDataKey(TargetPlatform);
			TargetLODKey = FString::FromInt(TargetMeshMinLOD);
		}
	}
	else
	{
		BindingType = "GEOCACHE_";
		SourceKey = BindingAsset.GetSourceGeometryCache() ? BindingAsset.GetSourceGeometryCache()->GetHash() : FString();
		TargetKey = BindingAsset.GetTargetGeometryCache() ? BindingAsset.GetTargetGeometryCache()->GetHash() : FString();
	}
	// When possible, use the GroomAsset 'cached DDC key'. This allows to avoid a bug where the DDC key would change 
	// when loading GroomAsset's hair description, which would modify the hair description hash ID with legacy content.
	FString GroomKey  = BindingAsset.GetGroom() ? BindingAsset.GetGroom()->GetDerivedDataKey(true /*bUseCachedKey*/) : FString();
	FString PointKey  = FString::FromInt(BindingAsset.GetNumInterpolationPoints());
	FString SectionKey = FString::FromInt(BindingAsset.GetMatchingSection());
	FString TargetBindingAttributeKey = BindingAsset.GetTargetBindingAttribute().ToString();
	FString RBLocalSpaceKey = FString::FromInt(UE::Groom::IsRBFLocalSpaceEnabled());

	uint32 KeyLength  = BindingType.Len() + SourceKey.Len() + TargetKey.Len() + GroomKey.Len() + PointKey.Len() + SectionKey.Len() + SourceLODKey.Len() + TargetLODKey.Len() + TargetBindingAttributeKey.Len() + RBLocalSpaceKey.Len();

	FString KeySuffix;
	KeySuffix.Reserve(KeyLength);
	KeySuffix = BindingType + SourceKey + TargetKey + GroomKey + PointKey + SectionKey + SourceLODKey + TargetLODKey + TargetBindingAttributeKey + RBLocalSpaceKey;
	return KeySuffix;
}

static FString BuildDerivedDataKeyGroup(const FString& InDeriveDataKeySuffix , uint32 InGroupIndex)
{
	return GroomBindingDerivedDataCacheUtils::BuildGroomBindingDerivedDataKey(InDeriveDataKeySuffix + FString(TEXT("_Group")) + FString::FromInt(InGroupIndex));
}

static TArray<FString> GetGroupDerivedDataKeys(const UGroomBindingAsset* In, const ITargetPlatform* TargetPlatform, int32 SourceMeshLOD, int32 TargetMeshMinLOD);
static void CacheDerivedDatas(
	UGroomBindingAsset* In,
	const uint32 InGroupIndex,
	const FString& DerivedDataKey,
	bool& bOutValid,
	const ITargetPlatform* TargetPlatform,
	const int32 SourceMeshLOD,
	const int32 TargetMeshMinLOD,
	UGroomBindingAsset::FHairGroupPlatformData& OutPlatformData);

#endif // WITH_EDITORONLY_DATA


void UGroomBindingAsset::BeginCacheDerivedDatas(
	const FOnGroomBindingAssetBuildComplete& DynamicCompletionDelegate,
	const FOnGroomBindingAssetBuildCompleteNative& NativeCompletionDelegate)
{
	if (!GetGroom() || !GetGroom()->IsValid())
	{
		DynamicCompletionDelegate.ExecuteIfBound(this, EGroomBindingAssetBuildResult::Failed);
		NativeCompletionDelegate.ExecuteIfBound(this, EGroomBindingAssetBuildResult::Failed);
		return;
	}

	if (IsCompiling())
	{
		FGroomBindingCompilingManager::Get().FinishCompilation({ this });
	}

	if (HasAnyDependenciesCompiling())
	{
		FGroomBindingCompilingManager::Get().AddGroomBindingsWithPendingDependencies({ this });
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomBindingAsset::BeginCacheDerivedDatas);

	// Flush the rendering commands generated by the detachments.
	FlushRenderingCommandIfUsed(this);

	// Acquire everything in readonly so that any thread trying to write a property locked for read-only will wait until the build is finished
	AcquireAsyncProperty(MAX_uint64, EGroomBindingAsyncPropertyLockType::ReadOnly);

	// Acquire the resource modified by the build so that any thread trying to read those will wait until the build is finished
	// If we forget something here and the build uses it, we will get an assert on the build thread because we're writing to an unlocked property
	AcquireAsyncProperty((uint64)EGroomBindingAsyncProperties::HairGroupResources, EGroomBindingAsyncPropertyLockType::WriteOnly);
	AcquireAsyncProperty((uint64)EGroomBindingAsyncProperties::HairGroupPlatformData, EGroomBindingAsyncPropertyLockType::WriteOnly);
	AcquireAsyncProperty((uint64)EGroomBindingAsyncProperties::GroupInfos, EGroomBindingAsyncPropertyLockType::WriteOnly);
	AcquireAsyncProperty((uint64)EGroomBindingAsyncProperties::SourceMeshUsedLOD, EGroomBindingAsyncPropertyLockType::WriteOnly);
	AcquireAsyncProperty((uint64)EGroomBindingAsyncProperties::TargetMeshUsedMinLOD, EGroomBindingAsyncPropertyLockType::WriteOnly);

	FGroomBindingBuildContext Context;
	if (!TryInitializeContextForMeshes(Context))
	{
		DynamicCompletionDelegate.ExecuteIfBound(this, EGroomBindingAssetBuildResult::Failed);
		NativeCompletionDelegate.ExecuteIfBound(this, EGroomBindingAssetBuildResult::Failed);
		return;
	}

	Context.DynamicCompletionDelegate = DynamicCompletionDelegate;
	Context.NativeCompletionDelegate = NativeCompletionDelegate;

	// Dependency management should have been taken care of already. Just making sure.
	check(GetTargetSkeletalMesh() == nullptr || GetTargetSkeletalMesh()->IsCompiling() == false);
	check(GetSourceSkeletalMesh() == nullptr || GetSourceSkeletalMesh()->IsCompiling() == false);

	const IConsoleVariable* CVarStreaming = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HairStrands.Streaming"));
	const bool bIsStrandStreamingEnabled = CVarStreaming && CVarStreaming->GetBool();

	const bool bIsAsyncBuildSupported = 
		// Do synchronous build until GeometryCache is thread-safe
		GetTargetGeometryCache() == nullptr
		&& GetSourceGeometryCache() == nullptr
		// Strand streaming not supported for async build yet
		&& !bIsStrandStreamingEnabled;

	if (bIsAsyncBuildSupported && FGroomBindingCompilingManager::Get().IsAsyncCompilationAllowed(this))
	{
		FQueuedThreadPool* ThreadPool = FGroomBindingCompilingManager::Get().GetThreadPool();
		EQueuedWorkPriority BasePriority = FGroomBindingCompilingManager::Get().GetBasePriority(this);
		check(AsyncTask == nullptr);

		int64 RequiredMemory = 1024 * 1024 * CVarMemoryForGroomBindingAssetCompile.GetValueOnAnyThread();

		AsyncTask = MakeUnique<FGroomBindingAsyncBuildTask>(this, MoveTemp(Context));
		AsyncTask->StartBackgroundTask(ThreadPool, BasePriority, EQueuedWorkFlags::DoNotRunInsideBusyWait, RequiredMemory, TEXT("GroomBindingAsset"));
		FGroomBindingCompilingManager::Get().AddGroomBindings({ this });
	}
	else
	{
		ExecuteCacheDerivedDatas(Context);
		FinishCacheDerivedDatas(Context);
	}
}

static int32 GetClampedSkeletalMeshLOD(const USkeletalMesh* SkeletalMesh, int32 RequestedMinLOD, const ITargetPlatform* TargetPlatform)
{
	if (!SkeletalMesh)
	{
		return INDEX_NONE;
	}

	// BestAvailableLOD is the best (i.e. lowest index) LOD that can ever be used by this mesh.
	// E.g. in a cooked build, this is the best LOD that was cooked.
	// 
	// RequestedMinLOD is clamped to it here, so that if the mesh has a per-platform min LOD,
	// the caller of UGroomBindingAsset::Build doesn't need to take that into account. They can
	// simply set RequestedMinLOD to 0 and if min LOD on the mesh is 3, for example, it will
	// silently build the groom binding using LOD 3 as the source.
	#if WITH_EDITOR
	check(TargetPlatform);
	// NOTE: This is a temporary workaround, as GetPlatformMinLODIdx does not currently return the 
	// lowest possible LOD for the given platform. It will be fixed in a future release.
	const int32 BestAvailableLOD = 0;// SkeletalMesh->GetPlatformMinLODIdx(TargetPlatform);
	#else
	const bool bForceLowestLODIdx = true;
	const int32 BestAvailableLOD = SkeletalMesh->GetMinLodIdx(bForceLowestLODIdx);
	#endif

	return FMath::Max(RequestedMinLOD, BestAvailableLOD);
}

static bool GetSkeletalMeshLOD(
	const UGroomBindingAsset* Binding,
	const USkeletalMesh* SkeletalMesh, 
	const int32 RequestedMinLOD, 
	const bool bRequestSingleLOD,
	ITargetPlatform* InPlatform, 
	int32& OutMeshMinLOD, 
	TArray<TRefCountPtr<FSkeletalMeshLODRenderData>>& OutMeshLODReferences)
{	
	check(SkeletalMesh);
	OutMeshMinLOD = GetClampedSkeletalMeshLOD(SkeletalMesh, RequestedMinLOD, InPlatform);

	// Clamp SourceMeshLOD to the next best LOD that's currently loaded.
	const FStreamableRenderResourceState& SRRState = SkeletalMesh->GetStreamableResourceState();
	if (SRRState.IsValid())
	{
		const int32 NumValidLODs = FMath::Min(SRRState.NumRequestedLODs, SRRState.NumResidentLODs);
		const int32 CurrentFirstLOD = SRRState.LODCountToAssetFirstLODIdx(NumValidLODs);
		OutMeshMinLOD = FMath::Max(CurrentFirstLOD, OutMeshMinLOD);
	}

	if (FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering())
	{
		if (RenderData->LODRenderData.IsValidIndex(OutMeshMinLOD))
		{
			const int32 LODCount = bRequestSingleLOD ? FMath::Min(OutMeshMinLOD+1, SkeletalMesh->GetLODNum()) : SkeletalMesh->GetLODNum();
			for (int32 MeshLODIndex = OutMeshMinLOD; MeshLODIndex < LODCount; ++MeshLODIndex)
			{
				TRefCountPtr<FSkeletalMeshLODRenderData> LODReference;
				if (RenderData->LODRenderData.IsValidIndex(MeshLODIndex))
				{
					LODReference = &RenderData->LODRenderData[MeshLODIndex];
				}

				if (!ensureMsgf(LODReference.IsValid(),
					TEXT("Missing render data for mesh %s LOD %i, which is claimed to be valid by FStreamableRenderResourceState. Can't build groom binding %s."),
					*SkeletalMesh->GetPathName(), MeshLODIndex, *Binding->GetPathName()))
				{
					return false;
				}

				OutMeshLODReferences.Add(MoveTemp(LODReference));
			}
		}
		else
		{
			ensureMsgf(false,
			TEXT("Missing render data for mesh %s LOD %i, which is claimed to be valid by FStreamableRenderResourceState. Can't build groom binding %s."),
			*SkeletalMesh->GetPathName(), OutMeshMinLOD, *Binding->GetPathName());

			return false;
		}
	}
	else
	{
		UE_LOGFMT(LogHairStrands, Error, "Skeletal mesh {0} has no render data. Can't build groom binding {1}.", SkeletalMesh->GetPathName(), Binding->GetPathName());
		return false;
	}

	return true;
}

bool UGroomBindingAsset::TryInitializeContextForMeshes(FGroomBindingBuildContext& OutContext) const
{
	if (GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		ITargetPlatform* RunningPlatform = nullptr;
		#if WITH_EDITOR
		RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		check(RunningPlatform);
		#endif

		if (USkeletalMesh* SourceMesh = GetSourceSkeletalMesh())
		{
			const bool bSucceed = GetSkeletalMeshLOD(this, SourceMesh, GetSourceMeshRequestedLOD(), true /*bSingleLOD*/, RunningPlatform, OutContext.SourceMeshLOD, OutContext.MeshLODReferences);
			if (!bSucceed) { return false; }
		}

		if (USkeletalMesh* TargetMesh = GetTargetSkeletalMesh())
		{
			const bool bSucceed = GetSkeletalMeshLOD(this, TargetMesh, GetTargetMeshRequestedMinLOD(), false/*bSingleLOD*/, RunningPlatform, OutContext.TargetMeshMinLOD, OutContext.MeshLODReferences);
			if (!bSucceed) { return false; }
		}
	}
	else
	{
		// Geometry caches are treated as having a single LOD
		OutContext.SourceMeshLOD = 0;
		OutContext.TargetMeshMinLOD = 0;
	}

	return true;
}

void UGroomBindingAsset::ExecuteCacheDerivedDatas(FGroomBindingBuildContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomBindingAsset::ExecuteCacheDerivedDatas);

	FGroomBindingAsyncBuildScope BuildScope(this);

	// 1. Set group count to the groom target
	const uint32 GroupCount = GetGroom()->GetNumHairGroups();
	GetHairGroupsPlatformData().SetNum(GroupCount);
	GetGroupInfos().SetNum(GroupCount);
	ITargetPlatform* RunningPlatform = nullptr;
#if WITH_EDITOR
	RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	check(RunningPlatform);
	CachedDerivedDataKey.SetNum(GroupCount);


	// 2. Build the key for each group and check if any group needs to be rebuilt
	const TArray<FString> GroupDerivedDataKeys = GetGroupDerivedDataKeys(this, RunningPlatform, Context.SourceMeshLOD, Context.TargetMeshMinLOD);
	const bool bAnyGroupNeedRebuild = GroupDerivedDataKeys != CachedDerivedDataKey;
#else
	// Note that in the non-editor path, we don't detect if the build is redundant and it's up to
	// the user to avoid calling Build unnecessarily.
	const bool bAnyGroupNeedRebuild = true;
#endif

	const bool bDelayDataRelease = GHairStrandsDelayBindingDataRelease > 0;
	if (bIsValid && bAnyGroupNeedRebuild && bDelayDataRelease)
	{
		TArray<UGroomBindingAsset::FHairGroupPlatformData>& PlatformData = GetHairGroupsPlatformData();
		FHairGroupPlatformDataArray* DataToBeDeleted = new FHairGroupPlatformDataArray();
		DataToBeDeleted->Data = MoveTemp(PlatformData);
		HairGroupPlatformDataToDelete.Enqueue(DataToBeDeleted);

		GetHairGroupsPlatformData().SetNum(GroupCount);
	}

	// 3. Build or retrieve from cache, binding data for each group
	bIsValid = true;
	bool bReloadResource = false;
	if (bAnyGroupNeedRebuild)
	{
		for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
		{
			// 1. Build data
			bool bGroupValid = true;
			bool bGroupReloadResource = false;
#if WITH_EDITOR
			if (GroupDerivedDataKeys[GroupIndex] != CachedDerivedDataKey[GroupIndex] || bDelayDataRelease)
			{
				// Fetch the built data from the DDC or build it

				const FString Key = GroupDerivedDataKeys[GroupIndex];
				CacheDerivedDatas(this, GroupIndex, Key, bGroupValid, RunningPlatform, Context.SourceMeshLOD, Context.TargetMeshMinLOD, GetHairGroupsPlatformData()[GroupIndex]);
			}
#else
			// Build the data directly, as there is no DDC
			const FGroomBindingBuilder::FInput BuilderInput(this, RunningPlatform, Context.SourceMeshLOD, Context.TargetMeshMinLOD);
			bGroupValid = FGroomBindingBuilder::BuildBinding(
				BuilderInput,
				GroupIndex,
				RunningPlatform,
				GetHairGroupsPlatformData()[GroupIndex]);
#endif

			if (bGroupValid)
			{
				bGroupReloadResource = true;
#if WITH_EDITOR
				CachedDerivedDataKey[GroupIndex] = GroupDerivedDataKeys[GroupIndex];
#endif
			}
			else
			{
				UE_LOGFMT(LogHairStrands, Error, "[Groom] The binding asset ({0}) couldn't be built. This binding asset won't be used.", GetName());
			}
	
			// 2. Release existing resources data
			if (bGroupReloadResource)
			{
				UGroomBindingAsset::FHairGroupResources& OutHairGroupResources = GetHairGroupResources();
				if (OutHairGroupResources.Num() > 0)
				{
					for (UGroomBindingAsset::FHairGroupResource& GroupResources : OutHairGroupResources)
					{
						AddHairGroupResourcesToDelete(GroupResources);
					}
					OutHairGroupResources.Empty();
				}
				check(OutHairGroupResources.Num() == 0);
			}

			bIsValid = bIsValid && bGroupValid;
			bReloadResource = bReloadResource || bGroupReloadResource;
		}

		// 3. Update binding infos here as they need to be valid when RecreateRenderContext is deleted
		//    When RecreateRenderContext's Dtor is called, it will recreate component, which will run 
		//    the binding validation to assess if the binding asset is compatible. This validation logic 
		//    use the binding infos to know if curve count match between GroomAsset and GroomBindingAsset
		UpdateGroomBindingAssetInfos(this);

		Context.bReloadResource |= bReloadResource;
	}
	else
	{
		// 3. Patch hair group info if it does not match the DDC-read/deserialized data
		UpdateGroomBindingAssetInfos(this);
	}
}

#if WITH_EDITORONLY_DATA
static void CacheDerivedDatas(
	UGroomBindingAsset* In,
	const uint32 InGroupIndex,
	const FString& DerivedDataKey,
	bool& bOutValid,
	const ITargetPlatform* TargetPlatform,
	const int32 SourceMeshLOD,
	const int32 TargetMeshMinLOD,
	UGroomBindingAsset::FHairGroupPlatformData& OutPlatformData)
{
	{
		bOutValid = false;
		using namespace UE;
		using namespace UE::DerivedData;

		const FCacheKey HeaderKey = ConvertLegacyCacheKey(DerivedDataKey + FString(TEXT("_Header")));
		const FSharedString Name = MakeStringView(In->GetPathName());
		FSharedBuffer Data;
		{
			FRequestOwner Owner(EPriority::Blocking);
			GetCache().GetValue({ {Name, HeaderKey} }, Owner, [&Data](FCacheGetValueResponse&& Response)
			{
				Data = Response.Value.GetData().Decompress();
			});
			Owner.Wait();
		}

		// Populate key/name for streaming data request
		auto FillDrivedDataKey = [&DerivedDataKey, &Name](UGroomBindingAsset::FHairGroupPlatformData& In)
		{
			for (uint32 MeshLODIndex = 0, MeshLODCount=In.RenRootBulkDatas.Num(); MeshLODIndex < MeshLODCount; ++MeshLODIndex)
			{
				In.RenRootBulkDatas[MeshLODIndex].DerivedDataKey = DerivedDataKey + FString::Printf(TEXT("_RenRootData_MeshLOD%d"), MeshLODIndex);
			}
		};

		bool bHasDataInCache = false;
		if (Data)
		{
			UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[GroomBinding/DDC] Found (GroomBinding:%ls TargetPlatform:%ls Async:%ls).", *In->GetName(), *TargetPlatform->PlatformName(), IsInGameThread() ? TEXT("No") : TEXT("Yes"));

			// Header
			FMemoryReaderView Ar(Data, /*bIsPersistent*/ true);
			InternalSerializePlatformData(Ar, In, OutPlatformData, 0 /*Flags*/, true /*bHeader*/, false /*bData*/);
			bHasDataInCache = true;

			// Fill DDC key for each strands LOD root bulk data. Done after InternalSerializePlatformData(), as RenRootBulkDatas is not filled in yet, and OutPlatformData.RenRootBulkDatas.Num() == 0.
			FillDrivedDataKey(OutPlatformData);

			// Verify that all strands data are correctly cached into the DDC
			{
				for (int32 MeshLODIndex = 0, MeshLODCount = OutPlatformData.RenRootBulkDatas.Num(); MeshLODIndex < MeshLODCount; ++MeshLODIndex)
				{
					FHairStreamingRequest R; bHasDataInCache &= R.WarmCache(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, OutPlatformData.RenRootBulkDatas[MeshLODIndex]);
				}
			}

			bOutValid = true;
		}

		if (!bHasDataInCache)
		{
			UE_CLOGF(IsHairStrandsDDCLogEnable(), LogHairStrands, Log, "[GroomBinding/DDC] Not found (GroomBinding:%ls TargetPlatform:%ls Async:%ls).", *In->GetName(), *TargetPlatform->PlatformName(), IsInGameThread() ? TEXT("No") : TEXT("Yes"));

			// Build groom binding data
			FGroomBindingBuilder::FInput BuilderInput(In, TargetPlatform, SourceMeshLOD, TargetMeshMinLOD);
			bOutValid = FGroomBindingBuilder::BuildBinding(BuilderInput, InGroupIndex, TargetPlatform, OutPlatformData);

			if (bOutValid)
			{
				FillDrivedDataKey(OutPlatformData);

				// Header
				{
					TArray<uint8> WriteData;
					FMemoryWriter Ar(WriteData, /*bIsPersistent*/ true);
					InternalSerializePlatformData(Ar, In, OutPlatformData, 0 /*Flags*/, true /*bHeader*/, false /*bData*/);
	
					FRequestOwner AsyncOwner(EPriority::Normal);
					GetCache().PutValue({ {Name, HeaderKey, FValue::Compress(MakeSharedBufferFromArray(MoveTemp(WriteData)))} }, AsyncOwner);
					AsyncOwner.KeepAlive();
				}
	
				// Data
				for (uint32 MeshLODIndex=0, MeshLODCount = OutPlatformData.RenRootBulkDatas.Num(); MeshLODIndex<MeshLODCount;++MeshLODIndex)
				{
					TArray<FCachePutValueRequest> Out;
					OutPlatformData.RenRootBulkDatas[MeshLODIndex].Write_DDC(In, Out);

					FRequestOwner AsyncOwner(EPriority::Normal);
					GetCache().PutValue(Out, AsyncOwner);
					AsyncOwner.KeepAlive();
				}
			}
		}
	}
}
#endif // WITH_EDITORONLY_DATA

void UGroomBindingAsset::FinishCacheDerivedDatas(FGroomBindingBuildContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomBindingAsset::FinishCacheDerivedDatas);

	ReleaseAsyncProperty();

	EGroomBindingAssetBuildResult Result;
	if (!IsTemplate() && IsValid())
	{
		Result = EGroomBindingAssetBuildResult::Succeeded;

		if (Context.bReloadResource)
		{
			InitResource();
		}

		SetSourceMeshUsedLOD(Context.SourceMeshLOD);
		SetTargetMeshUsedMinLOD(Context.TargetMeshMinLOD);
	}
	else
	{
		Result = EGroomBindingAssetBuildResult::Failed;

		SetSourceMeshUsedLOD(INDEX_NONE);
		SetTargetMeshUsedMinLOD(INDEX_NONE);
	}

	for (TObjectIterator<UGroomComponent> HairStrandsComponentIt; HairStrandsComponentIt; ++HairStrandsComponentIt)
	{
		if (HairStrandsComponentIt->BindingAsset == this 
#if WITH_EDITOR
			|| HairStrandsComponentIt->BindingAssetBeingLoaded == this // A GroomAsset was set on the component while it was still loading
#endif
			) 
		{
			HairStrandsComponentIt->PostCompilation();

			if (HairStrandsComponentIt->IsRegistered())
			{
				HairStrandsComponentIt->RecreateRenderState_Concurrent();
			}
		}
	}

	Context.DynamicCompletionDelegate.ExecuteIfBound(this, Result);
	Context.NativeCompletionDelegate.ExecuteIfBound(this, Result);
}

int32 UGroomBindingAsset::GetClampedSourceMeshLOD(const ITargetPlatform* TargetPlatform) const
{
	if (GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		return GetClampedSkeletalMeshLOD(GetSourceSkeletalMesh(), GetSourceMeshRequestedLOD(), TargetPlatform);
	}
	return INDEX_NONE;
}

int32 UGroomBindingAsset::GetClampedTargetMeshMinLOD(const ITargetPlatform* TargetPlatform) const
{
	if (GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		return GetClampedSkeletalMeshLOD(GetTargetSkeletalMesh(), GetTargetMeshRequestedMinLOD(), TargetPlatform);
	}
	return INDEX_NONE;
}

#if WITH_EDITORONLY_DATA

static TArray<FString> GetGroupDerivedDataKeys(const UGroomBindingAsset* In, const ITargetPlatform* TargetPlatform, int32 SourceMeshLOD, int32 TargetMeshMinLOD)
{
	check(In);
	check(TargetPlatform);

	const FString KeySuffix = BuildDerivedDataKeySuffix(*In, TargetPlatform, SourceMeshLOD, TargetMeshMinLOD);
	const uint32 GroupCount = In->GetGroupInfos().Num();

	TArray<FString> Out;
	Out.SetNum(GroupCount);
	for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
	{
		Out[GroupIndex] = BuildDerivedDataKeyGroup(KeySuffix, GroupIndex);
	}
	return Out;
}

static UGroomBindingAsset::FCachedCookedPlatformData* FindCachedCookedPlatformData(const TArray<FString>& InGroupKeysWithoutMeshLOD, TArray<UGroomBindingAsset::FCachedCookedPlatformData*>& InCachedCookedData)
{
	for (UGroomBindingAsset::FCachedCookedPlatformData* CookedPlatformData : InCachedCookedData)
	{
		if (CookedPlatformData->GroupDerivedDataKeys_Query == InGroupKeysWithoutMeshLOD)
		{
			return CookedPlatformData;
		}
	}
	return nullptr;
}

void UGroomBindingAsset::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	// Early out if hair strands are disabled
	const bool bIsHairEnabled = IsGroomEnabled();
	if (!bIsHairEnabled)
	{
		return;
	}

	// Finish any async compilation of the editor target before building other platforms
	if (IsCompiling())
	{
		FGroomBindingCompilingManager::Get().FinishCompilation({ this });
	}

	Super::BeginCacheForCookedPlatformData(TargetPlatform);

	// 1. Build the key for each group
	const TArray<FString> GroupDerivedDataKeys_Query = GetGroupDerivedDataKeys(this, TargetPlatform, 0/*SourceMeshLOD*/, 0/*TargetMeshMinLOD*/);

	// 2. Find existing cached cooked data
	UGroomBindingAsset::FCachedCookedPlatformData* TargetPlatformData = FindCachedCookedPlatformData(GroupDerivedDataKeys_Query, CachedCookedPlatformDatas);

	// 3. If the target cooked data does not already exist, we build it
	if (TargetPlatformData == nullptr && GetGroom() != nullptr)
	{
		auto BuildTargetPlatformData = [TargetPlatform](const TArray<FString>& InGroupDerivedDataKeys_Query, const TArray<FString>& InGroupDerivedDataKeys_Cache, UGroomBindingAsset* BindingAsset, int32 InSourceMeshLOD, int32 InTargetMeshMinLOD)
		{
			// 3.a Build cooked derived data
			const uint32 GroupCount = InGroupDerivedDataKeys_Cache.Num();
			UGroomBindingAsset::FCachedCookedPlatformData* NewTargetPlatformData = new FCachedCookedPlatformData();
			NewTargetPlatformData->GroupDerivedDataKeys_Query = InGroupDerivedDataKeys_Query;
			NewTargetPlatformData->GroupDerivedDataKeys = InGroupDerivedDataKeys_Cache;
			NewTargetPlatformData->SourceMeshLOD = InSourceMeshLOD;
			NewTargetPlatformData->TargetMeshMinLOD = InTargetMeshMinLOD;
			NewTargetPlatformData->GroupPlatformDatas.SetNum(GroupCount);
			for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
			{
				bool bGroupValid = true;
				CacheDerivedDatas(
					BindingAsset, 
					GroupIndex, 
					NewTargetPlatformData->GroupDerivedDataKeys[GroupIndex], 
					bGroupValid, 
					TargetPlatform, 
					InSourceMeshLOD, 
					InTargetMeshMinLOD, 
					NewTargetPlatformData->GroupPlatformDatas[GroupIndex]);

				if (!bGroupValid)
				{
					UE_LOGF(LogHairStrands, Error, "[Groom] The binding asset (%ls) couldn't be built. This binding asset won't be used.", *BindingAsset->GetName());
				}
			}

			// 3.b Place cooked derived data into their bulk data. 
			// This is done only for strands, which support DDC streaming
			// When cooking data, force loading of *all* bulk data prior to saving them
			// Note: bFillBulkdata is true for filling in the bulkdata container prior to serialization. This also forces the resources loading 
			// from the 'start' (i.e., without offset)
			for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
			{
				TArray<FHairStrandsRootBulkData>& RenRootBulkDatas = NewTargetPlatformData->GroupPlatformDatas[GroupIndex].RenRootBulkDatas;
				for (int32 MeshLODIndex = 0, MeshLODCount = RenRootBulkDatas.Num(); MeshLODIndex < MeshLODCount; ++MeshLODIndex)
				{
					FHairStreamingRequest R; R.Request(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP, RenRootBulkDatas[MeshLODIndex], true /*bWait*/, true /*bFillBulkdata*/, true /*bWarmCache*/, BindingAsset->GetFName());
				}
			}

			BindingAsset->CachedCookedPlatformDatas.Add(NewTargetPlatformData);
		};

		// 3.1 Build binding data for the target platform
		{
			const int32 SourceMeshLOD = GetClampedSourceMeshLOD(TargetPlatform);
			const int32 TargetMeshMinLOD = GetClampedTargetMeshMinLOD(TargetPlatform);
			const TArray<FString> GroupDerivedDataKeys_Cache = GetGroupDerivedDataKeys(this, TargetPlatform, SourceMeshLOD, TargetMeshMinLOD);
			BuildTargetPlatformData(GroupDerivedDataKeys_Query, GroupDerivedDataKeys_Cache, this, SourceMeshLOD, TargetMeshMinLOD);
		}

		// 3.2 Check if the derived data keys are unchanged. If they have change, recompute the target platform binding data again.
		// This can happen when a skel. mesh asset hasn't be resaved correct. While being cached, the skel. mesh data will update/transform 
		// its DDC key, causing the binding data to be cached as the incorrect DDC key.
		const TArray<FString> UpdatedGroupDerivedDataKeys_Query = GetGroupDerivedDataKeys(this, TargetPlatform, 0/*SourceMeshLOD*/, 0/*TargetMeshMinLOD*/);
		if (UpdatedGroupDerivedDataKeys_Query != GroupDerivedDataKeys_Query)
		{
			const int32 UpdatedSourceMeshLOD = GetClampedSourceMeshLOD(TargetPlatform);
			const int32 UpdatedTargetMeshMinLOD = GetClampedTargetMeshMinLOD(TargetPlatform);
			const TArray<FString> UpdatedGroupDerivedDataKeys_Cache = GetGroupDerivedDataKeys(this, TargetPlatform, UpdatedSourceMeshLOD, UpdatedTargetMeshMinLOD);
			BuildTargetPlatformData(UpdatedGroupDerivedDataKeys_Query, UpdatedGroupDerivedDataKeys_Cache, this, UpdatedSourceMeshLOD, UpdatedTargetMeshMinLOD);
		}
	}
}

TArray<UGroomBindingAsset::FHairGroupPlatformData>* UGroomBindingAsset::GetCachedCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	// 1. Build the key for each group
	const TArray<FString> GroupDerivedDataKeys_Query = GetGroupDerivedDataKeys(this, TargetPlatform, 0/*SourceMeshLOD*/, 0/*TargetMeshMinLOD*/);

	// 2. Find existing cached cooked data
	if (UGroomBindingAsset::FCachedCookedPlatformData* CachedCookedPlatformData = FindCachedCookedPlatformData(GroupDerivedDataKeys_Query, CachedCookedPlatformDatas))
	{
		return &CachedCookedPlatformData->GroupPlatformDatas;
	}
	else
	{
		return nullptr;
	}
}

void UGroomBindingAsset::ClearAllCachedCookedPlatformData()
{
	for (UGroomBindingAsset::FCachedCookedPlatformData* CookedPlatformData : CachedCookedPlatformDatas)
	{
		delete CookedPlatformData;
	}
	CachedCookedPlatformDatas.Empty();

	Super::ClearAllCachedCookedPlatformData();
}

#endif

void UGroomBindingAsset::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GetHairGroupsPlatformData().GetAllocatedSize());

	for (const FHairGroupResource& Group : GetHairGroupResources())
	{
		if (Group.SimRootResources) CumulativeResourceSize.AddDedicatedVideoMemoryBytes(Group.SimRootResources->GetResourcesSize());
		if (Group.RenRootResources) CumulativeResourceSize.AddDedicatedVideoMemoryBytes(Group.RenRootResources->GetResourcesSize());
		for (const FHairStrandsRestRootResource* CardsRootResource : Group.CardsRootResources)
		{
			if (CardsRootResource) CumulativeResourceSize.AddDedicatedVideoMemoryBytes(CardsRootResource->GetResourcesSize());
		}
	}
}

FName UGroomBindingAsset::GetAssetPathName(int32 LODIndex)
{
#if RHI_ENABLE_RESOURCE_INFO
	if (LODIndex > -1)
	{
		return FName(FString::Printf(TEXT("%s [LOD%d]"), *GetPathName(), LODIndex));
	}
	else
	{
		return FName(GetPathName());
	}
#else
	return NAME_None;
#endif
}

#define DEFINE_GROOMBINDING_MEMBER_NAME(Name)\
	FName UGroomBindingAsset::Get##Name##MemberName()\
	{\
		PRAGMA_DISABLE_DEPRECATION_WARNINGS\
		return FName(TEXT(#Name));\
		PRAGMA_ENABLE_DEPRECATION_WARNINGS\
	}

#define DEFINE_GROOMBINDING_MEMBER_GETTER(Type, Access, Name, GetConst, SetConst)\
	GetConst Type Access UGroomBindingAsset::Get##Name() const\
	{\
		WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::Name, EGroomBindingAsyncPropertyLockType::ReadOnly); \
		PRAGMA_DISABLE_DEPRECATION_WARNINGS\
		return Name;\
		PRAGMA_ENABLE_DEPRECATION_WARNINGS\
	}

#define DEFINE_GROOMBINDING_MEMBER_SETTER(Type, Access, Name, GetConst, SetConst)\
	void UGroomBindingAsset::Set##Name(SetConst Type Access In##Name)\
	{\
		WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::Name, EGroomBindingAsyncPropertyLockType::WriteOnly); \
		PRAGMA_DISABLE_DEPRECATION_WARNINGS\
		Name = In##Name;\
		PRAGMA_ENABLE_DEPRECATION_WARNINGS\
	}

#define DEFINE_GROOMBINDING_MEMBER_ACCESSOR(Type, Access, Name, GetConst, SetConst)\
	DEFINE_GROOMBINDING_MEMBER_NAME(Name) \
	DEFINE_GROOMBINDING_MEMBER_GETTER(Type, Access, Name, GetConst, SetConst) \
	DEFINE_GROOMBINDING_MEMBER_SETTER(Type, Access, Name, GetConst, SetConst)

// Define most of the binding member accessor
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(EGroomBindingMeshType, , GroomBindingType, , );
DEFINE_GROOMBINDING_MEMBER_NAME(Groom);
DEFINE_GROOMBINDING_MEMBER_GETTER(UGroomAsset, *, Groom, , );
DEFINE_GROOMBINDING_MEMBER_NAME(SourceSkeletalMesh);
DEFINE_GROOMBINDING_MEMBER_NAME(TargetSkeletalMesh);
DEFINE_GROOMBINDING_MEMBER_GETTER(USkeletalMesh, *, SourceSkeletalMesh, , );
DEFINE_GROOMBINDING_MEMBER_GETTER(USkeletalMesh, *, TargetSkeletalMesh, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(int32, , SourceMeshRequestedLOD, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(int32, , SourceMeshUsedLOD, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(int32, , TargetMeshRequestedMinLOD, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(int32, , TargetMeshUsedMinLOD, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(UGeometryCache, *, SourceGeometryCache, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(UGeometryCache, *, TargetGeometryCache, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(int32, , NumInterpolationPoints, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(int32, , MatchingSection, , );
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(TArray<FGoomBindingGroupInfo>, &, GroupInfos, const, const);
DEFINE_GROOMBINDING_MEMBER_ACCESSOR(FName, , TargetBindingAttribute, , );

#if WITH_EDITORONLY_DATA

void UGroomBindingAsset::UnregisterGroomDelegates()
{
	if (GetGroom())
	{
		GetGroom()->GetOnGroomAssetResourcesChanged().RemoveAll(this);
		GetGroom()->GetOnGroomAssetChanged().RemoveAll(this);
	}
}

void UGroomBindingAsset::RegisterGroomDelegates()
{
	if (GetGroom())
	{
		GetGroom()->GetOnGroomAssetChanged().AddUObject(this, &UGroomBindingAsset::InvalidateBinding);
		GetGroom()->GetOnGroomAssetResourcesChanged().AddUObject(this, &UGroomBindingAsset::InvalidateBinding);
	}
}

void UGroomBindingAsset::UnregisterSkeletalMeshDelegates()
{
	if (GetTargetSkeletalMesh())
	{
		GetTargetSkeletalMesh()->OnPostMeshCached().RemoveAll(this);
	}

	if (GetSourceSkeletalMesh())
	{
		GetSourceSkeletalMesh()->OnPostMeshCached().RemoveAll(this);
	}
}

void UGroomBindingAsset::RegisterSkeletalMeshDelegates()
{
	if (GetTargetSkeletalMesh())
	{
		GetTargetSkeletalMesh()->OnPostMeshCached().AddUObject(this, &UGroomBindingAsset::InvalidateBinding);
	}

	// Do not register the same skeletal mesh twice as it will make us invalidate bindings twice in a row and cause a stall.
	if (GetSourceSkeletalMesh() && GetSourceSkeletalMesh() != GetTargetSkeletalMesh())
	{
		GetSourceSkeletalMesh()->OnPostMeshCached().AddUObject(this, &UGroomBindingAsset::InvalidateBinding);
	}
}

#endif // WITH_EDITORONLY_DATA

void UGroomBindingAsset::SetGroom(UGroomAsset* InGroom)
{
	if (GetGroom() == InGroom)
	{
		return;
	}

#if WITH_EDITOR
	UnregisterGroomDelegates();
#endif

	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::Groom, EGroomBindingAsyncPropertyLockType::WriteOnly);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Groom = InGroom;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	RegisterGroomDelegates();
#endif
}

void UGroomBindingAsset::SetSourceSkeletalMesh(USkeletalMesh* InSourceSkeletalMesh)
{
	if (GetSourceSkeletalMesh() == InSourceSkeletalMesh)
	{
		return;
	}

#if WITH_EDITOR
	UnregisterSkeletalMeshDelegates();
#endif

	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::SourceSkeletalMesh, EGroomBindingAsyncPropertyLockType::WriteOnly);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SourceSkeletalMesh = InSourceSkeletalMesh;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	RegisterSkeletalMeshDelegates();
#endif
}

void UGroomBindingAsset::SetTargetSkeletalMesh(USkeletalMesh* InTargetSkeletalMesh)
{
	if (GetTargetSkeletalMesh() == InTargetSkeletalMesh)
	{
		return;
	}

#if WITH_EDITOR
	UnregisterSkeletalMeshDelegates();
#endif

	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::TargetSkeletalMesh, EGroomBindingAsyncPropertyLockType::WriteOnly);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TargetSkeletalMesh = InTargetSkeletalMesh;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	RegisterSkeletalMeshDelegates();
#endif
}

TArray<FGoomBindingGroupInfo>& UGroomBindingAsset::GetGroupInfos()
{
	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::GroupInfos, EGroomBindingAsyncPropertyLockType::ReadWrite);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GroupInfos;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UGroomBindingAsset::AddHairGroupResourcesToDelete(FHairGroupResource& In)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HairGroupResourcesToDelete.Enqueue(In);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UGroomBindingAsset::RemoveHairGroupResourcesToDelete(FHairGroupResource& Out)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HairGroupResourcesToDelete.Dequeue(Out);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName UGroomBindingAsset::GetHairGroupResourcesMemberName()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GET_MEMBER_NAME_CHECKED(UGroomBindingAsset, HairGroupResources);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UGroomBindingAsset::FHairGroupResources& UGroomBindingAsset::GetHairGroupResources()
{
	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::HairGroupResources, EGroomBindingAsyncPropertyLockType::ReadWrite);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HairGroupResources;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const UGroomBindingAsset::FHairGroupResources& UGroomBindingAsset::GetHairGroupResources() const
{
	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::HairGroupResources, EGroomBindingAsyncPropertyLockType::ReadOnly);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HairGroupResources;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UGroomBindingAsset::SetHairGroupResources(UGroomBindingAsset::FHairGroupResources InHairGroupResources)
{
	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::HairGroupResources, EGroomBindingAsyncPropertyLockType::WriteOnly);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HairGroupResources = InHairGroupResources;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName UGroomBindingAsset::GetHairGroupPlatformDataMemberName()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GET_MEMBER_NAME_CHECKED(UGroomBindingAsset, HairGroupsPlatformData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TArray<UGroomBindingAsset::FHairGroupPlatformData>& UGroomBindingAsset::GetHairGroupsPlatformData() const
{
	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::HairGroupPlatformData, EGroomBindingAsyncPropertyLockType::ReadOnly);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HairGroupsPlatformData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<UGroomBindingAsset::FHairGroupPlatformData>& UGroomBindingAsset::GetHairGroupsPlatformData()
{
	WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties::HairGroupPlatformData, EGroomBindingAsyncPropertyLockType::ReadWrite);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return HairGroupsPlatformData;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
void UGroomBindingAsset::RecreateResources()
{
	ReleaseResource(true /*bResetLoadedSize*/);
	InitResource();
	OnGroomBindingAssetChanged.Broadcast();
}

void UGroomBindingAsset::ChangeFeatureLevel(ERHIFeatureLevel::Type In)
{
	ChangeShaderPlatform(GShaderPlatformForFeatureLevel[In]);
}

void UGroomBindingAsset::ChangePlatformLevel(ERHIFeatureLevel::Type In)
{
	ChangeShaderPlatform(GShaderPlatformForFeatureLevel[In]);
}

void UGroomBindingAsset::ChangeShaderPlatform(EShaderPlatform InShaderPlatform)
{
	// When changing platform preview, recreate resources to the correct platform settings (e.g., r.hairstrands.strands=0/1)
	if (CachedResourceShaderPlatform != InShaderPlatform)
	{
		RecreateResources();
		CachedResourceShaderPlatform = InShaderPlatform;
	}
}
#endif // WITH_EDITOR
