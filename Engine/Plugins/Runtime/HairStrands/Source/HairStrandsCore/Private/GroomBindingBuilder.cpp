// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBindingBuilder.h"
#include "GroomBindingCommon.h"
#include "GeometryCache.h"
#include "GeometryCacheMeshData.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "Logging/StructuredLog.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshAttributeVertexBuffer.h"
#include "HairStrandsMeshProjection.h"
#include "Async/ParallelFor.h"
#include "GlobalShader.h"
#include "Misc/CoreMisc.h"
#include "Misc/ScopedSlowTask.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "GroomRBFDeformer.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Interfaces/ITargetPlatform.h"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Eigen for large matrix inversion
// Just to be sure, also added this in Eigen.Build.cs
#ifndef EIGEN_MPL2_ONLY
#define EIGEN_MPL2_ONLY
#endif

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Core>
#include <Eigen/Dense>
#include <Eigen/Sparse>
#include <Eigen/SparseLU>
#include <Eigen/SVD>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

// Run the binding asset building in parallel (faster)
#define BINDING_PARALLEL_BUILDING 1

///////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY_STATIC(LogGroomBindingBuilder, Log, All);

#define LOCTEXT_NAMESPACE "GroomBindingBuilder"

static int32 GHairStrandsBindingBuilderWarningEnable = 1;
static FAutoConsoleVariableRef CVarHairStrandsBindingBuilderWarningEnable(TEXT("r.HairStrands.Log.BindingBuilderWarning"), GHairStrandsBindingBuilderWarningEnable, TEXT("Enable/disable warning during groom binding builder"));

static TAutoConsoleVariable<int32> CVarHairStrandsBFLocalSpace(
	TEXT("r.HairStrands.RBFLocalSpace"),
	1,
	TEXT("Enable/disable rbf computation in local space."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

namespace UE::Groom
{
	bool IsRBFLocalSpaceEnabled()
	{
		return CVarHairStrandsBFLocalSpace.GetValueOnAnyThread() > 0;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FGroomBindingBuilder::FInput::FInput(UGroomBindingAsset* BindingAsset, const ITargetPlatform* TargetPlatform, int32 InSourceMeshLOD, int32 InTargetMeshMinLOD)
: BindingType(BindingAsset->GetGroomBindingType())
, NumInterpolationPoints(BindingAsset->GetNumInterpolationPoints())
, MatchingSection(BindingAsset->GetMatchingSection())
, TargetBindingAttribute(BindingAsset->GetTargetBindingAttribute())
, SourceMeshLOD(InSourceMeshLOD)
, TargetMeshMinLOD(InTargetMeshMinLOD)
, bHasValidTarget(BindingAsset->HasValidTarget())
, GroomAsset(BindingAsset->GetGroom())
, SourceSkeletalMesh(BindingAsset->GetSourceSkeletalMesh())
, TargetSkeletalMesh(BindingAsset->GetTargetSkeletalMesh())
, SourceGeometryCache(BindingAsset->GetSourceGeometryCache())
, TargetGeometryCache(BindingAsset->GetTargetGeometryCache())
{
	check(BindingType == EGroomBindingMeshType::SkeletalMesh || (SourceMeshLOD == 0 && TargetMeshMinLOD == 0));
}

FString FGroomBindingBuilder::GetVersion()
{
	// Important to update the version when groom building changes
	return TEXT("6d_563D4DE9469C41D590AE5810A85AA71B");
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Common utils functions
// These utils function are a copy of function in HairStrandsBindingCommon.ush
uint32 FHairStrandsRootUtils::PackTriangleIndex(uint32 TriangleIndex, uint32 SectionIndex)
{
	return ((SectionIndex & 0xFF) << 24) | (TriangleIndex & 0xFFFFFF);
}

// This function is a copy of UnpackTriangleIndex in HairStrandsBindingCommon.ush
void FHairStrandsRootUtils::UnpackTriangleIndex(uint32 Encoded, uint32& OutTriangleIndex, uint32& OutSectionIndex)
{
	OutSectionIndex = (Encoded >> 24) & 0xFF;
	OutTriangleIndex = Encoded & 0xFFFFFF;
}

uint32 FHairStrandsRootUtils::PackBarycentrics(const FVector2f& B)
{
	return uint32(FFloat16(B.X).Encoded) | (uint32(FFloat16(B.Y).Encoded)<<16);
}

FVector2f FHairStrandsRootUtils::UnpackBarycentrics(uint32 B)
{
	FFloat16 BX;
	BX.Encoded = (B & 0xFFFF);

	FFloat16 BY;
	BY.Encoded = (B >> 16) & 0xFFFF;

	return FVector2f(BX, BY);
}

uint32 FHairStrandsRootUtils::PackUVs(const FVector2f& UV)
{
	return (FFloat16(UV.X).Encoded & 0xFFFF) | ((FFloat16(UV.Y).Encoded & 0xFFFF) << 16);
}

float FHairStrandsRootUtils::PackUVsToFloat(const FVector2f& UV)
{
	uint32 Encoded = PackUVs(UV);
	return BitCast<float>(Encoded);
}

static float PackNormalToFloat(const FVector3f& InN)
{
	FVector3f N = InN; 
	N.Normalize();
	const FVector3f NN = N * 0.5f + 0.5f;
	const uint32 Encoded = 
		(uint32(FMath::Clamp(NN.X, 0.f, 1.f) * 1023.0f) & 0x3FF)    |
		(uint32(FMath::Clamp(NN.Y, 0.f, 1.f) * 1023.0f) & 0x3FF)<<10|
		(uint32(FMath::Clamp(NN.Z, 0.f, 1.f) * 1023.0f) & 0x3FF)<<20;
	return BitCast<float>(Encoded);
}

//////////////////////////////////////////////////////////////////////////
// Intermediate data struct

namespace GroomBinding_Mesh
{
////////////////////////////////////////////////////////////////////////////////////////
// Data structures to allow curves to be read from either bulk data or mesh description
////////////////////////////////////////////////////////////////////////////////////////

struct FHairStrandRootPositions
{
	FHairStrandRootPositions() = default;
	explicit FHairStrandRootPositions(const FHairStrandsDatas& Source)
	{
		RootPositions.Reserve(Source.GetNumCurves());

		for (uint32 CurveIndex = 0; CurveIndex < Source.GetNumCurves(); CurveIndex++)
		{
			const uint32 PointIndex = Source.StrandsCurves.CurvesOffset[CurveIndex];
			RootPositions.Add(Source.StrandsPoints.PointsPosition[PointIndex]);
		}

		BoundingBox = Source.BoundingBox;
	}

	explicit FHairStrandRootPositions(const FHairStrandsBulkData& Source)
	{
		const uint32 NumPoints = Source.GetNumPoints();
		const uint32 NumCurves = Source.GetNumCurves();
		RootPositions.Reserve(NumCurves);

		const FVector BoundingBoxCenter = Source.GetPositionOffset();

		const FHairStrandsCurveFormat::Type* Curves = reinterpret_cast<const FHairStrandsCurveFormat::Type*>(Source.Data.Curves.Data.LockReadOnly());
		check(Source.Data.Curves.GetBulkDataSize() == NumCurves * sizeof(FHairStrandsCurveFormat::Type));

		const FHairStrandsPositionFormat::Type* Positions = reinterpret_cast<const FHairStrandsPositionFormat::Type*>(Source.Data.Positions.Data.LockReadOnly());
		check(Source.Data.Positions.GetBulkDataSize() == NumPoints * sizeof(FHairStrandsPositionFormat::Type));

		for (uint32 CurveIndex = 0; CurveIndex < NumCurves; CurveIndex++)
		{
			const uint32 PointIndex = Curves[CurveIndex].PointOffset;
			const FHairStrandsPositionFormat::Type& Vertex = Positions[PointIndex];
			RootPositions.Add(FVector3f(BoundingBoxCenter + FVector(Vertex.X, Vertex.Y, Vertex.Z)));
		}

		Source.Data.Positions.Data.Unlock();
		Source.Data.Curves.Data.Unlock();

		BoundingBox = FBox3f(Source.GetBounds());
	}

	uint32 GetNumCurves() const { return RootPositions.Num(); }

	TArray<FVector3f> RootPositions;

	FBox3f BoundingBox = FBox3f(ForceInit);
};

//////////////////////////////////////////////////////////////////////////
// Interfaces to query mesh data from different sources
//////////////////////////////////////////////////////////////////////////

// Interface to gather info about a mesh section
class IMeshSectionData
{
public:
	virtual uint32 GetNumVertices() const = 0;
	virtual uint32 GetNumTriangles() const = 0;
	virtual uint32 GetBaseIndex() const = 0;
	virtual uint32 GetBaseVertexIndex() const = 0;
	virtual uint32 GetMaterialIndex() const = 0;
	virtual ~IMeshSectionData() {}
};

// Interface to query mesh data per LOD
class IMeshLODData
{
public:
	virtual const FVector3f* GetVerticesBuffer() const = 0;
	virtual uint32 GetNumVertices() const = 0;
	virtual const TArray<uint32>& GetIndexBuffer() const = 0;
	virtual const int32 GetNumSections() const = 0;
	virtual const IMeshSectionData& GetSection(uint32 SectionIndex) const = 0;
	virtual bool IsSectionValid(uint32 SectionIndex) const = 0;
	virtual const FVector3f& GetVertexPosition(uint32 VertexIndex) const = 0;
	virtual FVector2f GetVertexUV(uint32 VertexIndex, uint32 ChannelIndex) const = 0;
	virtual FVector3f GetVertexNormal(uint32 VertexIndex) const = 0;
	virtual int32 GetSectionFromVertexIndex(uint32 InVertIndex) const = 0;
	virtual bool HasVertexAttribute(FName In) const = 0;
	virtual void AllocateVertexBindingData(FName In) = 0;
	virtual void SetVertexBinding(uint32 VertexIt, float InterpolatedAttribute) = 0;
	virtual float GetVertexBinding(uint32 VertexIt) const = 0;
	virtual ~IMeshLODData() {}

	bool IsValidMaterialIndex(int32 InMaterialIndex) const
	{
		for (int32 SectionIt = 0, NumSections = GetNumSections(); SectionIt < NumSections; ++SectionIt)
		{
			if (IsSectionValid(SectionIt) && uint32(InMaterialIndex) == GetSection(SectionIt).GetMaterialIndex())
			{
				return true;
			}
		}
		return false;
	}

	TArray<int32> GetSectionsMatchingMaterialIndex(int32 InMaterialIndex) const
	{
		// Find all the sections which are mapped onto the target material index
		TArray<int32> Out;
		Out.Reserve(GetNumSections());
		for (int32 SectionIndex=0, NumSections = GetNumSections(); SectionIndex<NumSections; ++SectionIndex)
		{
			if (IsSectionValid(SectionIndex) && GetSection(SectionIndex).GetMaterialIndex() == InMaterialIndex)
			{
				Out.Add(SectionIndex);
			}
		}
		return Out;
	}
};

// Interface to wrap the mesh source and query its LOD data
class IMeshData
{
public:
	virtual bool IsValid() const = 0;
	virtual int32 GetNumLODs() const = 0;
	virtual bool IsLODValid(int32 LODIndex) const = 0;
	virtual const IMeshLODData& GetMeshLODData(int32 LODIndex) const = 0;
	virtual IMeshLODData& GetMeshLODData(int32 LODIndex) = 0;
	virtual ~IMeshData() {}
};

//////////////////////////////////////////////////////////////////////////
// Implementation for SkeletalMesh as a mesh source

class FSkeletalMeshSection : public IMeshSectionData
{
public:
	FSkeletalMeshSection(const FSkeletalMeshRenderData* InMeshData=nullptr, int32 InLODIndex=-1, int32 InSectionIndex=-1)
	{
		LODIndex 	 = InLODIndex;
		SectionIndex = InSectionIndex;
		MeshData 	 = InMeshData; 
	}

	virtual uint32 GetNumVertices() const override
	{
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		check(MeshData->LODRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex));
		return MeshData->LODRenderData[LODIndex].RenderSections[SectionIndex].NumVertices;
	}

	virtual uint32 GetNumTriangles() const override
	{
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		check(MeshData->LODRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex));
		return MeshData->LODRenderData[LODIndex].RenderSections[SectionIndex].NumTriangles;
	}

	virtual uint32 GetBaseIndex() const override
	{
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		check(MeshData->LODRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex));
		return MeshData->LODRenderData[LODIndex].RenderSections[SectionIndex].BaseIndex;
	}

	virtual uint32 GetBaseVertexIndex() const override
	{
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		check(MeshData->LODRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex));
		return MeshData->LODRenderData[LODIndex].RenderSections[SectionIndex].BaseVertexIndex;
	}

	virtual uint32 GetMaterialIndex() const override
	{
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		check(MeshData->LODRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex));
		return MeshData->LODRenderData[LODIndex].RenderSections[SectionIndex].MaterialIndex;
	}

private:
	int32 LODIndex = -1;
	int32 SectionIndex = -1;
	const FSkeletalMeshRenderData* MeshData = nullptr;
};

class FSkeletalMeshLODData : public IMeshLODData
{
public:
	FSkeletalMeshLODData(const FSkeletalMeshRenderData* InMeshData=nullptr, int32 InLODIndex=-1)
	{
		LODIndex = InLODIndex;
		MeshData = InMeshData;

		if (IsSourceGeometryAvailable())
		{
			const uint32 SectionCount = MeshData->LODRenderData[LODIndex].RenderSections.Num();
			Sections.Reserve(SectionCount);
			for (uint32 SectionIt=0;SectionIt<SectionCount; ++SectionIt)
			{
				Sections.Add(FSkeletalMeshSection(MeshData, LODIndex, SectionIt));
			}

			if (IndexBuffer.IsEmpty())
			{
				check(MeshData->LODRenderData.IsValidIndex(LODIndex));
				IndexBuffer.SetNum(MeshData->LODRenderData[LODIndex].MultiSizeIndexContainer.GetIndexBuffer()->Num());
				MeshData->LODRenderData[LODIndex].MultiSizeIndexContainer.GetIndexBuffer(IndexBuffer);
			}
		}
	}

	virtual const FVector3f* GetVerticesBuffer() const override
	{
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		return static_cast<const FVector3f*>(MeshData->LODRenderData[LODIndex].StaticVertexBuffers.PositionVertexBuffer.GetVertexData());
	}

	virtual uint32 GetNumVertices() const override
	{
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		return MeshData->LODRenderData[LODIndex].StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	}

	virtual const TArray<uint32>& GetIndexBuffer() const override
	{
		return IndexBuffer;
	}

	virtual const int32 GetNumSections() const override
	{
		return Sections.Num();
	}

	virtual const IMeshSectionData& GetSection(uint32 InSectionIndex) const override
	{
		check(MeshData);
		check(Sections.IsValidIndex(InSectionIndex));
		return Sections[InSectionIndex];
	}

	virtual bool IsSectionValid(uint32 SectionIndex) const override
	{
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		check(MeshData->LODRenderData[LODIndex].RenderSections.IsValidIndex(SectionIndex));
		return !MeshData->LODRenderData[LODIndex].RenderSections[SectionIndex].bDisabled;
	}

	virtual const FVector3f& GetVertexPosition(uint32 InVertexIndex) const override
	{
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		return MeshData->LODRenderData[LODIndex].StaticVertexBuffers.PositionVertexBuffer.VertexPosition(InVertexIndex);
	}

	virtual FVector2f GetVertexUV(uint32 InVertexIndex, uint32 InChannelIndex) const override
	{
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		return FVector2f(MeshData->LODRenderData[LODIndex].StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(InVertexIndex, InChannelIndex));
	}

	virtual FVector3f GetVertexNormal(uint32 InVertexIndex) const override
	{
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		return FVector3f(MeshData->LODRenderData[LODIndex].StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(InVertexIndex));
	}

	virtual int32 GetSectionFromVertexIndex(uint32 InVertIndex) const override
	{
		int32 OutSectionIndex = 0;
		check(MeshData);
		check(MeshData->LODRenderData.IsValidIndex(LODIndex));
		int32 OutVertIndex = 0;
		MeshData->LODRenderData[LODIndex].GetSectionFromVertexIndex(InVertIndex, OutSectionIndex, OutVertIndex);
		return OutSectionIndex;
	}

	bool IsSourceGeometryAvailable() const
	{
		return MeshData 
			&& MeshData->LODRenderData.IsValidIndex(LODIndex)
			&& MeshData->LODRenderData[LODIndex].StaticVertexBuffers.PositionVertexBuffer.GetVertexData() != nullptr;
	}

	bool HasVertexAttribute(FName In) const override
	{
		bool bHasAttribute = false;
		if (MeshData)
		{
			check(MeshData->LODRenderData.IsValidIndex(LODIndex));
			bHasAttribute = MeshData->LODRenderData[LODIndex].VertexAttributeBuffers.GetAttributeBuffer(In) != nullptr;
		}
		return bHasAttribute;
	}

	void AllocateVertexBindingData(FName In)
	{
		// If the skeletal mesh has the desired attribute, populate the vertex binding data with it
		if (HasVertexAttribute(In))
		{
			check(MeshData);
			check(MeshData->LODRenderData.IsValidIndex(LODIndex));
			if (FSkeletalMeshAttributeVertexBuffer* VertexAttribute = MeshData->LODRenderData[LODIndex].VertexAttributeBuffers.GetAttributeBuffer(In))
			{
				const FSkeletalMeshAttributeVertexBuffer::FCPUData CPUData = VertexAttribute->GetCPUData();
				if (CPUData.Data && CPUData.Data->GetNumVertices() > 0)
				{
					const uint32 NumVertices = CPUData.Data->GetNumVertices();
					VertexBindingData.Init(0, NumVertices);

					for (uint32 VertIt = 0; VertIt < NumVertices; ++VertIt)
					{
						const uint8* Element = CPUData.Data->GetDataPointer() + VertIt * CPUData.Data->GetStride();
						switch (CPUData.PixelFormat)
						{
							case PF_R8:
								VertexBindingData[VertIt] = (*Element) * (1.f / 255.f);
								break;
							case PF_R16F:
								VertexBindingData[VertIt] = *(FFloat16*)Element;
								break;
							case PF_R32_FLOAT:
								VertexBindingData[VertIt] = *(float*)Element;
								break;
						}
					}
				}
			}
		}

		// Otherwise allocate dummy vertex binding data
		const uint32 NumVertices = MeshData ? MeshData->LODRenderData[LODIndex].GetNumVertices() : 0u;
		if (VertexBindingData.Num() != NumVertices)
		{
			VertexBindingData.Init(1.f, NumVertices);
		}
	}

	void SetVertexBinding(uint32 InVertexIndex, float In) override
	{
		check(VertexBindingData.IsValidIndex(InVertexIndex));
		VertexBindingData[InVertexIndex] = In;
	}

	float GetVertexBinding(uint32 InVertexIndex) const override
	{
		check(VertexBindingData.IsValidIndex(InVertexIndex));
		return VertexBindingData[InVertexIndex];
	}

private:
	int32 LODIndex = -1;
	const FSkeletalMeshRenderData* MeshData = nullptr;

	TArray<uint32> IndexBuffer;
	TArray<FSkeletalMeshSection> Sections;

	TArray<float> VertexBindingData;
};

class FSkeletalMeshData : public IMeshData
{
public:
	FSkeletalMeshData(const USkeletalMesh* InSkeletalMesh, const FSkeletalMeshRenderData* InRenderData, int32 MeshMinLOD) 
	{
		SkeletalMesh = InSkeletalMesh;
		MeshData = InRenderData;
		if (SkeletalMesh)
		{
			if (MeshData)
			{
				LODCount = SkeletalMesh->GetLODNum();
				MeshesLODData.Reserve(LODCount);
				for (int32 LODIt = 0; LODIt < LODCount; ++LODIt)
				{
					// Only pass in the mesh data if we're building for this LOD
					const FSkeletalMeshRenderData* MeshDataForLOD = LODIt >= MeshMinLOD ? MeshData : nullptr;
					MeshesLODData.Add(FSkeletalMeshLODData(MeshDataForLOD, LODIt));
				}
			}
			else
			{
				UE_LOGF(LogHairStrands, Warning, "Could not retrieve mesh data for SkeletalMesh %ls.", *SkeletalMesh->GetName());
			}
		}
	}

	virtual bool IsValid() const override
	{
		return SkeletalMesh != nullptr && MeshData != nullptr && MeshesLODData.Num() > 0;
	}

	virtual int32 GetNumLODs() const override
	{
		return LODCount;
	}

	virtual bool IsLODValid(int32 LODIndex) const override
	{
		if (!MeshesLODData.IsValidIndex(LODIndex))
		{
			return false;
		}

		return MeshesLODData[LODIndex].IsSourceGeometryAvailable();
	}

	virtual const IMeshLODData& GetMeshLODData(int32 InLODIndex) const override
	{
		check(MeshesLODData.IsValidIndex(InLODIndex));
		return MeshesLODData[InLODIndex];
	}

	virtual IMeshLODData& GetMeshLODData(int32 InLODIndex) override
	{
		check(MeshesLODData.IsValidIndex(InLODIndex));
		return MeshesLODData[InLODIndex];
	}

private:
	int32 LODCount = 0;
	const USkeletalMesh* SkeletalMesh = nullptr;
	const FSkeletalMeshRenderData* MeshData = nullptr;
	TArray<FSkeletalMeshLODData> MeshesLODData;
};

//////////////////////////////////////////////////////////////////////////
// Implementation for GeometryCache as a mesh source

class FGeometryCacheSection : public IMeshSectionData
{
public:
	FGeometryCacheSection(FGeometryCacheMeshBatchInfo& InSection, uint32 InNumVertices, uint32 InBaseVertexIdex, uint32 InMaterialIndex)
		: Section(InSection)
		, NumVertices(InNumVertices)
		, BaseVertexIndex(InBaseVertexIdex)
		, MaterialIndex(InMaterialIndex)
	{
	}

	virtual uint32 GetNumVertices() const override
	{
		return NumVertices;
	}

	virtual uint32 GetNumTriangles() const override
	{
		return Section.NumTriangles;
	}

	virtual uint32 GetBaseIndex() const override
	{
		return Section.StartIndex;
	}

	virtual uint32 GetBaseVertexIndex() const override
	{
		return BaseVertexIndex;
	}

	virtual uint32 GetMaterialIndex() const override
	{
		return MaterialIndex;
	}

private:
	FGeometryCacheMeshBatchInfo& Section;
	uint32 NumVertices;
	uint32 BaseVertexIndex;
	uint32 MaterialIndex;
};

// GeometryCache have only one LOD so FGeometryCacheData provides both mesh source and mesh LOD data
class FGeometryCacheData : public IMeshData, public IMeshLODData
{
public:
	FGeometryCacheData(UGeometryCache* InGeometryCache) 
		: GeometryCache(InGeometryCache)
	{
		if (GeometryCache)
		{
			TArray<FGeometryCacheMeshData> MeshesData;
			GeometryCache->GetMeshDataAtTime(0.0f, MeshesData);
			if (MeshesData.Num() > 1)
			{
				UE_LOGF(LogHairStrands, Warning, "Cannot use non-flattened GeometryCache %ls as input.", *GeometryCache->GetName());
			}
			else if (MeshesData.Num() == 0)
			{
				UE_LOGF(LogHairStrands, Warning, "Could not read mesh data from the GeometryCache %ls.", *GeometryCache->GetName());
			}
			else
			{
				if (MeshesData[0].Positions.Num() > 0)
				{
					MeshData = MoveTemp(MeshesData[0]);
					for (FGeometryCacheMeshBatchInfo& BatchInfo : MeshData.BatchesInfo)
					{
						FRange SectionRange;
						for (uint32 VertexIndex = BatchInfo.StartIndex; VertexIndex < BatchInfo.StartIndex + BatchInfo.NumTriangles * 3; ++VertexIndex)
						{
							SectionRange.Add(MeshData.Indices[VertexIndex]);
						}
						SectionRanges.Add(SectionRange);

						Sections.Add(FGeometryCacheSection(BatchInfo, SectionRange.Num(), SectionRange.Min, BatchInfo.MaterialIndex));
					}
				}
				else
				{
					UE_LOGF(LogHairStrands, Warning, "GeometryCache %ls has no valid mesh data.", *GeometryCache->GetName());
				}
			}
		}
	}

	virtual bool IsValid() const override
	{
		return GeometryCache != nullptr && Sections.Num() > 0;
	}

	virtual int32 GetNumLODs() const override
	{
		return 1;
	}

	virtual bool IsLODValid(int32 LODIndex) const override
	{
		// There is only 1 LOD and it's always considered valid
		return LODIndex == 0;
	}

	virtual IMeshLODData& GetMeshLODData(int32 LODIndex) override
	{
		return *this;
	}

	virtual const IMeshLODData& GetMeshLODData(int32 LODIndex) const override
	{
		return *this;
	}

	virtual const FVector3f* GetVerticesBuffer() const override
	{
		return MeshData.Positions.GetData();
	}

	virtual uint32 GetNumVertices() const override
	{
		return MeshData.Positions.Num();
	}

	virtual const TArray<uint32>& GetIndexBuffer() const override
	{
		return MeshData.Indices;
	}

	virtual const int32 GetNumSections() const override
	{
		return Sections.Num();
	}

	virtual const IMeshSectionData& GetSection(uint32 SectionIndex) const override
	{
		return Sections[SectionIndex];
	}

	virtual bool IsSectionValid(uint32 SectionIndex) const override
	{
		return true;
	}

	virtual const FVector3f& GetVertexPosition(uint32 VertexIndex) const override
	{
		return MeshData.Positions[VertexIndex];
	}

	virtual FVector2f GetVertexUV(uint32 VertexIndex, uint32 ChannelIndex) const override
	{
		return FVector2f(MeshData.TextureCoordinates[VertexIndex]);
	}

	virtual FVector3f GetVertexNormal(uint32 InVertexIndex) const override
	{
		return MeshData.TangentsZ[InVertexIndex].ToFVector3f();
	}

	virtual int32 GetSectionFromVertexIndex(uint32 InVertIndex) const override
	{
		for (int32 SectionIndex = 0; SectionIndex < SectionRanges.Num(); ++SectionIndex)
		{
			const FRange& SectionRange = SectionRanges[SectionIndex];
			if (InVertIndex >= SectionRange.Min && InVertIndex <= SectionRange.Max)
			{
				return SectionIndex;
			}
		}
		return 0;
	}

	bool HasVertexAttribute(FName In) const override
	{
		return false;
	}

	void AllocateVertexBindingData(FName In)
	{
		const uint32 NumVertices = MeshData.Positions.Num();
		VertexBindingData.Init(1.f, NumVertices);
	}

	void SetVertexBinding(uint32 InVertexIndex, float In) override
	{
		check(VertexBindingData.IsValidIndex(InVertexIndex));
		VertexBindingData[InVertexIndex] = In;
	}

	float GetVertexBinding(uint32 InVertexIndex) const override
	{
		check(VertexBindingData.IsValidIndex(InVertexIndex));
		return VertexBindingData[InVertexIndex];
	}

private:
	UGeometryCache* GeometryCache;
	FGeometryCacheMeshData MeshData;
	TArray<FGeometryCacheSection> Sections;

	struct FRange
	{
		uint32 Min = -1;
		uint32 Max = 0;
		void Add(uint32 Value)
		{
			Min = FMath::Min(Min, Value);
			Max = FMath::Max(Max, Value);
		}
		uint32 Num() { return Max - Min + 1; }
	};

	TArray<FRange> SectionRanges;
	TArray<float> VertexBindingData;
};

}

///////////////////////////////////////////////////////////////////////////////////////////////////
// RBF weighting

namespace GroomBinding_RBFWeighting
{
	int32 FPointsSampler::StartingPoint(const TArray<bool>& ValidPoints, int32& NumPoints) const
	{
		int32 StartIndex = -1;
		NumPoints = 0;
		for (int32 i = 0; i < ValidPoints.Num(); ++i)
		{
			if (ValidPoints[i])
			{
				++NumPoints;
				if (StartIndex == -1)
				{
					StartIndex = i;
				}
			}
		}
		return StartIndex;
	}

	void FPointsSampler::BuildPositions(const FVector3f* PointPositions)
	{
		const int32 NumSamples = SampleIndices.Num();
		SamplePositions.SetNum(NumSamples);
		for (int32 i = 0; i < NumSamples; ++i)
		{
			SamplePositions[i] = PointPositions[SampleIndices[i]];
		}
	}

	void FPointsSampler::FurthestPoint(const int32 NumPoints, const FVector3f* PointPositions, const uint32 SampleIndex, TArray<bool>& ValidPoints, TArray<float>& PointsDistance)
	{
		float FurthestDistance = 0.0;
		uint32 PointIndex = 0;
		for (int32 j = 0; j < NumPoints; ++j)
		{
			if (ValidPoints[j])
			{
				PointsDistance[j] = FMath::Min((PointPositions[SampleIndices[SampleIndex - 1]] - PointPositions[j]).Size(), PointsDistance[j]);
				if (PointsDistance[j] >= FurthestDistance)
				{
					PointIndex = j;
					FurthestDistance = PointsDistance[j];
				}
			}
		}
		ValidPoints[PointIndex] = false;
		SampleIndices[SampleIndex] = PointIndex;
	}

	FPointsSampler::FPointsSampler(TArray<bool>& ValidPoints, const FVector3f* PointPositions, const int32 NumSamples)
	{
		int32 NumPoints = 0;
		int32 StartIndex = StartingPoint(ValidPoints, NumPoints);

		const int32 SamplesCount = FMath::Min(NumPoints, NumSamples);
		if (SamplesCount != 0)
		{
			SampleIndices.SetNum(SamplesCount);
			SampleIndices[0] = StartIndex;
			ValidPoints[StartIndex] = false;

			TArray<float> PointsDistance;
			PointsDistance.Init(MAX_FLT, ValidPoints.Num());

			for (int32 i = 1; i < SamplesCount; ++i)
			{
				FurthestPoint(ValidPoints.Num(), PointPositions, i, ValidPoints, PointsDistance);
			}
			BuildPositions(PointPositions);
		}
	}
}

namespace GroomBinding_RBFWeighting
{

	FWeightsBuilder::FWeightsBuilder(const uint32 NumRows, const uint32 NumColumns,
		const FVector3f* SourcePositions, const FVector3f* TargetPositions, const bool bLocalSpace)
	{
		const uint32 PolyRows 	 = FGroomRBFDeformer::GetEntryCount(NumRows);
		const uint32 PolyColumns = FGroomRBFDeformer::GetEntryCount(NumColumns);

		const uint32 SampleCount = NumRows;
		const uint32 WeightCount = FGroomRBFDeformer::GetWeightCount(SampleCount);

		MatrixEntries.Init(0.0, PolyRows * PolyColumns);
		InverseEntries.Init(0.0, PolyRows * PolyColumns);

		// Sanity check
		check(NumRows == NumColumns);
		check(WeightCount == MatrixEntries.Num());
		check(WeightCount == InverseEntries.Num());

		TArray<float>& LocalEntries = MatrixEntries;
		ParallelFor(NumRows,
			[
				NumRows,
				NumColumns,
				PolyRows,
				PolyColumns,
				SourcePositions,
				TargetPositions,
				&LocalEntries,
				bLocalSpace
			] (uint32 RowIndex)
			{
				const FVector3f PositionOffset = (bLocalSpace && (NumRows > 0))? SourcePositions[0] : FVector3f::Zero();
				int32 EntryIndex = RowIndex * PolyColumns;
				for (uint32 j = 0; j < NumColumns; ++j)
				{
					const float FunctionScale = (SourcePositions[RowIndex] - TargetPositions[j]).Size();
					LocalEntries[EntryIndex++] = FMath::Sqrt(FunctionScale * FunctionScale + 1.0);
				}
				LocalEntries[EntryIndex++] = 1.0;
				LocalEntries[EntryIndex++] = (SourcePositions[RowIndex].X-PositionOffset.X);
				LocalEntries[EntryIndex++] = (SourcePositions[RowIndex].Y-PositionOffset.Y);
				LocalEntries[EntryIndex++] = (SourcePositions[RowIndex].Z-PositionOffset.Z);

				EntryIndex = NumRows * PolyColumns + RowIndex;
				LocalEntries[EntryIndex] = 1.0;

				EntryIndex += PolyColumns;
				LocalEntries[EntryIndex] = (SourcePositions[RowIndex].X-PositionOffset.X);

				EntryIndex += PolyColumns;
				LocalEntries[EntryIndex] = (SourcePositions[RowIndex].Y-PositionOffset.Y);

				EntryIndex += PolyColumns;
				LocalEntries[EntryIndex] = (SourcePositions[RowIndex].Z-PositionOffset.Z);
			});
		static constexpr float REGUL_VALUE = 1e-4f;
		int32 EntryIndex = NumRows * PolyColumns + NumColumns;
		LocalEntries[EntryIndex] = REGUL_VALUE;

		EntryIndex += PolyColumns + 1;
		LocalEntries[EntryIndex] = REGUL_VALUE;

		EntryIndex += PolyColumns + 1;
		LocalEntries[EntryIndex] = REGUL_VALUE;

		EntryIndex += PolyColumns + 1;
		LocalEntries[EntryIndex] = REGUL_VALUE;

		ComputeWeights(PolyRows, PolyColumns);
	}

	void FWeightsBuilder::ComputeWeights(const uint32 NumRows, const uint32 NumColumns)
	{
		using EigenMatrix = Eigen::Map<Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic, Eigen::RowMajor>>;
		EigenMatrix WeightsMatrix(MatrixEntries.GetData(), NumRows, NumColumns);
		EigenMatrix WeightsInverse(InverseEntries.GetData(), NumColumns, NumRows);

		auto MatrixSvd = WeightsMatrix.jacobiSvd(Eigen::ComputeThinU | Eigen::ComputeThinV);
		const Eigen::VectorXf& SingularValues = MatrixSvd.singularValues();
		if (SingularValues.size() > 0)
		{
			const float Tolerance = FLT_EPSILON * SingularValues.array().abs()(0);
			WeightsInverse = MatrixSvd.matrixV() * (SingularValues.array().abs() > Tolerance).select(
				SingularValues.array().inverse(), 0).matrix().asDiagonal() * MatrixSvd.matrixU().adjoint();
		}
	}

	void UpdateInterpolationWeights(
		const FWeightsBuilder& InterpolationWeights,
		const FPointsSampler& PointsSampler,
		const GroomBinding_Mesh::IMeshLODData& MeshData,
		FHairStrandsRootData& OutRootLODData)
	{
		OutRootLODData.MeshSampleIndicesBuffer.SetNum(PointsSampler.SampleIndices.Num());
		OutRootLODData.MeshInterpolationWeightsBuffer.SetNum(InterpolationWeights.InverseEntries.Num());
		OutRootLODData.RestSamplePositionsBuffer.SetNum(PointsSampler.SampleIndices.Num());
		OutRootLODData.MeshSampleSectionsBuffer.SetNum(PointsSampler.SampleIndices.Num());

		OutRootLODData.SampleCount = PointsSampler.SampleIndices.Num();
		OutRootLODData.MeshSampleIndicesBuffer = PointsSampler.SampleIndices;
		OutRootLODData.MeshInterpolationWeightsBuffer = InterpolationWeights.InverseEntries;
		for (int32 i = 0; i < PointsSampler.SamplePositions.Num(); ++i)
		{			
			OutRootLODData.RestSamplePositionsBuffer[i] = FVector4f(PointsSampler.SamplePositions[i], 1.0f);
			OutRootLODData.MeshSampleSectionsBuffer[i] = MeshData.GetSectionFromVertexIndex(PointsSampler.SampleIndices[i]);
		}
	}

	void FillLocalValidPoints(
		const GroomBinding_Mesh::IMeshLODData& MeshLODData, 
		const int32 TargetMaterialIndex,
		const FHairStrandsRootData& ProjectionLOD, 
		TArray<bool>& OutValidPoints)
	{
		const TArray<uint32>& TriangleIndices = MeshLODData.GetIndexBuffer();

		OutValidPoints.Init(false, MeshLODData.GetNumVertices());

		const bool ValidMaterialIndex = (TargetMaterialIndex >= 0 && MeshLODData.IsValidMaterialIndex(TargetMaterialIndex));
		const TArray<int32> MatchingSections = MeshLODData.GetSectionsMatchingMaterialIndex(TargetMaterialIndex);
		
		for (uint32 EncodedTriangleId : ProjectionLOD.UniqueTriangleIndexBuffer)
		{
			uint32 SectionIndex = 0;
			uint32 TriangleIndex = 0;
			FHairStrandsRootUtils::UnpackTriangleIndex(EncodedTriangleId, TriangleIndex, SectionIndex);
			if (!ValidMaterialIndex || (ValidMaterialIndex && (MatchingSections.Find(SectionIndex) != INDEX_NONE)))
			{
				const GroomBinding_Mesh::IMeshSectionData& Section = MeshLODData.GetSection(SectionIndex);
				for (uint32 VertexIt = 0; VertexIt < 3; ++VertexIt)
				{
					const uint32 VertexIndex = TriangleIndices[Section.GetBaseIndex() + 3 * TriangleIndex + VertexIt];
					OutValidPoints[VertexIndex] = (VertexIndex >= Section.GetBaseVertexIndex()) && (VertexIndex < Section.GetBaseVertexIndex() + Section.GetNumVertices());
				}
			}
		}
	}

	void FillGlobalValidPoints(const GroomBinding_Mesh::IMeshLODData& MeshLODData, const int32 TargetMaterialIndex, TArray<bool>& OutValidPoints)
	{
		const TArray<uint32>& TriangleIndices = MeshLODData.GetIndexBuffer(); 
		if (TargetMaterialIndex >= 0 && MeshLODData.IsValidMaterialIndex(TargetMaterialIndex))
		{
			const TArray<int32> MatchingSections = MeshLODData.GetSectionsMatchingMaterialIndex(TargetMaterialIndex);

			OutValidPoints.Init(false, MeshLODData.GetNumVertices());

			for (int32 MatchingSectionIt : MatchingSections)
			{
				const GroomBinding_Mesh::IMeshSectionData& Section = MeshLODData.GetSection(MatchingSectionIt);
				for (uint32 TriangleIt = 0; TriangleIt < Section.GetNumTriangles(); ++TriangleIt)
				{
					for (uint32 VertexIt = 0; VertexIt < 3; ++VertexIt)
					{
						const uint32 VertexIndex = TriangleIndices[Section.GetBaseIndex() + 3 * TriangleIt + VertexIt];
						OutValidPoints[VertexIndex] = true;
					}
				}
			}
		}
		else
		{
			OutValidPoints.Init(true, MeshLODData.GetNumVertices());
		}
	}

	void ResetSampleData(FHairStrandsRootData& Out)
	{
		Out.SampleCount = 0;
		Out.MeshInterpolationWeightsBuffer.SetNum(0);
		Out.MeshSampleIndicesBuffer.SetNum(0);
		Out.RestSamplePositionsBuffer.SetNum(0);
	}

	void ComputeInterpolationWeights(
		FHairRootGroupData& Out, 
		const bool bNeedStrandsRoot,
		const uint32 NumInterpolationPoints, 
		const int32 MatchingMaterialIndex,
		const GroomBinding_Mesh::IMeshData* MeshData,
		const int32 TargetMeshMinLOD)
	{
		const int32 MeshLODCount = MeshData->GetNumLODs();
		const uint32 MaxSamples  = NumInterpolationPoints;

		for (int32 MeshLODIndex = 0; MeshLODIndex < MeshLODCount; ++MeshLODIndex)
		{
			if (MeshLODIndex < TargetMeshMinLOD)
			{
				check(!Out.SimRootDatas[MeshLODIndex].IsValid());
				check(!bNeedStrandsRoot || !Out.RenRootDatas[MeshLODIndex].IsValid());
				continue;
			}

			const GroomBinding_Mesh::IMeshLODData& MeshLODData = MeshData->GetMeshLODData(MeshLODIndex);

			int32 TargetMaterialIndex = -1;
			bool GlobalSamples = false;
			const FVector3f* PositionsPointer = nullptr;
			if (Out.MeshPositions_Transferred.Num() == MeshLODCount)
			{
				PositionsPointer = Out.MeshPositions_Transferred[MeshLODIndex].GetData();
				GlobalSamples = true;
				TargetMaterialIndex = MatchingMaterialIndex;				
			}
			else
			{
				PositionsPointer = MeshLODData.GetVerticesBuffer();
			}

			if (!GlobalSamples)
			{
				TArray<bool> ValidPoints;
				{
					FillLocalValidPoints(MeshLODData, TargetMaterialIndex, Out.SimRootDatas[MeshLODIndex], ValidPoints);

					FPointsSampler PointsSampler(ValidPoints, PositionsPointer, MaxSamples);
					const uint32 SampleCount = PointsSampler.SamplePositions.Num();

					FWeightsBuilder InterpolationWeights(SampleCount, SampleCount, PointsSampler.SamplePositions.GetData(), PointsSampler.SamplePositions.GetData(), UE::Groom::IsRBFLocalSpaceEnabled());

					// Guides
					UpdateInterpolationWeights(InterpolationWeights, PointsSampler, MeshLODData, Out.SimRootDatas[MeshLODIndex]);

					// Strands
					// No sample data, only used/available for guides
					if (bNeedStrandsRoot)
					{
						ResetSampleData(Out.RenRootDatas[MeshLODIndex]);
					}
				}
			}
			else
			{
				TArray<bool> ValidPoints;

				FillGlobalValidPoints(MeshLODData, TargetMaterialIndex, ValidPoints);

				FPointsSampler PointsSampler(ValidPoints, PositionsPointer, MaxSamples);
				const uint32 SampleCount = PointsSampler.SamplePositions.Num();

				FWeightsBuilder InterpolationWeights(SampleCount, SampleCount, PointsSampler.SamplePositions.GetData(), PointsSampler.SamplePositions.GetData(), UE::Groom::IsRBFLocalSpaceEnabled());

				// Guides
				UpdateInterpolationWeights(InterpolationWeights, PointsSampler, MeshLODData, Out.SimRootDatas[MeshLODIndex]);

				// Strands 
				// No sample data, only used/available for guides
				if (bNeedStrandsRoot)
				{
					ResetSampleData(Out.RenRootDatas[MeshLODIndex]);
				}
			}
		}
	}
}// namespace GroomBinding_RBFWeighting

///////////////////////////////////////////////////////////////////////////////////////////////////
// Root projection

namespace GroomBinding_RootProjection
{
	struct FTriangleGrid
	{
		struct FTriangle
		{
			uint32  TriangleIndex;
			uint32  SectionIndex;
			uint32  SectionBaseIndex;

			uint32  I0;
			uint32  I1;
			uint32  I2;

			FVector3f P0;
			FVector3f P1;
			FVector3f P2;

			FVector3f P0_NonTransfered;
			FVector3f P1_NonTransfered;
			FVector3f P2_NonTransfered;

			FVector3f N0;
			FVector3f N1;
			FVector3f N2;

			FVector2f UV0;
			FVector2f UV1;
			FVector2f UV2;

			float Attribute0;
			float Attribute1;
			float Attribute2;
		};

		struct FCell
		{
			TArray<FTriangle> Triangles;
		};
		typedef TArray<const FCell*> FCells;

		FTriangleGrid(const FVector3f& InMinBound, const FVector3f& InMaxBound)
		{
			static const int32 CellsLimitCount = 1 << 27; // 134M
			float InVoxelWorldSize = 2; //cm

			MinBound = InMinBound;
			MaxBound = InMaxBound;

			// Compute the voxel volume resolution, and snap the max bound to the voxel grid
			// Increase voxel size until the total allocation fit into CellsLimitCount budget
			GridResolution = FIntVector::ZeroValue;
			InVoxelWorldSize *= 0.5;
			do
			{
				InVoxelWorldSize *= 2;
				FVector3f VoxelResolutionF = (MaxBound - MinBound) / InVoxelWorldSize;
				GridResolution = FIntVector(FMath::CeilToInt(VoxelResolutionF.X), FMath::CeilToInt(VoxelResolutionF.Y), FMath::CeilToInt(VoxelResolutionF.Z));
			} while (static_cast<int64>(GridResolution.X) * GridResolution.Y * GridResolution.Z > CellsLimitCount);

			MaxBound = MinBound + FVector3f(GridResolution) * InVoxelWorldSize;

			const uint32 CellCount = GridResolution.X * GridResolution.Y * GridResolution.Z;
			Cells.SetNum(CellCount);
		}

		FORCEINLINE bool IsValid(const FIntVector& P) const
		{
			return
				0 <= P.X && P.X < GridResolution.X &&
				0 <= P.Y && P.Y < GridResolution.Y &&
				0 <= P.Z && P.Z < GridResolution.Z;
		}

		FORCEINLINE bool IsOutside(const FVector3f& MinP, const FVector3f& MaxP) const
		{
			return
				(MaxP.X <= MinBound.X || MaxP.Y <= MinBound.Y || MaxP.Z <= MinBound.Z) ||
				(MinP.X >= MaxBound.X || MinP.Y >= MaxBound.Y || MinP.Z >= MaxBound.Z);
		}

		FORCEINLINE FIntVector ClampToVolume(const FIntVector& CellCoord, bool& bIsValid) const
		{
			bIsValid = IsValid(CellCoord);
			return FIntVector(
				FMath::Clamp(CellCoord.X, 0, GridResolution.X - 1),
				FMath::Clamp(CellCoord.Y, 0, GridResolution.Y - 1),
				FMath::Clamp(CellCoord.Z, 0, GridResolution.Z - 1));
		}

		FORCEINLINE FIntVector ToCellCoord(const FVector3f& P) const
		{
			bool bIsValid = false;
			const FVector3f F = ((P - MinBound) / (MaxBound - MinBound));
			const FIntVector CellCoord = FIntVector(FMath::FloorToInt(F.X * GridResolution.X), FMath::FloorToInt(F.Y * GridResolution.Y), FMath::FloorToInt(F.Z * GridResolution.Z));
			return ClampToVolume(CellCoord, bIsValid);
		}

		uint32 ToIndex(const FIntVector& CellCoord) const
		{
			uint32 CellIndex = CellCoord.X + CellCoord.Y * GridResolution.X + CellCoord.Z * GridResolution.X * GridResolution.Y;
			check(CellIndex < uint32(Cells.Num()));
			return CellIndex;
		}

		FCells ToCells(const FVector3f& P) const
		{
			FCells Out;

			bool bIsValid = false;
			const FIntVector Coord = ToCellCoord(P);
			{
				const uint32 LinearIndex = ToIndex(Coord);
				if (Cells[LinearIndex].Triangles.Num() > 0)
				{
					Out.Add(&Cells[LinearIndex]);
					bIsValid = true;
				}
			}
			
			int32 Kernel = 1;
			do
			{
				for (int32 Z = -Kernel; Z <= Kernel; ++Z)
				for (int32 Y = -Kernel; Y <= Kernel; ++Y)
				for (int32 X = -Kernel; X <= Kernel; ++X)
				{
					// Do kernel box filtering layer, by layer
					if (FMath::Abs(X) != Kernel && FMath::Abs(Y) != Kernel && FMath::Abs(Z) != Kernel)
						continue;

					const FIntVector Offset(X, Y, Z);
					FIntVector C = Coord + Offset;
					C.X = FMath::Clamp(C.X, 0, GridResolution.X - 1);
					C.Y = FMath::Clamp(C.Y, 0, GridResolution.Y - 1);
					C.Z = FMath::Clamp(C.Z, 0, GridResolution.Z - 1);

					const uint32 LinearIndex = ToIndex(C);
					if (Cells[LinearIndex].Triangles.Num() > 0)
					{
						Out.Add(&Cells[LinearIndex]);
						bIsValid = true;
					}
				}
				++Kernel;

				// If no cells have been found in the entire grid, return
				if (Kernel >= FMath::Max3(GridResolution.X, GridResolution.Y, GridResolution.Z))
				{
					break;
				}
			} while (!bIsValid);

			return Out;
		}

		// Reject degenerate (near-zero area) triangles via cross product to avoid NaN 
		// from division by zero in ComputeClosestPoint's barycentric computation.
		bool IsTriangleValid(const FTriangle& T) const
		{
			const FVector3f AB = T.P1 - T.P0;
			const FVector3f AC = T.P2 - T.P0;
			const FVector3f Cross = FVector3f::CrossProduct(AB, AC);
			return FVector3f::DotProduct(Cross, Cross) > UE_SMALL_NUMBER;
		}

		bool Insert(const FTriangle& T)
		{
			if (!IsTriangleValid(T))
			{
				return false;
			}

			FVector3f TriMinBound;
			TriMinBound.X = FMath::Min(T.P0.X, FMath::Min(T.P1.X, T.P2.X));
			TriMinBound.Y = FMath::Min(T.P0.Y, FMath::Min(T.P1.Y, T.P2.Y));
			TriMinBound.Z = FMath::Min(T.P0.Z, FMath::Min(T.P1.Z, T.P2.Z));

			FVector3f TriMaxBound;
			TriMaxBound.X = FMath::Max(T.P0.X, FMath::Max(T.P1.X, T.P2.X));
			TriMaxBound.Y = FMath::Max(T.P0.Y, FMath::Max(T.P1.Y, T.P2.Y));
			TriMaxBound.Z = FMath::Max(T.P0.Z, FMath::Max(T.P1.Z, T.P2.Z));

			if (IsOutside(TriMinBound, TriMaxBound))
			{
				return false;
			}

			const FIntVector MinCoord = ToCellCoord(TriMinBound);
			const FIntVector MaxCoord = ToCellCoord(TriMaxBound);

			// Insert triangle in all cell covered by the AABB of the triangle
			bool bInserted = false;
			for (int32 Z = MinCoord.Z; Z <= MaxCoord.Z; ++Z)
			{
				for (int32 Y = MinCoord.Y; Y <= MaxCoord.Y; ++Y)
				{
					for (int32 X = MinCoord.X; X <= MaxCoord.X; ++X)
					{
						const FIntVector CellIndex(X, Y, Z);
						if (IsValid(CellIndex))
						{
							const uint32 CellLinearIndex = ToIndex(CellIndex);
							Cells[CellLinearIndex].Triangles.Add(T);
							bInserted = true;
						}
					}
				}
			}
			return bInserted;
		}

		FVector3f MinBound;
		FVector3f MaxBound;
		FIntVector GridResolution;
		TArray<FCell> Cells;
	};

	// Closest point on A triangle from another point
	// Code from the book "Real-Time Collision Detection" by Christer Ericson
	struct FTrianglePoint
	{
		FVector3f P;
		FVector3f Barycentric;
	};

	static FTrianglePoint ComputeClosestPoint(const FTriangleGrid::FTriangle& Tri, const FVector3f& P)
	{
		const FVector3f A = Tri.P0;
		const FVector3f B = Tri.P1;
		const FVector3f C = Tri.P2;

		// Check if P is in vertex region outside A.
		FVector3f AB = B - A;
		FVector3f AC = C - A;
		FVector3f AP = P - A;
		float D1 = FVector3f::DotProduct(AB, AP);
		float D2 = FVector3f::DotProduct(AC, AP);
		if (D1 <= 0.f && D2 <= 0.f)
		{
			FTrianglePoint Out;
			Out.P = A;
			Out.Barycentric = FVector3f(1, 0, 0);
			return Out;
		}

		// Check if P is in vertex region outside B.
		FVector3f BP = P - B;
		float D3 = FVector3f::DotProduct(AB, BP);
		float D4 = FVector3f::DotProduct(AC, BP);
		if (D3 >= 0.f && D4 <= D3)
		{
			FTrianglePoint Out;
			Out.P = B;
			Out.Barycentric = FVector3f(0, 1, 0);
			return Out;
		}

		// Check if P is in edge region of AB, and if so, return the projection of P onto AB.
		float VC = D1 * D4 - D3 * D2;
		if (VC <= 0.f && D1 >= 0.f && D3 <= 0.f)
		{
			float V = D1 / (D1 - D3);

			FTrianglePoint Out;
			Out.P = A + V * AB;
			Out.Barycentric = FVector3f(1 - V, V, 0);
			return Out;
		}

		// Check if P is in vertex region outside C.
		FVector3f CP = P - C;
		float D5 = FVector3f::DotProduct(AB, CP);
		float D6 = FVector3f::DotProduct(AC, CP);
		if (D6 >= 0.f && D5 <= D6)
		{
			FTrianglePoint Out;
			Out.P = C;
			Out.Barycentric = FVector3f(0, 0, 1);
			return Out;
		}

		// Check if P is in edge region of AC, and if so, return the projection of P onto AC.
		float VB = D5 * D2 - D1 * D6;
		if (VB <= 0.f && D2 >= 0.f && D6 <= 0.f)
		{
			float W = D2 / (D2 - D6);
			FTrianglePoint Out;
			Out.P = A + W * AC;
			Out.Barycentric = FVector3f(1 - W, 0, W);
			return Out;
		}

		// Check if P is in edge region of BC, and if so, return the projection of P onto BC.
		float VA = D3 * D6 - D5 * D4;
		if (VA <= 0.f && D4 - D3 >= 0.f && D5 - D6 >= 0.f)
		{
			float W = (D4 - D3) / (D4 - D3 + D5 - D6);
			FTrianglePoint Out;
			Out.P = B + W * (C - B);
			Out.Barycentric = FVector3f(0, 1 - W, W);
			return Out;
		}

		// P must be inside the face region. Compute the closest point through its barycentric coordinates (u,V,W).
		const float DenomSum = VA + VB + VC;

		// Guard against near-zero denominator from degenerate triangles to prevent NaN propagation.
		// Fallback with picking the closest point
		if (DenomSum <= UE_SMALL_NUMBER)
		{
			const float DistA = FVector3f::DistSquared(P, A);
			const float DistB = FVector3f::DistSquared(P, B);
			const float DistC = FVector3f::DistSquared(P, C);

			FTrianglePoint Out;
			if (DistA <= DistB && DistA <= DistC)
			{
				Out.P = A;
				Out.Barycentric = FVector3f(1, 0, 0);
			}
			else if (DistB <= DistC)
			{
				Out.P = B;
				Out.Barycentric = FVector3f(0, 1, 0);
			}
			else
			{
				Out.P = C;
				Out.Barycentric = FVector3f(0, 0, 1);
			}
			return Out;
		}

		const float Denom = 1.f / DenomSum;
		float V = VB * Denom;
		float W = VC * Denom;

		FTrianglePoint Out;
		Out.P = A + AB * V + AC * W;
		Out.Barycentric = FVector3f(1 - V - W, V, W);
		return Out;
	}

	struct FQueryResult
	{
		FTriangleGrid::FTriangle Triangle;
		FTrianglePoint Point;
		bool bValid = false;
	};

	FQueryResult QueryClosestPoint(const FTriangleGrid& InGrid, const FVector3f& InQueryP)
	{
		FQueryResult Out;

		const FTriangleGrid::FCells Cells = InGrid.ToCells(InQueryP);
		if (Cells.Num() > 0)
		{
			float ClosestDistance = FLT_MAX;
			FTrianglePoint ClosestPointOnTriangle = {};
			FTriangleGrid::FTriangle ClosestTriangle = {};
			for (const FTriangleGrid::FCell* Cell : Cells)
			{
				for (const FTriangleGrid::FTriangle& CellTriangle : Cell->Triangles)
				{
					const FTrianglePoint PointOnTriangle = ComputeClosestPoint(CellTriangle, InQueryP);
					const float Distance = FVector3f::Distance(PointOnTriangle.P, InQueryP);
					if (Distance < ClosestDistance)
					{
						ClosestDistance = Distance;
						ClosestTriangle = CellTriangle;
						ClosestPointOnTriangle = PointOnTriangle;
					}
				}
			}

			// Gracefully handle NaN distances to report an error via the existing bValid==false path.
			if (ClosestDistance < FLT_MAX)
			{
				Out.Triangle = ClosestTriangle;
				Out.Point = ClosestPointOnTriangle;
				Out.bValid = true;
			}
		}

		return Out;
	}

	FBox3f ComputeMeshLODBound(const GroomBinding_Mesh::IMeshLODData& MeshLODData, const TArray<FVector3f>& InTransferredPositions, const FVector3f& InRootsCenter, float& OutClosestTrianglePoint, bool bFilterBindableTriangle)
	{
		FBox3f Out;
		Out.Init();

		const uint32 SectionCount = MeshLODData.GetNumSections();
		const bool bHasTransferredPosition = InTransferredPositions.Num() > 0;

		const uint32 MaxSectionCount = GetHairStrandsMaxSectionCount();
		const uint32 MaxTriangleCount = GetHairStrandsMaxTriangleCount();
		const TArray<uint32>& IndexBuffer = MeshLODData.GetIndexBuffer();

		OutClosestTrianglePoint = FLT_MAX;
		check(SectionCount > 0);
		for (uint32 SectionIt = 0; SectionIt < SectionCount; ++SectionIt)
		{
			if (!MeshLODData.IsSectionValid(SectionIt))
			{
				continue;
			}

			// 2.2.1 Compute the bounding box of the skeletal mesh
			const GroomBinding_Mesh::IMeshSectionData& Section = MeshLODData.GetSection(SectionIt);
			const uint32 TriangleCount = Section.GetNumTriangles();
			const uint32 SectionBaseIndex = Section.GetBaseIndex();

			check(TriangleCount < MaxTriangleCount);
			check(SectionCount < MaxSectionCount);
			check(TriangleCount > 0);

			for (uint32 TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
			{
				FTriangleGrid::FTriangle T;
				T.TriangleIndex = TriangleIt;
				T.SectionIndex = SectionIt;
				T.SectionBaseIndex = SectionBaseIndex;

				T.I0 = IndexBuffer[T.SectionBaseIndex + T.TriangleIndex * 3 + 0];
				T.I1 = IndexBuffer[T.SectionBaseIndex + T.TriangleIndex * 3 + 1];
				T.I2 = IndexBuffer[T.SectionBaseIndex + T.TriangleIndex * 3 + 2];
			
				const bool bBindableTriangle = bFilterBindableTriangle ? (MeshLODData.GetVertexBinding(T.I0) > 0 || MeshLODData.GetVertexBinding(T.I1) > 0 || MeshLODData.GetVertexBinding(T.I2) > 0) : true;
				if (!bBindableTriangle)
				{
					continue;
				}

				if (bHasTransferredPosition)
				{
					T.P0 = InTransferredPositions[T.I0];
					T.P1 = InTransferredPositions[T.I1];
					T.P2 = InTransferredPositions[T.I2];
				}
				else
				{
					T.P0 = MeshLODData.GetVertexPosition(T.I0);
					T.P1 = MeshLODData.GetVertexPosition(T.I1);
					T.P2 = MeshLODData.GetVertexPosition(T.I2);
				}

				Out += T.P0;
				Out += T.P1;
				Out += T.P2;

				// Track closest point to the groom bound
				OutClosestTrianglePoint = FMath::Min(OutClosestTrianglePoint, (T.P0 - InRootsCenter).Length());
				OutClosestTrianglePoint = FMath::Min(OutClosestTrianglePoint, (T.P1 - InRootsCenter).Length());
				OutClosestTrianglePoint = FMath::Min(OutClosestTrianglePoint, (T.P2 - InRootsCenter).Length());
			}
		}

		return Out;
	}

	FBox3f ComputeMeshLODBound(const GroomBinding_Mesh::IMeshLODData& MeshLODData, bool bFilterBindableTriangle)
	{
		TArray<FVector3f> DummyTransferredPositions;
		float DummyClosestTrianglePoint;
		FVector3f DummyRootsCenter = FVector3f::ZeroVector; 		
		return ComputeMeshLODBound(MeshLODData, DummyTransferredPositions, DummyRootsCenter, DummyClosestTrianglePoint, bFilterBindableTriangle);
	}

	bool BuildMeshLODTriangleGrid(const GroomBinding_Mesh::IMeshLODData& MeshLODData, const TArray<FVector3f>& InTransferredPositions, FTriangleGrid& OutGrid, bool bFilterBindableTriangle)
	{
		const uint32 SectionCount = MeshLODData.GetNumSections();
		const bool bHasTransferredPosition = InTransferredPositions.Num() > 0;
		const uint32 ChannelIndex = 0;

		const uint32 MaxSectionCount = GetHairStrandsMaxSectionCount();
		const uint32 MaxTriangleCount = GetHairStrandsMaxTriangleCount();
		const TArray<uint32>& IndexBuffer = MeshLODData.GetIndexBuffer();

		bool bIsGridPopulated = false;
		for (uint32 SectionIt = 0; SectionIt < SectionCount; ++SectionIt)
		{
			if (!MeshLODData.IsSectionValid(SectionIt))
			{
				continue;
			}

			// 2.2.2 Insert all triangle within the grid
			const GroomBinding_Mesh::IMeshSectionData& Section = MeshLODData.GetSection(SectionIt);
			const uint32 TriangleCount = Section.GetNumTriangles();
			const uint32 SectionBaseIndex = Section.GetBaseIndex();

			check(TriangleCount < MaxTriangleCount);
			check(SectionCount < MaxSectionCount);
			check(TriangleCount > 0);

			for (uint32 TriangleIt = 0; TriangleIt < TriangleCount; ++TriangleIt)
			{
				FTriangleGrid::FTriangle T;
				T.TriangleIndex = TriangleIt;
				T.SectionIndex = SectionIt;
				T.SectionBaseIndex = SectionBaseIndex;

				T.I0 = IndexBuffer[T.SectionBaseIndex + T.TriangleIndex * 3 + 0];
				T.I1 = IndexBuffer[T.SectionBaseIndex + T.TriangleIndex * 3 + 1];
				T.I2 = IndexBuffer[T.SectionBaseIndex + T.TriangleIndex * 3 + 2];

				const bool bBindableTriangle = bFilterBindableTriangle ? (MeshLODData.GetVertexBinding(T.I0) > 0 || MeshLODData.GetVertexBinding(T.I1) > 0 || MeshLODData.GetVertexBinding(T.I2) > 0) : true;
				if (!bBindableTriangle)
				{
					continue;
				}

				if (bHasTransferredPosition)
				{
					T.P0 = InTransferredPositions[T.I0];
					T.P1 = InTransferredPositions[T.I1];
					T.P2 = InTransferredPositions[T.I2];
				}
				else
				{
					T.P0 = MeshLODData.GetVertexPosition(T.I0);
					T.P1 = MeshLODData.GetVertexPosition(T.I1);
					T.P2 = MeshLODData.GetVertexPosition(T.I2);
				}

				// Store non-transfered position as well for RBF remapping
				T.P0_NonTransfered = MeshLODData.GetVertexPosition(T.I0);
				T.P1_NonTransfered = MeshLODData.GetVertexPosition(T.I1);
				T.P2_NonTransfered = MeshLODData.GetVertexPosition(T.I2);

				T.UV0 = MeshLODData.GetVertexUV(T.I0, ChannelIndex);
				T.UV1 = MeshLODData.GetVertexUV(T.I1, ChannelIndex);
				T.UV2 = MeshLODData.GetVertexUV(T.I2, ChannelIndex);

				T.N0 = MeshLODData.GetVertexNormal(T.I0);
				T.N1 = MeshLODData.GetVertexNormal(T.I1);
				T.N2 = MeshLODData.GetVertexNormal(T.I2);

				T.Attribute0 = MeshLODData.GetVertexBinding(T.I0);
				T.Attribute1 = MeshLODData.GetVertexBinding(T.I1);
				T.Attribute2 = MeshLODData.GetVertexBinding(T.I2);
					
				bIsGridPopulated = OutGrid.Insert(T) || bIsGridPopulated;
			}
		}

		return bIsGridPopulated;
	}

	bool BuildMeshLODTriangleGrid(const GroomBinding_Mesh::IMeshLODData& MeshLODData, FTriangleGrid& OutGrid, bool bFilterBindableTriangle)
	{
		TArray<FVector3f> DummyTransferredPositions;
		return BuildMeshLODTriangleGrid(MeshLODData, DummyTransferredPositions, OutGrid, bFilterBindableTriangle);
	}

	void TransfertMeshAttribute(
		FName InAttributeName,
		GroomBinding_Mesh::IMeshData* InMeshData)
	{
		// Check if the attribute exist on any/all LODs to see if any transfer from LOD to LOD is necessary
		int32 SourceLODIndex = INDEX_NONE;
		bool bAny = false;
		bool bAll = true;
		const int32 MeshLODCount = InMeshData->GetNumLODs();
		for (int32 MeshLODIt = 0; MeshLODIt < MeshLODCount; ++MeshLODIt)
		{
			const bool bHasAttribute = InMeshData->GetMeshLODData(MeshLODIt).HasVertexAttribute(InAttributeName);
			bAny = bAny || bHasAttribute;
			bAll = bAll && bHasAttribute;

			if (SourceLODIndex == INDEX_NONE && bHasAttribute)
			{
				SourceLODIndex = MeshLODIt;
			}

			InMeshData->GetMeshLODData(MeshLODIt).AllocateVertexBindingData(InAttributeName);
		}

		// If no transfer is needed, early out (i.e., all LODs have the attribute, or no LOD has the attribute)
		const bool bNeedTransfer = bAny && !bAll;
		if (!bNeedTransfer || SourceLODIndex == INDEX_NONE)
		{
			return;
		}

		// 1. Build source grid for transfer
		FTriangleGrid* TriangleGrid = nullptr;
		{
			GroomBinding_Mesh::IMeshLODData& MeshLODData = InMeshData->GetMeshLODData(SourceLODIndex);

			if (MeshLODData.GetNumSections() == 0)
			{
				UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. MeshLODData has 0 sections.");
				return;
			}

			// Compute mesh bound
			// Take the smallest bounding box between the groom and the skeletal mesh
			const FBox3f MeshBound = ComputeMeshLODBound(MeshLODData, false /*bFIlterBindableTriangle*/);
			const FVector3f MeshExtent = MeshBound.Max - MeshBound.Min;
			const FVector3f GridMin = MeshBound.Min - MeshExtent * 0.1f;
			const FVector3f GridMax = MeshBound.Max + MeshExtent * 0.1f;
			
			// Build grid
			TriangleGrid = new FTriangleGrid(GridMin, GridMax);
			const bool bIsGridPopulated = BuildMeshLODTriangleGrid(MeshLODData, *TriangleGrid, false /*bFilterBindableTriangle*/);

			if (!bIsGridPopulated)
			{
				delete TriangleGrid;
				return;
			}
		}

		// 2. Transfer the attribute to all the LODs
		for (int32 MeshLODIt = 0; MeshLODIt < MeshLODCount; ++MeshLODIt)
		{
			const bool bHasAttribute = InMeshData->GetMeshLODData(MeshLODIt).HasVertexAttribute(InAttributeName);
			if (bHasAttribute)
			{
				continue;
			}

			GroomBinding_Mesh::IMeshLODData& MeshLODData = InMeshData->GetMeshLODData(MeshLODIt);
			const uint32 SectionCount = MeshLODData.GetNumSections();
			for (uint32 SectionIt = 0; SectionIt < SectionCount; ++SectionIt)
			{
				if (!MeshLODData.IsSectionValid(SectionIt))
				{
					continue;
				}
				// 2.2.2 Insert all triangle within the grid
				const GroomBinding_Mesh::IMeshSectionData& Section = MeshLODData.GetSection(SectionIt);
				const uint32 VertexCount = Section.GetNumVertices();
				const uint32 BaseVertexIndex = Section.GetBaseVertexIndex();
				
				GroomBinding_Mesh::IMeshLODData* LocalMeshLODData = &MeshLODData;
				ParallelFor(VertexCount, [InAttributeName, SectionIt, TriangleGrid, LocalMeshLODData, BaseVertexIndex] (uint32 VertexIt)
				{
					const uint32 VertexIndex = BaseVertexIndex + VertexIt;
					const FVector3f P = LocalMeshLODData->GetVertexPosition(VertexIndex);
					const FQueryResult Q = QueryClosestPoint(*TriangleGrid, P);

					if (Q.bValid)
					{
						const float InterpolatedAttribute = 
							Q.Triangle.Attribute0 * Q.Point.Barycentric.X +
							Q.Triangle.Attribute1 * Q.Point.Barycentric.Y +
							Q.Triangle.Attribute2 * Q.Point.Barycentric.Z;

						LocalMeshLODData->SetVertexBinding(VertexIndex,  InterpolatedAttribute);
					}
				});
			}
		}

		delete TriangleGrid;
	} // Transfer attribute

	static bool Project(
		const GroomBinding_Mesh::FHairStrandRootPositions& InRootPositions,
		const GroomBinding_Mesh::IMeshData* InMeshData,
		int32 TargetMeshMinLOD,
		const TArray<TArray<FVector3f>>& InTransferredPositions,
		TArray<FHairStrandsRootData>& OutRootData)
	{
		// 2. Project root for each mesh LOD
		const uint32 CurveCount = InRootPositions.GetNumCurves();
		const uint32 ChannelIndex = 0;
		const int32 MeshLODCount = InMeshData->GetNumLODs();
		check(MeshLODCount == OutRootData.Num());

		const bool bHasTransferredPosition = InTransferredPositions.Num() > 0;
		if (bHasTransferredPosition)
		{
			check(InTransferredPositions.Num() == MeshLODCount);
		}

		for (int32 MeshLODIt = 0; MeshLODIt < MeshLODCount; ++MeshLODIt)
		{
			check(MeshLODIt == OutRootData[MeshLODIt].LODIndex);

			if (MeshLODIt < TargetMeshMinLOD)
			{
				check(!OutRootData[MeshLODIt].IsValid());
				continue;
			}

			// 2.1. Build a grid around the hair AABB
			const GroomBinding_Mesh::IMeshLODData& MeshLODData = InMeshData->GetMeshLODData(MeshLODIt);
			const TArray<uint32>& IndexBuffer = MeshLODData.GetIndexBuffer();

			const uint32 MaxSectionCount = GetHairStrandsMaxSectionCount();
			const uint32 MaxTriangleCount = GetHairStrandsMaxTriangleCount();

			const uint32 SectionCount = MeshLODData.GetNumSections();
			if ( SectionCount == 0 )
			{
				UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. MeshLODData has 0 sections.");
				return false;
			}

			float ClosestTrianglePoint = FLT_MAX;
			const FBox3f MeshBound = ComputeMeshLODBound(MeshLODData, bHasTransferredPosition ? InTransferredPositions[MeshLODIt] : TArray<FVector3f>(), InRootPositions.BoundingBox.GetCenter(), ClosestTrianglePoint, true /*bFIlterBindableTriangle*/);

			// Take the smallest bounding box between the groom and the skeletal mesh
			const FVector3f MeshExtent = MeshBound.Max - MeshBound.Min;
			const FVector3f HairExtent = InRootPositions.BoundingBox.Max - InRootPositions.BoundingBox.Min;
			FVector3f GridMin;
			FVector3f GridMax;
			if (MeshExtent.Size() < HairExtent.Size())
			{
				GridMin = MeshBound.Min;
				GridMax = MeshBound.Max;
			}
			else
			{
				GridMin = InRootPositions.BoundingBox.Min;
				GridMax = InRootPositions.BoundingBox.Max;

				// By nature, it is possible that coarser LOD have positions which ressemble only very coarsly to 
				// LOD0. In this case we increase the hair bound to ensure that skel. mesh triangles will be intersect 
				// the groom bound to be correctly inserted.
				if (ClosestTrianglePoint < FLT_MAX)
				{
					GridMin -= FVector3f(ClosestTrianglePoint * 1.25f);
					GridMax += FVector3f(ClosestTrianglePoint * 1.25f);
				}
			}

			FTriangleGrid Grid(GridMin, GridMax);
			const bool bIsGridPopulated = BuildMeshLODTriangleGrid(MeshLODData, bHasTransferredPosition ? InTransferredPositions[MeshLODIt] : TArray<FVector3f>(), Grid, true /*bFIlterBindableTriangle*/);
			if (!bIsGridPopulated)
			{
				UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. The target skeletal mesh could be missing UVs.");
				return false;
			}

			OutRootData[MeshLODIt].RootBarycentricBuffer.SetNum(CurveCount);
			OutRootData[MeshLODIt].RootToUniqueTriangleIndexBuffer.SetNum(CurveCount);

			// 2.3. Compute the closest triangle for each root
			//InMeshRenderData->LODRenderData[LODIt].GetNumVertices();

			TArray<FHairStrandsUniqueTriangleIndexFormat::Type> RootTriangleIndexBuffer;
			RootTriangleIndexBuffer.SetNum(CurveCount);

			TArray<FHairStrandsMeshTrianglePositionFormat::Type> RestRootTrianglePositionBuffer;
			RestRootTrianglePositionBuffer.SetNum(CurveCount * 3);

			TArray<FHairStrandsMeshTrianglePositionFormat::Type> RestRootTrianglePositionBuffer_NonTransfered;
			RestRootTrianglePositionBuffer_NonTransfered.SetNum(CurveCount * 3);

		#if BINDING_PARALLEL_BUILDING
			TAtomic<uint32> bIsValid(1);
			ParallelFor(CurveCount,
				[
					MeshLODIt,
					&InRootPositions,
					&Grid,
					&RootTriangleIndexBuffer,
					&RestRootTrianglePositionBuffer,
					&RestRootTrianglePositionBuffer_NonTransfered,
					&OutRootData,
					&bIsValid
				] (uint32 CurveIndex)
		#else
			for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
		#endif
			{
				const FVector3f& RootP = InRootPositions.RootPositions[CurveIndex];
				const FQueryResult QueryResult = QueryClosestPoint(Grid, RootP);
				if (!QueryResult.bValid)
				{
					#if BINDING_PARALLEL_BUILDING
					bIsValid = 0; return;
					#else
					return false;
					#endif
				}
				const FTriangleGrid::FTriangle& ClosestTriangle = QueryResult.Triangle;
				const FVector2f ClosestBarycentrics(QueryResult.Point.Barycentric.X, QueryResult.Point.Barycentric.Y);

				// Record closest triangle and the root's barycentrics
				const uint32 EncodedBarycentrics = FHairStrandsRootUtils::PackBarycentrics(FVector2f(ClosestBarycentrics));	// LWC_TODO: Precision loss
				const uint32 EncodedTriangleIndex = FHairStrandsRootUtils::PackTriangleIndex(ClosestTriangle.TriangleIndex, ClosestTriangle.SectionIndex);
				OutRootData[MeshLODIt].RootBarycentricBuffer[CurveIndex] = EncodedBarycentrics;

				RootTriangleIndexBuffer[CurveIndex] = EncodedTriangleIndex;
				RestRootTrianglePositionBuffer[CurveIndex * 3 + 0] = FVector4f((FVector3f)ClosestTriangle.P0, PackNormalToFloat(ClosestTriangle.N0));
				RestRootTrianglePositionBuffer[CurveIndex * 3 + 1] = FVector4f((FVector3f)ClosestTriangle.P1, PackNormalToFloat(ClosestTriangle.N1));
				RestRootTrianglePositionBuffer[CurveIndex * 3 + 2] = FVector4f((FVector3f)ClosestTriangle.P2, PackNormalToFloat(ClosestTriangle.N2));

				RestRootTrianglePositionBuffer_NonTransfered[CurveIndex * 3 + 0] = FVector4f((FVector3f)ClosestTriangle.P0_NonTransfered, PackNormalToFloat(ClosestTriangle.N0));
				RestRootTrianglePositionBuffer_NonTransfered[CurveIndex * 3 + 1] = FVector4f((FVector3f)ClosestTriangle.P1_NonTransfered, PackNormalToFloat(ClosestTriangle.N1));
				RestRootTrianglePositionBuffer_NonTransfered[CurveIndex * 3 + 2] = FVector4f((FVector3f)ClosestTriangle.P2_NonTransfered, PackNormalToFloat(ClosestTriangle.N2));
			}
		#if BINDING_PARALLEL_BUILDING
			);
			if (bIsValid == 0)
			{
				return false;
			}
		#endif

			// Build list of unique triangles
			TArray<uint32> UniqueSectionId;
			TArray<uint32> UniqueTriangleToRootList;
			TMap<uint32, TArray<uint32>> UniqueTriangleToRootMap;
			for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
			{
				const uint32 EncodedTriangleId = RootTriangleIndexBuffer[CurveIndex];
				if (TArray<uint32>* CurvesList = UniqueTriangleToRootMap.Find(EncodedTriangleId))
				{
					CurvesList->Add(CurveIndex);
				}
				else
				{
					// Add unique section
					uint32 TriangleIndex;
					uint32 SectionIndex;
					FHairStrandsRootUtils::UnpackTriangleIndex(EncodedTriangleId, TriangleIndex, SectionIndex);
					UniqueSectionId.AddUnique(SectionIndex);

					// Add unique triangle
					UniqueTriangleToRootList.Add(EncodedTriangleId);
					TArray<uint32>& NewCurvesList = UniqueTriangleToRootMap.Add(EncodedTriangleId);
					NewCurvesList.Add(CurveIndex);
				}
			}

			// Sort unique triangle per section and triangle ID (encoded triangle ID stores section ID in high bits)
			UniqueTriangleToRootList.Sort();
			UniqueSectionId.Sort();

			// Build final unique triangle list and the root-to-unique-triangle mapping
			const uint32 UniqueTriangleCount = UniqueTriangleToRootList.Num();
			OutRootData[MeshLODIt].UniqueTriangleIndexBuffer.Reserve(UniqueTriangleCount );
			OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer.Reserve(UniqueTriangleCount * 3);
			OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer_TargetNonTransfered.Reserve(UniqueTriangleCount * 3);
			for (uint32 EncodedTriangleId : UniqueTriangleToRootList)
			{
				auto It = UniqueTriangleToRootMap.Find(EncodedTriangleId);
				check(It);

				OutRootData[MeshLODIt].UniqueTriangleIndexBuffer.Add(EncodedTriangleId);

				const uint32 FirstCurveIndex = (*It)[0];
				OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer.Add(RestRootTrianglePositionBuffer[FirstCurveIndex * 3 + 0]);
				OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer.Add(RestRootTrianglePositionBuffer[FirstCurveIndex * 3 + 1]);
				OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer.Add(RestRootTrianglePositionBuffer[FirstCurveIndex * 3 + 2]);

				OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer_TargetNonTransfered.Add(RestRootTrianglePositionBuffer_NonTransfered[FirstCurveIndex * 3 + 0]);
				OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer_TargetNonTransfered.Add(RestRootTrianglePositionBuffer_NonTransfered[FirstCurveIndex * 3 + 1]);
				OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer_TargetNonTransfered.Add(RestRootTrianglePositionBuffer_NonTransfered[FirstCurveIndex * 3 + 2]);

				// Write for each root, the index of the triangle
				const uint32 UniqueTriangleIndex = OutRootData[MeshLODIt].UniqueTriangleIndexBuffer.Num()-1;
				for (uint32 CurveIndex : *It)
				{
					OutRootData[MeshLODIt].RootToUniqueTriangleIndexBuffer[CurveIndex] = UniqueTriangleIndex;
				}
			}

			// Sanity check
			check(OutRootData[MeshLODIt].RootToUniqueTriangleIndexBuffer.Num() == CurveCount);
			check(OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer.Num() == UniqueTriangleCount * 3);
			check(OutRootData[MeshLODIt].RestUniqueTrianglePositionBuffer_TargetNonTransfered.Num() == UniqueTriangleCount * 3);
			check(OutRootData[MeshLODIt].UniqueTriangleIndexBuffer.Num() == UniqueTriangleCount);

			// Update the root mesh projection data with unique valid mesh section IDs, based on the projection data
			OutRootData[MeshLODIt].UniqueSectionIds = UniqueSectionId;
			OutRootData[MeshLODIt].MeshSectionCount = SectionCount;
		}

		return true;
	}
}// namespace GroomBinding_Project

///////////////////////////////////////////////////////////////////////////////////////////////////
// Mesh transfer

namespace GroomBinding_Transfer
{

	struct FTriangleGrid2D
	{
		struct FTriangle
		{
			uint32  TriangleIndex;
			uint32  SectionIndex;
			uint32  SectionBaseIndex;

			uint32  I0;
			uint32  I1;
			uint32  I2;

			FVector3f P0;
			FVector3f P1;
			FVector3f P2;

			FVector2f UV0;
			FVector2f UV1;
			FVector2f UV2;
		};

		struct FCell
		{
			TArray<FTriangle> Triangles;
		};
		typedef TArray<const FCell*> FCells;

		FTriangleGrid2D(uint32 Resolution)
		{
			GridResolution.X = Resolution;
			GridResolution.Y = Resolution;
			MinBound = FVector2f(0,0);
			MaxBound = FVector2f(1,1);

			Cells.SetNum(GridResolution.X * GridResolution.Y);
		}

		void Reset()
		{
			Cells.Empty();
			Cells.SetNum(GridResolution.X * GridResolution.Y);
		}

		FORCEINLINE bool IsValid(const FIntPoint& P) const
		{
			return
				0 <= P.X && P.X < GridResolution.X &&
				0 <= P.Y && P.Y < GridResolution.Y;
		}

		FORCEINLINE bool IsOutside(const FVector2f& MinP, const FVector2f& MaxP) const
		{
			return
				(MaxP.X <= MinBound.X || MaxP.Y <= MinBound.Y) ||
				(MinP.X >= MaxBound.X || MinP.Y >= MaxBound.Y);
		}

		FORCEINLINE FIntPoint ClampToVolume(const FIntPoint& CellCoord, bool& bIsValid) const
		{
			bIsValid = IsValid(CellCoord);
			return FIntPoint(
				FMath::Clamp(CellCoord.X, 0, GridResolution.X - 1),
				FMath::Clamp(CellCoord.Y, 0, GridResolution.Y - 1));
		}

		FORCEINLINE FIntPoint ToCellCoord(const FVector2f& P) const
		{
			bool bIsValid = false;
			FVector2f PP;
			PP.X = FMath::Clamp(P.X, 0.f, 1.f);
			PP.Y = FMath::Clamp(P.Y, 0.f, 1.f);
			const FIntPoint CellCoord = FIntPoint(FMath::FloorToInt(PP.X * GridResolution.X), FMath::FloorToInt(PP.Y * GridResolution.Y));
			return ClampToVolume(CellCoord, bIsValid);
		}

		uint32 ToIndex(const FIntPoint& CellCoord) const
		{
			uint32 CellIndex = CellCoord.X + CellCoord.Y * GridResolution.X;
			check(CellIndex < uint32(Cells.Num()));
			return CellIndex;
		}

		FCells ToCells(const FVector2f& P)
		{
			FCells Out;

			bool bIsValid = false;
			const FIntPoint Coord = ToCellCoord(P);
			{
				const uint32 LinearIndex = ToIndex(Coord);
				if (Cells[LinearIndex].Triangles.Num() > 0)
				{
					Out.Add(&Cells[LinearIndex]);
					bIsValid = true;
				}
			}

			int32 Kernel = 1;
			while (!bIsValid)
			{
				for (int32 Y = -Kernel; Y <= Kernel; ++Y)
				for (int32 X = -Kernel; X <= Kernel; ++X)
				{
					if (FMath::Abs(X) != Kernel && FMath::Abs(Y) != Kernel)
						continue;

					const FIntPoint Offset(X, Y);
					FIntPoint C = Coord + Offset;
					C.X = FMath::Clamp(C.X, 0, GridResolution.X - 1);
					C.Y = FMath::Clamp(C.Y, 0, GridResolution.Y - 1);

					const uint32 LinearIndex = ToIndex(C);
					if (Cells[LinearIndex].Triangles.Num() > 0)
					{
						Out.Add(&Cells[LinearIndex]);
						bIsValid = true;
					}
				}
				++Kernel;
			}

			return Out;
		}

		bool Insert(const FTriangle& T)
		{
			FVector2f TriMinBound;
			TriMinBound.X = FMath::Min(T.UV0.X, FMath::Min(T.UV1.X, T.UV2.X));
			TriMinBound.Y = FMath::Min(T.UV0.Y, FMath::Min(T.UV1.Y, T.UV2.Y));

			FVector2f TriMaxBound;
			TriMaxBound.X = FMath::Max(T.UV0.X, FMath::Max(T.UV1.X, T.UV2.X));
			TriMaxBound.Y = FMath::Max(T.UV0.Y, FMath::Max(T.UV1.Y, T.UV2.Y));

			if (IsOutside(TriMinBound, TriMaxBound))
			{
				return false;
			}

			const FIntPoint MinCoord = ToCellCoord(TriMinBound);
			const FIntPoint MaxCoord = ToCellCoord(TriMaxBound);

			// Insert triangle in all cell covered by the AABB of the triangle
			bool bInserted = false;
			for (int32 Y = MinCoord.Y; Y <= MaxCoord.Y; ++Y)
			{
				for (int32 X = MinCoord.X; X <= MaxCoord.X; ++X)
				{
					const FIntPoint CellIndex(X, Y);
					if (IsValid(CellIndex))
					{
						const uint32 CellLinearIndex = ToIndex(CellIndex);
						Cells[CellLinearIndex].Triangles.Add(T);
						bInserted = true;
					}
				}
			}
			return bInserted;
		}

		FVector2f MinBound;
		FVector2f MaxBound;
		FIntPoint GridResolution;
		TArray<FCell> Cells;
	};


	// Closest point on A triangle from another point in UV space
	// Code from the book "Real-Time Collision Detection" by Christer Ericson
	struct FTrianglePoint
	{
		FVector3f P;
		FVector3f Barycentric;
	};

	FTrianglePoint ComputeClosestPoint(const FVector2f& TriUV0, const FVector2f& TriUV1, const FVector2f& TriUV2, const FVector2f& UVs)
	{
		const FVector3f A = FVector3f(TriUV0, 0);
		const FVector3f B = FVector3f(TriUV1, 0);
		const FVector3f C = FVector3f(TriUV2, 0);
		const FVector3f P = FVector3f(UVs, 0);

		// Check if P is in vertex region outside A.
		FVector3f AB = B - A;
		FVector3f AC = C - A;
		FVector3f AP = P - A;
		float D1 = FVector3f::DotProduct(AB, AP);
		float D2 = FVector3f::DotProduct(AC, AP);
		if (D1 <= 0.f && D2 <= 0.f)
		{
			FTrianglePoint Out;
			Out.P = A;
			Out.Barycentric = FVector3f(1, 0, 0);
			return Out;
		}

		// Check if P is in vertex region outside B.
		FVector3f BP = P - B;
		float D3 = FVector3f::DotProduct(AB, BP);
		float D4 = FVector3f::DotProduct(AC, BP);
		if (D3 >= 0.f && D4 <= D3)
		{
			FTrianglePoint Out;
			Out.P = B;
			Out.Barycentric = FVector3f(0, 1, 0);
			return Out;
		}

		// Check if P is in edge region of AB, and if so, return the projection of P onto AB.
		float VC = D1 * D4 - D3 * D2;
		if (VC <= 0.f && D1 >= 0.f && D3 <= 0.f)
		{
			float V = FMath::Abs(D1 - D3) > 0.0001f? D1 / (D1 - D3) : 0.f;

			FTrianglePoint Out;
			Out.P = A + V * AB;
			Out.Barycentric = FVector3f(1 - V, V, 0);
			return Out;
		}

		// Check if P is in vertex region outside C.
		FVector3f CP = P - C;
		float D5 = FVector3f::DotProduct(AB, CP);
		float D6 = FVector3f::DotProduct(AC, CP);
		if (D6 >= 0.f && D5 <= D6)
		{
			FTrianglePoint Out;
			Out.P = C;
			Out.Barycentric = FVector3f(0, 0, 1);
			return Out;
		}

		// Check if P is in edge region of AC, and if so, return the projection of P onto AC.
		float VB = D5 * D2 - D1 * D6;
		if (VB <= 0.f && D2 >= 0.f && D6 <= 0.f)
		{
			float W = D2 / (D2 - D6);
			FTrianglePoint Out;
			Out.P = A + W * AC;
			Out.Barycentric = FVector3f(1 - W, 0, W);
			return Out;
		}

		// Check if P is in edge region of BC, and if so, return the projection of P onto BC.
		float VA = D3 * D6 - D5 * D4;
		if (VA <= 0.f && D4 - D3 >= 0.f && D5 - D6 >= 0.f)
		{
			float W = (D4 - D3) / (D4 - D3 + D5 - D6);
			FTrianglePoint Out;
			Out.P = B + W * (C - B);
			Out.Barycentric = FVector3f(0, 1 - W, W);
			return Out;
		}

		// P must be inside the face region. Compute the closest point through its barycentric coordinates (u,V,W).
		const float DenomSum = VA + VB + VC;

		// Guard against near-zero denominator from degenerate triangles to prevent NaN propagation.
		// Fallback with picking the closest point
		if (DenomSum <= UE_SMALL_NUMBER)
		{
			const float DistA = FVector3f::DistSquared(P, A);
			const float DistB = FVector3f::DistSquared(P, B);
			const float DistC = FVector3f::DistSquared(P, C);

			FTrianglePoint Out;
			if (DistA <= DistB && DistA <= DistC)
			{
				Out.P = A;
				Out.Barycentric = FVector3f(1, 0, 0);
			}
			else if (DistB <= DistC)
			{
				Out.P = B;
				Out.Barycentric = FVector3f(0, 1, 0);
			}
			else
			{
				Out.P = C;
				Out.Barycentric = FVector3f(0, 0, 1);
			}
			return Out;
		}

		const float Denom = 1.f / DenomSum;
		float V = VB * Denom;
		float W = VC * Denom;

		FTrianglePoint Out;
		Out.P = A + AB * V + AC * W;
		Out.Barycentric = FVector3f(1 - V - W, V, W);
		return Out;
	}

	bool Transfer(
		const GroomBinding_Mesh::IMeshData* InSourceMeshData,
		const GroomBinding_Mesh::IMeshData* InTargetMeshData,
		int32 TargetMeshMinLOD,
		int32 SourceMeshLODIndex,
		TArray<TArray<FVector3f>>& OutPositions_Original, 
		TArray<TArray<FVector3f>>& OutPositions_Transferred,
		const int32 InMatchingMaterialIndex)
	{
		// 1. Insert triangles into a 2D UV grid
		auto BuildGrid = [InSourceMeshData](
			int32 InSourceLODIndex,
			int32 InSourceMaterialIndex,
			int32 InChannelIndex,
			FTriangleGrid2D& OutGrid)
		{
			OutGrid.Reset();

			const GroomBinding_Mesh::IMeshLODData& MeshLODData = InSourceMeshData->GetMeshLODData(InSourceLODIndex);

			// Find all the sections which are mapped onto the target material index
			const TArray<int32> SourceTargetSections = MeshLODData.GetSectionsMatchingMaterialIndex(InSourceMaterialIndex);

			bool bIsGridPopulated = false;
			for (int32 SectionIndex : SourceTargetSections)
			{
				// Notes:
				// LODs are transfered using the LOD0 of the source mesh, as the LOD count can mismatch between source and target meshes.
				// Assume that the selected sections contains the head part, which is where the hair/facial hair should be projected on
				const uint32 SourceTriangleCount = MeshLODData.GetSection(SectionIndex).GetNumTriangles();
				const uint32 SourceSectionBaseIndex = MeshLODData.GetSection(SectionIndex).GetBaseIndex();

				const TArray<uint32>& SourceIndexBuffer = MeshLODData.GetIndexBuffer();

				for (uint32 SourceTriangleIt = 0; SourceTriangleIt < SourceTriangleCount; ++SourceTriangleIt)
				{
					FTriangleGrid2D::FTriangle T;
					T.SectionIndex = SectionIndex;
					T.SectionBaseIndex = SourceSectionBaseIndex;
					T.TriangleIndex = SourceTriangleIt;

					T.I0 = SourceIndexBuffer[T.SectionBaseIndex + SourceTriangleIt * 3 + 0];
					T.I1 = SourceIndexBuffer[T.SectionBaseIndex + SourceTriangleIt * 3 + 1];
					T.I2 = SourceIndexBuffer[T.SectionBaseIndex + SourceTriangleIt * 3 + 2];

					T.P0 = MeshLODData.GetVertexPosition(T.I0);
					T.P1 = MeshLODData.GetVertexPosition(T.I1);
					T.P2 = MeshLODData.GetVertexPosition(T.I2);

					T.UV0 = MeshLODData.GetVertexUV(T.I0, InChannelIndex);
					T.UV1 = MeshLODData.GetVertexUV(T.I1, InChannelIndex);
					T.UV2 = MeshLODData.GetVertexUV(T.I2, InChannelIndex);

					// Needed??
					//T.N0 = MeshLODData.GetVertexNormal(T.I0);
					//T.N1 = MeshLODData.GetVertexNormal(T.I1);
					//T.N2 = MeshLODData.GetVertexNormal(T.I2);

					bIsGridPopulated = OutGrid.Insert(T) || bIsGridPopulated;
				}
			}

			return bIsGridPopulated;
		};

		// 1. Insert triangles into a 2D UV grid
		const uint32 ChannelIndex = 0;
		const GroomBinding_Mesh::IMeshLODData& SourceMeshLODData = InSourceMeshData->GetMeshLODData(SourceMeshLODIndex);
		const bool bIsMatchingMaterialValid = SourceMeshLODData.IsValidMaterialIndex(InMatchingMaterialIndex);
		const int32 SourceMaterialIndex = bIsMatchingMaterialValid ? InMatchingMaterialIndex : 0;
		const int32 TargetMaterialIndex = SourceMaterialIndex;
		if (!bIsMatchingMaterialValid && GHairStrandsBindingBuilderWarningEnable > 0)
		{
			UE_LOGF(LogHairStrands, Warning, "[Groom] Binding asset will not respect the requested 'Matching material' %d. The source skeletal mesh does not have such a material. Instead 'Matching material' 0 will be used.", InMatchingMaterialIndex);
		}
		FTriangleGrid2D Grid(256);
		{
			const bool bIsGridPopulated = BuildGrid(SourceMeshLODIndex, SourceMaterialIndex, ChannelIndex, Grid);
			if (!bIsGridPopulated)
			{
				UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. The source skeletal mesh is missing or has invalid UVs.");
				return false;
			}
		}

		// 2. Look for closest triangle point in UV space
		// Make this run in parallel
		const int32 TargetLODCount = InTargetMeshData->GetNumLODs();
		OutPositions_Original.SetNum(TargetLODCount);
		OutPositions_Transferred.SetNum(TargetLODCount);
		for (int32 TargetLODIndex = 0; TargetLODIndex < TargetLODCount; ++TargetLODIndex)
		{
			if (TargetLODIndex < TargetMeshMinLOD)
			{
				OutPositions_Original[TargetLODIndex].Empty();
				OutPositions_Transferred[TargetLODIndex].Empty();
				continue;
			}

			// Check that the target SectionId is valid for the current LOD. 
			// If this is not the case, then fall back to section 0 and rebuild the source triangle grid to match the same section ID (1.)
			int32 LocalSourceMaterialIndex = SourceMaterialIndex;
			int32 LocalTargetMaterialIndex = TargetMaterialIndex;
			const GroomBinding_Mesh::IMeshLODData& TargetMeshLODData = InTargetMeshData->GetMeshLODData(TargetLODIndex);

			if (!TargetMeshLODData.IsValidMaterialIndex(LocalTargetMaterialIndex))
			{
				if ( TargetMeshLODData.GetNumSections() == 0 )
				{
					UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built for LOD %d. TargetMeshLODData.GetNumSections() == 0.", TargetLODIndex);
					return false;
				}

				if (GHairStrandsBindingBuilderWarningEnable > 0)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] Binding asset will not respect the requested 'Matching material' %d for LOD %d. The target skeletal mesh does not have such a material for this LOD. Instead material 0 will be used for this given LOD.", TargetMaterialIndex, TargetLODIndex);
				}

				LocalTargetMaterialIndex = 0;
				LocalSourceMaterialIndex = 0;
				const bool bIsGridPopulated = BuildGrid(SourceMeshLODIndex, LocalSourceMaterialIndex, ChannelIndex, Grid);
				if (!bIsGridPopulated)
				{
					UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built for LOD %d. The source skeletal mesh is missing or has invalid UVs.", TargetLODIndex);
					return false;
				}
			}

			const uint32 TargetVertexCount = TargetMeshLODData.GetNumVertices();

			TSet<FVector2f> UVs;
			for (uint32 TargetVertexIt = 0; TargetVertexIt < TargetVertexCount; ++TargetVertexIt)
			{
				const FVector2f Target_UV = TargetMeshLODData.GetVertexUV(TargetVertexIt, ChannelIndex);
				UVs.Add(Target_UV);
			}

			// Simple check to see if the target UVs are meaningful before doing the heavy work
			const int32 NumUVLowerLimit = FMath::Max(1, int32(TargetVertexCount * 0.01f));
			if (UVs.Num() < NumUVLowerLimit)
			{
				UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. The target skeletal mesh is missing or has invalid UVs.");
				return false;
			}

			OutPositions_Original[TargetLODIndex].SetNum(TargetVertexCount);
			OutPositions_Transferred[TargetLODIndex].SetNum(TargetVertexCount);
#if BINDING_PARALLEL_BUILDING
			ParallelFor(TargetVertexCount,
				[
					LocalTargetMaterialIndex,
					ChannelIndex,
					&TargetMeshLODData,
					TargetLODIndex,
					&Grid,
					&OutPositions_Original,
					&OutPositions_Transferred
				] (uint32 TargetVertexIt)
#else
			for (uint32 TargetVertexIt = 0; TargetVertexIt < TargetVertexCount; ++TargetVertexIt)
#endif
			{
				// This code is only about transferring position from source to target. There is no need to remove 'disabled' sections
				const int32 SectionIt = TargetMeshLODData.GetSectionFromVertexIndex(TargetVertexIt);
				const int32 MaterialIndex = TargetMeshLODData.GetSection(SectionIt).GetMaterialIndex();
				if (MaterialIndex != LocalTargetMaterialIndex)
				{
					
					OutPositions_Original[TargetLODIndex][TargetVertexIt] = FVector3f(0,0,0);
					OutPositions_Transferred[TargetLODIndex][TargetVertexIt] = FVector3f(0,0,0);
#if BINDING_PARALLEL_BUILDING
					return;
#else
					continue;
#endif
				}

				const FVector3f Target_P  = TargetMeshLODData.GetVertexPosition(TargetVertexIt);
				const FVector2f Target_UV = TargetMeshLODData.GetVertexUV(TargetVertexIt, ChannelIndex);

				// 2.1 Query closest triangles
				FVector3f RetargetedVertexPosition = Target_P;
				FTriangleGrid2D::FCells Cells = Grid.ToCells(Target_UV);

				// 2.2 Compute the closest triangle and comput the retarget position 
				float ClosestUVDistance = FLT_MAX;
				for (const FTriangleGrid2D::FCell* Cell : Cells)
				{
					for (const FTriangleGrid2D::FTriangle& CellTriangle : Cell->Triangles)
					{
						const FTrianglePoint ClosestPoint = ComputeClosestPoint(CellTriangle.UV0, CellTriangle.UV1, CellTriangle.UV2, Target_UV);
						const float UVDistanceToTriangle = FVector2f::Distance(FVector2f(ClosestPoint.P.X, ClosestPoint.P.Y), Target_UV);
						if (UVDistanceToTriangle < ClosestUVDistance)
						{
							RetargetedVertexPosition = FVector3f(
								ClosestPoint.Barycentric.X * CellTriangle.P0 +
								ClosestPoint.Barycentric.Y * CellTriangle.P1 +
								ClosestPoint.Barycentric.Z * CellTriangle.P2);
							ClosestUVDistance = UVDistanceToTriangle;
						}
					}
				}
				check(ClosestUVDistance < FLT_MAX);
				OutPositions_Original[TargetLODIndex][TargetVertexIt] = Target_P;
				OutPositions_Transferred[TargetLODIndex][TargetVertexIt] = RetargetedVertexPosition;
			}
#if BINDING_PARALLEL_BUILDING
			);
#endif
		}
		return true;
	}
}
// namespace GroomBinding_Transfer

static void InitHairStrandsRootData(TArray<FHairStrandsRootData>& Out, uint32 CurveCount, uint32 PointCount, uint32 LODCount, uint32 MinLOD, uint32 NumSamples, uint32 InFlags, const FHairStrandsDatas::FHash& InHash)
{
	Out.SetNum(LODCount);
	uint32 LODIndex = 0;
	for (FHairStrandsRootData& OutLOD : Out)
	{
		if (LODIndex >= MinLOD)
		{
			OutLOD.RootCount = CurveCount;
			OutLOD.PointCount = PointCount;
			OutLOD.SampleCount = NumSamples;
		}
		else
		{
			OutLOD.RootCount = 0;
			OutLOD.PointCount = 0;
			OutLOD.SampleCount = 0;
		}

		OutLOD.MeshInterpolationWeightsBuffer.Empty();
		OutLOD.MeshSampleIndicesBuffer.Empty();
		OutLOD.RestSamplePositionsBuffer.Empty();
		OutLOD.Flags = InFlags;
		OutLOD.Hash = InHash;
		OutLOD.LODIndex = LODIndex++;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Convert data into bulk data

namespace GroomBinding_BulkCopy
{

template<typename TFormatType>
void CopyToBulkData(FByteBulkData& Out, const TArray<typename TFormatType::Type>& Data)
{
	const uint32 DataSizeInByte = Data.Num() * sizeof(typename TFormatType::BulkType);

	// The buffer is then stored into bulk data
	Out.Lock(LOCK_READ_WRITE);
	void* BulkBuffer = Out.Realloc(DataSizeInByte);
	FMemory::Memcpy(BulkBuffer, Data.GetData(), DataSizeInByte);
	Out.Unlock();
}

template<typename TFormatType>
void CopyToBulkData(FHairBulkContainer& Out, const TArray<typename TFormatType::Type>& Data)
{
	CopyToBulkData<TFormatType>(Out.Data, Data);
}

template<typename TFormatType>
void CopyFromBulkData(TArray<typename TFormatType::Type>& Out, const FByteBulkData& In)
{
	check(In.IsBulkDataLoaded());

	const uint32 InDataSize = In.GetBulkDataSize();
	const uint32 ElementCount = InDataSize / sizeof(typename TFormatType::BulkType);
	Out.SetNum(ElementCount);

	// The buffer is then stored into bulk data
	const uint8* InData = (const uint8*)In.LockReadOnly();
	FMemory::Memcpy(Out.GetData(), InData, InDataSize);
	In.Unlock();
}

template<typename TFormatType>
void CopyFromBulkData(TArray<typename TFormatType::Type>& Out, const FHairBulkContainer& In)
{
	CopyFromBulkData<TFormatType>(Out, In.Data);
}


// Convert "root data" -> "root bulk data"
static void BuildRootBulkData(
	FHairStrandsRootBulkData& Out,
	const FHairStrandsRootData& In)
{
	// Header
	Out.Header.RootCount  = In.RootCount;
	Out.Header.PointCount = In.PointCount;

	Out.Header.Strides.RootToUniqueTriangleIndexBufferStride 	= FHairStrandsRootToUniqueTriangleIndexFormat::SizeInByte;
	Out.Header.Strides.RootBarycentricBufferStride 				= FHairStrandsRootBarycentricFormat::SizeInByte;
	Out.Header.Strides.UniqueTriangleIndexBufferStride 			= FHairStrandsUniqueTriangleIndexFormat::SizeInByte;
	Out.Header.Strides.RestUniqueTrianglePositionBufferStride 	= FHairStrandsMeshTrianglePositionFormat::SizeInByte * 3; // 3 vertices per triangle

	Out.Header.Strides.MeshInterpolationWeightsBufferStride 	= FHairStrandsWeightFormat::SizeInByte;
	Out.Header.Strides.MeshSampleIndicesAndSectionsBufferStride = FHairStrandsRBFSampleIndexFormat::SizeInByte;
	Out.Header.Strides.RestSamplePositionsBufferStride 			= FHairStrandsMeshTrianglePositionFormat::SizeInByte;

	{
		const bool bHasValidSamples =
			In.MeshInterpolationWeightsBuffer.Num() > 0 &&
			In.MeshSampleIndicesBuffer.Num() > 0 &&
			In.RestSamplePositionsBuffer.Num() > 0;

		Out.Header.LODIndex 			= In.LODIndex;
		Out.Header.SampleCount 			= bHasValidSamples ? In.SampleCount : 0u;
		Out.Header.UniqueTriangleCount 	= In.UniqueTriangleIndexBuffer.Num();
		Out.Header.UniqueSectionIndices	= In.UniqueSectionIds;
		Out.Header.MeshSectionCount		= In.MeshSectionCount;
		Out.Header.Flags				= FHairStrandsBulkData::ConvertDataFlagsFromHairStrandsFlags(In.Flags);
		Out.Header.Hash					= In.Hash;
	}

	// Data
	{
		const bool bHasValidSamples =
			In.MeshInterpolationWeightsBuffer.Num() > 0 &&
			In.MeshSampleIndicesBuffer.Num() > 0 &&
			In.RestSamplePositionsBuffer.Num() > 0;

		CopyToBulkData<FHairStrandsUniqueTriangleIndexFormat>(Out.Data.UniqueTriangleIndexBuffer, In.UniqueTriangleIndexBuffer);
		CopyToBulkData<FHairStrandsRootBarycentricFormat>(Out.Data.RootBarycentricBuffer, In.RootBarycentricBuffer);
		CopyToBulkData<FHairStrandsRootToUniqueTriangleIndexFormat>(Out.Data.RootToUniqueTriangleIndexBuffer, In.RootToUniqueTriangleIndexBuffer);
		CopyToBulkData<FHairStrandsMeshTrianglePositionFormat>(Out.Data.RestUniqueTrianglePositionBuffer, In.RestUniqueTrianglePositionBuffer);

		if (bHasValidSamples)
		{
			check(In.MeshSampleIndicesBuffer.Num() == In.MeshSampleSectionsBuffer.Num());

			const uint32 SampleCount = In.MeshSampleIndicesBuffer.Num();
			TArray<uint32> MeshSampleIndicesAndSectionBuffer;
			MeshSampleIndicesAndSectionBuffer.SetNum(SampleCount);
			for (uint32 SampleIt=0;SampleIt<SampleCount;++SampleIt)
			{
				const uint32 Index = In.MeshSampleIndicesBuffer[SampleIt];
				const uint32 SectionIndex = In.MeshSampleSectionsBuffer[SampleIt];
				MeshSampleIndicesAndSectionBuffer[SampleIt] = FHairStrandsRootUtils::PackTriangleIndex(Index, SectionIndex);

				// Update the unique section indices with section containing RBF samples
				// This allows faster update at runtime when not using skin cache
				// This is done only for guides, which is the only root data containing RBF data
				Out.Header.UniqueSectionIndices.AddUnique(SectionIndex);
			}

			CopyToBulkData<FHairStrandsWeightFormat>(Out.Data.MeshInterpolationWeightsBuffer, In.MeshInterpolationWeightsBuffer);
			CopyToBulkData<FHairStrandsRBFSampleIndexFormat>(Out.Data.MeshSampleIndicesAndSectionsBuffer, MeshSampleIndicesAndSectionBuffer);
			CopyToBulkData<FHairStrandsMeshTrianglePositionFormat>(Out.Data.RestSamplePositionsBuffer, In.RestSamplePositionsBuffer);
		}
		else
		{
			Out.Data.MeshInterpolationWeightsBuffer.RemoveBulkData();
			Out.Data.MeshSampleIndicesAndSectionsBuffer.RemoveBulkData();
			Out.Data.RestSamplePositionsBuffer.RemoveBulkData();
		}
	}
}

// Convert "root data" <- "root bulk data"
//
// Bulk data must be loaded before calling this function
static void BuildRootData(
	FHairStrandsRootData& Out,
	const FHairStrandsRootBulkData& In)
{
	Out.RootCount = In.Header.RootCount;
	Out.PointCount = In.Header.PointCount;
	{
		const bool bHasValidSamples = In.Header.SampleCount > 0;

		Out.LODIndex = In.Header.LODIndex;
		Out.SampleCount = bHasValidSamples ? In.Header.SampleCount : 0u;
		Out.UniqueSectionIds = In.Header.UniqueSectionIndices;
		Out.MeshSectionCount = In.Header.MeshSectionCount;

		CopyFromBulkData<FHairStrandsUniqueTriangleIndexFormat>(Out.UniqueTriangleIndexBuffer, In.Data.UniqueTriangleIndexBuffer);
		CopyFromBulkData<FHairStrandsRootToUniqueTriangleIndexFormat>(Out.RootToUniqueTriangleIndexBuffer, In.Data.RootToUniqueTriangleIndexBuffer);
		CopyFromBulkData<FHairStrandsRootBarycentricFormat>(Out.RootBarycentricBuffer, In.Data.RootBarycentricBuffer);
		CopyFromBulkData<FHairStrandsMeshTrianglePositionFormat>(Out.RestUniqueTrianglePositionBuffer, In.Data.RestUniqueTrianglePositionBuffer);

		if (bHasValidSamples)
		{
			TArray<uint32> MeshSampleIndicesAndSectionBuffer;

			CopyFromBulkData<FHairStrandsWeightFormat>(Out.MeshInterpolationWeightsBuffer, In.Data.MeshInterpolationWeightsBuffer);
			CopyFromBulkData<FHairStrandsRBFSampleIndexFormat>(MeshSampleIndicesAndSectionBuffer, In.Data.MeshSampleIndicesAndSectionsBuffer);
			CopyFromBulkData<FHairStrandsMeshTrianglePositionFormat>(Out.RestSamplePositionsBuffer, In.Data.RestSamplePositionsBuffer);

			// Split indices and sections
			const uint32 SampleCount = MeshSampleIndicesAndSectionBuffer.Num();
			Out.MeshSampleIndicesBuffer.SetNum(SampleCount);
			Out.MeshSampleSectionsBuffer.SetNum(SampleCount);
			for (uint32 SampleIt=0;SampleIt<SampleCount;++SampleIt)
			{
				FHairStrandsRootUtils::UnpackTriangleIndex(
					MeshSampleIndicesAndSectionBuffer[SampleIt], 
					Out.MeshSampleIndicesBuffer[SampleIt],
					Out.MeshSampleSectionsBuffer[SampleIt]);
			}
		}
		else
		{
			Out.MeshInterpolationWeightsBuffer.Empty();
			Out.MeshSampleIndicesBuffer.Empty();
			Out.MeshSampleSectionsBuffer.Empty();
			Out.RestSamplePositionsBuffer.Empty();
		}
	}
}

// Convert the root data into root bulk data
static void BuildGroupPlatformData(
	UGroomBindingAsset::FHairGroupPlatformData& Out,
	const FHairRootGroupData& In)
{
	// Guides
	{
		const int32 MeshLODCount = In.SimRootDatas.Num();
		Out.SimRootBulkDatas.SetNum(MeshLODCount);
		for (int32 MeshLODIt = In.TargetMeshMinLOD; MeshLODIt < MeshLODCount; ++MeshLODIt)
		{
			if (MeshLODIt < In.TargetMeshMinLOD)
			{
				continue;
			}

			BuildRootBulkData(Out.SimRootBulkDatas[MeshLODIt], In.SimRootDatas[MeshLODIt]);
		}
	}

	// Strands
	{
		const int32 MeshLODCount = In.RenRootDatas.Num();
		Out.RenRootBulkDatas.SetNum(MeshLODCount);
		for (int32 MeshLODIt = In.TargetMeshMinLOD; MeshLODIt < MeshLODCount; ++MeshLODIt)
		{
			if (MeshLODIt < In.TargetMeshMinLOD)
			{
				continue;
			}

			BuildRootBulkData(Out.RenRootBulkDatas[MeshLODIt], In.RenRootDatas[MeshLODIt]);
		}
	}

	// Cards
	const int32 LODCount = In.CardsRootDatas.Num();
	Out.CardsRootBulkDatas.SetNum(LODCount);
	for (int32 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		const int32 MeshLODCount = In.CardsRootDatas[LODIt].Num();
		Out.CardsRootBulkDatas[LODIt].SetNum(MeshLODCount);
		for (int32 MeshLODIt = In.TargetMeshMinLOD; MeshLODIt < MeshLODCount; ++MeshLODIt)
		{
			BuildRootBulkData(Out.CardsRootBulkDatas[LODIt][MeshLODIt], In.CardsRootDatas[LODIt][MeshLODIt]);
		}
	}

	Out.TargetMeshMinLOD = In.TargetMeshMinLOD;
}

} // namespace GroomBinding_BulkCopy

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper functions for binding validation code

bool GetHairStrandsBindingDataValidation();

#if WITH_EDITORONLY_DATA

static FHairStrandsCommonDatas::FHash GetHash(const FHairStrandsDatas& In)
{
	return In.GetHash();
}

static uint32 GetFlags(const FHairStrandsDatas& In)
{
	return In.Flags;
}

#else

static FHairStrandsCommonDatas::FHash GetHash(const FHairStrandsBulkData& In)
{
	return FHairStrandsCommonDatas::FHash();
}

static uint32 GetFlags(const FHairStrandsBulkData& In)
{
	return FHairStrandsBulkData::ConvertDataFlagsToHairStrandsFlags(In.Header.Flags);
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Main entry (CPU path)
bool BuildHairRootGroupData(const FGroomBindingBuilder::FInput& In, uint32 InGroupIndex, const ITargetPlatform* TargetPlatform, FHairRootGroupData& OutData)
{
	const bool bIsAssetValid = In.GroomAsset && In.bHasValidTarget && In.GroomAsset->GetNumHairGroups() > 0;
	if (!bIsAssetValid)
	{
		if (!In.GroomAsset)							{ UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset cannot be created/rebuilt - The groom binding has no groom asset."); return false; }
		if (!In.bHasValidTarget)					{ UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset cannot be created/rebuilt - The groom binding has no valid skel./geom cache. target"); return false; }
		if (In.GroomAsset->GetNumHairGroups() == 0)	{ UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset cannot be created/rebuilt - The groom asset has no groups."); return false; }
	}

	// 1. Build groom root data
	OutData = FHairRootGroupData();
	{
		// If we're currently running on a worker thread, all this preloading
		// stuff should have happened on the game-thread part of the build
		// before going async so these should be no-ops in that case.
		In.GroomAsset->ConditionalPostLoad();

		// Ensure the skeletal meshes / geom caches are built
		if (In.BindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			In.TargetSkeletalMesh->ConditionalPostLoad();
			In.TargetSkeletalMesh->GetLODNum();
			if (In.SourceSkeletalMesh)
			{
				In.SourceSkeletalMesh->ConditionalPostLoad();
				In.SourceSkeletalMesh->GetLODNum();
			}
		}
		else
		{
			In.TargetGeometryCache->ConditionalPostLoad();
			if (In.SourceGeometryCache)
			{
				In.SourceGeometryCache->ConditionalPostLoad();
			}
		}

		// If a skeletal mesh build or preedit is called while we're async compiling groom bindings, the groom binding compiler
		// will take care of finishing any groom binding that depends on the skeletal mesh being modified. So this is 
		// safe to do asynchronously without locks on the render data. We assume that the skeletal mesh's render data is
		// immutable once it has been built and can only be rebuilt throught a call to PreEditChange first.
#if WITH_EDITOR
		const bool bUseRunningPlatform = In.bForceUseRunningPlatform || !IsInGameThread();
#else
		const bool bUseRunningPlatform = true;
#endif

		// * Only for SkeletalMesh: Take scoped lock on the skeletal render mesh data during the entire groom binding building
		// * Then use an async build scope to allow accessing skeletal mesh property safely.
		//   If skel.meshes are nullptr, this will act as a NOP
		USkeletalMesh* InSourceSkeletalMesh = In.SourceSkeletalMesh == In.TargetSkeletalMesh ? nullptr : In.SourceSkeletalMesh;
#if WITH_EDITOR
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FScopedSkeletalMeshRenderData SourceSkeletalMeshScopedData(bUseRunningPlatform ? nullptr : InSourceSkeletalMesh);
		FScopedSkeletalMeshRenderData TargetSkeletalMeshScopedData(bUseRunningPlatform ? nullptr : In.TargetSkeletalMesh);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

		TUniquePtr<GroomBinding_Mesh::IMeshData> SourceMeshData;
		TUniquePtr<GroomBinding_Mesh::IMeshData> TargetMeshData;
		if (In.BindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			if (InSourceSkeletalMesh)
			{
				if (bUseRunningPlatform)
				{
#if WITH_EDITOR
					const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
					checkf(TargetPlatform == RunningPlatform, TEXT("It is only safe to query the running platform's render data asynchronously from the skeletal mesh"));
#endif
					SourceMeshData = TUniquePtr<GroomBinding_Mesh::FSkeletalMeshData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FSkeletalMeshData(InSourceSkeletalMesh, InSourceSkeletalMesh->GetResourceForRendering(), In.SourceMeshLOD));
				}
#if WITH_EDITOR
				else
				{
					FSkinnedAssetAsyncBuildScope AsyncBuildScope(InSourceSkeletalMesh);
					USkeletalMesh::GetPlatformSkeletalMeshRenderData(TargetPlatform, SourceSkeletalMeshScopedData);
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					SourceMeshData = TUniquePtr<GroomBinding_Mesh::FSkeletalMeshData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FSkeletalMeshData(InSourceSkeletalMesh, SourceSkeletalMeshScopedData.GetData(), In.SourceMeshLOD));
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
#endif
			}
			else
			{
				SourceMeshData = TUniquePtr<GroomBinding_Mesh::FSkeletalMeshData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FSkeletalMeshData(nullptr, nullptr, In.SourceMeshLOD));
			}

			if (In.TargetSkeletalMesh)
			{
				if (bUseRunningPlatform)
				{
#if WITH_EDITOR
					const ITargetPlatform* RunningPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
					checkf(TargetPlatform == RunningPlatform, TEXT("It is only safe to query the running platform's render data asynchronously from the skeletal mesh"));
#endif
					TargetMeshData = TUniquePtr<GroomBinding_Mesh::FSkeletalMeshData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FSkeletalMeshData(In.TargetSkeletalMesh, In.TargetSkeletalMesh->GetResourceForRendering(), In.TargetMeshMinLOD));
				}
#if WITH_EDITOR
				else
				{
					FSkinnedAssetAsyncBuildScope AsyncBuildScope(In.TargetSkeletalMesh);
					USkeletalMesh::GetPlatformSkeletalMeshRenderData(TargetPlatform, TargetSkeletalMeshScopedData);
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					TargetMeshData = TUniquePtr<GroomBinding_Mesh::FSkeletalMeshData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FSkeletalMeshData(In.TargetSkeletalMesh, TargetSkeletalMeshScopedData.GetData(), In.TargetMeshMinLOD));
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
#endif
			}
			else
			{
				TargetMeshData = TUniquePtr<GroomBinding_Mesh::FSkeletalMeshData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FSkeletalMeshData(nullptr, nullptr, In.TargetMeshMinLOD));
			}
		}
		else
		{
			// Geometry caches only have one LOD
			check(In.SourceMeshLOD == 0);
			check(In.TargetMeshMinLOD == 0);

			In.TargetGeometryCache->ConditionalPostLoad();
			if (In.SourceGeometryCache)
			{
				In.SourceGeometryCache->ConditionalPostLoad();
			}

			SourceMeshData = TUniquePtr<GroomBinding_Mesh::FGeometryCacheData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FGeometryCacheData(In.SourceGeometryCache));
			TargetMeshData = TUniquePtr<GroomBinding_Mesh::FGeometryCacheData, TDefaultDelete<GroomBinding_Mesh::IMeshData>>(new GroomBinding_Mesh::FGeometryCacheData(In.TargetGeometryCache));
		}

		if (!TargetMeshData->IsValid())
		{
			UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. Target mesh is not valid.");
			return false;
		}

		const uint32 GroupCount = In.GroomAsset->GetNumHairGroups();
		const uint32 MeshLODCount = TargetMeshData->GetNumLODs();

		check(InGroupIndex < uint32(In.GroomAsset->GetHairGroupsPlatformData().Num()));

		if (SourceMeshData->IsValid() && !SourceMeshData->IsLODValid(In.SourceMeshLOD))
		{
			UE_LOGFMT(LogHairStrands, Error, "[Groom] Binding asset could not be built. Source mesh LOD {0} is not valid.", In.SourceMeshLOD);
			return false;
		}

		for (uint32 MeshLODIndex = In.TargetMeshMinLOD; MeshLODIndex < MeshLODCount; MeshLODIndex++)
		{
			if (!TargetMeshData->IsLODValid(MeshLODIndex))
			{
				UE_LOGFMT(LogHairStrands, Error, "[Groom] Binding asset could not be built. Target mesh LOD {0} is not valid.", MeshLODIndex);
				return false;
			}
		}

		// Check if root data are needs for strands
		//
		// For now we process all groom LODs here. In future we could skip any LODs that will be
		// stripped on the target platform.
		bool bNeedStrandsRoot = false;
		for (const FHairLODSettings& LODSettings : In.GroomAsset->GetHairGroupsLOD()[InGroupIndex].LODs)
		{
			if (LODSettings.GeometryType == EGroomGeometryType::Strands)
			{
				bNeedStrandsRoot = true;
				break;
			}
		}

		const FHairGroupPlatformData& GroupData = In.GroomAsset->GetHairGroupsPlatformData()[InGroupIndex];

		// 1.1 Build guide/strands data
#if WITH_EDITORONLY_DATA
		FHairStrandsDatas StrandsDataSource;
		FHairStrandsDatas GuidesDataSource;
		if (!In.GroomAsset->GetHairStrandsDatas(InGroupIndex, StrandsDataSource, GuidesDataSource))
		{
			UE_LOGFMT(LogHairStrands, Error, "[Groom] Binding asset could not be built. Failed to read from {0}", In.GroomAsset->GetFullName());
			return false;
		}
#else
		const FHairStrandsBulkData& StrandsDataSource = GroupData.Strands.BulkData;
		const FHairStrandsBulkData& GuidesDataSource = GroupData.Guides.BulkData;

		if (!GroupData.Strands.HasValidData())
		{
			// Strands have been cooked out
			bNeedStrandsRoot = false;
		}
#endif

		// 1.2 Init root data for guides/strands/cards
		{
			// Guides
			const uint32 GuidesCurveCount = GuidesDataSource.GetNumCurves();
			const uint32 GuidesPointCount = GuidesDataSource.GetNumPoints();
			InitHairStrandsRootData(OutData.SimRootDatas, GuidesCurveCount, GuidesPointCount, MeshLODCount, In.TargetMeshMinLOD, In.NumInterpolationPoints, GetFlags(GuidesDataSource), GetHash(GuidesDataSource));

			// Strands
			if (bNeedStrandsRoot)
			{
				const uint32 StrandsCurveCount = StrandsDataSource.GetNumCurves();
				const uint32 StrandsPointCount = StrandsDataSource.GetNumPoints();
				InitHairStrandsRootData(OutData.RenRootDatas, StrandsCurveCount, StrandsPointCount, MeshLODCount, In.TargetMeshMinLOD, In.NumInterpolationPoints, GetFlags(StrandsDataSource), GetHash(StrandsDataSource));
			}

			// Cards
			const uint32 CardsLODCount = GroupData.Cards.LODs.Num();
			OutData.CardsRootDatas.SetNum(GroupData.Cards.LODs.Num());
			for (uint32 CardsLODIt = 0; CardsLODIt < CardsLODCount; ++CardsLODIt)
			{
				if (GroupData.Cards.IsValid(CardsLODIt))
				{
#if WITH_EDITORONLY_DATA
					FHairStrandsDatas LODGuidesData;
					const bool bIsValid = In.GroomAsset->GetHairCardsGuidesDatas(InGroupIndex, CardsLODIt, LODGuidesData);
#else
					const FHairStrandsBulkData& LODGuidesData = GroupData.Cards.LODs[CardsLODIt].GuideBulkData;
					const bool bIsValid = GroupData.Cards.LODs[CardsLODIt].BulkData.IsValid();
#endif
					if (bIsValid)
					{
						const uint32 CardsCurveCount = LODGuidesData.GetNumCurves();
						const uint32 CardsPointCount = LODGuidesData.GetNumPoints(); 

						InitHairStrandsRootData(OutData.CardsRootDatas[CardsLODIt], CardsCurveCount, CardsPointCount, MeshLODCount, In.TargetMeshMinLOD, In.NumInterpolationPoints, GetFlags(LODGuidesData), GetHash(LODGuidesData));
					}
				}
			}
		}

		// Transfer requires root UV embedded into the groom asset. It is not possible to read safely hair description here to extra this data.
		const bool bNeedTransferPosition = SourceMeshData->IsValid();

		// Create mapping between the source & target using their UV
		uint32 WorkItemCount = 1 + (bNeedTransferPosition ? 1 : 0); // RBF + optional position transfer
		{
			// Guides
			WorkItemCount += 1;
			// Strands
			WorkItemCount += bNeedStrandsRoot ? 1 : 0;
			// Cards
			WorkItemCount += OutData.CardsRootDatas.Num();
		}
	
		uint32 WorkItemIndex = 0;
		FScopedSlowTask SlowTask(WorkItemCount, LOCTEXT("BuildBindingData", "Building groom binding data"), true /*bInEnabled*/);
		SlowTask.MakeDialog();

		// 1.3 Transfer positions
		if (bNeedTransferPosition)
		{
			if (!GroomBinding_Transfer::Transfer(
				SourceMeshData.Get(),
				TargetMeshData.Get(),
				In.TargetMeshMinLOD,
				In.SourceMeshLOD,
				OutData.MeshPositions, 
				OutData.MeshPositions_Transferred, 
				In.MatchingSection))
			{
				UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. Positions transfer between source and target mesh failed.");
				return false;
			}
			SlowTask.EnterProgressFrame();
		}

		// 1.3bis Populate binding attribute
		{
			GroomBinding_RootProjection::TransfertMeshAttribute(
				In.TargetBindingAttribute,
				TargetMeshData.Get());
		}

		// 1.4 Build root data for guides/strands/cards
		{
			// Guides
			{
				const GroomBinding_Mesh::FHairStrandRootPositions GuidesData(GuidesDataSource);

				// Sanity check
				#if WITH_EDITORONLY_DATA
				if (GetHairStrandsBindingDataValidation())
				{
					for (const FHairStrandsRootData& RootData : OutData.SimRootDatas)
					{
						check(RootData.Hash == GuidesDataSource.GetHash());
					}
				}
				#endif

				if (!GroomBinding_RootProjection::Project(
					GuidesData,
					TargetMeshData.Get(),
					In.TargetMeshMinLOD,
					OutData.MeshPositions_Transferred,
					OutData.SimRootDatas))
				{
					UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. Some guide roots are not close enough to the target mesh to be projected onto it.");
					return false; 
				}
				SlowTask.EnterProgressFrame();
			}

			// Strands
			if (bNeedStrandsRoot)
			{
				const GroomBinding_Mesh::FHairStrandRootPositions StrandsData(StrandsDataSource);

				// Sanity check
				#if WITH_EDITORONLY_DATA
				if (GetHairStrandsBindingDataValidation())
				{
					for (const FHairStrandsRootData& RootData : OutData.RenRootDatas)
					{
						check(RootData.Hash == StrandsDataSource.GetHash());
					}
				}
				#endif

				if (!GroomBinding_RootProjection::Project(
					StrandsData,
					TargetMeshData.Get(),
					In.TargetMeshMinLOD,
					OutData.MeshPositions_Transferred,
					OutData.RenRootDatas))
				{
					UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. Some strand roots are not close enough to the target mesh to be projected onto it.");
					return false;
				}
				SlowTask.EnterProgressFrame();
			}

			// Cards
			const uint32 CardsLODCount = OutData.CardsRootDatas.Num();
			for (uint32 CardsLODIt = 0; CardsLODIt < CardsLODCount; ++CardsLODIt)
			{
				if (In.GroomAsset->GetHairGroupsPlatformData()[InGroupIndex].Cards.IsValid(CardsLODIt))
				{
#if WITH_EDITORONLY_DATA
					FHairStrandsDatas LODGuidesDataSource;
					const bool bIsValid = In.GroomAsset->GetHairCardsGuidesDatas(InGroupIndex, CardsLODIt, LODGuidesDataSource);

					// Sanity check
					if (GetHairStrandsBindingDataValidation())
					{
						const FHairStrandsCommonDatas::FHash SourceHash = LODGuidesDataSource.GetHash();
						for (const FHairStrandsRootData& RootData : OutData.CardsRootDatas[CardsLODIt])
						{
							check(RootData.Hash == SourceHash);
						}
					}
#else
					const bool bIsValid = GroupData.Cards.LODs[CardsLODIt].IsValid();
					const FHairStrandsBulkData& LODGuidesDataSource = GroupData.Cards.LODs[CardsLODIt].GuideBulkData;
#endif
					if (bIsValid)
					{
						const GroomBinding_Mesh::FHairStrandRootPositions LODGuidesData(LODGuidesDataSource);

						if (!GroomBinding_RootProjection::Project(
							LODGuidesData,
							TargetMeshData.Get(),
							In.TargetMeshMinLOD,
							OutData.MeshPositions_Transferred,
							OutData.CardsRootDatas[CardsLODIt]))
						{
							UE_LOGF(LogHairStrands, Error, "[Groom] Binding asset could not be built. Some cards guide roots are not close enough to the target mesh to be projected onto it.");
							return false; 
						}
					}
				}
				SlowTask.EnterProgressFrame();
			}
		}
		
		// 1.5 RBF building
		{
			GroomBinding_RBFWeighting::ComputeInterpolationWeights(OutData, bNeedStrandsRoot, In.NumInterpolationPoints, In.MatchingSection, TargetMeshData.Get(), In.TargetMeshMinLOD);
			SlowTask.EnterProgressFrame();
		}
	}

	OutData.TargetMeshMinLOD = In.TargetMeshMinLOD;
	return true;
}

bool FGroomBindingBuilder::BuildBinding(
	const FGroomBindingBuilder::FInput& In,
	uint32 InGroupIndex,
	const ITargetPlatform* TargetPlatform,
	UGroomBindingAsset::FHairGroupPlatformData& OutPlatformData)
{
	// 1. Build root Data
	FHairRootGroupData RootData;
	bool bSucceed = BuildHairRootGroupData(In, InGroupIndex, TargetPlatform, RootData);

	// 2. Convert data to bulk data
	if (bSucceed)
	{
		GroomBinding_BulkCopy::BuildGroupPlatformData(OutPlatformData, RootData);
	}

	return bSucceed;
}

bool FGroomBindingBuilder::BuildBinding(class UGroomBindingAsset* BindingAsset, bool bInitResource)
{
#if WITH_EDITORONLY_DATA
	BindingAsset->BeginCacheDerivedDatas();
#endif
	return true;
}

bool FGroomBindingBuilder::BuildBinding(class UGroomBindingAsset* BindingAsset, uint32 InGroupIndex)
{
#if WITH_EDITORONLY_DATA
	BindingAsset->BeginCacheDerivedDatas();
#endif
	return true;
}

void FGroomBindingBuilder::GetRootData(
	FHairStrandsRootData& Out,
	const FHairStrandsRootBulkData& In)
{
	GroomBinding_BulkCopy::BuildRootData(Out, In);
}

#undef LOCTEXT_NAMESPACE
