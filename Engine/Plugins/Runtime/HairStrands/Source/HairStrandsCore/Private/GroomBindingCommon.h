// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomBindingBuilder.h"

class ITargetPlatform;

/** Binding data */
struct FHairRootGroupData
{
	TArray<FHairStrandsRootData>		SimRootDatas;
	TArray<FHairStrandsRootData>		RenRootDatas;
	TArray<TArray<FHairStrandsRootData>>CardsRootDatas;

	// (Skel.) Mesh Positions for each LODs
	TArray<TArray<FVector3f>> 			MeshPositions;
	TArray<TArray<FVector3f>> 			MeshPositions_Transferred;

	// The index of the best LOD on the target mesh that is supported by this data
	int32 TargetMeshMinLOD = INDEX_NONE;
};
bool BuildHairRootGroupData(const FGroomBindingBuilder::FInput& In, uint32 InGroupIndex, const ITargetPlatform* TargetPlatform, FHairRootGroupData& OutData);