// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBindingCompiler.h"

#include "AssetCompilingManager.h"
#include "GroomBindingAsset.h"
#include "ObjectCacheContext.h"
#include "GroomComponent.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "EngineModule.h"
#include "Misc/ScopedSlowTask.h"
#include "GroomComponent.h"
#include "UObject/UObjectIterator.h"
#include "Containers/Set.h"
#include "SkeletalMeshCompiler.h"
#include "Misc/IQueuedWork.h"
#include "Components/PrimitiveComponent.h"
#include "ContentStreaming.h"
#include "EngineUtils.h"
#include "ProfilingDebugging/CountersTrace.h"

#if WITH_EDITOR
#include "Settings/EditorExperimentalSettings.h"
#endif

#define LOCTEXT_NAMESPACE "GroomBindingCompiler"

#if WITH_EDITOR
static AsyncCompilationHelpers::FAsyncCompilationStandardCVars CVarAsyncGroomBindingAssetStandard(
	TEXT("GroomBinding"),
	TEXT("groom bindings"),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			FGroomBindingCompilingManager::Get().FinishAllCompilation();
		}
	));
#endif

namespace GroomBindingAssetCompilingManagerImpl
{
	static void EnsureInitializedCVars()
	{
		static bool bIsInitialized = false;

		if (!bIsInitialized)
		{
			bIsInitialized = true;
#if WITH_EDITOR
			AsyncCompilationHelpers::EnsureInitializedCVars(
				TEXT("groombinding"),
				CVarAsyncGroomBindingAssetStandard.AsyncCompilation,
				CVarAsyncGroomBindingAssetStandard.AsyncCompilationMaxConcurrency,
				GET_MEMBER_NAME_CHECKED(UEditorExperimentalSettings, bEnableAsyncGroomBindingCompilation));
#endif
		}
	}
}

FGroomBindingCompilingManager::FGroomBindingCompilingManager()
	: Notification(GetAssetNameFormat())
{
	GroomBindingAssetCompilingManagerImpl::EnsureInitializedCVars();

	PostReachabilityAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddRaw(this, &FGroomBindingCompilingManager::OnPostReachabilityAnalysis);
}

void FGroomBindingCompilingManager::AttachDependencies(UGroomBindingAsset* GroomBindingAsset)
{
	if (USkeletalMesh* TargetSkeletalMesh = GroomBindingAsset->GetTargetSkeletalMesh())
	{
		RegisteredSkeletalMeshes.Add(TargetSkeletalMesh, GroomBindingAsset);
	}

	if (USkeletalMesh* SourceSkeletalMesh = GroomBindingAsset->GetSourceSkeletalMesh())
	{
		RegisteredSkeletalMeshes.Add(SourceSkeletalMesh, GroomBindingAsset);
	}

	if (UGroomAsset* GroomAsset = GroomBindingAsset->GetGroom())
	{
		RegisteredGroomAssets.Add(GroomAsset, GroomBindingAsset);
	}
}

void FGroomBindingCompilingManager::DetachDependencies(UGroomBindingAsset* GroomBindingAsset)
{
	if (USkeletalMesh* TargetSkeletalMesh = GroomBindingAsset->GetTargetSkeletalMesh())
	{
		RegisteredSkeletalMeshes.Remove(TargetSkeletalMesh, GroomBindingAsset);
	}

	if (USkeletalMesh* SourceSkeletalMesh = GroomBindingAsset->GetSourceSkeletalMesh())
	{
		RegisteredSkeletalMeshes.Remove(SourceSkeletalMesh, GroomBindingAsset);
	}

	if (UGroomAsset* GroomAsset = GroomBindingAsset->GetGroom())
	{
		RegisteredGroomAssets.Remove(GroomAsset, GroomBindingAsset);
	}
}

void FGroomBindingCompilingManager::OnPostReachabilityAnalysis()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::CancelUnreachableGroomBindings);

	TArray<UGroomBindingAsset*> PendingAssets;
	PendingAssets.Reserve(GetNumRemainingJobs());
		
	// Compilation has not started yet... just remove it from our pending list
	for (auto Iterator = GroomBindingWithPendingDependencies.CreateIterator(); Iterator; ++Iterator)
	{
		UGroomBindingAsset* GroomBinding = *Iterator;
		if (GroomBinding->IsUnreachable())
		{
			Iterator.RemoveCurrent();
		}
	}

	for (auto Iterator = RegisteredGroomBindingAssets.CreateIterator(); Iterator; ++Iterator)
	{
		UGroomBindingAsset* GroomBinding = *Iterator;
		if (GroomBinding->IsUnreachable())
		{
			UE_LOGF(LogHairStrands, Verbose, "Cancelling groom binding %ls async compilation because it's being garbage collected", *GroomBinding->GetName());

			if (GroomBinding->TryCancelAsyncTasks())
			{
				Iterator.RemoveCurrent();
				DetachDependencies(GroomBinding);
			}
			else
			{
				PendingAssets.Add(GroomBinding);
			}
		}
	}

	for (auto Iterator = RegisteredSkeletalMeshes.CreateIterator(); Iterator; ++Iterator)
	{
		if (Iterator->Key->IsUnreachable() || Iterator->Value->IsUnreachable())
		{
			Iterator.RemoveCurrent();
		}
	}

	for (auto Iterator = RegisteredGroomAssets.CreateIterator(); Iterator; ++Iterator)
	{
		if (Iterator->Key->IsUnreachable() || Iterator->Value->IsUnreachable())
		{
			Iterator.RemoveCurrent();
		}
	}

	if (PendingAssets.Num())
	{
		FinishCompilation(PendingAssets);
	}
}

FName FGroomBindingCompilingManager::GetAssetTypeName() const
{
	return TEXT("UE-GroomBinding");
}

FTextFormat FGroomBindingCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("GroomBindingNameFormat", "{0}|plural(one=Groom Binding,other=Groom Bindings)");
}

TArrayView<FName> FGroomBindingCompilingManager::GetDependentTypeNames() const
{
	// GroomBindings may have dependencies to SkeletalMesh so we prefer processing them before we get called
	// so a single FinishAllCompilation is able to finish everything in a single pass.
#if WITH_EDITOR
	static FName DependentTypeNames[] = 
	{
		FSkinnedAssetCompilingManager::GetStaticAssetTypeName(),
	};
	return TArrayView<FName>(DependentTypeNames);
#else
	return TArrayView<FName>();
#endif
}

int32 FGroomBindingCompilingManager::GetNumRemainingAssets() const
{
	return GetNumRemainingJobs();
}

EQueuedWorkPriority FGroomBindingCompilingManager::GetBasePriority(UGroomBindingAsset* InGroomBindingAsset) const
{
	return EQueuedWorkPriority::Low;
}

FQueuedThreadPool* FGroomBindingCompilingManager::GetThreadPool() const
{
	static FQueuedThreadPoolDynamicWrapper* GGroomBindingAssetThreadPool = nullptr;
	if (GGroomBindingAssetThreadPool == nullptr && FAssetCompilingManager::Get().GetThreadPool() != nullptr)
	{
		TFunction<EQueuedWorkPriority(EQueuedWorkPriority)> PriorityMapper = [](EQueuedWorkPriority) { return EQueuedWorkPriority::Low; };

		// GroomBinding assets will be scheduled on the asset thread pool, where concurrency limits might by dynamically adjusted depending on memory constraints.
		GGroomBindingAssetThreadPool = new FQueuedThreadPoolDynamicWrapper(FAssetCompilingManager::Get().GetThreadPool(), -1, PriorityMapper);

#if WITH_EDITOR
		AsyncCompilationHelpers::BindThreadPoolToCVar(
			GGroomBindingAssetThreadPool,
			CVarAsyncGroomBindingAssetStandard.AsyncCompilation,
			CVarAsyncGroomBindingAssetStandard.AsyncCompilationResume,
			CVarAsyncGroomBindingAssetStandard.AsyncCompilationMaxConcurrency
		);
#endif
	}

	return GGroomBindingAssetThreadPool;
}

void FGroomBindingCompilingManager::Shutdown()
{
	bHasShutdown = true;
	if (GetNumRemainingJobs())
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::Shutdown);

		for (auto Iterator = RegisteredGroomBindingAssets.CreateIterator(); Iterator; ++Iterator)
		{
			UGroomBindingAsset* GroomBindingAsset = *Iterator;
			if (!GroomBindingAsset->IsAsyncTaskComplete())
			{
				if (GroomBindingAsset->AsyncTask->Cancel())
				{
					GroomBindingAsset->AsyncTask.Reset();
				}
			}

			if (!GroomBindingAsset->AsyncTask)
			{
				Iterator.RemoveCurrent();
				DetachDependencies(GroomBindingAsset);
			}
		}

		FinishCompilation(RegisteredGroomBindingAssets.Array());
	}
}

bool FGroomBindingCompilingManager::IsAsyncCompilationEnabled() const
{
	if (bHasShutdown)
	{
		return false;
	}

#if WITH_EDITOR
	return CVarAsyncGroomBindingAssetStandard.AsyncCompilation.GetValueOnAnyThread() != 0;
#else
	return true;
#endif
}

TRACE_DECLARE_INT_COUNTER(QueuedGroomBindingAssetCompilation, TEXT("AsyncCompilation/QueuedGroomBinding"));
void FGroomBindingCompilingManager::UpdateCompilationNotification()
{
	TRACE_COUNTER_SET(QueuedGroomBindingAssetCompilation, GetNumRemainingJobs());
	Notification.Update(GetNumRemainingJobs());
}

void FGroomBindingCompilingManager::PostCompilation(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets)
{
	if (InGroomBindingAssets.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnAssetPostCompileEvent);

		TSet<UGroomBindingAsset*> Set;
		TArray<FAssetCompileData> AssetsData;
		AssetsData.Reserve(InGroomBindingAssets.Num());

		for (UGroomBindingAsset* GroomBindingAsset : InGroomBindingAssets)
		{
			Set.Emplace(GroomBindingAsset);
			AssetsData.Emplace(GroomBindingAsset);
		}

		FAssetCompilingManager::Get().OnAssetPostCompileEvent().Broadcast(AssetsData);
	}
}

void FGroomBindingCompilingManager::PostCompilation(UGroomBindingAsset* GroomBindingAsset)
{
	using namespace GroomBindingAssetCompilingManagerImpl;
	
	// If AsyncTask is null here, the task got canceled so we don't need to do anything
	if (GroomBindingAsset->AsyncTask)
	{
		check(IsInGameThread());
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::PostCompilation);

		UE_LOGF(LogHairStrands, Verbose, "Refreshing groom binding asset %ls because it is ready", *GroomBindingAsset->GetName());

		FObjectCacheContextScope ObjectCacheScope;

		// The scope is important here to destroy the FGroomBindingAsyncBuildScope before broadcasting events
		{
			// Acquire the async task locally to protect against re-entrance
			TUniquePtr<FGroomBindingAsyncBuildTask> LocalAsyncTask = MoveTemp(GroomBindingAsset->AsyncTask);
			LocalAsyncTask->EnsureCompletion();

			FGroomBindingAsyncBuildScope AsyncBuildScope(GroomBindingAsset);

			if (LocalAsyncTask->GetTask().BuildContext.IsSet())
			{
				GroomBindingAsset->FinishCacheDerivedDatas(*LocalAsyncTask->GetTask().BuildContext);

				LocalAsyncTask->GetTask().BuildContext.Reset();
			}
		}

#if WITH_EDITOR
		// Calling this delegate during app exit might be quite dangerous and lead to crash
		// if the content browser wants to refresh a thumbnail it might try to load a package
		// which will then fail due to various reasons related to the editor shutting down.
		// Triggering this callback while garbage collecting can also result in listeners trying to look up objects
		if (!GExitPurge && !IsGarbageCollecting())
		{
			// Generate an empty property changed event, to force the asset registry tag
			// to be refreshed now that RenderData is available.
			FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(GroomBindingAsset, EmptyPropertyChangedEvent);
		}
#endif
	}
}

bool FGroomBindingCompilingManager::IsAsyncCompilationAllowed(UGroomBindingAsset* GroomBindingAsset) const
{
	return IsAsyncCompilationEnabled();
}

FGroomBindingCompilingManager& FGroomBindingCompilingManager::Get()
{
	static FGroomBindingCompilingManager Singleton;
	return Singleton;
}

int32 FGroomBindingCompilingManager::GetNumRemainingJobs() const
{
	return RegisteredGroomBindingAssets.Num() + GroomBindingWithPendingDependencies.Num();
}

void FGroomBindingCompilingManager::AddGroomBindingsWithPendingDependencies(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::AddGroomBindingsWithPendingDependencies);
	check(IsInGameThread());

	for (UGroomBindingAsset* GroomBindingAsset : InGroomBindingAssets)
	{
		GroomBindingWithPendingDependencies.Emplace(GroomBindingAsset);
	}

	UpdateCompilationNotification();
}

void FGroomBindingCompilingManager::AddGroomBindings(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::AddGroomBindings)
	check(IsInGameThread());

	for (UGroomBindingAsset* GroomBindingAsset : InGroomBindingAssets)
	{
		// If the compilation is launched while we still have it in our list, we don't want to schedule it again so remove it now.
		GroomBindingWithPendingDependencies.Remove(GroomBindingAsset);

		check(GroomBindingAsset->AsyncTask != nullptr);
		RegisteredGroomBindingAssets.Emplace(GroomBindingAsset);

		AttachDependencies(GroomBindingAsset);
	}

	UpdateCompilationNotification();
}

void FGroomBindingCompilingManager::FinishCompilation(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::FinishCompilation);

	check(IsInGameThread());

	TSet<USkinnedAsset*> Dependencies;
	for (UGroomBindingAsset* GroomBindingAsset : InGroomBindingAssets)
	{
		if (GroomBindingAsset->GetTargetSkeletalMesh() && GroomBindingAsset->GetTargetSkeletalMesh()->IsCompiling())
		{
			Dependencies.Emplace(GroomBindingAsset->GetTargetSkeletalMesh());
		}

		if (GroomBindingAsset->GetSourceSkeletalMesh() && GroomBindingAsset->GetSourceSkeletalMesh()->IsCompiling())
		{
			Dependencies.Emplace(GroomBindingAsset->GetSourceSkeletalMesh());
		}
	}

#if WITH_EDITOR
	FSkinnedAssetCompilingManager::Get().FinishCompilation(Dependencies.Array());
#endif
	
	// Now that dependencies have finished, we can launch the compilations for the groom bindings.
	SchedulePendingCompilations();

	TArray<UGroomBindingAsset*> PendingGroomBindingAssets;
	PendingGroomBindingAssets.Reserve(InGroomBindingAssets.Num());

	for (UGroomBindingAsset* GroomBindingAsset : InGroomBindingAssets)
	{
		if (RegisteredGroomBindingAssets.Contains(GroomBindingAsset))
		{
			PendingGroomBindingAssets.Emplace(GroomBindingAsset);
		}
	}

	if (PendingGroomBindingAssets.Num())
	{
		class FCompilableGroomBindingAsset : public AsyncCompilationHelpers::TCompilableAsyncTask<FGroomBindingAsyncBuildTask>
		{
		public:
			FCompilableGroomBindingAsset(UGroomBindingAsset* InGroomBindingAsset)
				: GroomBindingAsset(InGroomBindingAsset)
			{
			}

			FGroomBindingAsyncBuildTask* GetAsyncTask() override
			{
				return GroomBindingAsset->AsyncTask.Get();
			}

			TStrongObjectPtr<UGroomBindingAsset> GroomBindingAsset;
			FName GetName() override { return GroomBindingAsset->GetFName(); }
		};

		TArray<FCompilableGroomBindingAsset> CompilableGroomBindingAsset(PendingGroomBindingAssets);

		FObjectCacheContextScope ObjectCacheScope;
		AsyncCompilationHelpers::FinishCompilation(
			[&CompilableGroomBindingAsset](int32 Index) -> AsyncCompilationHelpers::ICompilable& { return CompilableGroomBindingAsset[Index]; },
			CompilableGroomBindingAsset.Num(),
#if WITH_EDITOR
			LOCTEXT("GroomBindings", "Groom Bindings"),
			LogHairStrands,
#endif
			[this](AsyncCompilationHelpers::ICompilable* Object)
			{
				UGroomBindingAsset* GroomBindingAsset = static_cast<FCompilableGroomBindingAsset*>(Object)->GroomBindingAsset.Get();
				PostCompilation(GroomBindingAsset);
				RegisteredGroomBindingAssets.Remove(GroomBindingAsset);
				DetachDependencies(GroomBindingAsset);
			}
		);

		PostCompilation(PendingGroomBindingAssets);

		UpdateCompilationNotification();
	}
}

void FGroomBindingCompilingManager::SchedulePendingCompilations()
{
	TArray<UGroomBindingAsset*> ReadyToSchedule;
	for (auto It = GroomBindingWithPendingDependencies.CreateIterator(); It; ++It)
	{
		UGroomBindingAsset* GroomBindingAsset = *It;
		if (!GroomBindingAsset->HasAnyDependenciesCompiling())
		{
			ReadyToSchedule.Emplace(GroomBindingAsset);
			It.RemoveCurrent();
		}
	}

	// Call CacheDerivedDatas again so it's scheduled for real this time.
	for (UGroomBindingAsset* GroomBindingAsset : ReadyToSchedule)
	{
		GroomBindingAsset->BeginCacheDerivedDatas();
	}
}

void FGroomBindingCompilingManager::Reschedule()
{
	// Implement if we ever want to prioritize nearest visible grooms first
}

void FGroomBindingCompilingManager::FinishCompilationsForGame()
{
	// Implement if grooms ever become necessary for gameplay to work correctly
}

void FGroomBindingCompilingManager::FinishAllCompilation()
{
	check(IsInGameThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::FinishAllCompilation);

	if (GetNumRemainingJobs())
	{
		FinishCompilation(RegisteredGroomBindingAssets.Array());
	}
}

void FGroomBindingCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::FinishCompilationForObjects);

	TSet<UGroomBindingAsset*> AssetToFinish;
	for (UObject* Object : InObjects)
	{
		if (UGroomBindingAsset* GroomBindingAsset = Cast<UGroomBindingAsset>(Object))
		{
			AssetToFinish.Add(GroomBindingAsset);
		}
		else if (UGroomComponent* GroomComponent = Cast<UGroomComponent>(Object))
		{
			if (GroomComponent->BindingAsset)
			{
				AssetToFinish.Add(GroomComponent->BindingAsset);
			}
		}
		else if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
		{
			for (auto It = RegisteredSkeletalMeshes.CreateKeyIterator(SkeletalMesh); It; ++It)
			{
				AssetToFinish.Add(It.Value());
			}
		}
		else if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
		{
			for (auto It = RegisteredGroomAssets.CreateKeyIterator(GroomAsset); It; ++It)
			{
				AssetToFinish.Add(It.Value());
			}
		}
	}

	if (AssetToFinish.Num())
	{
		FinishCompilation(AssetToFinish.Array());
	}
}

void FGroomBindingCompilingManager::ProcessGroomBindingAssets(bool bLimitExecutionTime, int32 MinBatchSize)
{
	using namespace GroomBindingAssetCompilingManagerImpl;
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBindingCompilingManager::ProcessGroomBindingAssets);
	const int32 NumRemainingJobs = GetNumRemainingJobs();
	// Spread out the load over multiple frames but if too many assets, convergence is more important than frame time
	const int32 MaxMeshUpdatesPerFrame = bLimitExecutionTime ? FMath::Max(64, NumRemainingJobs / 10) : INT32_MAX;

	FObjectCacheContextScope ObjectCacheScope;
	if (NumRemainingJobs && NumRemainingJobs >= MinBatchSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFinishedGroomBindingAssets);

		const double TickStartTime = FPlatformTime::Seconds();

		TArray<UGroomBindingAsset*> ProcessedGroomBindingAssets;
		int32 ProcessedCount = 0;
		for (auto It = RegisteredGroomBindingAssets.CreateIterator(); It; ++It)
		{
			UGroomBindingAsset* GroomBindingAsset = *It;
			if (GroomBindingAsset->IsAsyncTaskComplete())
			{
				PostCompilation(GroomBindingAsset);
				ProcessedGroomBindingAssets.Add(GroomBindingAsset);
				It.RemoveCurrent();

				if (++ProcessedCount > MaxMeshUpdatesPerFrame)
				{
					break;
				}
			}
		}

		PostCompilation(ProcessedGroomBindingAssets);
	}
}

void FGroomBindingCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	FObjectCacheContextScope ObjectCacheScope;
	FinishCompilationsForGame();

	Reschedule();
	
	SchedulePendingCompilations();

	ProcessGroomBindingAssets(bLimitExecutionTime);

	UpdateCompilationNotification();
}

#undef LOCTEXT_NAMESPACE
