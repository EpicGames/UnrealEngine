// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBlueprintLibrary.h"
#include "GeometryCache.h"
#include "HairStrandsCore.h"
#include "GroomBindingAsset.h"
#include "HairStrandsInterface.h"
#include "Engine/Engine.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomBlueprintLibrary)

UGroomBindingAsset* UGroomBlueprintLibrary::CreateNewGroomBindingAssetWithPath(
	const FString& InDesiredPackagePath,
	UGroomAsset* InGroomAsset,
	USkeletalMesh* InSkeletalMesh,
	int32 InNumInterpolationPoints,
	USkeletalMesh* InSourceSkeletalMeshForTransfer,
	int32 InMatchingSection)
{
#if WITH_EDITOR
	if (!InGroomAsset || !InSkeletalMesh)
	{
		return nullptr;
	}

	UGroomBindingAsset* BindingAsset = FHairStrandsCore::CreateGroomBindingAsset(EGroomBindingMeshType::SkeletalMesh, InDesiredPackagePath, nullptr, InGroomAsset, InSourceSkeletalMeshForTransfer, InSkeletalMesh, InNumInterpolationPoints, InMatchingSection);
	if (BindingAsset)
	{
		BindingAsset->Build();
	}
	FHairStrandsCore::SaveAsset(BindingAsset);
	return BindingAsset;
#else
	return nullptr;
#endif
}

UGroomBindingAsset* UGroomBlueprintLibrary::CreateNewGroomBindingAsset(
	UGroomAsset* InGroomAsset,
	USkeletalMesh* InSkeletalMesh,
	int32 InNumInterpolationPoints,
	USkeletalMesh* InSourceSkeletalMeshForTransfer,
	int32 InMatchingSection)
{
#if WITH_EDITOR
	if (!InGroomAsset || !InSkeletalMesh)
	{
		return nullptr;
	}

	UGroomBindingAsset* BindingAsset = FHairStrandsCore::CreateGroomBindingAsset(EGroomBindingMeshType::SkeletalMesh, InGroomAsset, InSourceSkeletalMeshForTransfer, InSkeletalMesh, InNumInterpolationPoints, InMatchingSection);
	if (BindingAsset)
	{
		BindingAsset->Build();
	}
	FHairStrandsCore::SaveAsset(BindingAsset);
	return BindingAsset;
#else
	return nullptr;
#endif
}

UGroomBindingAsset* UGroomBlueprintLibrary::CreateNewGeometryCacheGroomBindingAssetWithPath(
	const FString& InDesiredPackagePath,
	UGroomAsset* InGroomAsset,
	UGeometryCache* InGeometryCache,
	int32 InNumInterpolationPoints,
	UGeometryCache* InSourceGeometryCacheForTransfer,
	int32 InMatchingSection)
{
#if WITH_EDITOR
	if (!InGroomAsset || !InGeometryCache)
	{
		return nullptr;
	}

	UGroomBindingAsset* BindingAsset = FHairStrandsCore::CreateGroomBindingAsset(EGroomBindingMeshType::GeometryCache, InDesiredPackagePath, nullptr, InGroomAsset, InSourceGeometryCacheForTransfer, InGeometryCache, InNumInterpolationPoints, InMatchingSection);
	if (BindingAsset)
	{
		BindingAsset->Build();
	}
	FHairStrandsCore::SaveAsset(BindingAsset);
	return BindingAsset;
#else
	return nullptr;
#endif
}

UGroomBindingAsset* UGroomBlueprintLibrary::CreateNewGeometryCacheGroomBindingAsset(
	UGroomAsset* InGroomAsset,
	UGeometryCache* InGeometryCache,
	int32 InNumInterpolationPoints,
	UGeometryCache* InSourceGeometryCacheForTransfer,
	int32 InMatchingSection)
{
#if WITH_EDITOR
	if (!InGroomAsset || !InGeometryCache)
	{
		return nullptr;
	}

	UGroomBindingAsset* BindingAsset = FHairStrandsCore::CreateGroomBindingAsset(EGroomBindingMeshType::GeometryCache, InGroomAsset, InSourceGeometryCacheForTransfer, InGeometryCache, InNumInterpolationPoints, InMatchingSection);
	if (BindingAsset)
	{
		BindingAsset->Build();
	}
	FHairStrandsCore::SaveAsset(BindingAsset);
	return BindingAsset;
#else
	return nullptr;
#endif
}

bool UGroomBlueprintLibrary::IsHairStrandsSupportedInWorld(const UObject* WorldContextObject)
{
	bool bHairEnabledSupported = false;

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (FSceneInterface* Scene = World->Scene)
		{
			bHairEnabledSupported = IsHairStrandsSupported(EHairStrandsShaderType::Strands, Scene->GetShaderPlatform());
		}
	}

	return bHairEnabledSupported;
}