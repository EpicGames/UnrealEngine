// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBuilder.h"
#include "GroomAsset.h"
#include "GroomComponent.h"
#include "GroomSettings.h"
#include "HairDescription.h"
#include "GroomSettings.h"
#include "GroomBindingBuilder.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "GlobalShader.h"
#include "RHIGPUReadback.h"
#include "ShaderParameterStruct.h"
#include "SceneView.h"
#include "Math/MathFwd.h"
#include "RenderCurveDefinitions.h"

#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/ScopedSlowTask.h"
#include "Clustering/KMeans.h"

DEFINE_LOG_CATEGORY_STATIC(LogGroomBuilder, Log, All);

#define LOCTEXT_NAMESPACE "GroomBuilder"

/////////////////////////////////////////////////////////////////////////////////////////
// CVars

// For debug purpose
static float GHairInterpolationMetric_Distance = 1;
static float GHairInterpolationMetric_Angle = 0;
static float GHairInterpolationMetric_Length = 0;
static float GHairInterpolationMetric_AngleAttenuation = 5;
static FAutoConsoleVariableRef CVarHairInterpolationMetric_Distance(TEXT("r.HairStrands.InterpolationMetric.Distance"), GHairInterpolationMetric_Distance, TEXT("Hair strands interpolation metric weights for distance"));
static FAutoConsoleVariableRef CVarHairInterpolationMetric_Angle(TEXT("r.HairStrands.InterpolationMetric.Angle"), GHairInterpolationMetric_Angle, TEXT("Hair strands interpolation metric weights for angle"));
static FAutoConsoleVariableRef CVarHairInterpolationMetric_Length(TEXT("r.HairStrands.InterpolationMetric.Length"), GHairInterpolationMetric_Length, TEXT("Hair strands interpolation metric weights for length"));
static FAutoConsoleVariableRef CVarHairInterpolationMetric_AngleAttenuation(TEXT("r.HairStrands.InterpolationMetric.AngleAttenuation"), GHairInterpolationMetric_AngleAttenuation, TEXT("Hair strands interpolation angle attenuation"));

static int32 GHairClusterBuilder_MaxVoxelResolution = 256;
static FAutoConsoleVariableRef CVarHairClusterBuilder_MaxVoxelResolution(TEXT("r.HairStrands.ClusterBuilder.MaxVoxelResolution"), GHairClusterBuilder_MaxVoxelResolution, TEXT("Max voxel resolution used when building hair strands cluster data to avoid too long building time (default:128).  "));

static int32 GHairGroupIndexBuilder_MaxVoxelResolution = 64;
static FAutoConsoleVariableRef CVarHairGroupIndexBuilder_MaxVoxelResolution(TEXT("r.HairStrands.HairGroupBuilder.MaxVoxelResolution"), GHairGroupIndexBuilder_MaxVoxelResolution, TEXT("Max voxel resolution used when voxelizing hair strands to transfer group index grom strands to cards. This avoids too long building time (default:64).  "));

/////////////////////////////////////////////////////////////////////////////////////////
// Forward declaration

bool DoesHairStrandsSupportCompressedPosition();

/////////////////////////////////////////////////////////////////////////////////////////
// Helpers

FString FGroomBuilder::GetVersion()
{
	FString Out = TEXT("v21_102031A3EE7E4570B3093414EF3956F3");
	if (IsRenderCurveEnabled())
	{
		Out += TEXT("_Curve4a");
	}
	return Out;
}

namespace GroomBuilder_Voxelization
{
	struct FCoverageScale
	{
		float ScreenSize = 0;
		float CoverageScale = 0;
	};

	void ComputeHairCoverageScale(const FHairStrandsDatas& In, TArray<FCoverageScale>& Out);
}

// For debug purpose
template<typename T>
void ReportSize(const TCHAR* InName, const TArray<T>& In)
{
#if 0
	const uint32 TotalSize = In.GetTypeSize() * In.Num();
	const uint32 TotalKBytes = TotalSize / 1024;
	UE_LOGF(LogGroomBuilder, Log, "Size: %10d Kbytes - Num: %8d - Name: %ls", TotalKBytes, In.Num(), InName);
#endif
}

/////////////////////////////////////////////////////////////////////////////////////////
// Build functions

namespace FHairStrandsDecimation
{
	void Decimate(
		const FHairStrandsDatas& InData,
		float CurveDecimationPercentage,
		float VertexDecimationPercentage,
		bool bContinuousDecimationReordering,
		FHairStrandsDatas& OutData,
		const bool bBuildSourceMapping);

	void Decimate(
		const FHairStrandsDatas& InData, 
		const uint32 NumCurves,
		const int32 NumVertices,
		FHairStrandsDatas& OutData,
		const bool bBuildSourceMapping);
}

namespace HairStrandsBuilder
{
	/** Build FHairStrandsDatas from a FHairStrandsRawDatas */
	FHairStrandsDatas BuildInternalData(const FHairStrandsCommonDatas& In)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HairStrandsBuilder::BuildInternalData);
		FHairStrandsDatas Out;

		Out.StrandsCurves = In.StrandsCurves;
		Out.StrandsPoints = In.StrandsPoints;
		Out.Flags = In.Flags;
		Out.BoundingBox.Min = {  FLT_MAX,  FLT_MAX ,  FLT_MAX };
		Out.BoundingBox.Max = { -FLT_MAX, -FLT_MAX , -FLT_MAX };
		Out.BoundingBox.IsValid = 0;

		if (Out.GetNumCurves() > 0 && Out.GetNumPoints() > 0)
		{
			TArray<FVector3f>::TIterator PositionIterator = Out.StrandsPoints.PointsPosition.CreateIterator();
			TArray<float>::TIterator RadiusIterator = Out.StrandsPoints.PointsRadius.CreateIterator();
			TArray<float>::TIterator CoordUIterator = Out.StrandsPoints.PointsCoordU.CreateIterator();

			TArray<uint16>::TIterator CountIterator = Out.StrandsCurves.CurvesCount.CreateIterator();
			TArray<uint32>::TIterator OffsetIterator = Out.StrandsCurves.CurvesOffset.CreateIterator();
			TArray<float>::TIterator LengthIterator = Out.StrandsCurves.CurvesLength.CreateIterator();

			uint32 StrandOffset = 0;
			*OffsetIterator = StrandOffset; ++OffsetIterator;

			for (uint32 CurveIndex = 0; CurveIndex < Out.GetNumCurves(); ++CurveIndex, ++OffsetIterator, ++LengthIterator, ++CountIterator)
			{
				const uint16& StrandCount = *CountIterator;

				StrandOffset += StrandCount;
				*OffsetIterator = StrandOffset;

				float StrandLength = 0.0;
				FVector3f PreviousPosition(0.0, 0.0, 0.0);
				for (uint32 PointIndex = 0; PointIndex < StrandCount; ++PointIndex, ++PositionIterator, ++RadiusIterator, ++CoordUIterator)
				{
					const FVector3f CurrentPosition = *PositionIterator;
					Out.BoundingBox += CurrentPosition;
					Out.BoundingBox.IsValid = 1;
					if (PointIndex > 0)
					{
						StrandLength += (CurrentPosition - PreviousPosition).Size();
					}
					*CoordUIterator = StrandLength;
					PreviousPosition = CurrentPosition;
				}
				*LengthIterator = StrandLength;
			}

			CountIterator.Reset();
			LengthIterator.Reset();
			RadiusIterator.Reset();
			CoordUIterator.Reset();

			for (uint32 CurveIndex = 0; CurveIndex < Out.GetNumCurves(); ++CurveIndex, ++LengthIterator, ++CountIterator)
			{
				const uint16& StrandCount = *CountIterator;

				for (uint32 PointIndex = 0; PointIndex < StrandCount; ++PointIndex, ++RadiusIterator, ++CoordUIterator)
				{
					if (*LengthIterator > 0.0f)
					{
						*CoordUIterator /= *LengthIterator;
					}
				}
			}
		}
		return Out;
	}

	inline void CopyVectorToPosition(const FVector3f& InVector, FHairStrandsPositionFormat::Type& OutPosition)
	{
		OutPosition.X = InVector.X;
		OutPosition.Y = InVector.Y;
		OutPosition.Z = InVector.Z;
	}

	template<typename TFormatType>
	void CopyToBulkData(FByteBulkData& Out, const TArray<typename TFormatType::Type>& Data)
	{
		static_assert(TFormatType::SizeInByte == sizeof(typename TFormatType::BulkType));
		const uint32 ElementSizeInByte = sizeof(typename TFormatType::BulkType);
		const uint32 DataSizeInByte = Data.Num() * ElementSizeInByte;

		// The buffer is then stored into bulk data
		Out.Lock(LOCK_READ_WRITE);
		void* BulkBuffer = Out.Realloc(DataSizeInByte);
		FMemory::Memcpy(BulkBuffer, Data.GetData(), DataSizeInByte);
		Out.Unlock();
	}

	template<typename TFormatType>
	void CopyToBulkData(FHairBulkContainer& Out, const TArray<typename TFormatType::Type>& Data)
	{
		CopyToBulkData<TFormatType>(Out.Data, Data);
	}

	/** Build the bulk/packed datas for gpu rendering/simulation */
	void BuildBulkData(const FHairStrandsDatas& HairStrands, const TArray<uint8>& RandomSeeds, const TArray<GroomBuilder_Voxelization::FCoverageScale>& InCoverageScales, FHairStrandsBulkData& OutBulkData, bool bAllowTranscoding)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HairStrandsBuilder::BuildBulkData);

		const uint32 NumCurves = HairStrands.GetNumCurves();
		const uint32 NumPoints = HairStrands.GetNumPoints();
		if (!(NumCurves > 0 && NumPoints > 0))
		{
			return;
		}

		TArray<FHairStrandsPositionFormat::Type> OutPackedPositions;
		TArray<FHairStrandsCurveFormat::Type> OutPackedCurves;

		OutPackedPositions.SetNum(NumPoints * FHairStrandsPositionFormat::ComponentCount);
		OutPackedCurves.SetNum(NumCurves * FHairStrandsCurveFormat::ComponentCount);

		const uint32 PointToCurveChunkElementCount = 8u;
		TArray<FHairStrandsPointToCurveFormat::Type> OutPointToCurve;
		OutPointToCurve.SetNum(FMath::DivideAndRoundUp(NumPoints, PointToCurveChunkElementCount));

		const uint32 Attributes = HairStrands.GetAttributes();
		const uint32 AttributeFlags = HairStrands.GetAttributeFlags();
		const bool bHasMultipleClumpIDs = HasHairAttributeFlags(AttributeFlags, EHairAttributeFlags::HasMultipleClumpIDs);

		// Byte arrays
		TArray<uint8> AttributeRootUV;
		TArray<uint8> AttributeSeed;
		TArray<uint8> AttributeLength;
		TArray<uint8> AttributeClumpIDs;
		TArray<uint8> AttributeColor;
		TArray<uint8> AttributeRoughness;
		TArray<uint8> AttributeAO;

		// Stride data in bytes
		const uint32 Stride_Seed 		= 1;
		const uint32 Stride_Length 		= 2;
		const uint32 Stride_RootUV 		= 4;
		const uint32 Stride_ClumpID 	= bHasMultipleClumpIDs ? 8 : 2;
		const uint32 Stride_Color		= 4;
		const uint32 Stride_Roughness	= 1;
		const uint32 Stride_AO			= 1;

		// Ensure all data array are 4bytes aligned so that data are properly padded
																	{ AttributeSeed		.SetNum(FMath::DivideAndRoundUp(NumCurves * Stride_Seed, 		4u) * 4u); }
																	{ AttributeLength	.SetNum(FMath::DivideAndRoundUp(NumCurves * Stride_Length, 		4u) * 4u); }
		if (HasHairAttribute(Attributes, EHairAttribute::RootUV)) 	{ AttributeRootUV	.SetNum(FMath::DivideAndRoundUp(NumCurves * Stride_RootUV, 		4u) * 4u); }
		if (HasHairAttribute(Attributes, EHairAttribute::ClumpID))	{ AttributeClumpIDs	.SetNum(FMath::DivideAndRoundUp(NumCurves * Stride_ClumpID,		4u) * 4u); }
		if (HasHairAttribute(Attributes, EHairAttribute::Color))	{ AttributeColor	.SetNum(FMath::DivideAndRoundUp(NumPoints * Stride_Color, 		4u) * 4u); }
		if (HasHairAttribute(Attributes, EHairAttribute::Roughness)){ AttributeRoughness.SetNum(FMath::DivideAndRoundUp(NumPoints * Stride_Roughness, 	4u) * 4u); }
		if (HasHairAttribute(Attributes, EHairAttribute::AO))		{ AttributeAO		.SetNum(FMath::DivideAndRoundUp(NumPoints * Stride_AO, 			4u) * 4u); }

		const FVector3f HairBoxCenter = HairStrands.BoundingBox.GetCenter();

		const FHairStrandsCurves& Curves = HairStrands.StrandsCurves;
		const FHairStrandsPoints& Points = HairStrands.StrandsPoints;

		// Allocate remapping data if needed
		TArray<FHairStrandsIndexFormat::Type> OutPointsMapping;
		if (Points.PointsMapping.Num() > 0)
		{
			OutPointsMapping.SetNum(NumPoints * FHairStrandsIndexFormat::ComponentCount);
		}
		TArray<FHairStrandsIndexFormat::Type> OutCurvesMapping;
		if (Curves.CurvesMapping.Num() > 0)
		{
			OutCurvesMapping.SetNum(NumCurves * FHairStrandsIndexFormat::ComponentCount);
		}

		const float MaxLength = GetHairStrandsMaxLength(HairStrands);
		const float MaxRadius = GetHairStrandsMaxRadius(HairStrands);

		static_assert(sizeof(FPackedHairVertex) == sizeof(FPackedHairVertex::BulkType));
		static_assert(sizeof(FPackedHairVertex) == FPackedHairPositionStrideInBytes);
		static_assert(sizeof(FPackedHairVertex) == 8u);

		uint32 MinPointPerCurve = HAIR_MAX_NUM_POINT_PER_CURVE;
		uint32 MaxPointPerCurve = 0;
		uint32 AccPointPerCurve = 0;

		const bool bSeedValid = RandomSeeds.Num() > 0;
		for (uint32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			const int32 IndexOffset = Curves.CurvesOffset[CurveIndex];
			const uint16 PointCount = Curves.CurvesCount[CurveIndex];

			MinPointPerCurve = FMath::Min(MinPointPerCurve, uint32(PointCount));
			MaxPointPerCurve = FMath::Max(MaxPointPerCurve, uint32(PointCount));
			AccPointPerCurve += PointCount;

			for (int32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
			{
				const uint32 PrevIndex = FMath::Max(0, PointIndex - 1);
				const uint32 NextIndex = FMath::Min(PointCount + 1, PointCount - 1);
				const FVector3f& PointPosition = Points.PointsPosition[PointIndex + IndexOffset];

				// Packed Position|CoordU|NormalizedRadius|Type
				{
					const float CoordU = Points.PointsCoordU[PointIndex + IndexOffset];
					const float NormalizedRadius = Points.PointsRadius[PointIndex + IndexOffset] / MaxRadius;

					FHairStrandsPositionFormat::Type& PackedPosition = OutPackedPositions[PointIndex + IndexOffset];
					CopyVectorToPosition(PointPosition - HairBoxCenter, PackedPosition);
					PackedPosition.UCoord = uint8(FMath::Clamp(CoordU * 255.f, 0.f, 255.f));
					PackedPosition.Radius = uint8(FMath::Clamp(NormalizedRadius * 63.f, 0.f, 63.f));
					PackedPosition.Type = (PointIndex == 0) ? HAIR_CONTROLPOINT_START : (PointIndex == (PointCount - 1) ? HAIR_CONTROLPOINT_END : HAIR_CONTROLPOINT_INSIDE);
				}

				// Point to Curve
				{
					static_assert(PointToCurveChunkElementCount == 8u);

					const uint32 CurrentIndex    = PointIndex + IndexOffset;
					const uint32 CurrentIndex4   = CurrentIndex >> 3u;
					const uint32 LocalPointIndex = CurrentIndex & 0x7;
					const bool bIsFirstPoint     = LocalPointIndex == 0;

					// Format
					// Encode 8 points per uint with a base curve index + delta bits
					// [                         32bits                         ]
					// [     24bits     ][                8bits                 ]
					// [ BaseCurveIndex ][Pt0][Pt0][Pt2][Pt3][Pt4][Pt5][Pt6][Pt7]
					if (bIsFirstPoint)
					{
						OutPointToCurve[CurrentIndex4] = CurveIndex;
					}
					else
					{
						const uint32 BaseCurveIndex = (OutPointToCurve[CurrentIndex4] & 0xFFFFFF);
						uint32 PackedDeltas   = (OutPointToCurve[CurrentIndex4]>>24u) & 0xFF;

						// Rebuilt curve indices from the detla encoding
						uint32 CurveIndices[8] = {0,0,0,0,0,0,0,0};
						CurveIndices[0] = BaseCurveIndex;
						for (uint32 PointIt = 1; PointIt <= LocalPointIndex; ++PointIt)
						{
							CurveIndices[PointIt] = CurveIndices[PointIt-1] + ((PackedDeltas >> PointIt) & 0x1);
						}

						// Sanity check
						check(CurveIndices[LocalPointIndex] == CurveIndex || CurveIndices[LocalPointIndex]+1 == CurveIndex);

						// Add curve bit if needed
						if (CurveIndices[LocalPointIndex] != CurveIndex)
						{
							PackedDeltas |= 1u << LocalPointIndex;
						}

						OutPointToCurve[CurrentIndex4] = BaseCurveIndex | PackedDeltas<<24u;
					}
				}

				// Per-Vertex Color
				if (HasHairAttribute(Attributes, EHairAttribute::Color))
				{
					uint32* Data = (uint32*)AttributeColor.GetData();
					uint32& Packed = Data[PointIndex + IndexOffset];
					Packed = 0;
					Packed |= FMath::Clamp(uint32(FMath::Sqrt(Points.PointsBaseColor[PointIndex + IndexOffset].R) * 0x7FF), 0u, 0x7FFu);
					Packed |= FMath::Clamp(uint32(FMath::Sqrt(Points.PointsBaseColor[PointIndex + IndexOffset].G) * 0x7FF), 0u, 0x7FFu)<<11;
					Packed |= FMath::Clamp(uint32(FMath::Sqrt(Points.PointsBaseColor[PointIndex + IndexOffset].B) * 0x3FF), 0u, 0x3FFu)<<22;
				}

				// Per-Vertex Roughness
				if (HasHairAttribute(Attributes, EHairAttribute::Roughness))
				{
					uint8* Data = (uint8*)AttributeRoughness.GetData();
					Data[PointIndex + IndexOffset] = FMath::Clamp(uint32(Points.PointsRoughness[PointIndex + IndexOffset] * 0xFF), 0u, 0xFFu);
				}

				// Per-Vertex AO
				if (HasHairAttribute(Attributes, EHairAttribute::AO))
				{
					uint8* Data = (uint8*)AttributeAO.GetData();
					Data[PointIndex + IndexOffset] = FMath::Clamp(uint32(Points.PointsAO[PointIndex + IndexOffset] * 0xFF), 0u, 0xFFu);
				}

				// Per-Vertex Source Mapping
				{
					if(!Points.PointsMapping.IsEmpty())
					{
						OutPointsMapping[PointIndex + IndexOffset] = Points.PointsMapping[PointIndex + IndexOffset];
					}
				}
			}

			// Per-Curve Length
			{
				FFloat16* Packed = (FFloat16*)AttributeLength.GetData();
				Packed[CurveIndex] = Curves.CurvesLength[CurveIndex];
			}

			// Per-Curve Seed
			{
				uint8* Packed = (uint8*)AttributeSeed.GetData();
				Packed[CurveIndex] = bSeedValid ? RandomSeeds[CurveIndex] : 0;
			}

			// Per-Curve Source Mapping
			{
				if(!Curves.CurvesMapping.IsEmpty())
				{
					OutCurvesMapping[CurveIndex] = Curves.CurvesMapping[CurveIndex];
				}
			}

			// Per-Curve Root UVs
			if (HasHairAttribute(Attributes, EHairAttribute::RootUV))
			{
				FVector2D RootUV = FVector2D(Curves.CurvesRootUV[CurveIndex]);
				// Root UV support UDIM texture coordinate but limit the spans of the UDIM to be in 256x256 instead of 9999x9999.
				// The internal UV coords are also limited to 8bits, which means if sampling need to be super precise, this is no enough.
				// Specal case for UV == 1.0f, as we don't need UDIM data in this case, so force the value to be in [0..1)
				const float SmallEpsilon = 1e-05f;
				RootUV.X = RootUV.X == 1.0f ? RootUV.X - SmallEpsilon : RootUV.X;
				RootUV.Y = RootUV.Y == 1.0f ? RootUV.Y - SmallEpsilon : RootUV.Y;
				const FVector2D TextureRootUV(FMath::Fractional(RootUV.X), FMath::Fractional(RootUV.Y));
				const FVector2D TextureIndexUV = RootUV - TextureRootUV;

				// UDIM
				uint32* Data = (uint32*)AttributeRootUV.GetData();
				uint32& Packed = Data[CurveIndex];
				Packed = 0;
				Packed |= (uint32(FMath::Clamp(TextureRootUV.X * 2047.f, 0.f, 2047.f)) & 0x7FFu);
				Packed |= (uint32(FMath::Clamp(TextureRootUV.Y * 2047.f, 0.f, 2047.f)) & 0x7FFu) << 11u;
				Packed |= (uint32(FMath::Clamp(TextureIndexUV.X, 0.f, 31.f)) & 0x1F) << 22u;
				Packed |= (uint32(FMath::Clamp(TextureIndexUV.Y, 0.f, 31.f)) & 0x1F) << 27u;
			}

			// Per-Curve Clump ID (16 bits max)
			if (HasHairAttribute(Attributes, EHairAttribute::ClumpID))
			{
				if (bHasMultipleClumpIDs)
				{
					uint64* Packed = (uint64*)AttributeClumpIDs.GetData();
					Packed[CurveIndex]  = 0;
					Packed[CurveIndex] |= uint64(FMath::Clamp(Curves.ClumpIDs[CurveIndex].X, 0, 65535));
					Packed[CurveIndex] |= uint64(FMath::Clamp(Curves.ClumpIDs[CurveIndex].Y, 0, 65535))<<16;
					Packed[CurveIndex] |= uint64(FMath::Clamp(Curves.ClumpIDs[CurveIndex].Z, 0, 65535))<<32;
				}
				else
				{
					uint16* Packed = (uint16*)AttributeClumpIDs.GetData();
					Packed[CurveIndex] = uint16(FMath::Clamp(Curves.ClumpIDs[CurveIndex].X, 0, 65535));
				}
			}

			// Per-Curve Strand ID
			{
				// At the moment we do not store this data within the bulk data
			}

			// Curves
			{
				// If the limit change, update this encoding
				static_assert(HAIR_MAX_NUM_POINT_PER_CURVE == 0xFF);
				static_assert(HAIR_MAX_NUM_POINT_PER_GROUP == 0xFFFFFF);
				UE_CLOGF(PointCount > HAIR_MAX_NUM_POINT_PER_CURVE, LogGroomBuilder, Warning, "[Groom] Curve point count is greater than the allowed limit (%d/%d)", PointCount, uint32(HAIR_MAX_NUM_POINT_PER_CURVE));
				UE_CLOGF(IndexOffset > HAIR_MAX_NUM_POINT_PER_GROUP, LogGroomBuilder, Warning, "[Groom] Curve point offset is greater than the allowed limit (%d/%d)", IndexOffset, uint32(HAIR_MAX_NUM_POINT_PER_GROUP));
				FHairStrandsCurveFormat::Type& Curve = OutPackedCurves[CurveIndex];
				Curve.PointCount  = FMath::Min(uint32(PointCount), HAIR_MAX_NUM_POINT_PER_CURVE);
				Curve.PointOffset = FMath::Min(uint32(IndexOffset), HAIR_MAX_NUM_POINT_PER_GROUP);
			}
		}
		
		// Transcoded positions (optional)
		const FVector3f TranscodingPositionOffset = -HairStrands.BoundingBox.GetExtent();
		const FVector3f TranscodingPositionScale  =  HairStrands.BoundingBox.GetExtent() * 2.f;
		const uint32 TranscodedPositionChunkElementCount = 3u;
		const uint32 TranscodedPositionChunkStride = sizeof(FTranscodedHairPositions);
		TArray<FTranscodedHairPositions> OutTranscodedPositions;
		if (DoesHairStrandsSupportCompressedPosition() && bAllowTranscoding)
		{
			bool bHasTranscodedPositions = false;
			static_assert(sizeof(FTranscodedHairPositions) == 16u);
			static_assert(sizeof(FTranscodedHairPositions) == FCompressedHairPositionsStrideInBytes);

			auto To10bitsPosition = [TranscodingPositionOffset, TranscodingPositionScale](const FHairStrandsPositionFormat::Type& In, bool& bIsResconstructionErrorLowerThanThreshold)
			{
				const FVector3f PF = FVector3f(In.X, In.Y, In.Z);
				const FVector3f NP = (PF - TranscodingPositionOffset) / TranscodingPositionScale;

				const float bPrecisionError = 0.001f;
				check(NP.X >= 0-bPrecisionError && NP.X <= 1+bPrecisionError); 
				check(NP.Y >= 0-bPrecisionError && NP.Y <= 1+bPrecisionError); 
				check(NP.Z >= 0-bPrecisionError && NP.Z <= 1+bPrecisionError); 

				FTranscodedHairPositions::FPosition Out;
				Out.X 	= NP.X * 1023.f;
				Out.Y 	= NP.Y * 1023.f;
				Out.Z 	= NP.Z * 1023.f;
				Out.Type= In.Type;

				// Relative error
				{
					const FVector3f ReconstructedNP = FVector3f(Out.X / 1023.f, Out.Y / 1023.f, Out.Z / 1023.f);
					const float fMaxError = (ReconstructedNP - NP).GetAbsMax();

					const float ErrorThreshold = 0.01f; // 1%
					const bool bIsValid = fMaxError <= ErrorThreshold;
					if (!bIsValid) { bIsResconstructionErrorLowerThanThreshold = false; }
				}

				// Absolute error
				#if 0
				{
					const FVector3f ReconstructedPF = FVector3f(Out.X / 1023.f, Out.Y / 1023.f, Out.Z / 1023.f) * TranscodingPositionScale + TranscodingPositionOffset;
					const float fMaxError = (ReconstructedPF - PF).GetAbsMax();

					const float ErrorThreshold = 0.1f; // 1mm
					const bool bIsValid = fMaxError <= ErrorThreshold;
					if (!bIsValid) { bIsResconstructionErrorLowerThanThreshold = false; }
				}
				#endif

				return Out;
			};

			const FHairStrandsPositionFormat::Type DummInPoint = {0,0,0,0,0,0};
			const FTranscodedHairPositions::FPosition DummyOutPoint = {0,0,0,0};
			const uint32 TranscodedPositionChunkCount = FMath::DivideAndRoundUp(uint32(OutPackedPositions.Num()), TranscodedPositionChunkElementCount);
			OutTranscodedPositions.SetNum(TranscodedPositionChunkCount);
			for (uint32 ChunkIt=0;ChunkIt<TranscodedPositionChunkCount;++ChunkIt)
			{
				static_assert(TranscodedPositionChunkElementCount == 3u);
				const uint32 I0 = ChunkIt * TranscodedPositionChunkElementCount + 0;
				const uint32 I1 = ChunkIt * TranscodedPositionChunkElementCount + 1;
				const uint32 I2 = ChunkIt * TranscodedPositionChunkElementCount + 2;
	
				const FHairStrandsPositionFormat::Type P0 = OutPackedPositions.IsValidIndex(I0) ? OutPackedPositions[I0] : DummInPoint;
				const FHairStrandsPositionFormat::Type P1 = OutPackedPositions.IsValidIndex(I1) ? OutPackedPositions[I1] : DummInPoint;
				const FHairStrandsPositionFormat::Type P2 = OutPackedPositions.IsValidIndex(I2) ? OutPackedPositions[I2] : DummInPoint;
	
				bool bCanUseCompressedPosition = true;
				FTranscodedHairPositions Out;
				Out.CP0 = OutPackedPositions.IsValidIndex(I0) ? To10bitsPosition(P0, bCanUseCompressedPosition) : DummyOutPoint;
				Out.CP1 = OutPackedPositions.IsValidIndex(I1) ? To10bitsPosition(P1, bCanUseCompressedPosition) : DummyOutPoint;
				Out.CP2 = OutPackedPositions.IsValidIndex(I2) ? To10bitsPosition(P2, bCanUseCompressedPosition) : DummyOutPoint;

				if (!bCanUseCompressedPosition)
				{
					OutTranscodedPositions.SetNum(0);
					break;
				}

				const float UCoord0 = P0.UCoord / 255.f;
				const float UCoord1 = P1.UCoord / 255.f;
				const float UCoord2 = P2.UCoord / 255.f;

				const bool bInnerCurve = P0.Type == 0 && P1.Type == 0 && P2.Type == 0;
				if (bInnerCurve)
				{
					// Radius
					Out.Attribute0.Radius0 = P0.Radius;
					Out.Attribute0.Radius1 = P1.Radius;
					Out.Attribute0.Radius2 = P2.Radius;	

					// UCoord
					Out.Attribute0.UCoord0 = UCoord0 * 63.f; // 6 bits
					Out.Attribute0.UCoord2 = UCoord2 * 63.f; // 6 bits
					Out.Attribute0.Interp  = float(UCoord1 - UCoord0) / FMath::Max(0.001f, UCoord2 - UCoord0) * 3.f; // 2 bits
				}
				else
				{
					// Radius
					Out.Attribute1.Radius0 = P0.Radius;
					Out.Attribute1.Radius1 = P1.Radius;
					Out.Attribute1.Radius2 = P2.Radius;

					// UCoord
					bool bFirst = true;
					switch (P0.Type)
					{
						case HAIR_CONTROLPOINT_START : /* = 0*/ break;
						case HAIR_CONTROLPOINT_INSIDE: if (bFirst) { Out.Attribute1.UCoord0 = UCoord0 * 127u; } else { Out.Attribute1.UCoord1 = UCoord0 * 127u; } bFirst = false; break;
						case HAIR_CONTROLPOINT_END   : /* = 1*/ break;
					}

					switch (P1.Type)
					{
						case HAIR_CONTROLPOINT_START : /* = 0*/ break;
						case HAIR_CONTROLPOINT_INSIDE: if (bFirst) { Out.Attribute1.UCoord0 = UCoord1 * 127u; } else { Out.Attribute1.UCoord1 = UCoord1 * 127u; } bFirst = false; break;
						case HAIR_CONTROLPOINT_END   : /* = 1*/ break;
					}

					switch (P2.Type)
					{
						case HAIR_CONTROLPOINT_START : /* = 0*/ break;
						case HAIR_CONTROLPOINT_INSIDE: if (bFirst) { Out.Attribute1.UCoord0 = UCoord2 * 127u; } else { Out.Attribute1.UCoord1 = UCoord2 * 127u; } bFirst = false; break;
						case HAIR_CONTROLPOINT_END   : /* = 1*/ break;
					}
				}

				OutTranscodedPositions[ChunkIt] = Out;
			}
		}

		OutBulkData.Header.BoundingBox.Min = (FVector)HairStrands.BoundingBox.Min;
		OutBulkData.Header.BoundingBox.Max = (FVector)HairStrands.BoundingBox.Max;
		OutBulkData.Header.BoundingBox.IsValid = 1;
		OutBulkData.Header.CurveCount = HairStrands.GetNumCurves();
		OutBulkData.Header.PointCount = HairStrands.GetNumPoints();
		OutBulkData.Header.SourceCurveCount = HairStrands.StrandsCurves.SourceCurveCount;
		OutBulkData.Header.SourcePointCount = HairStrands.StrandsPoints.SourcePointCount;
		OutBulkData.Header.MinPointPerCurve = MinPointPerCurve;
		OutBulkData.Header.MaxPointPerCurve = MaxPointPerCurve;
		OutBulkData.Header.AvgPointPerCurve = FMath::DivideAndRoundDown(AccPointPerCurve, NumCurves);
		OutBulkData.Header.MaxLength = MaxLength;
		OutBulkData.Header.MaxRadius = MaxRadius;
		OutBulkData.Header.Flags = FHairStrandsBulkData::DataFlags_HasData;
		OutBulkData.Header.ImportedAttributes = HairStrands.GetAttributes();
		OutBulkData.Header.ImportedAttributeFlags = HairStrands.GetAttributeFlags();
		OutBulkData.Header.Transcoding.PositionOffset = FVector3f::ZeroVector;
		OutBulkData.Header.Transcoding.PositionScale = FVector3f::ZeroVector;
		OutBulkData.Header.Hash = HairStrands.GetHash();

		// Transfer group trimmed pt./curve info from group to header. This allows to display this information 
		// in the groom editor by reading the bulk data header
		if (HairStrands.HasFlags(EHairStrandsFlags::HasTrimmedCurve))
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasTrimmedCurve;
		}
		if (HairStrands.HasFlags(EHairStrandsFlags::HasTrimmedPoint))
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasTrimmedPoint;
		}
		if (HairStrands.HasFlags(EHairStrandsFlags::HasShuffledCurve))
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasShuffledCurves;
		}
		if (HairStrands.HasFlags(EHairStrandsFlags::HasExtraEndingPoint))
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasExtraEndingPoint;
		}
	
		// Transcoding
		const bool bHasTranscodedPosition = OutTranscodedPositions.Num() > 0;
		if (bHasTranscodedPosition)
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasTranscodedPosition;
			OutBulkData.Header.Transcoding.PositionOffset = TranscodingPositionOffset;
			OutBulkData.Header.Transcoding.PositionScale = TranscodingPositionScale;
		}

		const uint32 UintToByte = 4;

		const uint32 CurveAttributeChunkElementCount = 1024; // Make it asset dependent?
		const uint32 PointAttributeChunkElementCount = 8192; // Make it asset dependent?

		// Concatenate all curve-attributes
		uint32 CurveAttributeChunkStride = 0;
		{
			TArray<FHairStrandsAttributeFormat::Type> OutPackedAttributes;
			OutPackedAttributes.Reserve(FMath::DivideAndRoundUp(
				AttributeRootUV.Num() + 
				AttributeSeed.Num() +
				AttributeLength.Num() +
				AttributeClumpIDs.Num(), 4));

			for (uint32 AttributeIt=0; AttributeIt< HAIR_CURVE_ATTRIBUTE_COUNT; ++AttributeIt)
			{
				OutBulkData.Header.CurveAttributeOffsets[AttributeIt] = HAIR_ATTRIBUTE_INVALID_OFFSET;
			}

			auto AppendAttribute = [&](uint32 InAttributeIndex, uint32 InStrideInBytes, const TArray<uint8>& InData, uint32 InChunkIndex, uint32 InChunkCount)
			{
				const uint32 ElementIndex 	= InChunkIndex * CurveAttributeChunkElementCount;
				const uint32 ElementCount 	= FMath::Min(OutBulkData.Header.CurveCount - ElementIndex, CurveAttributeChunkElementCount);

				const uint32 OffsetInBytes	= ElementIndex * InStrideInBytes;
				const uint32 SizeInBytes	= ElementCount * InStrideInBytes;
				const uint32*DataInUints	= (uint32*)(InData.GetData() + OffsetInBytes);
				const uint32 SizeInUints	= FMath::DivideAndRoundUp(SizeInBytes, 4u);

				OutPackedAttributes.Append(MakeArrayView(DataInUints, SizeInUints));

				// If the attribute has several chunk, pad the last chunk with 0u, so that 
				// the attribute offset remain correct for this last chunk
				if (InChunkCount > 1 && InChunkIndex == InChunkCount-1)
				{
					const uint32 PaddingElementCount = CurveAttributeChunkElementCount - ElementCount;
					const uint32 PaddingSizeInBytes	 = PaddingElementCount * InStrideInBytes;
					const uint32 PaddingSizeInUints	 = FMath::DivideAndRoundUp(PaddingSizeInBytes, 4u);
					for (uint32 PadIt = 0; PadIt < PaddingSizeInUints; ++PadIt)
					{
						OutPackedAttributes.Add(0u);
					}
				}

				if (InChunkIndex == 0)
				{
					OutBulkData.Header.CurveAttributeOffsets[InAttributeIndex] = CurveAttributeChunkStride;
					CurveAttributeChunkStride = OutPackedAttributes.Num() * 4u;
				}
			};

			// Concatenate all curve-attribute into chunks
			// [            Chunk0            ] [            Chunk1            ] [ ... ]
			// [   RootUV     ][     Seed     ] [   RootUV     ][     Seed     ] [ ... ]
			// [C0][C1][C2][C3][C0][C1][C2][C3] [C4][C5][C6][C7][C4][C5][C6][C7] [ ... ]
			const uint32 ChunkCount = FMath::DivideAndRoundUp(NumCurves, CurveAttributeChunkElementCount);
			const uint32 ChunkElementCount = FMath::Min(CurveAttributeChunkElementCount, OutBulkData.Header.CurveCount);
			for (uint32 ChunkIt=0; ChunkIt<ChunkCount; ++ChunkIt)
			{
				if (HasHairAttribute(Attributes, EHairAttribute::RootUV))
				{
					AppendAttribute(HAIR_CURVE_ATTRIBUTE_ROOTUV, Stride_RootUV, AttributeRootUV, ChunkIt, ChunkCount);
				}
	
				{
					AppendAttribute(HAIR_CURVE_ATTRIBUTE_SEED, Stride_Seed, AttributeSeed, ChunkIt, ChunkCount);
				}
	
				{
					AppendAttribute(HAIR_CURVE_ATTRIBUTE_LENGTH, Stride_Length, AttributeLength, ChunkIt, ChunkCount);
				}
	
				if (HasHairAttribute(Attributes, EHairAttribute::ClumpID))
				{
					AppendAttribute(bHasMultipleClumpIDs ? HAIR_CURVE_ATTRIBUTE_CLUMPID3 : HAIR_CURVE_ATTRIBUTE_CLUMPID, Stride_ClumpID, AttributeClumpIDs, ChunkIt, ChunkCount);
				}
			}

			CopyToBulkData<FHairStrandsAttributeFormat>(OutBulkData.Data.CurveAttributes, OutPackedAttributes);
			ReportSize(TEXT("Curve Attributes"), OutPackedAttributes);
		}

		// Concatenate all point-attributes
		uint32 PointAttributeChunkStride = 0;
		{
			TArray<FHairStrandsAttributeFormat::Type> OutPackedAttributes;
			OutPackedAttributes.Reserve(FMath::DivideAndRoundUp(
				AttributeColor.Num() +
				AttributeRoughness.Num() +
				AttributeAO.Num(), 4));

			for (uint32 AttributeIt=0; AttributeIt< HAIR_POINT_ATTRIBUTE_COUNT; ++AttributeIt)
			{
				OutBulkData.Header.PointAttributeOffsets[AttributeIt] = HAIR_ATTRIBUTE_INVALID_OFFSET;
			}
			
			auto AppendAttribute = [&](uint32 InAttributeIndex, uint32 InStrideInBytes, const TArray<uint8>& InData, uint32 InChunkIndex, uint32 InChunkCount)
			{
				const uint32 ElementIndex 	= InChunkIndex * PointAttributeChunkElementCount;
				const uint32 ElementCount 	= FMath::Min(OutBulkData.Header.PointCount - ElementIndex, PointAttributeChunkElementCount);

				const uint32 OffsetInBytes	= ElementIndex * InStrideInBytes;
				const uint32 SizeInBytes	= ElementCount * InStrideInBytes;
				const uint32*DataInUints	= (uint32*)(InData.GetData() + OffsetInBytes);
				const uint32 SizeInUints	= FMath::DivideAndRoundUp(SizeInBytes, 4u);

				OutPackedAttributes.Append(MakeArrayView(DataInUints, SizeInUints));

				// If the attribute has several chunk, pad the last chunk with 0u, so that 
				// the attribute offset remain correct for this last chunk
				if (InChunkCount > 1 && InChunkIndex == InChunkCount-1)
				{
					const uint32 PaddingElementCount = PointAttributeChunkElementCount - ElementCount;
					const uint32 PaddingSizeInBytes	 = PaddingElementCount * InStrideInBytes;
					const uint32 PaddingSizeInUints	 = FMath::DivideAndRoundUp(PaddingSizeInBytes, 4u);
					for (uint32 PadIt = 0; PadIt < PaddingSizeInUints; ++PadIt)
					{
						OutPackedAttributes.Add(0u);
					}
				}

				if (InChunkIndex == 0)
				{
					OutBulkData.Header.PointAttributeOffsets[InAttributeIndex] = PointAttributeChunkStride;
					PointAttributeChunkStride = OutPackedAttributes.Num() * 4u;
				}
			};

			// Concatenate all point-attributes into chunks
			// [            Chunk0            ] [            Chunk1            ] [ ... ]
			// [    Color     ][      AO      ] [    Color     ][      AO      ] [ ... ]
			// [P0][P1][P2][P3][P0][P1][P2][P3] [P4][P5][P6][P7][P4][P5][P6][P7] [ ... ]
			const uint32 ChunkCount = FMath::DivideAndRoundUp(NumPoints, PointAttributeChunkElementCount);
			const uint32 ChunkElementCount = FMath::Min(PointAttributeChunkElementCount, OutBulkData.Header.PointCount);
			for (uint32 ChunkIt=0; ChunkIt<ChunkCount; ++ChunkIt)
			{
				if (HasHairAttribute(Attributes, EHairAttribute::Color))
				{
					AppendAttribute(HAIR_POINT_ATTRIBUTE_COLOR, Stride_Color, AttributeColor, ChunkIt, ChunkCount);
				}
	
				if (HasHairAttribute(Attributes, EHairAttribute::Roughness))
				{
					AppendAttribute(HAIR_POINT_ATTRIBUTE_ROUGHNESS, Stride_Roughness, AttributeRoughness, ChunkIt, ChunkCount);
				}
	
				if (HasHairAttribute(Attributes, EHairAttribute::AO))
				{
					AppendAttribute(HAIR_POINT_ATTRIBUTE_AO, Stride_AO, AttributeAO, ChunkIt, ChunkCount);
				}
			}

			const bool bHasPointAttribute = OutPackedAttributes.Num() > 0;
			if (bHasPointAttribute)
			{
				OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasPointAttribute;
				CopyToBulkData<FHairStrandsAttributeFormat>(OutBulkData.Data.PointAttributes, OutPackedAttributes);
				ReportSize(TEXT("Point Attributes"), OutPackedAttributes);
			}
		}

		if (bHasTranscodedPosition)
		{
			CopyToBulkData<FHairStrandsTranscodedPositionFormat>(OutBulkData.Data.TranscodedPositions, OutTranscodedPositions);
			ReportSize(TEXT("Positions"), OutTranscodedPositions);
		}
		else
		{
			CopyToBulkData<FHairStrandsPositionFormat>(OutBulkData.Data.Positions, OutPackedPositions);
			ReportSize(TEXT("Positions"), OutPackedPositions);
		}

		CopyToBulkData<FHairStrandsCurveFormat>(OutBulkData.Data.Curves, OutPackedCurves);
		ReportSize(TEXT("PackedCurves"), OutPackedCurves);
		CopyToBulkData<FHairStrandsPointToCurveFormat>(OutBulkData.Data.PointToCurve, OutPointToCurve);
		ReportSize(TEXT("PointToCurve"), OutPointToCurve);

		if (!OutPointsMapping.IsEmpty())
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasPointMapping;
			CopyToBulkData<FHairStrandsIndexFormat>(OutBulkData.Data.PointMapping, OutPointsMapping);
			ReportSize(TEXT("PointsMapping"), OutPointsMapping);
		}
		if (!OutCurvesMapping.IsEmpty())
		{
			OutBulkData.Header.Flags |= FHairStrandsBulkData::DataFlags_HasCurveMapping;
			CopyToBulkData<FHairStrandsIndexFormat>(OutBulkData.Data.CurveMapping, OutCurvesMapping);
			ReportSize(TEXT("CurvesMapping"), OutCurvesMapping);
		}

		// Build curve to point count mapping for runtime CLOD
		OutBulkData.Header.CurveToPointCount.Reserve(OutPackedCurves.Num());
		for (const FHairStrandsCurveFormat::Type& Curve : OutPackedCurves)
		{
			OutBulkData.Header.CurveToPointCount.Add(Curve.PointOffset + Curve.PointCount);
		}

		OutBulkData.Header.CoverageScales.Reserve(InCoverageScales.Num());
		for (const GroomBuilder_Voxelization::FCoverageScale& Scale : InCoverageScales)
		{
			OutBulkData.Header.CoverageScales.Add(Scale.CoverageScale);
		}

		// Stride datas
		OutBulkData.Header.Strides.PositionStride = FHairStrandsPositionFormat::SizeInByte;
		OutBulkData.Header.Strides.CurveStride = FHairStrandsCurveFormat::SizeInByte;

		OutBulkData.Header.Strides.PointToCurveChunkStride = FHairStrandsPointToCurveFormat::SizeInByte;
		OutBulkData.Header.Strides.PointToCurveChunkElementCount = PointToCurveChunkElementCount;

		OutBulkData.Header.Strides.CurveAttributeChunkStride = CurveAttributeChunkStride;
		OutBulkData.Header.Strides.PointAttributeChunkStride = PointAttributeChunkStride;

		OutBulkData.Header.Strides.CurveAttributeChunkElementCount = CurveAttributeChunkElementCount;
		OutBulkData.Header.Strides.PointAttributeChunkElementCount = PointAttributeChunkElementCount;

		OutBulkData.Header.Strides.TranscodedPositionChunkElementCount = TranscodedPositionChunkElementCount;
		OutBulkData.Header.Strides.TranscodedPositionChunkStride = sizeof(FTranscodedHairPositions);

		OutBulkData.Header.Strides.CurveMappingStride = FHairStrandsIndexFormat::SizeInByte;
		OutBulkData.Header.Strides.PointMappingStride = FHairStrandsIndexFormat::SizeInByte;
	}
} // namespace HairStrandsBuilder

namespace HairInterpolationBuilder
{
	struct FHairRoot
	{
		FVector3f Position;
		uint32    VertexCount;
		FVector3f Normal;
		uint32    Index;
		float     Length;
	};

	struct FHairInterpolationMetric
	{	
		// Total/combined metrics
		float Metric;

		// Debug info
		float DistanceMetric;
		float AngularMetric;
		float LengthMetric;

		float CosAngle;
		float Distance;
		float GuideLength;
		float RenderLength;

	};

	inline FHairInterpolationMetric ComputeInterpolationMetric(const FHairRoot& RenderRoot, const FHairRoot& GuideRoot)
	{
		FHairInterpolationMetric Out;
		Out.Distance = FVector3f::Dist(RenderRoot.Position, GuideRoot.Position);
		Out.CosAngle = FVector3f::DotProduct(RenderRoot.Normal, GuideRoot.Normal);
		Out.GuideLength = GuideRoot.Length;
		Out.RenderLength = RenderRoot.Length;

		// Metric takes into account the following properties to find guides which are close, share similar orientation, and 
		// have similar length for better interpolation
		// * distance
		// * orientation 
		// * length 
		const float AngularAttenuation = GHairInterpolationMetric_AngleAttenuation > 1 ? GHairInterpolationMetric_AngleAttenuation : 0;
		Out.DistanceMetric	= Out.Distance * GHairInterpolationMetric_Distance;
		Out.AngularMetric	= AngularAttenuation == 0 ? 0 : (FMath::Clamp((1 - FMath::Pow(Out.CosAngle, AngularAttenuation)), 0.f, 1.f) * GHairInterpolationMetric_Angle);
		Out.LengthMetric	= FMath::Abs(FMath::Max(Out.GuideLength / float(Out.RenderLength), Out.RenderLength / float(Out.GuideLength)) - 1) * GHairInterpolationMetric_Length; // Ratio
		Out.Metric			= Out.DistanceMetric + Out.AngularMetric + Out.LengthMetric;

		return Out;
	}

	template<uint32 NumSamples>
	inline void GetCurvePositions(const FHairStrandsDatas& CurvesDatas, const uint32 CurveIndex, TStaticArray<FVector3f,NumSamples>& Out)
	{
		const float  PointCount = CurvesDatas.StrandsCurves.CurvesCount[CurveIndex]-1.0;
		const uint32 PointOffset = CurvesDatas.StrandsCurves.CurvesOffset[CurveIndex];

		for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{	
			const float CurvePoint = static_cast<float>(SampleIndex) * PointCount / (static_cast<float>(NumSamples)-1.0f);
			const uint32 PointPrev = (SampleIndex == 0) ? 0 : (SampleIndex == (NumSamples-1)) ? PointCount - 1 : floor(CurvePoint);
			const uint32 PointNext = PointPrev + 1;	
			const float PointAlpha = CurvePoint - static_cast<float>(PointPrev);

			Out[SampleIndex] = CurvesDatas.StrandsPoints.PointsPosition[PointOffset + PointPrev]*(1.0f-PointAlpha) + CurvesDatas.StrandsPoints.PointsPosition[PointOffset + PointNext]*PointAlpha;
		}
	}
	
	template<uint32 NumSamples>
	inline float ComputeCurvesMetric(const FHairStrandsDatas& RenCurvesDatas, const uint32 RenCurveIndex, 
	                                 const FHairStrandsDatas& SimCurvesDatas, const uint32 SimCurveIndex, 
	                                 const TStaticArray<FVector3f,NumSamples>& RenSamplePoints,
	                                 const TStaticArray<FVector3f,NumSamples>& SimSamplePoints,
	                                 const float RootImportance, 
	                                 const float ShapeImportance, 
	                                 const float ProximityImportance)
	{
		const float AverageLength = FMath::Max(0.5f * (RenCurvesDatas.StrandsCurves.CurvesLength[RenCurveIndex] + SimCurvesDatas.StrandsCurves.CurvesLength[SimCurveIndex]), SMALL_NUMBER);

		static const float DeltaCoord = 1.0f / static_cast<float>(NumSamples-1);

		const FVector3f& RenRoot = RenCurvesDatas.StrandsPoints.PointsPosition[RenCurvesDatas.StrandsCurves.CurvesOffset[RenCurveIndex]];
		const FVector3f& SimRoot = SimCurvesDatas.StrandsPoints.PointsPosition[SimCurvesDatas.StrandsCurves.CurvesOffset[SimCurveIndex]];

		float CurveProximityMetric = 0.0f;
		float CurveShapeMetric = 0.0f;
		for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{
			const FVector3f& SimPosition  = SimSamplePoints[SampleIndex];
			const FVector3f& RenPosition = RenSamplePoints[SampleIndex];
			const float RootWeight = FMath::Exp(-RootImportance*SampleIndex*DeltaCoord);

			CurveProximityMetric += (SimPosition - RenPosition).Size() * RootWeight;
			CurveShapeMetric += (SimPosition - SimRoot - RenPosition + RenRoot).Size() * RootWeight;
		}
		CurveShapeMetric *= DeltaCoord / AverageLength;
		CurveProximityMetric *= DeltaCoord / AverageLength;

		return FMath::Exp(-ShapeImportance*CurveShapeMetric) * FMath::Exp(-ProximityImportance * CurveProximityMetric);
	}

	typedef TStaticArray<FVector3f,16> FCurvePositions;

	inline void PrintInterpolationMetric(const FHairInterpolationMetric& In)
	{
		UE_LOGF(LogGroomBuilder, Log, "------------------------------------------------------------------------------------------");
		UE_LOGF(LogGroomBuilder, Log, "Total Metric = %f", In.Metric);
		UE_LOGF(LogGroomBuilder, Log, "Distance     = %f (%f)", In.Distance, In.DistanceMetric);
		UE_LOGF(LogGroomBuilder, Log, "Angle        = %f (%f)", FMath::RadiansToDegrees(FMath::Acos(In.CosAngle)), In.AngularMetric);
		UE_LOGF(LogGroomBuilder, Log, "Length       = %f/%f (%f)", In.RenderLength, In.GuideLength, In.LengthMetric);
	}

	template<typename T>
	void SwapValue(T& A, T& B)
	{
		T Temp = A;
		A = B;
		B = Temp;
	}

	struct FMetrics
	{
		static const uint32 Count = HAIR_INTERPOLATION_MAX_GUIDE_COUNT;
		float KMinMetrics[Count];
		int32 KClosestGuideIndices[Count];
	};

	struct FClosestGuides
	{
		static const uint32 Count = HAIR_INTERPOLATION_MAX_GUIDE_COUNT;
		int32 Indices[Count];
	};

	// Randomize influence guide to break interpolation coherence, and create a more random/natural pattern
	static void SelectFinalGuides(
		FClosestGuides& ClosestGuides, 
		const FIntVector& RandomIndices,
		const FMetrics& InMetric, 
		const bool bRandomizeInterpolation, 
		const bool bUseUniqueGuide)
	{
		FMetrics Metric = InMetric;
		check(Metric.KClosestGuideIndices[0] >= 0);
		static_assert(HAIR_INTERPOLATION_MAX_GUIDE_COUNT == 2);

		// If some indices are invalid (for instance, found a valid single guide, fill in the rest with the valid ones)
		if (Metric.KClosestGuideIndices[1] < 0)
		{
			Metric.KClosestGuideIndices[1] = Metric.KClosestGuideIndices[0];
			Metric.KMinMetrics[1] = Metric.KMinMetrics[0];
		}

		uint32 RandIndex0 = 0;
		uint32 RandIndex1 = 1;
		if (bRandomizeInterpolation)
		{
			// This randomization makes certain strands being affected by 1, 2, or 3 guides
			RandIndex0 = RandomIndices[0];
			RandIndex1 = RandomIndices[1];
		}

		ClosestGuides.Indices[0] = Metric.KClosestGuideIndices[RandIndex0];
		ClosestGuides.Indices[1] = Metric.KClosestGuideIndices[RandIndex1];

		if (bUseUniqueGuide)
		{
			ClosestGuides.Indices[1] = Metric.KClosestGuideIndices[RandIndex0];
			RandIndex1 = RandIndex0;
		}

		float MinMetrics[FMetrics::Count];
		MinMetrics[0] = Metric.KMinMetrics[RandIndex0];
		MinMetrics[1] = Metric.KMinMetrics[RandIndex1];

		if (MinMetrics[0] > MinMetrics[1])
		{
			SwapValue(MinMetrics[0], MinMetrics[1]);
			SwapValue(ClosestGuides.Indices[0], ClosestGuides.Indices[1]);
		}
		
		// If there less than 3 valid guides, fill the rest with existing valid guides
		// This can happen due to the normal-orientation based rejection above
		if (ClosestGuides.Indices[1] < 0)
		{
			ClosestGuides.Indices[1] = ClosestGuides.Indices[0];
			MinMetrics[1] = MinMetrics[0];
		}

		check(MinMetrics[0] <= MinMetrics[1]);
	}

	// Simple closest distance metric
	static void ComputeSimpleMetric(
		FMetrics& Metrics1, 
		const FHairRoot& RenRoot, 
		const FHairRoot& GuideRoot, 
		const int32 RenCurveIndex,
		const int32 SimCurveIndex)
	{
		const float Metric = FVector3f::Dist(GuideRoot.Position, RenRoot.Position);
		if (Metric < Metrics1.KMinMetrics[FMetrics::Count - 1])
		{
			int32 LastGuideIndex = SimCurveIndex;
			float LastMetric = Metric;
			for (uint32 Index = 0; Index < FMetrics::Count; ++Index)
			{
				if (Metric < Metrics1.KMinMetrics[Index])
				{
					SwapValue(Metrics1.KClosestGuideIndices[Index], LastGuideIndex);
					SwapValue(Metrics1.KMinMetrics[Index], LastMetric);
				}
			}
		}
	}

	// Complex pairing metric
	static void ComputeAdvandedMetric(FMetrics& Metrics0,
		const FHairStrandsDatas& RenStrandsData,
		const FHairStrandsDatas& SimStrandsData,
		const int32 RenCurveIndex,
		const int32 SimCurveIndex,
		const FCurvePositions& RenCurvePositions)
	{
		// Sample sim strands at fix rate posisions (similarly is done render strands)
		FCurvePositions SimCurvePositions;
		GetCurvePositions(SimStrandsData, SimCurveIndex, SimCurvePositions);

		const float Metric = 1.0 - ComputeCurvesMetric<16>(RenStrandsData, RenCurveIndex, SimStrandsData, SimCurveIndex, RenCurvePositions, SimCurvePositions, 0.0f, 1.0f, 1.0f);
		if (Metric < Metrics0.KMinMetrics[FMetrics::Count - 1])
		{
			int32 LastGuideIndex = SimCurveIndex;
			float LastMetric = Metric;
			for (uint32 Index = 0; Index < FMetrics::Count; ++Index)
			{
				if (Metric < Metrics0.KMinMetrics[Index])
				{
					SwapValue(Metrics0.KClosestGuideIndices[Index], LastGuideIndex);
					SwapValue(Metrics0.KMinMetrics[Index], LastMetric);
				}
			}
		}
	}

	struct FRootsGrid
	{
		FVector3f MinBound = FVector3f::ZeroVector;
		FVector3f MaxBound = FVector3f::ZeroVector;
		
		const uint32 MaxLookupDistance = 31;
		const FIntVector GridResolution = FIntVector(32, 32, 32);

		TArray<int32> GridIndirection;
		TArray<TArray<int32>> RootIndices;
		
		FORCEINLINE bool IsValid(const FIntVector& P) const
		{
			return	0 <= P.X && P.X < GridResolution.X &&
					0 <= P.Y && P.Y < GridResolution.Y &&
					0 <= P.Z && P.Z < GridResolution.Z;
		}

		FORCEINLINE FIntVector ClampToVolume(const FIntVector& CellCoord, bool& bIsValid) const
		{
			bIsValid = IsValid(CellCoord);
			return FIntVector(
				FMath::Clamp(CellCoord.X, 0, GridResolution.X - 1),
				FMath::Clamp(CellCoord.Y, 0, GridResolution.Y - 1),
				FMath::Clamp(CellCoord.Z, 0, GridResolution.Z - 1));
		}

		FORCEINLINE FIntVector ToCellCoord(const FVector3f& P) const
		{
			bool bIsValid = false;
			const FVector3f F = ((P - MinBound) / (MaxBound - MinBound));
			const FIntVector CellCoord = FIntVector(FMath::FloorToInt(F.X * GridResolution.X), FMath::FloorToInt(F.Y * GridResolution.Y), FMath::FloorToInt(F.Z * GridResolution.Z));			
			return ClampToVolume(CellCoord, bIsValid);
		}

		uint32 ToIndex(const FIntVector& CellCoord) const
		{
			uint32 CellIndex = CellCoord.X + CellCoord.Y * GridResolution.X + CellCoord.Z * GridResolution.X * GridResolution.Y;
			check(CellIndex < uint32(GridIndirection.Num()));
			return CellIndex;
		}

		void InsertRoots(TArray<FHairRoot>& Roots, const FVector3f& InMinBound, const FVector3f& InMaxBound)
		{
			MinBound = InMinBound;
			MaxBound = InMaxBound;
			GridIndirection.SetNumZeroed(GridResolution.X*GridResolution.Y*GridResolution.Z);
			RootIndices.Empty();
			RootIndices.AddDefaulted(); // Add a default empty list for the null element

			const uint32 RootCount = Roots.Num();
			for (uint32 RootIt = 0; RootIt < RootCount; ++RootIt)
			{
				const FHairRoot& Root = Roots[RootIt];
				const FIntVector CellCoord = ToCellCoord(Root.Position);
				const uint32 Index = ToIndex(CellCoord);
				if (GridIndirection[Index] == 0)
				{
					GridIndirection[Index] = RootIndices.Num();
					RootIndices.AddDefaulted();
				}
				
				TArray<int32>& CellGuideIndices = RootIndices[GridIndirection[Index]];
				CellGuideIndices.Add(RootIt);
			}
		}

		FORCEINLINE void SearchCell(
			const FIntVector& CellCoord,
			const uint32 RenCurveIndex,
			const FHairRoot& RenRoot,
			const TArray<FHairRoot>& SimRoots,
			FMetrics& Metrics) const 
		{
			const uint32 CellIndex = ToIndex(CellCoord);

			if (GridIndirection[CellIndex] == 0)
				return;

			const TArray<int32>& Elements = RootIndices[GridIndirection[CellIndex]];

			for (int32 SimCurveIndex : Elements)
			{
				const FHairRoot& GuideRoot = SimRoots[SimCurveIndex];
				{
					ComputeSimpleMetric(Metrics, RenRoot, GuideRoot, RenCurveIndex, SimCurveIndex);
				}
			}
		}

		FClosestGuides FindClosestRoots(
			const uint32 RenCurveIndex,
			const TArray<FHairRoot>& RenRoots,
			const TArray<FHairRoot>& SimRoots,
			const FHairStrandsDatas& RenStrandsData,
			const FHairStrandsDatas& SimStrandsData,
			const bool bRandomized,
			const bool bUnique,
			const FIntVector& RandomIndices) const
		{
			const FHairRoot& RenRoot = RenRoots[RenCurveIndex];
			const FIntVector PointCoord = ToCellCoord(RenRoot.Position);

			FMetrics Metrics;
			for (uint32 ClearIt = 0; ClearIt < FMetrics::Count; ++ClearIt)
			{
				Metrics.KMinMetrics[ClearIt] = FLT_MAX;
				Metrics.KClosestGuideIndices[ClearIt] = -1;
			}

			for (int32 Offset = 1; Offset <= int32(MaxLookupDistance); ++Offset)
			{
				// Center
				{
					bool bIsValid = false;
					const FIntVector CellCoord = ClampToVolume(PointCoord, bIsValid);
					if (bIsValid) SearchCell(PointCoord, RenCurveIndex, RenRoot, SimRoots, Metrics);
				}

				// Top & Bottom
				for (int32 X = -Offset; X <= Offset; ++X)
				for (int32 Y = -Offset; Y <= Offset; ++Y)
				{
					bool bIsValid0 = false, bIsValid1 = false;
					const FIntVector CellCoord0 = ClampToVolume(PointCoord + FIntVector(X, Y, Offset), bIsValid0);
					const FIntVector CellCoord1 = ClampToVolume(PointCoord + FIntVector(X, Y,-Offset), bIsValid1);
					if (bIsValid0) SearchCell(CellCoord0, RenCurveIndex, RenRoot, SimRoots, Metrics);
					if (bIsValid1) SearchCell(CellCoord1, RenCurveIndex, RenRoot, SimRoots, Metrics);
				}

				const int32 OffsetMinusOne = Offset - 1;
				// Front & Back
				for (int32 X = -Offset; X <= Offset; ++X)
				for (int32 Z = -OffsetMinusOne; Z <= OffsetMinusOne; ++Z)
				{
					bool bIsValid0 = false, bIsValid1 = false;
					const FIntVector CellCoord0 = ClampToVolume(PointCoord + FIntVector(X,  Offset, Z), bIsValid0);
					const FIntVector CellCoord1 = ClampToVolume(PointCoord + FIntVector(X, -Offset, Z), bIsValid1);
					if (bIsValid0) SearchCell(CellCoord0, RenCurveIndex, RenRoot, SimRoots, Metrics);
					if (bIsValid1) SearchCell(CellCoord1, RenCurveIndex, RenRoot, SimRoots, Metrics);
				}
				
				// Left & Right
				for (int32 Y = -OffsetMinusOne; Y <= OffsetMinusOne; ++Y)
				for (int32 Z = -OffsetMinusOne; Z <= OffsetMinusOne; ++Z)
				{
					bool bIsValid0 = false, bIsValid1 = false;
					const FIntVector CellCoord0 = ClampToVolume(PointCoord + FIntVector( Offset, Y, Z), bIsValid0);
					const FIntVector CellCoord1 = ClampToVolume(PointCoord + FIntVector(-Offset, Y, Z), bIsValid1);
					if (bIsValid0) SearchCell(CellCoord0, RenCurveIndex, RenRoot, SimRoots, Metrics);
					if (bIsValid1) SearchCell(CellCoord1, RenCurveIndex, RenRoot, SimRoots, Metrics);
				}

				// Early out if we have found all closest guide during a ring/layer step.
				// This early out is not conservative, as the complex metric might find better guides one or multiple step further.
				if (Metrics.KClosestGuideIndices[FMetrics::Count-1] != -1 && Offset >= 2)
				{
					break;
				}
			}

			// If no valid guide have been found, switch to a simpler metric
			FClosestGuides ClosestGuides;
			SelectFinalGuides(ClosestGuides, RandomIndices, Metrics, bRandomized, bUnique);

			check(ClosestGuides.Indices[0] >= 0);
			check(ClosestGuides.Indices[1] >= 0);

			return ClosestGuides;
		}


		FORCEINLINE void SearchCell(
			const FIntVector& CellCoord,
			const uint32 RenCurveIndex,
			const FHairRoot& RenRoot,
			const TArray<FHairRoot>& SimRoots,
			const FHairStrandsDatas& RenStrandsData,
			const FHairStrandsDatas& SimStrandsData,
			const FCurvePositions& RenCurvePositions,
			FMetrics& Metrics0,
			FMetrics& Metrics1) const
		{
			const uint32 CellIndex = ToIndex(CellCoord);

			if (GridIndirection[CellIndex] == 0)
				return;

			const TArray<int32>& Elements = RootIndices[GridIndirection[CellIndex]];

			for (int32 SimCurveIndex : Elements)
			{
				const FHairRoot& GuideRoot = SimRoots[SimCurveIndex];
				{
					ComputeSimpleMetric(Metrics1, RenRoot, GuideRoot, RenCurveIndex, SimCurveIndex);
					ComputeAdvandedMetric(Metrics0, RenStrandsData, SimStrandsData, RenCurveIndex, SimCurveIndex, RenCurvePositions);
				}
			}
		}

		FClosestGuides FindBestClosestRoots(
			const uint32 RenCurveIndex,
			const TArray<FHairRoot>& RenRoots,
			const TArray<FHairRoot>& SimRoots,
			const FHairStrandsDatas& RenStrandsData,
			const FHairStrandsDatas& SimStrandsData,
			const FCurvePositions& RenCurvePositions,
			const bool bRandomized,
			const bool bUnique,
			const FIntVector& RandomIndices) const
		{
			const FHairRoot& RenRoot = RenRoots[RenCurveIndex];
			const FIntVector PointCoord = ToCellCoord(RenRoot.Position);

			FMetrics Metrics0;
			FMetrics Metrics1;
			for (uint32 ClearIt = 0; ClearIt < FMetrics::Count; ++ClearIt)
			{
				Metrics0.KMinMetrics[ClearIt] = FLT_MAX;
				Metrics0.KClosestGuideIndices[ClearIt] = -1;
				Metrics1.KMinMetrics[ClearIt] = FLT_MAX;
				Metrics1.KClosestGuideIndices[ClearIt] = -1;
			}

			for (int32 Offset = 1; Offset <= int32(MaxLookupDistance); ++Offset)
			{
				// Center
				{
					bool bIsValid = false;
					const FIntVector CellCoord = ClampToVolume(PointCoord, bIsValid);
					if (bIsValid) SearchCell(CellCoord, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
				}

				// Top & Bottom
				for (int32 X = -Offset; X <= Offset; ++X)
				for (int32 Y = -Offset; Y <= Offset; ++Y)
				{
					bool bIsValid0 = false, bIsValid1 = false;
					const FIntVector CellCoord0 = ClampToVolume(PointCoord + FIntVector(X, Y, Offset), bIsValid0);
					const FIntVector CellCoord1 = ClampToVolume(PointCoord + FIntVector(X, Y,-Offset), bIsValid1);
					if (bIsValid0) SearchCell(CellCoord0, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
					if (bIsValid1) SearchCell(CellCoord1, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
				}

				const int32 OffsetMinusOne = Offset - 1;
				// Front & Back
				for (int32 X = -Offset; X <= Offset; ++X)
				for (int32 Z = -OffsetMinusOne; Z <= OffsetMinusOne; ++Z)
				{
					bool bIsValid0 = false, bIsValid1 = false;
					const FIntVector CellCoord0 = ClampToVolume(PointCoord + FIntVector(X, Offset, Z), bIsValid0);
					const FIntVector CellCoord1 = ClampToVolume(PointCoord + FIntVector(X, -Offset, Z), bIsValid1);
					if (bIsValid0) SearchCell(CellCoord0, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
					if (bIsValid1) SearchCell(CellCoord1, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
				}
				
				// Left & Right
				for (int32 Y = -OffsetMinusOne; Y <= OffsetMinusOne; ++Y)
				for (int32 Z = -OffsetMinusOne; Z <= OffsetMinusOne; ++Z)
				{
					bool bIsValid0 = false, bIsValid1 = false;
					const FIntVector CellCoord0 = ClampToVolume(PointCoord + FIntVector( Offset, Y, Z), bIsValid0);
					const FIntVector CellCoord1 = ClampToVolume(PointCoord + FIntVector(-Offset, Y, Z), bIsValid1);
					if (bIsValid0) SearchCell(CellCoord0, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
					if (bIsValid1) SearchCell(CellCoord1, RenCurveIndex, RenRoot, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Metrics0, Metrics1);
				}

				// Early out if we have found all closest guide during a ring/layer step.
				// This early out is not conservative, as the complex metric might find better guides one or multiple step further.
				if ((Metrics0.KClosestGuideIndices[FMetrics::Count-1] != -1 || Metrics1.KClosestGuideIndices[FMetrics::Count - 1] != -1) && Offset >= 2)
				{
					break;
				}
			}

			// If no valid guide have been found, switch to a simpler metric
			FClosestGuides ClosestGuides;
			if (Metrics0.KClosestGuideIndices[0] != -1)
			{
				SelectFinalGuides(ClosestGuides, RandomIndices, Metrics0, bRandomized, bUnique);
			}
			else
			{
				SelectFinalGuides(ClosestGuides, RandomIndices, Metrics1, bRandomized, bUnique);
			}

			check(ClosestGuides.Indices[0] >= 0);
			check(ClosestGuides.Indices[1] >= 0);

			return ClosestGuides;
		}
	};

	static FClosestGuides FindBestRoots(
		const uint32 RenCurveIndex,
		const TArray<FHairRoot>& RenRoots,
		const TArray<FHairRoot>& SimRoots,
		const FHairStrandsDatas& RenStrandsData,
		const FHairStrandsDatas& SimStrandsData,
		const FCurvePositions& RenCurvePositions,
		const bool bRandomized,
		const bool bUnique,
		const FIntVector& RandomIndices)
	{
		FMetrics Metrics;
		for (uint32 ClearIt = 0; ClearIt < FMetrics::Count; ++ClearIt)
		{
			Metrics.KMinMetrics[ClearIt] = FLT_MAX;
			Metrics.KClosestGuideIndices[ClearIt] = -1;
		}

		const uint32 SimRootsCount = SimRoots.Num();
		for (uint32 SimCurveIndex =0; SimCurveIndex<SimRootsCount; ++SimCurveIndex)
		{
			ComputeAdvandedMetric(Metrics, RenStrandsData, SimStrandsData, RenCurveIndex, SimCurveIndex, RenCurvePositions);
		}
			
		FClosestGuides ClosestGuides;
		SelectFinalGuides(ClosestGuides, RandomIndices, Metrics, bRandomized, bUnique);

		check(ClosestGuides.Indices[0] >= 0);
		check(ClosestGuides.Indices[1] >= 0);

		return ClosestGuides;
	}

	// Extract strand roots
	static void ExtractRoots(const FHairStrandsDatas& InData, TArray<FHairRoot>& OutRoots, FVector3f& MinBound, FVector3f& MaxBound)
	{
		MinBound = FVector3f(FLT_MAX, FLT_MAX, FLT_MAX);
		MaxBound = FVector3f(-FLT_MAX, -FLT_MAX, -FLT_MAX);
		const uint32 CurveCount = InData.StrandsCurves.Num();
		OutRoots.Reserve(CurveCount);
		for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
		{
			const uint32 PointOffset = InData.StrandsCurves.CurvesOffset[CurveIndex];
			const uint32 PointCount = InData.StrandsCurves.CurvesCount[CurveIndex];
			const float  CurveLength = InData.StrandsCurves.CurvesLength[CurveIndex];
			check(PointCount > 1);
			const FVector3f& P0 = InData.StrandsPoints.PointsPosition[PointOffset];
			const FVector3f& P1 = InData.StrandsPoints.PointsPosition[PointOffset + 1];
			FVector3f N = (P1 - P0).GetSafeNormal();

			// Fallback in case the initial points are too close (this happens on certain assets)
			if (FVector3f::DotProduct(N, N) == 0)
			{
				N = FVector3f(0, 0, 1);
			}
			OutRoots.Add({ P0, PointCount, N, PointOffset, CurveLength });

			MinBound = MinBound.ComponentMin(P0);
			MaxBound = MaxBound.ComponentMax(P0);
		}
	}

	struct FVertexInterpolationDesc
	{
		uint32 Index0 = 0;
		uint32 Index1 = 0;
		float T = 0;
	};

	// Find the vertex along a sim curve 'SimCurveIndex', which has the same parametric distance than the render distance 'RenPointDistance'
	static FVertexInterpolationDesc FindMatchingVertex(const float RenPointDistance, const FHairStrandsDatas& SimStrandsData, const uint32 SimCurveIndex)
	{
		const uint32 SimOffset = SimStrandsData.StrandsCurves.CurvesOffset[SimCurveIndex];

		const float CurveLength = SimStrandsData.StrandsCurves.CurvesLength[SimCurveIndex];

		// Find with with vertex the vertex should be paired
		const uint32 SimPointCount = SimStrandsData.StrandsCurves.CurvesCount[SimCurveIndex];
		for (uint32 SimPointIndex = 0; SimPointIndex < SimPointCount-1; ++SimPointIndex)
		{
			const float SimPointDistance0 = SimStrandsData.StrandsPoints.PointsCoordU[SimPointIndex + SimOffset] * CurveLength;
			const float SimPointDistance1 = SimStrandsData.StrandsPoints.PointsCoordU[SimPointIndex + SimOffset + 1] * CurveLength;
			if (SimPointDistance0 <= RenPointDistance && RenPointDistance <= SimPointDistance1)
			{
				const float SegmentLength = SimPointDistance1 - SimPointDistance0;
				FVertexInterpolationDesc Out;
				Out.Index0 = SimPointIndex;
				Out.Index1 = SimPointIndex+1;
				Out.T = (RenPointDistance - SimPointDistance0) / (SegmentLength>0? SegmentLength : 1);
				Out.T = FMath::Clamp(Out.T, 0.f, 1.f);
				return Out;
			}
		}
		FVertexInterpolationDesc Desc;
		Desc.Index0 = SimPointCount-2;
		Desc.Index1 = SimPointCount-1;
		Desc.T = 1;
		return Desc;
	}

	// Find the vertex along a sim curve 'SimCurveIndex', which has the smallest euclidean distance than the render vertex
	static FVertexInterpolationDesc FindMatchingVertex(const FVector3f& RenPointPosition, const FHairStrandsDatas& SimStrandsData, const uint32 SimCurveIndex)
	{
		const uint32 SimOffset = SimStrandsData.StrandsCurves.CurvesOffset[SimCurveIndex];
		float MinPointDistance = FLT_MAX;
		int32 MinPointIndex = 0;

		// Find with with vertex the vertex should be paired
		const uint32 SimPointCount = SimStrandsData.StrandsCurves.CurvesCount[SimCurveIndex];
		for (uint32 SimPointIndex = 0; SimPointIndex < SimPointCount-1; ++SimPointIndex)
		{
			
			const float SimRenDistance = (SimStrandsData.StrandsPoints.PointsPosition[SimPointIndex + SimOffset] - RenPointPosition).Size();
			if (SimRenDistance <= MinPointDistance)
			{
				MinPointDistance = SimRenDistance;
				MinPointIndex = SimPointIndex;
			}
		}
		FVertexInterpolationDesc Desc;
		const FVector3f EdgeDirection = (SimStrandsData.StrandsPoints.PointsPosition[MinPointIndex +1+ SimOffset] - SimStrandsData.StrandsPoints.PointsPosition[MinPointIndex + SimOffset]).GetSafeNormal();
		if(((RenPointPosition- SimStrandsData.StrandsPoints.PointsPosition[MinPointIndex + SimOffset]).Dot(EdgeDirection) > 0.0) || (MinPointIndex == 0))
		{
			Desc.Index0 = MinPointIndex;
			Desc.Index1 = MinPointIndex + 1;
		}
		else
		{
			Desc.Index0 = MinPointIndex-1;
			Desc.Index1 = MinPointIndex;
		}
		
		const float SimRenDistance0 = (SimStrandsData.StrandsPoints.PointsPosition[Desc.Index0 + SimOffset] - RenPointPosition).Size();
		const float SimRenDistance1 = (SimStrandsData.StrandsPoints.PointsPosition[Desc.Index1 + SimOffset] - RenPointPosition).Size();
		
		Desc.T = SimRenDistance0 / (SimRenDistance0+SimRenDistance1);
		return Desc;
	}

	static void BuildInterpolationData(
		FHairStrandsInterpolationDatas& InterpolationData,
		const FHairStrandsDatas& SimStrandsData,
		const FHairStrandsDatas& RenStrandsData,
		const FHairInterpolationSettings& Settings,
		const TArray<FIntVector>& RandomGuideIndices)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HairInterpolationBuilder::BuildInterpolationData);

		InterpolationData.SetNum(RenStrandsData.GetNumCurves(), RenStrandsData.GetNumPoints());
		InterpolationData.bUseUniqueGuide = Settings.bUseUniqueGuide;

		typedef TArray<FHairRoot> FRoots;

		// Build acceleration structure for fast nearest-neighbors lookup.
		// This is used only for low quality interpolation as high quality 
		// interpolation require broader search
		FRoots RenRoots, SimRoots;
		FRootsGrid RootsGrid;
		{
			FVector3f RenMinBound, RenMaxBound;
			FVector3f SimMinBound, SimMaxBound;
			ExtractRoots(RenStrandsData, RenRoots, RenMinBound, RenMaxBound);
			ExtractRoots(SimStrandsData, SimRoots, SimMinBound, SimMaxBound);

			if (Settings.InterpolationQuality == EHairInterpolationQuality::Low || Settings.InterpolationQuality == EHairInterpolationQuality::Medium)
			{
				// Build a conservative bound, to insure all queries will fall 
				// into the grid volume.
				const FVector3f MinBound = RenMinBound.ComponentMin(SimMinBound);
				const FVector3f MaxBound = RenMaxBound.ComponentMax(SimMaxBound);
				RootsGrid.InsertRoots(SimRoots, MinBound, MaxBound);
			}
		}

		// Find k-closest guide:
		uint32 TotalInvalidInterpolationCount = 0;
		const static float MinWeightDistance = 0.0001f;

		const uint32 RenCurveCount = RenStrandsData.GetNumCurves();
		const uint32 SimCurveCount = SimStrandsData.GetNumCurves();

		TAtomic<uint32> CompletedTasks(0);
		FScopedSlowTask SlowTask(RenCurveCount, LOCTEXT("BuildInterpolationData", "Building groom simulation data"));
		SlowTask.MakeDialog();

		const FDateTime StartTime = FDateTime::UtcNow();
		ParallelFor(RenCurveCount, 
		[
			StartTime,
			Settings,
			RenCurveCount, &RenRoots, &RenStrandsData,
			SimCurveCount, &SimRoots, &SimStrandsData, 
			&RootsGrid,
			&TotalInvalidInterpolationCount,  
			&InterpolationData, 
			&RandomGuideIndices,
			&CompletedTasks,
			&SlowTask
		] (uint32 RenCurveIndex) 
		//for (uint32 RenCurveIndex = 0; RenCurveIndex < RenCurveCount; ++RenCurveIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(HairInterpolationBuilder::ComputingClosestGuidesAndWeights);

			++CompletedTasks;

			if (IsInGameThread())
			{
				const uint32 CurrentCompletedTasks = CompletedTasks.Exchange(0);
				const float RemainingTasks = FMath::Clamp(SlowTask.TotalAmountOfWork - SlowTask.CompletedWork, 1.f, SlowTask.TotalAmountOfWork);
				const FTimespan ElaspedTime = FDateTime::UtcNow() - StartTime;
				const double RemainingTimeInSeconds = RemainingTasks * double(ElaspedTime.GetSeconds() / SlowTask.CompletedWork);

				FTextBuilder TextBuilder;
				TextBuilder.AppendLineFormat(LOCTEXT("ComputeGuidesAndWeights", "Computing closest guides and weights ({0})"), FText::AsTimespan(FTimespan::FromSeconds(RemainingTimeInSeconds)));
				SlowTask.EnterProgressFrame(CurrentCompletedTasks, TextBuilder.ToText());
			}

			const FHairRoot& StrandRoot = RenRoots[RenCurveIndex];

			FClosestGuides ClosestGuides;
			if (Settings.InterpolationQuality == EHairInterpolationQuality::Low)
			{
				ClosestGuides = RootsGrid.FindClosestRoots(RenCurveIndex, RenRoots, SimRoots, RenStrandsData, SimStrandsData, Settings.bRandomizeGuide, Settings.bUseUniqueGuide, RandomGuideIndices[RenCurveIndex]);
			}
			else if (Settings.InterpolationQuality == EHairInterpolationQuality::Medium)
			{
				FCurvePositions RenCurvePositions; 
				GetCurvePositions(RenStrandsData, RenCurveIndex, RenCurvePositions);
				ClosestGuides = RootsGrid.FindBestClosestRoots(RenCurveIndex, RenRoots, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Settings.bRandomizeGuide, Settings.bUseUniqueGuide, RandomGuideIndices[RenCurveIndex]);
			}
			else // (Settings.Quality == EHairInterpolationQuality::High)
			{
				FCurvePositions RenCurvePositions; 
				GetCurvePositions(RenStrandsData, RenCurveIndex, RenCurvePositions);
				ClosestGuides = FindBestRoots(RenCurveIndex, RenRoots, SimRoots, RenStrandsData, SimStrandsData, RenCurvePositions, Settings.bRandomizeGuide, Settings.bUseUniqueGuide, RandomGuideIndices[RenCurveIndex]);
			}

			const uint32 RendPointCount	= RenStrandsData.StrandsCurves.CurvesCount[RenCurveIndex];
			const uint32 RenOffset		= RenStrandsData.StrandsCurves.CurvesOffset[RenCurveIndex];
			const FVector3f& RenPointPositionRoot = RenStrandsData.StrandsPoints.PointsPosition[RenOffset];

			// Set per-curve index
			float LocalCurveSimWeights[FClosestGuides::Count]; 
			for (uint32 KIndex = 0; KIndex < FClosestGuides::Count; ++KIndex)
			{
				const uint32 SimCurveIndex = ClosestGuides.Indices[KIndex];
				InterpolationData.CurveSimIndices[RenCurveIndex][KIndex] = SimCurveIndex;
				LocalCurveSimWeights[KIndex] = 0;

				const uint32 SimOffset = SimStrandsData.StrandsCurves.CurvesOffset[SimCurveIndex];
				InterpolationData.CurveSimRootPointIndex[RenCurveIndex][KIndex] = SimOffset;
			}

			// Set per-point 'local' point index and lerp value
			for (uint32 RenPointIndex = 0; RenPointIndex < RendPointCount; ++RenPointIndex)
			{
				const uint32 PointGlobalIndex = RenPointIndex + RenOffset;
				const FVector3f& RenPointPosition = RenStrandsData.StrandsPoints.PointsPosition[PointGlobalIndex];
				const float RenPointDistance = RenStrandsData.StrandsPoints.PointsCoordU[PointGlobalIndex] * RenStrandsData.StrandsCurves.CurvesLength[RenCurveIndex];

				for (uint32 KIndex = 0; KIndex < FClosestGuides::Count; ++KIndex)
				{
					// Find the closest vertex on the guide which matches the strand vertex distance along its curve
					if (Settings.InterpolationDistance == EHairInterpolationWeight::Parametric || Settings.InterpolationDistance == EHairInterpolationWeight::Distance)
					{
						const uint32 SimCurveIndex = ClosestGuides.Indices[KIndex];
						const uint32 SimOffset = SimStrandsData.StrandsCurves.CurvesOffset[SimCurveIndex];
						const FVertexInterpolationDesc Desc =  (Settings.InterpolationDistance == EHairInterpolationWeight::Parametric) ?
							FindMatchingVertex(RenPointDistance, SimStrandsData, SimCurveIndex) :
							FindMatchingVertex(RenStrandsData.StrandsPoints.PointsPosition[PointGlobalIndex], SimStrandsData, SimCurveIndex);
						
						const FVector3f& SimPointPosition0 = SimStrandsData.StrandsPoints.PointsPosition[Desc.Index0 + SimOffset];
						const FVector3f& SimPointPosition1 = SimStrandsData.StrandsPoints.PointsPosition[Desc.Index1 + SimOffset];
						const float Weight = 1.0f / FMath::Max(MinWeightDistance, FVector3f::Distance(RenPointPosition, FMath::LerpStable(SimPointPosition0, SimPointPosition1, Desc.T)));

						InterpolationData.PointSimIndices[PointGlobalIndex][KIndex] = Desc.Index0/* + SimOffset*/;
						InterpolationData.PointSimLerps[PointGlobalIndex][KIndex] = Desc.T;
						LocalCurveSimWeights[KIndex] += Weight;
					}

					// Use only the root as a *constant* weight for deformation along each vertex
					// Still compute the closest vertex (in parametric distance) to know on which vertex the offset/delta should be computed
					if (Settings.InterpolationDistance == EHairInterpolationWeight::Root)
					{
						const uint32 SimCurveIndex = ClosestGuides.Indices[KIndex];
						const uint32 SimOffset = SimStrandsData.StrandsCurves.CurvesOffset[SimCurveIndex];
						const FVector3f& SimRootPointPosition = SimStrandsData.StrandsPoints.PointsPosition[SimOffset];
						const FVector3f& RenRootPointPosition = RenStrandsData.StrandsPoints.PointsPosition[RenOffset];
						const float Weight = 1.0f / FMath::Max(MinWeightDistance, FVector3f::Distance(RenRootPointPosition, SimRootPointPosition));
						const FVertexInterpolationDesc Desc = FindMatchingVertex(RenPointDistance, SimStrandsData, SimCurveIndex);

						InterpolationData.PointSimIndices[PointGlobalIndex][KIndex] = Desc.Index0/* + SimOffset*/;
						InterpolationData.PointSimLerps[PointGlobalIndex][KIndex] = Desc.T;
						LocalCurveSimWeights[KIndex] += Weight;
					}

					// Use the *same vertex index* to match guide vertex with strand vertex
					if (Settings.InterpolationDistance == EHairInterpolationWeight::Index)
					{
						const uint32 SimCurveIndex = ClosestGuides.Indices[KIndex];
						const uint32 SimOffset = SimStrandsData.StrandsCurves.CurvesOffset[SimCurveIndex];
						const uint32 SimPointCount = SimStrandsData.StrandsCurves.CurvesCount[SimCurveIndex];
						const uint32 SimPointIndex = FMath::Clamp(RenPointIndex, 0u, SimPointCount - 1);
						const FVector3f& SimPointPosition = SimStrandsData.StrandsPoints.PointsPosition[SimPointIndex + SimOffset];
						const float Weight = 1.0f / FMath::Max(MinWeightDistance, FVector3f::Distance(RenPointPosition, SimPointPosition));

						InterpolationData.PointSimIndices[PointGlobalIndex][KIndex] = SimPointIndex/* + SimOffset*/;
						InterpolationData.PointSimLerps[PointGlobalIndex][KIndex] = 1;
						LocalCurveSimWeights[KIndex] += Weight;
					}
				}

			}

			// Set per-curve weights
			float TotalWeight = 0;
			for (int32 KIndex = 0; KIndex < FClosestGuides::Count; ++KIndex)
			{
				TotalWeight += LocalCurveSimWeights[KIndex];
			}
			for (int32 KIndex = 0; KIndex < FClosestGuides::Count; ++KIndex)
			{
				InterpolationData.CurveSimWeights[RenCurveIndex][KIndex] = LocalCurveSimWeights[KIndex] / FMath::Max(0.001f, TotalWeight);
			}
		});
	}

	/** Build data for interpolation between simulation and rendering */
	void BuildInterplationBulkData(
		const FHairStrandsDatas& SimDatas, 
		const FHairStrandsInterpolationDatas& HairInterpolation, 
		FHairStrandsInterpolationBulkData& OutBulkData)
	{
		OutBulkData.Reset();

		TRACE_CPUPROFILER_EVENT_SCOPE(HairInterpolationBuilder::BuildInterplationBulkData);

		const uint32 PointCount = HairInterpolation.GetPointCount();
		const uint32 CurveCount = HairInterpolation.GetCurveCount();
		if (PointCount == 0 || CurveCount == 0)
		{
			return;
		}

		OutBulkData.Header.Flags = FHairStrandsInterpolationBulkData::DataFlags_HasData;
		OutBulkData.Header.PointCount = PointCount;
		OutBulkData.Header.CurveCount = CurveCount;
		OutBulkData.Header.Strides.CurveInterpolationStride = 0;
		OutBulkData.Header.Strides.PointInterpolationStride = 0;

		struct FHairInterpolationCurve
		{
			uint32 CurveIndex : 24;
			uint32 CurveWeight : 8;
			uint32 PointIndex;
		};

		struct FHairInterpolationPoint
		{
			uint8 VertexIndex;
			uint8 VertexLerp;
		};

		static_assert(sizeof(FHairInterpolationCurve) == HAIR_INTERPOLATION_CURVE_STRIDE);
		static_assert(sizeof(FHairInterpolationPoint) == HAIR_INTERPOLATION_POINT_STRIDE);
		static_assert(HAIR_INTERPOLATION_MAX_GUIDE_COUNT == 2u);

		// Header
		const uint32 KCount = HairInterpolation.bUseUniqueGuide ? 1u : HAIR_INTERPOLATION_MAX_GUIDE_COUNT;
		if (HairInterpolation.bUseUniqueGuide)
		{
			OutBulkData.Header.Flags |= FHairStrandsInterpolationBulkData::DataFlags_HasSingleGuideData;
		}
		OutBulkData.Header.Strides.CurveInterpolationStride = sizeof(FHairInterpolationCurve) * KCount;
		OutBulkData.Header.Strides.PointInterpolationStride = sizeof(FHairInterpolationPoint) * KCount;
		
		// Data
		{
			// Fill curve data
			TArray<FHairStrandsInterpolationFormat::Type> OutInterpolationCurves;
			{
				const uint32 uint32Count = FMath::DivideAndRoundUp(CurveCount * OutBulkData.Header.Strides.CurveInterpolationStride, 4u);
				OutInterpolationCurves.Init(0u, uint32Count);

				FHairInterpolationCurve* Out = (FHairInterpolationCurve*)OutInterpolationCurves.GetData();
				for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
				{
					for (uint32 KIndex = 0; KIndex < KCount; ++KIndex)
					{
						FHairInterpolationCurve& OutInterp = Out[CurveIndex*KCount + KIndex];
						OutInterp.CurveIndex  = HairInterpolation.CurveSimIndices[CurveIndex][KIndex];
						OutInterp.CurveWeight = FMath::Clamp(HairInterpolation.CurveSimWeights[CurveIndex][KIndex] * 255.f, 0, 255);
						OutInterp.PointIndex  = HairInterpolation.CurveSimRootPointIndex[CurveIndex][KIndex];
					}
				}
			}

			// Fill in point data
			TArray<FHairStrandsInterpolationFormat::Type> OutInterpolationPoints;
			{
				const uint32 uint32Count = FMath::DivideAndRoundUp(PointCount * OutBulkData.Header.Strides.PointInterpolationStride, 4u);
				OutInterpolationPoints.Init(0u, uint32Count);

				FHairInterpolationPoint* Out = (FHairInterpolationPoint*)OutInterpolationPoints.GetData();
				for (uint32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
				{
					const FIntVector2& Indices 	= HairInterpolation.PointSimIndices[PointIndex];
					const FVector2f& S 			= HairInterpolation.PointSimLerps[PointIndex];
					for (uint32 KIndex = 0; KIndex < KCount; ++KIndex)
					{		
						FHairInterpolationPoint& OutInterp = Out[PointIndex*KCount + KIndex];
						OutInterp.VertexIndex 		= Indices[KIndex];
						OutInterp.VertexLerp		= FMath::Clamp(S[KIndex] * 255.f, 0, 255);
					}
				}
			}

			// Sanity check
			{
				// Check on data size aligned on 4-byte address
				const uint32 PointTotalSize = OutInterpolationPoints.GetTypeSize() * OutInterpolationPoints.Num();
				check((PointTotalSize & 0x3) == 0);

				// Check on point count to ensure address is 4-byte aligned
				// This is needed as resources are allocated from header description. This optionally adds dummy point.
				uint32 PointSizeFromHeader = OutBulkData.Header.PointCount * OutBulkData.Header.Strides.PointInterpolationStride;
				if ((PointSizeFromHeader & 0x3) != 0)
				{
					if (HairInterpolation.bUseUniqueGuide)
					{
						OutBulkData.Header.PointCount = FMath::DivideAndRoundUp(OutBulkData.Header.PointCount, 2u) * 2u;
					}
					else
					{
						// This shouldn't happen
						check(false);
					}
				}
				PointSizeFromHeader = OutBulkData.Header.PointCount * OutBulkData.Header.Strides.PointInterpolationStride;
				check(PointSizeFromHeader == PointTotalSize);
			}
			

			HairStrandsBuilder::CopyToBulkData<FHairStrandsInterpolationFormat>(OutBulkData.Data.CurveInterpolation, OutInterpolationCurves);
			HairStrandsBuilder::CopyToBulkData<FHairStrandsInterpolationFormat>(OutBulkData.Data.PointInterpolation, OutInterpolationPoints);
			ReportSize(TEXT("Interpolation Curves"), OutInterpolationCurves);
			ReportSize(TEXT("Interpolation Points"), OutInterpolationPoints);
		}
	}

	/** Fill the GroomAsset with the interpolation data that exists in the HairDescription */
	void FillInterpolationData(
		const FHairStrandsDatas& RenData,
		const FHairStrandsDatas& SimData,
		FHairStrandsInterpolationDatas& OutInterpolation)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HairInterpolationBuilder::FillInterpolationData);

		check(RenData.StrandsCurves.CurvesClosestGuideIDs.Num() != 0);
		check(RenData.StrandsCurves.CurvesClosestGuideWeights.Num() != 0);
		check(SimData.GetNumCurves() > 0);
		int32 SimCurveCount = SimData.GetNumCurves();

		OutInterpolation.SetNum(RenData.GetNumCurves(), RenData.GetNumPoints());
		for (uint32 RenCurveIndex = 0; RenCurveIndex < RenData.GetNumCurves(); ++RenCurveIndex)
		{
			const uint32 RenCurveOffset = RenData.StrandsCurves.CurvesOffset[RenCurveIndex];
			const uint16 RenCurveNumVertices = RenData.StrandsCurves.CurvesCount[RenCurveIndex];

			const FIntVector StrandClosestGuides = RenData.StrandsCurves.CurvesClosestGuideIDs[RenCurveIndex];
			const FVector3f StrandGuideWeights  = RenData.StrandsCurves.CurvesClosestGuideWeights[RenCurveIndex];


			float CurveSimWeights[FClosestGuides::Count];
			for (int32 GuideIndex = 0; GuideIndex < FClosestGuides::Count; ++GuideIndex)
			{
				CurveSimWeights[GuideIndex] = 0;
			}

			for (uint16 RenPointIndex = 0; RenPointIndex < RenCurveNumVertices; ++RenPointIndex)
			{
				const uint32 RenPointGlobalIndex = RenPointIndex + RenCurveOffset;
				const FVector3f& RenPointPosition = RenData.StrandsPoints.PointsPosition[RenPointGlobalIndex];
				const float RenPointDistance = RenData.StrandsPoints.PointsCoordU[RenPointGlobalIndex] * RenData.StrandsCurves.CurvesLength[RenCurveIndex];

				bool bHasValidGuide = false;
				for (uint32 GuideIndex = 0; GuideIndex < FClosestGuides::Count; ++GuideIndex)
				{
					int32 ImportedGroomID = StrandClosestGuides[GuideIndex];
					const int* SimCurveIndex = SimData.StrandsCurves.GroomIDToIndex.Find(ImportedGroomID);

					if (SimCurveIndex && *SimCurveIndex >= 0 && *SimCurveIndex < SimCurveCount)
					{
						// Fill the interpolation data using the ParametricDistance algorithm with a constant weight for all vertices along the strand
						const uint32 SimOffset = SimData.StrandsCurves.CurvesOffset[*SimCurveIndex];
						const FVertexInterpolationDesc Desc = FindMatchingVertex(RenPointDistance, SimData, *SimCurveIndex);

						// Point data
						OutInterpolation.PointSimIndices[RenPointGlobalIndex][GuideIndex] = Desc.Index0 + SimOffset;
						OutInterpolation.PointSimLerps[RenPointGlobalIndex][GuideIndex] = Desc.T;

						// Curve data
						OutInterpolation.CurveSimIndices[RenCurveIndex][GuideIndex] = *SimCurveIndex;
						CurveSimWeights[GuideIndex] += OutInterpolation.CurveSimWeights[RenCurveIndex][GuideIndex];
						bHasValidGuide = true;
					}
				}

				// To not have invalid data, filled in interpolation data with first guide
				if (!bHasValidGuide)
				{
					for (uint32 GuideIndex = 0; GuideIndex < FClosestGuides::Count; ++GuideIndex)
					{
						const uint32 SimCurveIndex = 0; // Take the first guide

						// Fill the interpolation data using the ParametricDistance algorithm with a constant weight for all vertices along the strand
						const uint32 SimOffset = SimData.StrandsCurves.CurvesOffset[SimCurveIndex];
						const FVertexInterpolationDesc Desc = FindMatchingVertex(RenPointDistance, SimData, SimCurveIndex);

						// Point data
						OutInterpolation.PointSimIndices[RenPointGlobalIndex][GuideIndex] = Desc.Index0 + SimOffset;
						OutInterpolation.PointSimLerps[RenPointGlobalIndex][GuideIndex] = Desc.T;

						// Curve data
						OutInterpolation.CurveSimIndices[RenCurveIndex][GuideIndex] = SimCurveIndex;
						CurveSimWeights[GuideIndex] += 1.0f;
					}
				}
			}

			// Normalize the curve weights
			float TotalWeight = 0;
			for (int32 GuideIndex = 0; GuideIndex < FClosestGuides::Count; ++GuideIndex)
			{
				TotalWeight += CurveSimWeights[GuideIndex];
			}
			for (int32 GuideIndex = 0; GuideIndex < FClosestGuides::Count; ++GuideIndex)
			{
				OutInterpolation.CurveSimWeights[RenCurveIndex][GuideIndex] = CurveSimWeights[GuideIndex] / FMath::Max(0.001f, TotalWeight);
			}
		}
	}
}

FORCEINLINE FIntVector VectorToIntVector(const FVector3f& Index)
{
	return FIntVector(Index.X, Index.Y, Index.Z);
}

static FName ToGroupName(const FStrandID& InStrandID, const uint32 InGroupID, const TStrandAttributesConstRef<FName>& InGroupNames)
{
	if (InGroupNames.IsValid() && InStrandID < InGroupNames.GetNumElements())
	{
		return InGroupNames[InStrandID];
	}
	else
	{
		return FName(FString::Printf(TEXT("Group_%d"), InGroupID));
	}
}

bool NeedsToAddEndingControlPoint(uint32 CurveNumVertices)
{
	return GetHairStrandsUsesTriangleStrips() && uint32(CurveNumVertices + 1) <= HAIR_MAX_NUM_POINT_PER_CURVE;
}

bool FGroomBuilder::BuildHairDescriptionGroups(const FHairDescription& HairDescription, FHairDescriptionGroups& Out, const bool bAllowAddEndControlPoint)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBuilder::BuildHairDescriptionGroups);

	// Convert HairDescription to HairStrandsDatas
	// For now, just convert HairDescription to HairStrandsDatas
	int32 NumCurves = HairDescription.GetNumStrands();
	int32 NumVertices = HairDescription.GetNumVertices();

	// Check for required attributes
	TGroomAttributesConstRef<int> MajorVersion = HairDescription.GroomAttributes().GetAttributesRef<int>(HairAttribute::Groom::MajorVersion);
	TGroomAttributesConstRef<int> MinorVersion = HairDescription.GroomAttributes().GetAttributesRef<int>(HairAttribute::Groom::MinorVersion);

	// Major/Minor version check is currently disabled as this is not used at the moment and create false positive
	#if 0
	if (!MajorVersion.IsValid() || !MinorVersion.IsValid())
	{
		UE_LOGF(LogGroomBuilder, Warning, "No version number attributes found. The groom may be missing attributes even if it imports.");
	}
	#endif

	FGroomID GroomID(0);

	TGroomAttributesConstRef<float> GroomHairWidthAttribute = HairDescription.GroomAttributes().GetAttributesRef<float>(HairAttribute::Groom::Width);
	TOptional<float> GroomHairWidth;
	if (GroomHairWidthAttribute.IsValid())
	{
		GroomHairWidth = GroomHairWidthAttribute[GroomID];
	}

	TVertexAttributesConstRef<FVector3f> VertexPositions	= HairDescription.VertexAttributes().GetAttributesRef<FVector3f>(HairAttribute::Vertex::Position);
	TVertexAttributesConstRef<FVector3f> VertexBaseColor	= HairDescription.VertexAttributes().GetAttributesRef<FVector3f>(HairAttribute::Vertex::Color);
	TVertexAttributesConstRef<float> VertexRoughness		= HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Roughness);
	TVertexAttributesConstRef<float> VertexAO				= HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::AO);
	TStrandAttributesConstRef<int> StrandNumVertices		= HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::VertexCount);
	TStrandAttributesConstRef<int> ClumpIDs					= HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::ClumpID);
	TStrandAttributesConstRef<FVector3f> ClumpID3s			= HairDescription.StrandAttributes().GetAttributesRef<FVector3f>(HairAttribute::Strand::ClumpID);

	if (!VertexPositions.IsValid() || !StrandNumVertices.IsValid())
	{
		UE_LOGF(LogGroomBuilder, Warning, "Failed to import hair: No vertices or curves data found.");
		return false;
	}

	const bool bHasBaseColorAttribute = VertexBaseColor.IsValid();
	const bool bHasRoughnessAttribute = VertexRoughness.IsValid();
	const bool bHasAOAttribute = VertexAO.IsValid();
	const bool bHasClumpIDs = ClumpIDs.IsValid() || ClumpID3s.IsValid();

	// Sanity check
	check(bHasBaseColorAttribute == HairDescription.HasAttribute(EHairAttribute::Color));
	check(bHasRoughnessAttribute == HairDescription.HasAttribute(EHairAttribute::Roughness));
	check(bHasAOAttribute == HairDescription.HasAttribute(EHairAttribute::AO));
	check(bHasClumpIDs == HairDescription.HasAttribute(EHairAttribute::ClumpID)); 

	TVertexAttributesConstRef<float> VertexWidths = HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width);
	TStrandAttributesConstRef<float> StrandWidths = HairDescription.StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::Width);

	TStrandAttributesConstRef<FVector2f> StrandRootUV = HairDescription.StrandAttributes().GetAttributesRef<FVector2f>(HairAttribute::Strand::RootUV);
	const bool bHasUVData = StrandRootUV.IsValid();
	check (bHasUVData == HairDescription.HasAttribute(EHairAttribute::RootUV)); // Sanity check

	TStrandAttributesConstRef<int> StrandGuides = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::Guide);
	TStrandAttributesConstRef<int> GroupIDs = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::GroupID);
	TStrandAttributesConstRef<int> StrandIDs = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::ID);
	TStrandAttributesConstRef<FName> GroupNames = HairDescription.StrandAttributes().GetAttributesRef<FName>(HairAttribute::Strand::GroupName);

	// For precomputed weights, there is two variants with single or triplet IDs/weights
	// Single
	TStrandAttributesConstRef<int> ClosestGuide		  = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::ClosestGuides);
	TStrandAttributesConstRef<float> GuideWeight	  = HairDescription.StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::GuideWeights);
	// Triplet
	TStrandAttributesConstRef<FVector3f> ClosestGuides= HairDescription.StrandAttributes().GetAttributesRef<FVector3f>(HairAttribute::Strand::ClosestGuides);
	TStrandAttributesConstRef<FVector3f> GuideWeights = HairDescription.StrandAttributes().GetAttributesRef<FVector3f>(HairAttribute::Strand::GuideWeights);

	// To use ClosestGuides and GuideWeights attributes, guides must be imported from HairDescription and
	// must include StrandID attribute since ClosestGuides references those IDs
	const bool bPrecomputedWeight1 = ClosestGuide.IsValid()  && GuideWeight.IsValid();
	const bool bPrecomputedWeight3 = ClosestGuides.IsValid() && GuideWeights.IsValid();
	const bool bCanUseClosestGuidesAndWeights = StrandIDs.IsValid() && (bPrecomputedWeight1 || bPrecomputedWeight3);
	check(bCanUseClosestGuidesAndWeights == HairDescription.HasAttribute(EHairAttribute::PrecomputedGuideWeights)); // Sanity check

	auto FindOrAdd = [&GroupIDs, &GroupNames](FStrandID StrandID, FHairDescriptionGroups& OutGroups) -> FHairDescriptionGroup&
	{
		const bool bValidGroupId = GroupIDs.IsValid();
		const bool bValidGroupName = GroupNames.IsValid();

		// 1. If GroupID are valid, then use them in priority. GroupID can optionally be mapped to GroupName
		if (bValidGroupId)
		{
			const int32 GroupID = GroupIDs[StrandID];
			for (FHairDescriptionGroup& Group : OutGroups.HairGroups)
			{
				if (Group.Info.GroupID == GroupID)
				{
					return Group;
				}
			}
			FHairDescriptionGroup& Group = OutGroups.HairGroups.AddDefaulted_GetRef();
			Group.Info.GroupID = GroupID;
			Group.Info.GroupName = ToGroupName(StrandID, GroupID, GroupNames);
			return Group;
		}
		// 2. If no GroupID are valid, use unique GroupName to create GroupID
		else if (bValidGroupName)
		{
			const uint32 GroupID = OutGroups.HairGroups.Num();
			const FName GroupName = ToGroupName(StrandID, GroupID, GroupNames);
			for (FHairDescriptionGroup& Group : OutGroups.HairGroups)
			{
				if (Group.Info.GroupName == GroupName)
				{
					return Group;
				}
			}
			FHairDescriptionGroup& Group = OutGroups.HairGroups.AddDefaulted_GetRef();
			Group.Info.GroupID = GroupID;
			Group.Info.GroupName = GroupName;
			return Group;
		}
		// 3. If there are no GroupID and no GroupName, add a default group are valid, use unique group name to create GroupID
		else
		{
			if (OutGroups.HairGroups.IsEmpty())
			{
				const uint32 GroupID = 0;
				FHairDescriptionGroup& Group = OutGroups.HairGroups.AddDefaulted_GetRef();
				Group.Info.GroupID = GroupID;
				Group.Info.GroupName = ToGroupName(StrandID, GroupID, GroupNames);
			}
			return OutGroups.HairGroups[0];
		}
	};

	// Track the imported max hair width among all imported CVs. This information is displayed to artsits to set/tune groom asset later.
	float DDCMaxHairWidth = -1.f;

	bool bCurveCountWarningIssued = false;
	bool bPointCountWarningIssued = false;

	// Initialize Curve/Point index remapping 
	// * Per group source point offsets
	// * Per group points source mapping
	// * Per group curves source mapping
	TArray<uint32> GroupSourceOffsets;
	TArray<TArray<uint32>> GroupPointsMapping;
	TArray<TArray<uint32>> GroupCurvesMapping;
	{
		FHairDescriptionGroups LocalGroups;
		for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			FStrandID StrandID(CurveIndex);
			if (StrandNumVertices[StrandID] > 0)
			{
				FindOrAdd(StrandID, LocalGroups);
			}			
		}

		int32 MaxGroupID = 0;
		for(int32 GroupIndex = 0, GroupCount=LocalGroups.HairGroups.Num(); GroupIndex < GroupCount; ++GroupIndex)
		{
			MaxGroupID = FMath::Max(LocalGroups.HairGroups[GroupIndex].Info.GroupID, MaxGroupID);
		}
		
		const uint32 NumGroupID = MaxGroupID + 1;
		GroupSourceOffsets.Init(0, NumGroupID);
		GroupPointsMapping.Init(TArray<uint32>(), NumGroupID);	
		GroupCurvesMapping.Init(TArray<uint32>(), NumGroupID);
	}

	int32 GlobalVertexIndex = 0;
	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		FStrandID StrandID(CurveIndex);

		bool bIsGuide = false;
		if (StrandGuides.IsValid())
		{
			// Version 0.1 defines 1 as being guide
			bIsGuide = StrandGuides[StrandID] == 1;
		}

		int32 CurveNumVertices = StrandNumVertices[StrandID];

		FHairStrandsRawDatas* CurrentHairStrandsDatas = nullptr;
		FHairDescriptionGroup& Group = FindOrAdd(StrandID, Out);

		// Source curve/point count needs to be kept as vanilla as possible, so we count even invalid curve and invalid point
		if (bIsGuide)
		{
			Group.Guides.StrandsCurves.SourceCurveCount += 1;
			Group.Guides.StrandsPoints.SourcePointCount += CurveNumVertices;
		}
		else
		{
			Group.Strands.StrandsCurves.SourceCurveCount += 1;
			Group.Strands.StrandsPoints.SourcePointCount += CurveNumVertices;
		}

		// Check if the current curve has any data
		if (CurveNumVertices <= 0)
		{
			continue;
		}

		// Check if the current curve has invalid data
		bool bHasInvalidPosition = false;
		{
			for (int32 VertexIndex = 0; VertexIndex < CurveNumVertices; ++VertexIndex)
			{
				FVertexID VertexID(GlobalVertexIndex + VertexIndex);
				const FVector3f P = VertexPositions[VertexID];
				if (!(FMath::IsFinite(P.X) && FMath::IsFinite(P.Y) && FMath::IsFinite(P.Z)))
				{
					bHasInvalidPosition = true;
					break;
				}
			}
		}

		auto SetFlag = [bIsGuide, &Group](EHairStrandsFlags In)
		{
			if (bIsGuide)
			{
				Group.Guides.Flags |= uint32(In);
			}
			else
			{
				Group.Strands.Flags |= uint32(In);
			}
		};

		bool bNumCurveValid = false;
		if (bHasInvalidPosition)
		{
			// Remove curve with NaN positions
			GlobalVertexIndex += CurveNumVertices;
			Group.Info.Flags |= uint32(EHairGroupInfoFlags::HasInvalidPoint);
			SetFlag(EHairStrandsFlags::HasInvalidPoint);
			continue;
		}
		else if (!bIsGuide)
		{
			CurrentHairStrandsDatas = &Group.Strands;
			if (Group.Info.NumCurves < HAIR_MAX_NUM_CURVE_PER_GROUP)
			{
				bNumCurveValid = true;
				++Group.Info.NumCurves;
			}
		}
		else
		{
			CurrentHairStrandsDatas = &Group.Guides;
			if (Group.Info.NumGuides < HAIR_MAX_NUM_CURVE_PER_GROUP)
			{
				bNumCurveValid = true;
				++Group.Info.NumGuides;
			}
		}

		// If the current curve has more control point than allows, issue a warning
		if (CurveNumVertices > HAIR_MAX_NUM_POINT_PER_CURVE && !bPointCountWarningIssued)
		{
			// Do not display this during build time. 
			// * This is reported at import time
			// * This is visible in Groom editor
			//UE_LOGF(LogGroomBuilder, Warning, "[Groom] Groom contains strands with more than %d control points. Control points beyond that limit will be trimmed", uint32(HAIR_MAX_NUM_POINT_PER_CURVE));
			Group.Info.Flags |= uint32(EHairGroupInfoFlags::HasTrimmedPoint);
			SetFlag(EHairStrandsFlags::HasTrimmedPoint);
		}

		// If the current group has reached the max number of curve, skip them
		if (!bNumCurveValid)
		{
			if (!bCurveCountWarningIssued)
			{
				// Do not display this during build time. 
				// * This is reported at import time
				// * This is visible in Groom editor
				//UE_LOGF(LogGroomBuilder, Warning, "[Groom] Group has more than %d curves per group. Curve beyong that limit won't be part of the groom", uint32(HAIR_MAX_NUM_CURVE_PER_GROUP));
				bCurveCountWarningIssued = true;
			}
			Group.Info.Flags |= uint32(EHairGroupInfoFlags::HasTrimmedCurve);
			SetFlag(EHairStrandsFlags::HasTrimmedCurve);

			// Skip it
			GlobalVertexIndex += CurveNumVertices;
			continue;
		}

		if (!CurrentHairStrandsDatas)
		{
			continue;
		}

		const bool bAddEndingControlPoint = NeedsToAddEndingControlPoint(CurveNumVertices) && bAllowAddEndControlPoint && !bIsGuide;
		CurrentHairStrandsDatas->StrandsCurves.CurvesCount.Add(FMath::Min(uint32(CurveNumVertices + (bAddEndingControlPoint ? 1u : 0u)), HAIR_MAX_NUM_POINT_PER_CURVE));
		if (bAddEndingControlPoint)
		{
			SetFlag(EHairStrandsFlags::HasExtraEndingPoint);
		}

		if (bCanUseClosestGuidesAndWeights)
		{
			// ClosesGuides needs mapping of StrandID (the strand index in the HairDescription)
			CurrentHairStrandsDatas->StrandsCurves.StrandIDs.Add(CurveIndex);

			// and ImportedGroomID (the imported ID associated with a strand) to index in StrandCurves
			const int ImportedGroomID = StrandIDs[StrandID];
			const int StrandCurveIndex = CurrentHairStrandsDatas->StrandsCurves.CurvesCount.Num() - 1;
			CurrentHairStrandsDatas->StrandsCurves.GroomIDToIndex.Add(ImportedGroomID, StrandCurveIndex);

			if (!bIsGuide)
			{
				if (bPrecomputedWeight3)
				{
					CurrentHairStrandsDatas->StrandsCurves.CurvesClosestGuideIDs.Add(VectorToIntVector(ClosestGuides[StrandID]));
					CurrentHairStrandsDatas->StrandsCurves.CurvesClosestGuideWeights.Add(GuideWeights[StrandID]);
				}
				else
				{
					CurrentHairStrandsDatas->StrandsCurves.CurvesClosestGuideIDs.Add(FIntVector(ClosestGuide[StrandID]));
					CurrentHairStrandsDatas->StrandsCurves.CurvesClosestGuideWeights.Add(FVector3f(GuideWeight[StrandID]));
				}
			}
		}

		if (bHasClumpIDs)
		{
			if (ClumpID3s.IsValid())
			{
				CurrentHairStrandsDatas->StrandsCurves.ClumpIDs.Add(FIntVector(ClumpID3s[StrandID].X, ClumpID3s[StrandID].Y, ClumpID3s[StrandID].Z));
				SetHairAttributeFlags(CurrentHairStrandsDatas->StrandsCurves.AttributeFlags, EHairAttributeFlags::HasMultipleClumpIDs);
			}
			else
			{
				CurrentHairStrandsDatas->StrandsCurves.ClumpIDs.Add(FIntVector(ClumpIDs[StrandID]));
			}
		}

		if (bHasUVData)
		{
			CurrentHairStrandsDatas->StrandsCurves.CurvesRootUV.Add(StrandRootUV[StrandID]);
			if (StrandRootUV[StrandID].X > 1.f || StrandRootUV[StrandID].Y > 1.f)
			{
				SetHairAttributeFlags(CurrentHairStrandsDatas->StrandsCurves.AttributeFlags, EHairAttributeFlags::HasRootUDIM);
			}
		}

		// Groom 
		float StrandWidth = 0.01f;
		if (GroomHairWidth) 
		{
			StrandWidth = GroomHairWidth.GetValue();
		}

		// Curve
		if (StrandWidths.IsValid())
		{
			StrandWidth = StrandWidths[StrandID];
		}

		if (!bIsGuide)
		{
			GroupCurvesMapping[Group.Info.GroupID].Add(CurveIndex);
		}

		for (int32 VertexIndex = 0; VertexIndex < CurveNumVertices; ++VertexIndex, ++GlobalVertexIndex)
		{
			FVertexID VertexID(GlobalVertexIndex);
			if(!bIsGuide)
			{
				GroupPointsMapping[Group.Info.GroupID].Add(GroupSourceOffsets[Group.Info.GroupID]);
			}

			// Skip vertex beyond the supported limit. Continue to loop for incrementing GlobalVertexIndex
			if (VertexIndex >= HAIR_MAX_NUM_POINT_PER_CURVE)
			{
				continue;
			}

			CurrentHairStrandsDatas->StrandsPoints.PointsPosition.Add(VertexPositions[VertexID]);

			if (bHasBaseColorAttribute)
			{
				CurrentHairStrandsDatas->StrandsPoints.PointsBaseColor.Add(FLinearColor(VertexBaseColor[VertexID]));
			}
			if (bHasRoughnessAttribute)
			{
				CurrentHairStrandsDatas->StrandsPoints.PointsRoughness.Add(VertexRoughness[VertexID]);
			}
			if (bHasAOAttribute)
			{
				CurrentHairStrandsDatas->StrandsPoints.PointsAO.Add(VertexAO[VertexID]);
			}

			// Vertex
			float VertexWidth = 0.f;
			if (VertexWidths.IsValid())
			{
				VertexWidth = VertexWidths[VertexID];
			}
			else if (StrandWidth != 0.f)
			{
				// Fall back to strand width if there was no vertex width
				VertexWidth = StrandWidth;
			}

			// If the curve is a guide, and its width is 0, force to a constant width to insure the guides can be correctly rendered/displayed for debug purpose
			if (bIsGuide && VertexWidth == 0.f)
			{
				VertexWidth = 0.01f;
			}

			CurrentHairStrandsDatas->StrandsPoints.PointsRadius.Add(VertexWidth * 0.5f);

			// Add extra control point at the end of each curve when GetHairStrandsUsesTriangleStrips() 
			// is enabled to avoid loosing the last segment of each curve
			if (bAddEndingControlPoint && (VertexIndex == CurveNumVertices-1))
			{
				const int32 CurrentIndex = CurrentHairStrandsDatas->StrandsPoints.PointsPosition.Num();
				const int32 P0Index = CurrentIndex-1;
				const int32 P1Index = CurrentIndex-2;
				check(CurveNumVertices >= 2);
				check(P0Index > 0);
				check(P1Index >= 0);

				if(!bIsGuide)
				{
					GroupPointsMapping[Group.Info.GroupID].Add(GroupSourceOffsets[Group.Info.GroupID]);
				}

				// Add a small position offset to avoid superposed positions of the extra position
				const FVector3f P0 = CurrentHairStrandsDatas->StrandsPoints.PointsPosition[P0Index];
				const FVector3f P1 = CurrentHairStrandsDatas->StrandsPoints.PointsPosition[P1Index];
				const FVector3f PositionOffset = (P0 - P1) * 0.1f; // 10% of the distance

				CurrentHairStrandsDatas->StrandsPoints.PointsPosition.Add(VertexPositions[VertexID] + PositionOffset);
				if (bHasBaseColorAttribute)
				{
					CurrentHairStrandsDatas->StrandsPoints.PointsBaseColor.Add(FLinearColor(VertexBaseColor[VertexID]));
				}
				if (bHasRoughnessAttribute)
				{
					CurrentHairStrandsDatas->StrandsPoints.PointsRoughness.Add(VertexRoughness[VertexID]);
				}
				if (bHasAOAttribute)
				{
					CurrentHairStrandsDatas->StrandsPoints.PointsAO.Add(VertexAO[VertexID]);
				}
				CurrentHairStrandsDatas->StrandsPoints.PointsRadius.Add(VertexWidth * 0.5f);
			}
			if(!bIsGuide)
			{
				GroupSourceOffsets[Group.Info.GroupID]++;
			}
		}
	}

	// Initialize group index
	{
		int32 GroupIndex = 0;
		for (FHairDescriptionGroup& Group : Out.HairGroups)
		{
			Group.Info.GroupIndex = GroupIndex++;
		}
	}

	// Compute bound radius
	{
		FVector3f GroomBoundMin(FLT_MAX);
		FVector3f GroomBoundMax(-FLT_MAX);
		for (const FHairDescriptionGroup& Group : Out.HairGroups)
		{
			for (const FVector3f& P : Group.Strands.StrandsPoints.PointsPosition)
			{
				GroomBoundMin.X = FMath::Min(GroomBoundMin.X, P.X);
				GroomBoundMin.Y = FMath::Min(GroomBoundMin.Y, P.Y);
				GroomBoundMin.Z = FMath::Min(GroomBoundMin.Z, P.Z);

				GroomBoundMax.X = FMath::Max(GroomBoundMax.X, P.X);
				GroomBoundMax.Y = FMath::Max(GroomBoundMax.Y, P.Y);
				GroomBoundMax.Z = FMath::Max(GroomBoundMax.Z, P.Z);
			}
		}

		Out.Bounds = FBoxSphereBounds3f(FBox3f(GroomBoundMin, GroomBoundMax));
	}

	// Update GroupInfo
	for (FHairDescriptionGroup& Group : Out.HairGroups)
	{
		// Sanity check
		check(Group.Strands.GetNumCurves() == Group.Info.NumCurves);
		check(Group.Guides.GetNumCurves() == Group.Info.NumGuides);
		
		// Update infos
		Group.Info.NumCurveVertices = Group.Strands.GetNumPoints();
		Group.Info.NumGuideVertices = Group.Guides.GetNumPoints();

		// Prepare/Resize all attributes based on position/curves counts
		Group.Strands.StrandsPoints.SetNum(Group.Strands.GetNumPoints(), 0u, true /*bHasMapping*/);
		Group.Strands.StrandsCurves.SetNum(Group.Strands.GetNumCurves(), 0u, true /*bHasMapping*/);

		// Prepare/Resize all attributes based on position/curves counts
		Group.Guides.StrandsPoints.SetNum(Group.Guides.GetNumPoints(), 0u);
		Group.Guides.StrandsCurves.SetNum(Group.Guides.GetNumCurves(), 0u);

		// Set the strands points mapping
		Group.Strands.StrandsPoints.PointsMapping = GroupPointsMapping[Group.Info.GroupID];
		Group.Strands.StrandsCurves.CurvesMapping = GroupCurvesMapping[Group.Info.GroupID];
	}

	return true;
}

void FGroomBuilder::BuildData(const FHairStrandsRawDatas& In, FHairStrandsDatas& Out)
{
	Out = HairStrandsBuilder::BuildInternalData(In);
}

void FGroomBuilder::BuildData(
	const FHairDescriptionGroup& InHairDescriptionGroup,
	const FHairGroupsInterpolation& InSettings,
	FHairStrandsDatas& OutRen,
	FHairStrandsDatas& OutSim,
	const bool bAllowCurveReordering,
	const bool bApplyDecimation,
	const bool bBuildSourceMapping)
{
	FHairGroupInfo DummyGroupInfo;
	BuildData(
		InHairDescriptionGroup, 
		InSettings, 
		DummyGroupInfo, 
		OutRen,
		OutSim,
		bAllowCurveReordering,
		bApplyDecimation,
		bBuildSourceMapping);
}

void FGroomBuilder::BuildData(
	const FHairDescriptionGroup& InHairDescriptionGroup, 
	const FHairGroupsInterpolation& InSettings, 
	FHairGroupInfo& OutGroupInfo, 
	FHairStrandsDatas& OutRen,
	FHairStrandsDatas& OutSim,
	const bool bAllowCurveReordering,
	const bool bApplyDecimation,
	const bool bBuildSourceMapping)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomBuilder::BuildData);

	// Sanitize decimation values. Do not update the 'InSettings' values directly as this would change 
	// the groom asset and thus would change the DDC key
	const float CurveDecimation		= bApplyDecimation ? FMath::Clamp(InSettings.DecimationSettings.CurveDecimation, 0.f, 1.f) : 1.0f;
	const float VertexDecimation	= bApplyDecimation ? FMath::Clamp(InSettings.DecimationSettings.VertexDecimation, 0.f, 1.f) : 1.0f;
	const float HairToGuideDensity	= FMath::Clamp(InSettings.InterpolationSettings.HairToGuideDensity, 0.f, 1.f);	
	const bool bCurveReordering 	= bAllowCurveReordering;

	{
		OutGroupInfo.GroupIndex = InHairDescriptionGroup.Info.GroupIndex;
		OutGroupInfo.GroupID = InHairDescriptionGroup.Info.GroupID;
		OutGroupInfo.GroupName = InHairDescriptionGroup.Info.GroupName;

		// Rendering data
		OutRen.Reset();
		if (InHairDescriptionGroup.Strands.IsValid())
		{
			OutRen = HairStrandsBuilder::BuildInternalData(InHairDescriptionGroup.Strands);
			
			if (bCurveReordering)
			{
				OutRen.Flags |= uint32(EHairStrandsFlags::HasShuffledCurve);
			}
			if (CurveDecimation < 1)
			{
				OutRen.Flags |= uint32(EHairStrandsFlags::HasDecimatedCurve);
			}
			if (VertexDecimation < 1)
			{
				OutRen.Flags |= uint32(EHairStrandsFlags::HasDecimatedPoint);
			}
			
			// Decimate
			if (CurveDecimation < 1 || VertexDecimation < 1 || bCurveReordering)
			{
				FHairStrandsDatas FullData = OutRen;
				OutRen.Reset();
				FHairStrandsDecimation::Decimate(FullData, CurveDecimation, VertexDecimation, bCurveReordering, OutRen, bBuildSourceMapping);
			}
			
			OutGroupInfo.NumCurves			= OutRen.GetNumCurves();
			OutGroupInfo.NumCurveVertices	= OutRen.GetNumPoints();
			OutGroupInfo.MaxCurveLength		= GetHairStrandsMaxLength(OutRen);

			// Sanity check
			check(OutGroupInfo.NumCurves		<= InHairDescriptionGroup.Info.NumCurves);
			check(OutGroupInfo.NumCurveVertices <= InHairDescriptionGroup.Info.NumCurveVertices);
		}

		// Simulation data
		OutSim.Reset();
		{
			const bool bHasImportedGuides = InHairDescriptionGroup.Info.NumGuides > 0;
			if (bHasImportedGuides && InSettings.InterpolationSettings.GuideType == EGroomGuideType::Imported)
			{
				OutSim = HairStrandsBuilder::BuildInternalData(InHairDescriptionGroup.Guides);
			}
			else
			{
				if(InSettings.InterpolationSettings.GuideType == EGroomGuideType::Rigged)
				{
					if (InHairDescriptionGroup.Info.NumGuides > 0)
					{
						// Pick the new guides among the imported ones
						FHairStrandsDatas TempSim = HairStrandsBuilder::BuildInternalData(InHairDescriptionGroup.Guides);
						FHairStrandsDecimation::Decimate(TempSim, InSettings.InterpolationSettings.RiggedGuideNumCurves, InSettings.InterpolationSettings.RiggedGuideNumPoints, OutSim, bBuildSourceMapping);
					}
					else
					{
						// Otherwise let s pick the guides among the rendered strands
						check(OutRen.GetNumPoints() > 0);
						FHairStrandsDecimation::Decimate(OutRen, InSettings.InterpolationSettings.RiggedGuideNumCurves, InSettings.InterpolationSettings.RiggedGuideNumPoints, OutSim, bBuildSourceMapping);
					}
				}
				else if (!bHasImportedGuides || InSettings.InterpolationSettings.GuideType == EGroomGuideType::Generated)
				{
					check(OutRen.GetNumPoints() > 0);
					FHairStrandsDecimation::Decimate(OutRen, HairToGuideDensity, 1, false, OutSim, bBuildSourceMapping);
				}
				else
				{
					checkNoEntry();
				}
			}

			OutGroupInfo.NumGuides			= OutSim.GetNumCurves();
			OutGroupInfo.NumGuideVertices	= OutSim.GetNumPoints();

			// Sanity check
			check(OutGroupInfo.NumGuides > 0);
			check(OutGroupInfo.NumGuideVertices > 0);
		}
	}
}

void FGroomBuilder::BuildBulkData(
	const FHairGroupInfo& InInfo,
	const FHairStrandsDatas& InData,
	FHairStrandsBulkData& OutBulkData, 
	bool bAllowTranscoding)
{
	const bool bIsNotGuide = bAllowTranscoding;

	OutBulkData.Reset();

	// 1. Initialize curve seed
	const int32 NumCurves = InData.GetNumCurves();
	FRandomStream Random;
	Random.Initialize(InInfo.GroupID);
	TArray<uint8> CurveSeeds;
	CurveSeeds.SetNumUninitialized(NumCurves);
	for (int32 Index = 0; Index < NumCurves; ++Index)
	{
		CurveSeeds[Index] = Random.RandHelper(255);
	}

	// 2. Compute coverage scaling based on screen size & decimation
	TArray<GroomBuilder_Voxelization::FCoverageScale> CoverageScales;
	if (bIsNotGuide)
	{
		GroomBuilder_Voxelization::ComputeHairCoverageScale(InData, CoverageScales); // TODO This should be done during the build step, not the bulk data build step
	}
	else
	{
		GroomBuilder_Voxelization::FCoverageScale Dummy;
		Dummy.ScreenSize = 1;
		Dummy.CoverageScale = 1;
		CoverageScales.Init(Dummy, 10);
	}

	// 3. Build bulk data
	HairStrandsBuilder::BuildBulkData(InData, CurveSeeds, CoverageScales, OutBulkData, bAllowTranscoding);
}

void FGroomBuilder::BuildInterplationData(
	const FHairGroupInfo& InInfo,
	const FHairStrandsDatas& InRenData,
	const FHairStrandsDatas& InSimData,
	const FHairInterpolationSettings& InInterpolationSettings,
	FHairStrandsInterpolationDatas& OutInterpolationData)
{
	// Build Rendering data for InterpolationData
	// Build InterpolationData from render and simulation HairStrandsDatas
	// Skip building if interpolation data was provided by the source file
	if (OutInterpolationData.GetPointCount() == 0)
	{
		// If there's usable closest guides and guide weights attributes, fill them into the asset
		// This step requires the HairSimulationData (guides) to be filled prior to this
		const bool bUsePrecomputedWeights = InInterpolationSettings.GuideType == EGroomGuideType::Imported && InRenData.StrandsCurves.CurvesClosestGuideIDs.Num() > 0 && InRenData.StrandsCurves.CurvesClosestGuideWeights.Num() > 0;
		if (bUsePrecomputedWeights)
		{
			FHairStrandsInterpolationDatas OutInterpolation;
			HairInterpolationBuilder::FillInterpolationData(InRenData, InSimData, OutInterpolationData);
		}
		else
		{
			// This randomization makes certain strands being affected by 1, 2, or 3 guides
			FRandomStream Random;
			Random.Initialize(InInfo.GroupID);
			const int32 NumRenderCurves = InRenData.GetNumCurves();
			TArray<FIntVector> GuideIndices;
			GuideIndices.SetNumUninitialized(NumRenderCurves);
			for (int32 Index = 0; Index < NumRenderCurves; ++Index)
			{
				FIntVector& RandomIndices = GuideIndices[Index];
				for (int GuideIndex = 0; GuideIndex < HairInterpolationBuilder::FMetrics::Count; ++GuideIndex)
				{
					RandomIndices[GuideIndex] = Random.RandRange(0, HairInterpolationBuilder::FMetrics::Count - 1);
				}
			}

			HairInterpolationBuilder::BuildInterpolationData(OutInterpolationData, InSimData, InRenData, InInterpolationSettings, GuideIndices);
		}
	}
}

void FGroomBuilder::BuildInterplationBulkData(
	const FHairStrandsDatas& InSimData,
	const FHairStrandsInterpolationDatas& InInterpolationData,
	FHairStrandsInterpolationBulkData& OutInterpolationData)
{
	HairInterpolationBuilder::BuildInterplationBulkData(InSimData, InInterpolationData, OutInterpolationData);
}

namespace FHairStrandsDecimation
{

inline uint32 DecimatePointCount(uint32 InCount, float InDecimationFactor)
{
	return FMath::Clamp(uint32(FMath::CeilToInt(InCount * InDecimationFactor)), 2u, InCount);
}

static void DecimateCurve(
	const TArray<FVector3f>& InPoints, 
	const uint32 InOffset, 
	const uint32 InCount, 
	const float InDecimationFactor, 
	TArray<uint32>& OutIndices)
{
	check(InCount > 2);
	const int32 OutCount = DecimatePointCount(InCount, InDecimationFactor);

	OutIndices.SetNum(InCount);
	for (uint32 CurveIt = 0; CurveIt < InCount; ++CurveIt)
	{
		OutIndices[CurveIt] = CurveIt;
	}

	while (OutIndices.Num() > OutCount)
	{
		float MinError = FLT_MAX;
		uint32 ElementToRemove = 0;
		const uint32 Count = OutIndices.Num();
		for (uint32 IndexIt = 1; IndexIt < Count - 1; ++IndexIt)
		{
			const FVector3f& P0 = InPoints[InOffset + OutIndices[IndexIt - 1]];
			const FVector3f& P1 = InPoints[InOffset + OutIndices[IndexIt]];
			const FVector3f& P2 = InPoints[InOffset + OutIndices[IndexIt + 1]];

			const float Area = FVector3f::CrossProduct(P0 - P1, P2 - P1).Size() * 0.5f;

			if (Area < MinError)
			{
				MinError = Area;
				ElementToRemove = IndexIt;
			}
		}
		OutIndices.RemoveAt(ElementToRemove);
	}
}
	
void Decimate(
		const FHairStrandsDatas& InData, 
		const uint32 NumCurves,
		const int32 NumVertices,
		FHairStrandsDatas& OutData,
		const bool bBuildSourceMapping)
{
	const uint32 InCurveCount = InData.StrandsCurves.Num();
	const uint32 OutCurveCount = FMath::Clamp(NumCurves, 1u, InCurveCount);
	
	// Fill root positions and curves offsets for sampling
	TArray<FVector3f> RootPositions;
	RootPositions.Init(FVector3f::ZeroVector, InData.GetNumCurves());
	for(uint32 StrandIndex = 0; StrandIndex < InData.GetNumCurves(); ++StrandIndex)
	{
		RootPositions[StrandIndex] = InData.StrandsPoints.PointsPosition[InData.StrandsCurves.CurvesOffset[StrandIndex]];
	}
				
	TArray<bool> ValidPoints;
	ValidPoints.Init(true, InData.GetNumCurves());

	const uint32 InAttributes = InData.GetAttributes();

	// Pick NumCurves strands from the guides
	GroomBinding_RBFWeighting::FPointsSampler PointsSampler(ValidPoints, RootPositions.GetData(), OutCurveCount);

	const uint32 CurveCount = PointsSampler.SampleIndices.Num();
	uint32 PointCount = CurveCount * NumVertices;

	OutData.StrandsCurves.SetNum(CurveCount, InAttributes, bBuildSourceMapping);
	OutData.StrandsPoints.SetNum(PointCount, InAttributes, bBuildSourceMapping);
	OutData.HairDensity = InData.HairDensity;

	PointCount = 0;
	for(uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
	{
		const int32 GuideIndex = PointsSampler.SampleIndices[CurveIndex];
		const int32 GuideOffset = InData.StrandsCurves.CurvesOffset[GuideIndex];
		const int32 GuideCount = InData.StrandsCurves.CurvesCount[GuideIndex];
		
		const int32 MaxPoints = FMath::Min(GuideCount, NumVertices);
		const float BoneSpace = FMath::Max(1.0,float(GuideCount-1) / (MaxPoints-1));

		OutData.StrandsCurves.CurvesCount[CurveIndex]  = MaxPoints;
		FHairStrandsDatas::CopyCurve(InData, OutData, InAttributes, GuideIndex, CurveIndex);
	
		for(int32 PointIndex = 0; PointIndex < MaxPoints; ++PointIndex, ++PointCount)
		{
			const float PointSpace = GuideOffset + PointIndex * BoneSpace;
	
			const int32 PointBegin = FMath::Min(FMath::FloorToInt32(PointSpace), GuideOffset + GuideCount-2);
			const int32 PointEnd = PointBegin+1;	
			const float PointAlpha = PointSpace - PointBegin;
			
			FHairStrandsDatas::CopyPointLerp(InData, OutData, InAttributes, PointBegin, PointEnd, PointAlpha, PointCount);
		}
	}
	OutData = HairStrandsBuilder::BuildInternalData(OutData);
}

void GetCurveShuffleIndices(TArray<uint32>& CurveRandomizedIndex, uint32 InCurveCount)
{
	FRandomStream Random;
	Random.Initialize(0xdeedbeed);

	// this is the proof of concept version - which just does a shuffle - TODO: improved voxel based version which caters for both long hair and short hair
	CurveRandomizedIndex.SetNum(InCurveCount);

	//initialize value
	for (uint32 CurveIndex = 0; CurveIndex < InCurveCount; CurveIndex++)
	{
		CurveRandomizedIndex[CurveIndex] = CurveIndex;
	}
	//shuffle
	for (uint32 CurveIndex = 0; CurveIndex < InCurveCount; CurveIndex++)
	{
		uint32 RandIndex = Random.RandRange(0, InCurveCount - 1);
		CurveRandomizedIndex.Swap(CurveIndex, RandIndex);
	}
}

void Decimate(
	const FHairStrandsDatas& InData, 
	float CurveDecimationPercentage, 
	float VertexDecimationPercentage, 
	bool bContinuousDecimationReordering,
	FHairStrandsDatas& OutData,
	const bool bBuildSourceMapping)
{
	CurveDecimationPercentage = FMath::Clamp(CurveDecimationPercentage, 0.f, 1.f);
	VertexDecimationPercentage = FMath::Clamp(VertexDecimationPercentage, 0.f, 1.f);

	// Pick randomly strand as guide 
	// Divide strands in buckets and pick randomly one stand per bucket
	const uint32 InCurveCount = InData.StrandsCurves.Num();
	const uint32 OutCurveCount = FMath::Clamp(uint32(InCurveCount * CurveDecimationPercentage), 1u, InCurveCount);
	const uint32 InAttributes = InData.GetAttributes();

	TArray<uint32> CurveRandomizedIndex;
	if (bContinuousDecimationReordering)
	{
		GetCurveShuffleIndices(CurveRandomizedIndex, InCurveCount);
	}

	FRandomStream Random;
	Random.Initialize(0xdeedbeed);

	uint32 OutTotalPointCount = InData.GetNumPoints();
	TArray<uint32> CurveIndices;
	const bool bNeedDecimation = CurveDecimationPercentage < 1 || VertexDecimationPercentage < 1;
	if (bNeedDecimation)
	{
		CurveIndices.SetNum(OutCurveCount);
		OutTotalPointCount = 0;
		const float CurveBucketSize = float(InCurveCount) / float(OutCurveCount);
		int32 LastCurveIndex = -1;
		for (uint32 BucketIndex = 0; BucketIndex < OutCurveCount; BucketIndex++)
		{
			const float MinBucket = FMath::Max(BucketIndex * CurveBucketSize, float(LastCurveIndex + 1));
			const float MaxBucket = (BucketIndex + 1) * CurveBucketSize;
			const float AdjustedBucketSize = MaxBucket - MinBucket;
			if (AdjustedBucketSize > 0)
			{
				const uint32 CurveIndex = Random.RandRange(MinBucket, FMath::FloorToInt(MinBucket + AdjustedBucketSize) - 1);
				CurveIndices[BucketIndex] = CurveIndex;
				LastCurveIndex = CurveIndex;

				const uint32 EffectiveCurveIndex = bContinuousDecimationReordering ? CurveRandomizedIndex[CurveIndex] : CurveIndex;

				const uint32 InPointCount = InData.StrandsCurves.CurvesCount[EffectiveCurveIndex];
				const uint32 OutPointCount = DecimatePointCount(InPointCount, VertexDecimationPercentage);
				OutTotalPointCount += OutPointCount;
			}
		}
	}

	OutData.StrandsCurves.SetNum(OutCurveCount, InAttributes, bBuildSourceMapping);
	OutData.StrandsPoints.SetNum(OutTotalPointCount, InAttributes, bBuildSourceMapping);
	OutData.HairDensity = InData.HairDensity;
	OutData.StrandsCurves.SourceCurveCount = InData.StrandsCurves.SourceCurveCount;
	OutData.StrandsPoints.SourcePointCount = InData.StrandsPoints.SourcePointCount;

	uint32 OutPointOffset = 0; 
	for (uint32 OutCurveIndex = 0; OutCurveIndex < OutCurveCount; ++OutCurveIndex)
	{
		const uint32 InCurveIndex		 = bNeedDecimation ? CurveIndices[OutCurveIndex] : OutCurveIndex;
		const uint32 EffectiveCurveIndex = bContinuousDecimationReordering ? CurveRandomizedIndex[InCurveIndex] : InCurveIndex;

		const uint32 InPointOffset	= InData.StrandsCurves.CurvesOffset[EffectiveCurveIndex];
		const uint32 InPointCount	= InData.StrandsCurves.CurvesCount[EffectiveCurveIndex];

		// Decimation using area metric
		TArray<uint32> OutPointIndices;
		if (bNeedDecimation)
		{
			// Don't need to DecimateCurve if there are only 2 control vertices
			if (InPointCount > 2)
			{
				DecimateCurve(
					InData.StrandsPoints.PointsPosition,
					InPointOffset,
					InPointCount,
					VertexDecimationPercentage,
					OutPointIndices);
			}
			else
			{
				// Just pass the start and end point of the curve, no decimation needed
				OutPointIndices.Add(0);
				OutPointIndices.Add(1);
			}
		}

		const uint32 OutPointCount = bNeedDecimation ? OutPointIndices.Num() : InPointCount;
		for (uint32 OutPointIndex = 0; OutPointIndex < OutPointCount; ++OutPointIndex)
		{
			const uint32 InPointIndex = bNeedDecimation ? OutPointIndices[OutPointIndex] : OutPointIndex;

			FHairStrandsDatas::CopyPoint(InData, OutData, InAttributes, InPointIndex + InPointOffset, OutPointIndex + OutPointOffset);
		}

		OutData.StrandsCurves.CurvesCount[OutCurveIndex]  = OutPointCount;
		OutData.StrandsCurves.CurvesOffset[OutCurveIndex] = OutPointOffset;
		FHairStrandsDatas::CopyCurve(InData, OutData, InAttributes, EffectiveCurveIndex, OutCurveIndex);

		OutPointOffset += OutPointCount;
	}

	// Copy flags from the source (trimmed curve/point, added ending point, shuffle curve, ...)
	OutData.Flags = InData.Flags;
	OutData = HairStrandsBuilder::BuildInternalData(OutData);
}
} // namespace FHairStrandsDecimation

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair voxelization
namespace GroomBuilder_Voxelization
{

FORCEINLINE uint32 ToLinearCoord(const FIntVector& T, const FIntVector& Resolution)
{
	// Morton instead for better locality?
	return T.X + T.Y * Resolution.X + T.Z * Resolution.X * Resolution.Y;
}

FORCEINLINE FIntVector ToCoord(const FVector3f& T, const FIntVector& Resolution, const FVector3f& MinBound, const float VoxelSize)
{
	const FVector3f C = (T - MinBound) / VoxelSize;
	return FIntVector(
		FMath::Clamp(FMath::FloorToInt(C.X), 0, Resolution.X - 1),
		FMath::Clamp(FMath::FloorToInt(C.Y), 0, Resolution.Y - 1),
		FMath::Clamp(FMath::FloorToInt(C.Z), 0, Resolution.Z - 1));
}

FORCEINLINE bool IsValid(const FIntVector& P, const FIntVector& Resolution)
{
	return	0 <= P.X && P.X < Resolution.X &&
			0 <= P.Y && P.Y < Resolution.Y &&
			0 <= P.Z && P.Z < Resolution.Z;
}

FORCEINLINE FIntVector ClampToVolume(const FIntVector& CellCoord, const FIntVector& Resolution, bool& bIsValid)
{
	bIsValid = IsValid(CellCoord, Resolution);
	return FIntVector(
		FMath::Clamp(CellCoord.X, 0, Resolution.X - 1),
		FMath::Clamp(CellCoord.Y, 0, Resolution.Y - 1),
		FMath::Clamp(CellCoord.Z, 0, Resolution.Z - 1));
}

FORCEINLINE FIntVector ToCellCoord(const FVector3f& P, const FVector3f& MinBound, const FVector4f& MaxBound, const FIntVector& Resolution)
{
	bool bIsValid = false;
	const FVector3f F = FVector3f((P - MinBound) / (MaxBound - MinBound));
	const FIntVector CellCoord = FIntVector(FMath::FloorToInt(F.X * Resolution.X), FMath::FloorToInt(F.Y * Resolution.Y), FMath::FloorToInt(F.Z * Resolution.Z));
	return ClampToVolume(CellCoord, Resolution, bIsValid);
}

FORCEINLINE uint32 ToIndex(const FIntVector& CellCoord, const FIntVector& Resolution)
{
	uint32 CellIndex = CellCoord.X + CellCoord.Y * Resolution.X + CellCoord.Z * Resolution.X * Resolution.Y;
	return CellIndex;
}

struct FHairGrid
{
	struct FCurve
	{
		FVector3f BaseColor = FVector3f::ZeroVector;
		float     Roughness = 0;
		uint32    CurveIndex = 0;
		float     Radius = 0;
		uint8     GroupIndex = 0;
	};

	struct FVoxel
	{
		TArray<FCurve> VoxelCurves;
	};

	float VoxelSize;
	FVector3f MinBound;
	FVector3f MaxBound;
	FIntVector Resolution;
	TArray<FVoxel> Voxels;
};

static void AllocateVoxels(const FBox3f& InBoundingBox, FHairGrid& Out)
{
	// 1. Compute the overal bound of for all hair groups
	Out.MinBound = InBoundingBox.Min;
	Out.MaxBound = InBoundingBox.Max;
	Out.Resolution = FIntVector::ZeroValue;

	// 2. Based on the bound, determine the voxel grid resolution
	Out.VoxelSize = 0.5f; // 5mm
	{
		const int32 MaxResolution = FMath::Max(GHairGroupIndexBuilder_MaxVoxelResolution, 2);

		bool bIsValid = false;
		while (!bIsValid)
		{
			FVector3f VoxelResolutionF = (Out.MaxBound - Out.MinBound) / Out.VoxelSize;
			Out.Resolution = FIntVector(FMath::CeilToInt(VoxelResolutionF.X), FMath::CeilToInt(VoxelResolutionF.Y), FMath::CeilToInt(VoxelResolutionF.Z));
			bIsValid = Out.Resolution.X <= MaxResolution && Out.Resolution.Y <= MaxResolution && Out.Resolution.Z <= MaxResolution;
			if (!bIsValid)
			{
				Out.VoxelSize *= 2;
			}
		}
		Out.MaxBound = Out.MinBound + FVector3f(Out.Resolution) * Out.VoxelSize;
	}
	Out.Voxels.SetNum(FMath::Max(Out.Resolution.X * Out.Resolution.Y * Out.Resolution.Z, 1));
}

static void VoxelizeCurves(const FHairStrandsDatas& In, uint32 InGroupIndex, FHairGrid& Out)
{
	const uint32 Attributes = In.GetAttributes();

	// Fill in voxel (TODO: make it parallel)
	for (uint32 CurveIt = 0, CurveCount = In.GetNumCurves(); CurveIt < CurveCount; ++CurveIt)
	{
		const uint32 PointOffset = In.StrandsCurves.CurvesOffset[CurveIt];
		const uint32 PointCount = In.StrandsCurves.CurvesCount[CurveIt];

		uint32 PrevLinearCoord = ~0;
		for (uint32 PointIndex = 0; PointIndex < PointCount - 1; ++PointIndex)
		{
			const uint32 Index0 = PointOffset + PointIndex;
			const uint32 Index1 = PointOffset + PointIndex + 1;
			const FVector3f& P0 = In.StrandsPoints.PointsPosition[Index0];
			const FVector3f& P1 = In.StrandsPoints.PointsPosition[Index1];
			const float R0 = In.StrandsPoints.PointsRadius[Index0];
			const float R1 = In.StrandsPoints.PointsRadius[Index1];
			const FVector3f C0 = HasHairAttribute(Attributes, EHairAttribute::Color) ? FVector3f(In.StrandsPoints.PointsBaseColor[Index0].R, In.StrandsPoints.PointsBaseColor[Index0].G, In.StrandsPoints.PointsBaseColor[Index0].B) : FVector3f(0);
			const FVector3f C1 = HasHairAttribute(Attributes, EHairAttribute::Color) ? FVector3f(In.StrandsPoints.PointsBaseColor[Index1].R, In.StrandsPoints.PointsBaseColor[Index1].G, In.StrandsPoints.PointsBaseColor[Index1].B) : FVector3f(0);
			const float Rough0 = HasHairAttribute(Attributes, EHairAttribute::Roughness) ? In.StrandsPoints.PointsRoughness[Index0] : 0;
			const float Rough1 = HasHairAttribute(Attributes, EHairAttribute::Roughness) ? In.StrandsPoints.PointsRoughness[Index1] : 0;
			const FVector3f Segment = P1 - P0;

			// This is a coarse/non-conservative voxelization, by ray-marching segment, instead of walking intersected voxels
			const float Length = Segment.Size();
			const uint32 StepCount = FMath::CeilToInt(Length / Out.VoxelSize);
			for (uint32 StepIt = 0; StepIt < StepCount + 1; ++StepIt)
			{
				const float T = float(StepIt) / float(StepCount);
				const FVector3f P = P0 + Segment * T;
				const FIntVector Coord = ToCoord(P, Out.Resolution, Out.MinBound, Out.VoxelSize);
				const uint32 LinearCoord = ToLinearCoord(Coord, Out.Resolution);
				if (LinearCoord != PrevLinearCoord)
				{
					FHairGrid::FCurve RCurve;
					RCurve.CurveIndex = CurveIt;
					RCurve.GroupIndex = InGroupIndex;
					RCurve.Radius = FMath::LerpStable(R0, R1, T);
					RCurve.BaseColor = FVector3f(FMath::LerpStable(C0.X, C1.X, T), FMath::LerpStable(C0.Y, C1.Y, T), FMath::LerpStable(C0.Z, C1.Z, T));
					RCurve.Roughness = FMath::LerpStable(Rough0, Rough1, T);
					Out.Voxels[LinearCoord].VoxelCurves.Add(RCurve);

					PrevLinearCoord = LinearCoord;
				}
			}
		}
	}
}

static void Voxelize(const FHairDescriptionGroups& InGroups, FHairGrid& Out)
{
	AllocateVoxels(InGroups.Bounds.GetBox(), Out);
	
	uint32 GroupIndex = 0;
	for (const FHairDescriptionGroup& Group : InGroups.HairGroups)
	{
		// Build the derived data so that curve offset are available
		FHairStrandsDatas BuiltData = HairStrandsBuilder::BuildInternalData(Group.Strands);
		VoxelizeCurves(BuiltData, GroupIndex, Out);
		++GroupIndex;
	}
}

static void Voxelize(const FHairStrandsDatas& InData, FHairGrid& Out)
{
	AllocateVoxels(InData.BoundingBox, Out);
	VoxelizeCurves(InData, 0u /*GroupIndex not used*/, Out);
}

FORCEINLINE void SearchCell(const FHairStrandsVoxelData& In, const FIntVector& C, FHairStrandsVoxelData::FData& Out)
{
	const uint32 I = GroomBuilder_Voxelization::ToIndex(C, In.Resolution);
	if (In.Datas[I].GroupIndex != FHairStrandsVoxelData::InvalidGroupIndex)
	{
		Out = In.Datas[I];
	}
}

struct FVoxelCoverage
{
	uint32 CurveCount = 0;
	float AvgRadius = 0;
	float Coverage = 0;
};

static void ComputeCoverage(const float InGridVoxelSize, const TArray<GroomBuilder_Voxelization::FHairGrid::FVoxel>& InNonEmptyVoxels, TArray<FVoxelCoverage>& OutVoxelCoverages)
{	
	const uint32 VoxelCount = InNonEmptyVoxels.Num();
	OutVoxelCoverages.SetNum(VoxelCount);

	#if 1
	ParallelFor(VoxelCount, 
	[
		InGridVoxelSize,
		&InNonEmptyVoxels,
		&OutVoxelCoverages
	] (uint32 VoxelIt) 
	#else
	for (uint32 VoxelIt=0;VoxelIt<VoxelCount;++VoxelIt)
	#endif
	{
		const GroomBuilder_Voxelization::FHairGrid::FVoxel& Voxel = InNonEmptyVoxels[VoxelIt];
		FVoxelCoverage& Out = OutVoxelCoverages[VoxelIt];

		Out.CurveCount = Voxel.VoxelCurves.Num();
		for (const GroomBuilder_Voxelization::FHairGrid::FCurve& Curve : Voxel.VoxelCurves)
		{
			const float NormalizedRadius = Curve.Radius / InGridVoxelSize;
			Out.AvgRadius += NormalizedRadius;
		}
		Out.AvgRadius /= FMath::Max(1u, Out.CurveCount);
		Out.Coverage = GetHairCoverage(Out.CurveCount, Out.AvgRadius);
	});
}

static float RemoveCurveAndComputeCoverageScaling(
	const float InGridVoxelSize,
	TArray<GroomBuilder_Voxelization::FHairGrid::FVoxel>& InNonEmptyVoxels,
	const TArray<FVoxelCoverage>& InReferenceCoverage, 
	uint32 InCurveIndexStart,
	uint32 InCurveIndexEnd)
{
	check(InNonEmptyVoxels.Num() == InReferenceCoverage.Num());

	// Certain platform don't support atomic<float>. To support these, we rely on fixed-point math for coverage computation.
	// The (Radius/Count -> Coverage) LUT has precision up to 6 decimals, so quantizing to 10e6 should be enough
	const uint64 FixedPointCoverageScale = 1000000u;
	std::atomic<uint64> uAvgCoverage = 0;
	//std::atomic<float> AvgCoverage = 0;

	const uint32 VoxelCount = InNonEmptyVoxels.Num();
	#if 1
	ParallelFor(VoxelCount, 
	[
		InGridVoxelSize,
		&InNonEmptyVoxels,
		&InReferenceCoverage,
		InCurveIndexStart,
		InCurveIndexEnd,
		FixedPointCoverageScale,
		&uAvgCoverage
	] (uint32 VoxelIt) 
	#else
	for (uint32 VoxelIt=0;VoxelIt<VoxelCount;++VoxelIt)
	#endif
	{
		GroomBuilder_Voxelization::FHairGrid::FVoxel& Voxel = InNonEmptyVoxels[VoxelIt];

		uint32 CurveCount = 0;
		float AvgRadius = 0;

		uint32 RemoveCount = 0;
		for (GroomBuilder_Voxelization::FHairGrid::FCurve& Curve : Voxel.VoxelCurves)
		{
			if (InCurveIndexStart <= Curve.CurveIndex  && Curve.CurveIndex <= InCurveIndexEnd)
			{
				RemoveCount++;
			}
			else
			{
				CurveCount++;
				const float NormalizedRadius = Curve.Radius / InGridVoxelSize;
				AvgRadius += NormalizedRadius;
			}
		}

		AvgRadius /= FMath::Max(CurveCount, 1u);

		const float VoxelCoverage = GetHairCoverage(CurveCount, AvgRadius);
		const float CoverageScale = VoxelCoverage > 0 ? InReferenceCoverage[VoxelIt].Coverage / VoxelCoverage : 1.f;
		//AvgCoverage.fetch_add(CoverageScale, std::memory_order_relaxed);
		//AvgCoverage += CoverageScale;

		const uint64 uCoverageScale = CoverageScale * FixedPointCoverageScale;
		uAvgCoverage  += uCoverageScale;

		if (RemoveCount > 0)
		{
			Voxel.VoxelCurves.RemoveAll([InCurveIndexStart, InCurveIndexEnd](GroomBuilder_Voxelization::FHairGrid::FCurve& Curve) 
			{ 
				return InCurveIndexStart <= Curve.CurveIndex && Curve.CurveIndex <= InCurveIndexEnd;
			});
		}
	});

	const float AvgCoverage = uAvgCoverage / float(FixedPointCoverageScale);
	return AvgCoverage / float(FMath::Max(VoxelCount, 1u));
}

void ComputeHairCoverageScale(const FHairStrandsDatas& In, TArray<FCoverageScale>& OutCoverageScales)
{
	// 1. Voxelize curves
	GroomBuilder_Voxelization::FHairGrid Grid;
	GroomBuilder_Voxelization::Voxelize(In, Grid);

	// 2. Collect non-empty voxels
	TArray<GroomBuilder_Voxelization::FHairGrid::FVoxel> NonEmptyVoxels;
	NonEmptyVoxels.Reserve(Grid.Voxels.Num());
	for (const GroomBuilder_Voxelization::FHairGrid::FVoxel& Voxel : Grid.Voxels)
	{
		if (Voxel.VoxelCurves.Num() > 0)
		{
			NonEmptyVoxels.Add(Voxel);
		}
	}

	// 3. Compute reference coverage
	TArray<FVoxelCoverage> ReferenceCoverage;
	ComputeCoverage(Grid.VoxelSize, NonEmptyVoxels, ReferenceCoverage);

	// 4. Compute coverage scale for every X curves removed
	//    Compute scale for each decimated percent (i.e. 100 buckets)
	const uint32 CurveCount = In.GetNumCurves();
	uint32 BucketCount = FMath::Min(100u, CurveCount);
	const uint32 BucketSize = FMath::DivideAndRoundUp(CurveCount, BucketCount);
	BucketCount = FMath::DivideAndRoundUp(CurveCount, BucketSize);

	check(BucketSize > 0);
	check(BucketCount > 0);

	FCoverageScale DefaultValue;
	DefaultValue.ScreenSize = 1;
	DefaultValue.CoverageScale = 1;
	OutCoverageScales.Init(DefaultValue, BucketCount);
	if (BucketCount > 1u)
	{
		float MaxCoverageScale = 1.f;
		for (int32 BucketIt = BucketCount-1; BucketIt >= 0; --BucketIt)
		{
			const uint32 CurveIndexStart =  BucketIt * BucketSize;
			const uint32 CurveIndexEnd   = FMath::Min(CurveCount-1u, (BucketIt+1) * BucketSize - 1u);
			const uint32 CurveBucketCount = (CurveIndexEnd-CurveIndexStart)+1;
			const float CoverageScale = RemoveCurveAndComputeCoverageScaling(Grid.VoxelSize, NonEmptyVoxels, ReferenceCoverage, CurveIndexStart, CurveIndexEnd);
			MaxCoverageScale = FMath::Max(CoverageScale, MaxCoverageScale);
			OutCoverageScales[BucketIt].CoverageScale = MaxCoverageScale;
			OutCoverageScales[BucketIt].ScreenSize = 1.f; // TODO: compute the screen size / pixel area at which this decimation rate should be applied
		}
	}
}

} // namespace GroomBuilder_Voxelization

FHairStrandsVoxelData::FData FHairStrandsVoxelData::GetData(const FVector3f& P) const
{
	const uint32 MaxLookupDistance = FMath::Max3(Resolution.X, Resolution.Y, Resolution.Z) * 0.5f;
	const FIntVector C = GroomBuilder_Voxelization::ToCellCoord(P, MinBound, MaxBound, Resolution);

	FHairStrandsVoxelData::FData Out;
	Out.GroupIndex = FHairStrandsVoxelData::InvalidGroupIndex;
	Out.BaseColor = FVector3f::ZeroVector;
	Out.Roughness = 0.f;
	for (int32 Offset = 1; Offset <= int32(MaxLookupDistance); ++Offset)
	{
		// Center
		{
			bool bIsValid = false;
			const FIntVector CellCoord = GroomBuilder_Voxelization::ClampToVolume(C, Resolution, bIsValid);
			if (bIsValid) GroomBuilder_Voxelization::SearchCell(*this, CellCoord, Out);
		}

		// Top & Bottom
		for (int32 X = -Offset; X <= Offset; ++X)
		for (int32 Y = -Offset; Y <= Offset; ++Y)
		{
			bool bIsValid0 = false, bIsValid1 = false;
			const FIntVector CellCoord0 = GroomBuilder_Voxelization::ClampToVolume(C + FIntVector(X, Y, Offset), Resolution, bIsValid0);
			const FIntVector CellCoord1 = GroomBuilder_Voxelization::ClampToVolume(C + FIntVector(X, Y, -Offset), Resolution, bIsValid1);
			if (bIsValid0) GroomBuilder_Voxelization::SearchCell(*this, CellCoord0, Out);
			if (bIsValid1) GroomBuilder_Voxelization::SearchCell(*this, CellCoord1, Out);
		}

		const int32 OffsetMinusOne = Offset - 1;
		// Front & Back
		for (int32 X = -Offset; X <= Offset; ++X)
		for (int32 Z = -OffsetMinusOne; Z <= OffsetMinusOne; ++Z)
		{
			bool bIsValid0 = false, bIsValid1 = false;
			const FIntVector CellCoord0 = GroomBuilder_Voxelization::ClampToVolume(C + FIntVector(X, Offset, Z), Resolution, bIsValid0);
			const FIntVector CellCoord1 = GroomBuilder_Voxelization::ClampToVolume(C + FIntVector(X, -Offset, Z), Resolution, bIsValid1);
			if (bIsValid0) GroomBuilder_Voxelization::SearchCell(*this, CellCoord0, Out);
			if (bIsValid1) GroomBuilder_Voxelization::SearchCell(*this, CellCoord1, Out);
		}

		// Left & Right
		for (int32 Y = -OffsetMinusOne; Y <= OffsetMinusOne; ++Y)
		for (int32 Z = -OffsetMinusOne; Z <= OffsetMinusOne; ++Z)
		{
			bool bIsValid0 = false, bIsValid1 = false;
			const FIntVector CellCoord0 = GroomBuilder_Voxelization::ClampToVolume(C + FIntVector(Offset, Y, Z), Resolution, bIsValid0);
			const FIntVector CellCoord1 = GroomBuilder_Voxelization::ClampToVolume(C + FIntVector(-Offset, Y, Z), Resolution, bIsValid1);
			if (bIsValid0) GroomBuilder_Voxelization::SearchCell(*this, CellCoord0, Out);
			if (bIsValid1) GroomBuilder_Voxelization::SearchCell(*this, CellCoord1, Out);
		}

		// Early out if we have found a valid group index during a ring/layer step.
		if (Out.GroupIndex != InvalidGroupIndex)
		{
			break;
		}
	}

	// No valid group index has been found. In this case force the group index to 0
	if (Out.GroupIndex == FHairStrandsVoxelData::InvalidGroupIndex) { Out.GroupIndex = 0; }

	return Out;
}

void FGroomBuilder::VoxelizeGroupIndex(const FHairDescriptionGroups& In, FHairStrandsVoxelData& Out)
{
	GroomBuilder_Voxelization::FHairGrid Grid;
	GroomBuilder_Voxelization::Voxelize(In, Grid);
	Out.MinBound = Grid.MinBound;
	Out.MaxBound = Grid.MaxBound;
	Out.Resolution = Grid.Resolution;
	Out.Datas.SetNum(Grid.Voxels.Num());

	TArray<uint16> GroupBins;
	for (uint32 I=0,VoxelCount=Grid.Voxels.Num();I<VoxelCount;++I)
	{
		Out.Datas[I].GroupIndex = FHairStrandsVoxelData::InvalidGroupIndex;
		Out.Datas[I].BaseColor = FVector3f::ZeroVector;
		Out.Datas[I].Roughness = 0.f;

		// 1. For a given voxel, build an histogram of all the groups falling into this voxel
		GroupBins.Init(0, 64u);
		for (const GroomBuilder_Voxelization::FHairGrid::FCurve& Curve : Grid.Voxels[I].VoxelCurves)
		{
			check(Curve.GroupIndex != FHairStrandsVoxelData::InvalidGroupIndex);
			check(Curve.GroupIndex < GroupBins.Num());
			GroupBins[Curve.GroupIndex]++;
			Out.Datas[I].BaseColor += Curve.BaseColor;
			Out.Datas[I].Roughness += Curve.Roughness;
		}

		const uint32 CurveCount = Grid.Voxels[I].VoxelCurves.Num();
		if (CurveCount > 0)
		{
			Out.Datas[I].BaseColor /= float(CurveCount);
			Out.Datas[I].Roughness /= float(CurveCount);
		}

		// 2. Select the hair group having the largest count
		if (CurveCount > 1)
		{
			uint32 MaxBinCount = 0u;
			for (uint8 GroupIndex = 0, GroupCount = GroupBins.Num(); GroupIndex < GroupCount; ++GroupIndex)
			{
				if (GroupBins[GroupIndex] > MaxBinCount)
				{
					Out.Datas[I].GroupIndex = GroupIndex;
					MaxBinCount = GroupBins[GroupIndex];
				}
			}
		}
		else if (CurveCount == 1)
		{
			Out.Datas[I].GroupIndex = Grid.Voxels[I].VoxelCurves[0].GroupIndex;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Cluster culling data

namespace GroomBuilder_Cluster
{
struct FClusterGrid
{
	struct FCurve
	{
		FCurve()
		{
			for (uint8 LODIt = 0; LODIt < FHairClusterInfo::MaxLOD; ++LODIt)
			{
				CountPerLOD[LODIt] = 0;
			}
		}
		uint32 Offset = 0;
		uint32 Count = 0;
		uint32 CurveIndex = 0;
		float Area = 0;
		float AvgRadius = 0;
		float MaxRadius = 0;
		uint32 CountPerLOD[FHairClusterInfo::MaxLOD];
	};

	struct FCluster
	{
		float CurveAvgRadius = 0;
		float CurveMaxRadius = 0;
		float RootBoundRadius = 0;
		float Area = 0;
		TArray<FCurve> ClusterCurves;
	};

	FClusterGrid(const FIntVector& InResolution, const FVector3f& InMinBound, const FVector3f& InMaxBound)
	{
		MinBound = InMinBound;
		MaxBound = InMaxBound;
		GridResolution = InResolution;
		Clusters.SetNum(FMath::Max(GridResolution.X * GridResolution.Y * GridResolution.Z, 1));
	}

	FORCEINLINE bool IsValid(const FIntVector& P) const
	{
		return	0 <= P.X && P.X < GridResolution.X &&
			0 <= P.Y && P.Y < GridResolution.Y &&
			0 <= P.Z && P.Z < GridResolution.Z;
	}

	FORCEINLINE FIntVector ClampToVolume(const FIntVector& CellCoord, bool& bIsValid) const
	{
		bIsValid = IsValid(CellCoord);
		return FIntVector(
			FMath::Clamp(CellCoord.X, 0, GridResolution.X - 1),
			FMath::Clamp(CellCoord.Y, 0, GridResolution.Y - 1),
			FMath::Clamp(CellCoord.Z, 0, GridResolution.Z - 1));
	}

	FORCEINLINE FIntVector ToCellCoord(const FVector3f& P) const
	{
		bool bIsValid = false;
		const FVector3f F = ((P - MinBound) / (MaxBound - MinBound));
		const FIntVector CellCoord = FIntVector(FMath::FloorToInt(F.X * GridResolution.X), FMath::FloorToInt(F.Y * GridResolution.Y), FMath::FloorToInt(F.Z * GridResolution.Z));
		return ClampToVolume(CellCoord, bIsValid);
	}

	uint32 ToIndex(const FIntVector& CellCoord) const
	{
		uint32 CellIndex = CellCoord.X + CellCoord.Y * GridResolution.X + CellCoord.Z * GridResolution.X * GridResolution.Y;
		check(CellIndex < uint32(Clusters.Num()));
		return CellIndex;
	}

	void InsertRenderingCurve(FCurve& Curve, const FVector3f& Root)
	{
		FIntVector CellCoord = ToCellCoord(Root);
		uint32 Index = ToIndex(CellCoord);
		FCluster& Cluster = Clusters[Index];
		Cluster.ClusterCurves.Add(Curve);
	}

	FVector3f MinBound;
	FVector3f MaxBound;
	FIntVector GridResolution;
	TArray<FCluster> Clusters;
};

static void DecimateCurveLOD(
	const TArray<FVector3f>& InPoints,
	const uint32 InOffset,
	const uint32 InCount,
	const TArray<FHairLODSettings>& InSettings,
	uint32* OutCountPerLOD,
	TArray<uint8>& OutVertexLODMask)
{
	OutVertexLODMask.SetNum(InCount);

	// Insure that all settings have more and more agressive, and rectify it is not the case.
	TArray<FHairLODSettings> Settings = InSettings;
	{
		float PrevFactor = 1;
		float PrevAngle = 0;
		for (FHairLODSettings& S : Settings)
		{
			// Sanitize the decimation values
			S.CurveDecimation  = FMath::Clamp(S.CurveDecimation,  0.f,  1.f);
			S.VertexDecimation = FMath::Clamp(S.VertexDecimation, 0.f,  1.f);
			S.AngularThreshold = FMath::Clamp(S.AngularThreshold, 0.f, 90.f);

			if (S.VertexDecimation > PrevFactor)
			{
				S.VertexDecimation = PrevFactor;
			}

			if (S.AngularThreshold < PrevAngle)
			{
				S.AngularThreshold = PrevAngle;
			}

			PrevFactor = S.VertexDecimation;
			PrevAngle = S.AngularThreshold;
		}
	}

	check(InCount > 2);

	// Array containing the remaining vertex indices. This list get trimmed down as we process over all LODs.
	TArray<uint32> OutIndices;
	OutIndices.SetNum(InCount);
	for (uint32 CurveIt = 0; CurveIt < InCount; ++CurveIt)
	{
		OutIndices[CurveIt] = CurveIt;
	}

	const uint32 LODCount = Settings.Num();
	check(LODCount <= FHairClusterInfo::MaxLOD);

	for (uint8 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		const int32 LODTargetVertexCount = FMath::Clamp(uint32(FMath::CeilToInt(InCount * Settings[LODIt].VertexDecimation)), 2u, InCount);
		const float LODAngularThreshold  = FMath::DegreesToRadians(Settings[LODIt].AngularThreshold);

		// 'bCanDecimate' tracks if it is possible to reduce the remaining vertives even more while respecting the user angular constrain
		bool bCanDecimate = true;
		while (OutIndices.Num() > LODTargetVertexCount && bCanDecimate)
		{
			float MinError = FLT_MAX;
			int32 ElementToRemove = -1;
			const uint32 Count = OutIndices.Num();
			for (uint32 IndexIt = 1; IndexIt < Count - 1; ++IndexIt)
			{
				const FVector3f& P0 = InPoints[InOffset + OutIndices[IndexIt - 1]];
				const FVector3f& P1 = InPoints[InOffset + OutIndices[IndexIt]];
				const FVector3f& P2 = InPoints[InOffset + OutIndices[IndexIt + 1]];

				const float Area = FVector3f::CrossProduct(P0 - P1, P2 - P1).Size() * 0.5f;

				//     P0 .       . P2
				//         \Inner/
				//   ` .    \   /
				// Thres(` . \^/ ) Angle
				//    --------.---------
				//            P1
				const FVector3f V0 = (P0 - P1).GetSafeNormal();
				const FVector3f V1 = (P2 - P1).GetSafeNormal();
				const float InnerAngle = FMath::Abs(FMath::Acos(FVector3f::DotProduct(V0, V1)));
				const float Angle = (PI - InnerAngle) * 0.5f;

				if (Area < MinError && Angle < LODAngularThreshold)
				{
					MinError = Area;
					ElementToRemove = IndexIt;
				}
			}
			bCanDecimate = ElementToRemove >= 0;
			if (bCanDecimate)
			{
				OutIndices.RemoveAt(ElementToRemove);
			}
		}

		OutCountPerLOD[LODIt] = OutIndices.Num();

		// For all remaining vertices, we mark them as 'used'/'valid' for the current LOD levl
		for (uint32 LocalIndex : OutIndices)
		{
			OutVertexLODMask[LocalIndex] |= 1 << LODIt;
		}
	}

	// Sanity check to insure that vertex LOD in a continuous fashion.
	for (uint32 PointIt = 0; PointIt < InCount; ++PointIt)
	{
		const uint8 Mask = OutVertexLODMask[PointIt];
		check(Mask == 0 || Mask == 1 || Mask == 3 || Mask == 7 || Mask == 15 || Mask == 31 || Mask == 63 || Mask == 127 || Mask == 255);
	}
}


inline uint32 to10Bits(float V)
{
	return FMath::Clamp(uint32(V * 1024), 0u, 1023u);
}

static void BuildClusterData(
	const FHairStrandsDatas& InRenStrandsData,
	const float InGroomAssetRadius, 
	const FHairGroupsLOD& InSettings, 
	FHairStrandsClusterData& Out)
{
	// 0. Rest existing culling data
	Out.Reset();

	const uint32 LODCount = FMath::Min(uint32(InSettings.LODs.Num()), FHairClusterInfo::MaxLOD);
	check(LODCount > 0);

	const uint32 RenCurveCount = InRenStrandsData.GetNumCurves();
	Out.PointCount = InRenStrandsData.GetNumPoints();
	Out.CurveCount = InRenStrandsData.GetNumCurves();
	check(Out.PointCount > 0 && Out.CurveCount > 0);

	// 1. Compute the number of curves per LOD
	Out.LODInfos.SetNum(LODCount);
	for (uint8 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		const float CurveDecimation = FMath::Clamp(InSettings.LODs[LODIt].CurveDecimation, 0.0f, 1.0f);
		Out.LODInfos[LODIt].CurveCount = FMath::Clamp(FMath::CeilToInt(Out.CurveCount * CurveDecimation), 1, InRenStrandsData.GetNumCurves());
		Out.LODInfos[LODIt].PointCount = 0;
		Out.LODInfos[LODIt].RadiusScale= 1;
		Out.LODInfos[LODIt].ScreenSize = InSettings.LODs[LODIt].ScreenSize;
		Out.LODInfos[LODIt].bIsVisible = InSettings.LODs[LODIt].bVisible;
	}

	// 2. Compute control point LOD
	static_assert(FHairClusterInfo::MaxLOD == 8);
	TAtomic<uint32> ConcurrentPointCountPerLOD[FHairClusterInfo::MaxLOD] = { 0, 0, 0, 0, 0, 0, 0, 0 };
	Out.PointLODs.SetNum(InRenStrandsData.GetNumPoints());
	ParallelFor(Out.CurveCount, 
	[
		LODCount,
		&ConcurrentPointCountPerLOD,
		&InSettings,
		&InRenStrandsData,
		&Out
	] (uint32 CurveIt) 
	//for (uint32 CurveIt = 0; CurveIt < Out.CurveCount; ++CurveIt)
	{
		const uint32 PointOffset = InRenStrandsData.StrandsCurves.CurvesOffset[CurveIt];
		const uint32 PointCount  = InRenStrandsData.StrandsCurves.CurvesCount[CurveIt];

		uint32 CountPerLOD[FHairClusterInfo::MaxLOD];

		// 2.1 Write out cluster information
		// Don't need to DecimateCurve if there are only 2 control vertices
		TArray<uint8> PointLODMasks;
		PointLODMasks.SetNum(PointCount);
		if (PointCount > 2)
		{
			DecimateCurveLOD(
				InRenStrandsData.StrandsPoints.PointsPosition,
				PointOffset,
				PointCount,
				InSettings.LODs,
				CountPerLOD,
				PointLODMasks);

			// For stats
			for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
			{
				ConcurrentPointCountPerLOD[LODIt] += CountPerLOD[LODIt];
				//Out.LODInfos[LODIt].PointCount += CountPerLOD[LODIt];
			}
		}

		// 2.2 Write LOD mask per control point
		for (uint8 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
			{
				const uint8 LODMask = PointLODMasks[PointIt];
				if (LODMask & (1 << LODIt))
				{
					// Store the minimum/coarser LOD at which the current point is visible
					const uint32 GlobalPointIndex = PointOffset + PointIt;
					Out.PointLODs[GlobalPointIndex] = LODIt;
				}
			}
		}
	});
	// Copy back the 'concurrent PointCountPerLOD' into the final array
	for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
	{
		Out.LODInfos[LODIt].PointCount = ConcurrentPointCountPerLOD[LODIt];
	}

	// 3. Compute radius scale per LOD
	{
		struct FCurveInfo
		{
			float Area = 0;
			float MinRadius = FLT_MAX;
			float MaxRadius = 0;
			float AvgRadius = 0;
		};
		// Compute curves area
		TArray<FCurveInfo> CurveInfos;
		CurveInfos.SetNum(RenCurveCount);
		float AvgCurveArea = 0;
		for (uint32 CurveIt = 0; CurveIt < Out.CurveCount; ++CurveIt)
		{
			FCurveInfo& CurveInfo = CurveInfos[CurveIt];
			const uint32 PointOffset = InRenStrandsData.StrandsCurves.CurvesOffset[CurveIt];
			const uint32 PointCount  = InRenStrandsData.StrandsCurves.CurvesCount[CurveIt];

			// Compute area of each curve to later compute area correction
			for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
			{
				uint32 PointGlobalIndex = PointIt + PointOffset;
				const FVector3f& V0 = InRenStrandsData.StrandsPoints.PointsPosition[PointGlobalIndex];
				if (PointIt > 0)
				{
					const FVector3f& V1 = InRenStrandsData.StrandsPoints.PointsPosition[PointGlobalIndex - 1];
					FVector3f OutDir;
					float OutLength;
					(V1 - V0).ToDirectionAndLength(OutDir, OutLength);
					CurveInfo.Area += InRenStrandsData.StrandsPoints.PointsRadius[PointGlobalIndex] * OutLength;
				}

				const float PointRadius = InRenStrandsData.StrandsPoints.PointsRadius[PointGlobalIndex];
				CurveInfo.AvgRadius += PointRadius;
				CurveInfo.MaxRadius = FMath::Max(CurveInfo.MaxRadius, PointRadius);
				CurveInfo.MinRadius = FMath::Max(CurveInfo.MinRadius, PointRadius);
			}
			CurveInfo.AvgRadius /= FMath::Max(1u, PointCount);

			AvgCurveArea += CurveInfo.Area;
		}
		AvgCurveArea /= FMath::Max(1u, Out.CurveCount);

		const FVector3f GroupMinBound(InRenStrandsData.BoundingBox.Min);
		const FVector3f GroupMaxBound(InRenStrandsData.BoundingBox.Max);
		const FVector3f GroupSize = GroupMaxBound - GroupMinBound;
		const FVector3f Area3(GroupSize.X * GroupSize.Y, GroupSize.X * GroupSize.Z, GroupSize.Y * GroupSize.Z);
		const float MaxArea = FMath::Max3(Area3.X, Area3.Y, Area3.Z);
		const float AvgArea = (Area3.X + Area3.Y + Area3.Z) / 3.f;
		//		const float MaxArea = FMath::Max3(GroupSize.X, GroupSize.Y, GroupSize.Z);

		// Reference: "Stochastic Simplification of Aggregate Detail"
		const float DepthComplexity = Out.CurveCount * AvgCurveArea / MaxArea;
		const float r = DepthComplexity / Out.CurveCount;

		for (uint8 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			const float Lambda = InSettings.LODs[LODIt].CurveDecimation;
			// For now, for legacy reason, do not use Stochastic simplification, but rather use manual thickness compensation if provided (default=1.f)
			//Out.LODInfos[LODIt].RadiusScale = (1.f - FMath::Pow(1.f - r, 1.f / Lambda)) / r;
			Out.LODInfos[LODIt].RadiusScale = InSettings.LODs[LODIt].ThicknessScale;
		}
	}

	// Compute the number of cluster based on a voxelization of the groom.
	uint32 ClusterCount = 0;
	{
		// 1. Allocate cluster per voxel containing contains >=1 render curve root
		FVector3f GroupMinBound(InRenStrandsData.BoundingBox.Min);
		FVector3f GroupMaxBound(InRenStrandsData.BoundingBox.Max);

		// Compute the voxel volume resolution, and snap the max bound to the voxel grid
		// Iterate until voxel size are below max resolution, so that computation is not too long
		FIntVector VoxelResolution = FIntVector::ZeroValue;
		{
			const int32 MaxResolution = FMath::Max(GHairClusterBuilder_MaxVoxelResolution, 2);

			float ClusterWorldSize = 1.f; //cm
			bool bIsValid = false;
			while (!bIsValid)
			{
				FVector3f VoxelResolutionF = (GroupMaxBound - GroupMinBound) / ClusterWorldSize;
				VoxelResolution = FIntVector(FMath::CeilToInt(VoxelResolutionF.X), FMath::CeilToInt(VoxelResolutionF.Y), FMath::CeilToInt(VoxelResolutionF.Z));
				bIsValid = VoxelResolution.X <= MaxResolution && VoxelResolution.Y <= MaxResolution && VoxelResolution.Z <= MaxResolution;
				if (!bIsValid)
				{
					ClusterWorldSize *= 2;
				}
			}
			GroupMaxBound = GroupMinBound + FVector3f(VoxelResolution) * ClusterWorldSize;
		}

		// 2. Insert all rendering curves into the voxel structure
		FClusterGrid ClusterGrid(VoxelResolution, GroupMinBound, GroupMaxBound);
		for (uint32 RenCurveIndex = 0; RenCurveIndex < RenCurveCount; ++RenCurveIndex)
		{
			FClusterGrid::FCurve RCurve;
			RCurve.CurveIndex = RenCurveIndex;
			RCurve.Count = InRenStrandsData.StrandsCurves.CurvesCount[RenCurveIndex];
			RCurve.Offset = InRenStrandsData.StrandsCurves.CurvesOffset[RenCurveIndex];
			RCurve.Area = 0.0f;
			RCurve.AvgRadius = 0;
			RCurve.MaxRadius = 0;

			// Compute area of each curve to later compute area correction
			for (uint32 RenPointIndex = 0; RenPointIndex < RCurve.Count; ++RenPointIndex)
			{
				uint32 PointGlobalIndex = RenPointIndex + RCurve.Offset;
				const FVector3f& V0 = InRenStrandsData.StrandsPoints.PointsPosition[PointGlobalIndex];
				if (RenPointIndex > 0)
				{
					const FVector3f& V1 = InRenStrandsData.StrandsPoints.PointsPosition[PointGlobalIndex - 1];
					FVector3f OutDir;
					float OutLength;
					(V1 - V0).ToDirectionAndLength(OutDir, OutLength);
					RCurve.Area += InRenStrandsData.StrandsPoints.PointsRadius[PointGlobalIndex] * OutLength;
				}

				const float PointRadius = InRenStrandsData.StrandsPoints.PointsRadius[PointGlobalIndex];
				RCurve.AvgRadius += PointRadius;
				RCurve.MaxRadius = FMath::Max(RCurve.MaxRadius, PointRadius);
			}
			RCurve.AvgRadius /= FMath::Max(1u, RCurve.Count);

			const FVector3f Root = InRenStrandsData.StrandsPoints.PointsPosition[RCurve.Offset];
			ClusterGrid.InsertRenderingCurve(RCurve, Root);
		}

		// 3. Count non-empty clusters
		{
			for (FClusterGrid::FCluster& Cluster : ClusterGrid.Clusters)
			{
				if (Cluster.ClusterCurves.Num() > 0)
				{
					++ClusterCount;
				}
			}
		}
	}

	// 4. Fill in cluster Info for legacy compatibility
	const uint32 MinClusterCount = FMath::Max(FMath::DivideAndRoundUp(InRenStrandsData.GetNumCurves(), 100u), 128u); // 1%
	const uint32 MaxClusterCount = 4096;
	Out.ClusterCount = FMath::Max(MinClusterCount, ClusterCount);
	Out.ClusterCount = FMath::Min(Out.ClusterCount, InRenStrandsData.GetNumCurves()); // Can't have more clusters than curve, since we use curves as proxy for cluster AABB
	Out.ClusterScale = 1.f;
	if (ClusterCount > MaxClusterCount)
	{
		Out.ClusterScale = float(ClusterCount) / float(MaxClusterCount);
		Out.ClusterCount = MaxClusterCount;
	}
	Out.CurveToClusterIds.SetNum(InRenStrandsData.GetNumCurves());
	Out.ClusterInfos.SetNum(Out.ClusterCount);
	const uint32 CurvePerCluster = FMath::DivideAndRoundUp(InRenStrandsData.GetNumCurves(), Out.ClusterCount);
	// Every 'CurvePerCluster' is assigned to the current cluster
	for (uint32 CurveIt = 0, CurveCount=InRenStrandsData.GetNumCurves(); CurveIt < CurveCount; ++CurveIt)
	{
		Out.CurveToClusterIds[CurveIt] = CurveIt % CurvePerCluster;
	}
	for (uint32 ClusterIt = 0; ClusterIt < Out.ClusterCount; ++ClusterIt)
	{
		Out.ClusterInfos[ClusterIt].LODCount = LODCount;
		for (uint8 LODIt = 0; LODIt < LODCount; ++LODIt)
		{
			Out.ClusterInfos[ClusterIt].ScreenSize[LODIt]  = Out.LODInfos[LODIt].ScreenSize;
			Out.ClusterInfos[ClusterIt].RadiusScale[LODIt] = Out.LODInfos[LODIt].RadiusScale;
			Out.ClusterInfos[ClusterIt].bIsVisible[LODIt]  = Out.LODInfos[LODIt].bIsVisible;
		}
	}
}

static void BuildClusterBulkData(
	const FHairStrandsClusterData& In,
	FHairStrandsClusterBulkData& Out)
{
	Out.Reset();
	Out.Header.ClusterCount		= In.ClusterCount;
	Out.Header.ClusterScale		= In.ClusterScale;
	Out.Header.PointCount		= In.PointCount;
	Out.Header.CurveCount		= In.CurveCount;
	Out.Header.LODInfos			= In.LODInfos;

	Out.Header.Strides.PackedClusterInfoStride = FHairClusterInfoFormat::SizeInByte;
	Out.Header.Strides.CurveToClusterIdStride = FHairClusterIndexFormat::SizeInByte;
	Out.Header.Strides.PointLODStride = sizeof(uint32);

	// Sanity check
	check(Out.Header.ClusterCount	== uint32(In.ClusterInfos.Num()));
	check(Out.Header.CurveCount		== uint32(In.CurveToClusterIds.Num()));
	
	HairStrandsBuilder::CopyToBulkData<FHairClusterIndexFormat>(Out.Data.CurveToClusterIds, In.CurveToClusterIds);
	ReportSize(TEXT("CurveToClusterIds"), In.CurveToClusterIds);

	// Pack point LOD data
	// |                          32bits                               |
	// |      4bits    |      4bits    |      4bits    |      4bits    |
	// | Point0 MinLOD | Point1 MinLOD | Point2 MinLOD | Point3 MinLOD |
	static_assert(HAIR_POINT_LOD_BIT_COUNT == 4);
	TArray<uint32> PackedPointLODs;
	PackedPointLODs.SetNum(FMath::DivideAndRoundUp(uint32(In.PointLODs.Num()), HAIR_POINT_LOD_COUNT_PER_UINT));
	for (uint32 PackedIt = 0, PackedCount = PackedPointLODs.Num(); PackedIt<PackedCount; ++PackedIt)
	{
		uint32 Packed = 0;
		for (uint32 It = 0; It < HAIR_POINT_LOD_COUNT_PER_UINT; ++It)
		{
			const uint32 InIndex = FMath::Min(PackedIt * HAIR_POINT_LOD_COUNT_PER_UINT + It, uint32(In.PointLODs.Num()-1));
			Packed = Packed | ((In.PointLODs[InIndex] & 0xF) << (It * HAIR_POINT_LOD_BIT_COUNT));
		}
		PackedPointLODs[PackedIt] = Packed;
	}
	HairStrandsBuilder::CopyToBulkData<FHairClusterIndexFormat>(Out.Data.PointLODs, PackedPointLODs);
	ReportSize(TEXT("PointLODs"), PackedPointLODs);

	// Pack LODInfo into GPU format
	{
		check(uint32(In.ClusterInfos.Num()) == Out.Header.ClusterCount);
		check(uint32(In.CurveToClusterIds.Num()) == Out.Header.CurveCount);
	
		float MinScreenSize = FLT_MAX;
		float MaxScreenSize = 0.f; 

		float MinRadiusScale = FLT_MAX; 
		float MaxRadiusScale = 0.f; 
		for (const FHairClusterInfo& Info : In.ClusterInfos)
		{
			for (uint32 LODIt = 0; LODIt < Info.LODCount; ++LODIt)
			{
				MinScreenSize = FMath::Min(MinScreenSize, Info.ScreenSize[LODIt]);
				MaxScreenSize = FMath::Max(MaxScreenSize, Info.ScreenSize[LODIt]);

				MinRadiusScale = FMath::Min(MinRadiusScale, Info.RadiusScale[LODIt]);
				MaxRadiusScale = FMath::Max(MaxRadiusScale, Info.RadiusScale[LODIt]);
			}
		}
		check(MinScreenSize <= MaxScreenSize);
		check(MinRadiusScale <= MaxRadiusScale);

		const float ScreenOffset = MinScreenSize;
		const float ScreenScale = FMath::Abs(MaxScreenSize - MinScreenSize) > 1e-4f ? (MaxScreenSize - MinScreenSize) : 1.f;
		const float InvScreenScale = 1.f / ScreenScale;

		const float RadiusOffset = MinRadiusScale;
		const float RadiusScale = FMath::Abs(MaxRadiusScale - MinRadiusScale) > 1e-4f ? (MaxRadiusScale - MinRadiusScale) : 1.f;
		const float InvRadiusScale = 1.f / RadiusScale;

		Out.Header.ClusterInfoParameters = FVector4f(ScreenScale, ScreenOffset, RadiusScale, RadiusOffset);

		auto Quantize8Bits = [](float In, float InScale, float InOffset)
		{
			const float N = (In - InOffset) * InScale;
			return FMath::Clamp(uint32(N * 0xFFu), 0u, 0xFFu);
		};

		TArray<FHairClusterInfo::Packed> PackedClusterInfos;
		PackedClusterInfos.Reserve(In.ClusterInfos.Num());
		for (const FHairClusterInfo& Info : In.ClusterInfos)
		{
			FHairClusterInfo::Packed& Packed = PackedClusterInfos.AddDefaulted_GetRef();

			uint32 bIsVisible = 0;
			for (uint32 LODIt = 0; LODIt < FHairClusterInfo::MaxLOD; ++LODIt)
			{
				if (Info.bIsVisible[LODIt])
				{
					bIsVisible = bIsVisible | (1 << LODIt);
				}
			}

			// Sanity check
			check(Info.LODCount <= 8);

			Packed.Screen0 = Info.LODCount;
			Packed.Screen0 = Packed.Screen0 | (Quantize8Bits(Info.ScreenSize[1], InvScreenScale, ScreenOffset) << 8);
			Packed.Screen0 = Packed.Screen0 | (Quantize8Bits(Info.ScreenSize[2], InvScreenScale, ScreenOffset) << 16);
			Packed.Screen0 = Packed.Screen0 | (Quantize8Bits(Info.ScreenSize[3], InvScreenScale, ScreenOffset) << 24);

			Packed.Screen1 = 0;
			Packed.Screen1 = Packed.Screen1 | (Quantize8Bits(Info.ScreenSize[4], InvScreenScale, ScreenOffset) << 0);
			Packed.Screen1 = Packed.Screen1 | (Quantize8Bits(Info.ScreenSize[5], InvScreenScale, ScreenOffset) << 8);
			Packed.Screen1 = Packed.Screen1 | (Quantize8Bits(Info.ScreenSize[6], InvScreenScale, ScreenOffset) << 16);
			Packed.Screen1 = Packed.Screen1 | (Quantize8Bits(Info.ScreenSize[7], InvScreenScale, ScreenOffset) << 24);

			Packed.Radius0 = bIsVisible & 0xFF;
			Packed.Radius0 = Packed.Radius0 | (Quantize8Bits(Info.RadiusScale[1], InvRadiusScale, RadiusOffset) << 8);
			Packed.Radius0 = Packed.Radius0 | (Quantize8Bits(Info.RadiusScale[2], InvRadiusScale, RadiusOffset) << 16);
			Packed.Radius0 = Packed.Radius0 | (Quantize8Bits(Info.RadiusScale[3], InvRadiusScale, RadiusOffset) << 24);

			Packed.Radius1 = 0;
			Packed.Radius1 = Packed.Radius1 | (Quantize8Bits(Info.RadiusScale[4], InvRadiusScale, RadiusOffset) << 0);
			Packed.Radius1 = Packed.Radius1 | (Quantize8Bits(Info.RadiusScale[5], InvRadiusScale, RadiusOffset) << 8);
			Packed.Radius1 = Packed.Radius1 | (Quantize8Bits(Info.RadiusScale[6], InvRadiusScale, RadiusOffset) << 16);
			Packed.Radius1 = Packed.Radius1 | (Quantize8Bits(Info.RadiusScale[7], InvRadiusScale, RadiusOffset) << 24);

			static_assert(sizeof(FHairClusterInfo::Packed) == sizeof(FHairClusterInfo::BulkType));
		}

		HairStrandsBuilder::CopyToBulkData<FHairClusterInfoFormat>(Out.Data.PackedClusterInfos, PackedClusterInfos);
		ReportSize(TEXT("ClusterInfos"), PackedClusterInfos);
	}
}

} // namespace GroomBuilder_Cluster

void FGroomBuilder::BuildClusterBulkData(
	const FHairStrandsDatas& InRenStrandsData,
	const float InGroomAssetRadius,
	const FHairGroupsLOD& InSettings,
	FHairStrandsClusterBulkData& Out)
{
	FHairStrandsClusterData ClusterData;
	GroomBuilder_Cluster::BuildClusterData(InRenStrandsData, InGroomAssetRadius, InSettings, ClusterData);
	GroomBuilder_Cluster::BuildClusterBulkData(ClusterData, Out);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Asynchronous queuing for hair strands positions readback

class FHairStrandCopyPositionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandCopyPositionCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandCopyPositionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MaxVertexCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InPositions)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InPositionOffset)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutPositions)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_READ_POSITIONS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandCopyPositionCS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainCS", SF_Compute);

static FRDGBufferRef ReadPositions(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FHairGroupInstance* Instance,
	const bool bReadGuides, bool& bHasValidResource)
{
	check(Instance);
	const uint32 NumVertices = bReadGuides ? Instance->Guides.GetData().GetNumPoints() : Instance->HairGroupPublicData->GetActiveStrandsPointCount();
	
	auto ReadData = [](FRDGBuilder& GraphBuilder, FHairStrandsRestResource* InRest, FHairStrandsDeformedResource* InDeformedResource, FRDGBufferSRVRef& InPositions, FRDGBufferSRVRef& InPositionOffset)
	{
		if (InDeformedResource)
		{
			InPositions = Register(GraphBuilder, InDeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateSRV).SRV;
			InPositionOffset = Register(GraphBuilder, InDeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateSRV).SRV;
		}
		else
		{
			InPositions = Register(GraphBuilder, InRest->PositionBuffer, ERDGImportedBufferFlags::CreateSRV).SRV;
			InPositionOffset = Register(GraphBuilder, InRest->PositionOffsetBuffer, ERDGImportedBufferFlags::CreateSRV).SRV;
		}
	};

	FRDGBufferSRVRef InPositions = nullptr;
	FRDGBufferSRVRef InPositionOffset = nullptr;
	if (bReadGuides)
	{
		ReadData(GraphBuilder, Instance->Guides.RestResource,  Instance->Guides.DeformedResource, InPositions, InPositionOffset);
	}
	else
	{
		ReadData(GraphBuilder, Instance->Strands.RestResource, Instance->Strands.DeformedResource, InPositions, InPositionOffset);
	}
	
	FRDGBufferRef OutPositions = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FVector4f), NumVertices), TEXT("StrandsPositionForReadback"));
	bHasValidResource = InPositions && InPositionOffset;
	if(bHasValidResource)
	{
		FHairStrandCopyPositionCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandCopyPositionCS::FParameters>();
		Parameters->MaxVertexCount = NumVertices;
		Parameters->InPositionOffset = InPositionOffset;
		Parameters->InPositions = InPositions;
		Parameters->OutPositions = GraphBuilder.CreateUAV(OutPositions, PF_A32B32G32R32F);

		FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(NumVertices, 128);

		TShaderMapRef<FHairStrandCopyPositionCS> ComputeShader(ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::CopyPositions"),
			ComputeShader,
			Parameters,
			DispatchCount);
	}

	return OutPositions;
}

struct FStrandsPositionData
{
	TUniquePtr<FRHIGPUBufferReadback> GPUPositions;
	uint32 GroupIt = 0;
	uint32 GroupCount = 0;
	uint32 ReadbackDelay = 0;
	uint32 CurveCount = 0;
	uint32 PointCount = 0;
	FSceneInterface* Scene;
	TRefCountPtr<FHairGroupInstance> GroupInstance = nullptr;
	TSharedPtr<FStrandsPositionOutput> Output;
	bool bReadGuides = false;
};

TQueue<FStrandsPositionData*>		GStrandsPositionQueries;
TQueue<FStrandsPositionData*>		GStrandsPositionReadbacks;

bool HasHairStrandsPositionQueries()
{
	return !GStrandsPositionQueries.IsEmpty() || !GStrandsPositionReadbacks.IsEmpty();
}

void RunHairStrandsPositionQueries(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const struct FShaderPrintData* DebugShaderData, const FSceneView* SceneView)
{
	// Operations are ordered in reverse to ensure they are processed on independent frames to avoid TDR on heavy grooms.

	// 2. Readback positions
	{
		TArray<FStrandsPositionData*> QueryNotReady;
		FStrandsPositionData* Q = nullptr;
		while (GStrandsPositionReadbacks.Dequeue(Q))
		{
			if(Q && Q->Scene && Q->GroupInstance &&
				SceneView && SceneView->Family && (Q->Scene == SceneView->Family->Scene))
			{
				if (Q->GPUPositions->IsReady() || Q->ReadbackDelay == 0)
				{
					const TRefCountPtr<FHairGroupInstance> Instance = Q->GroupInstance;

					check(Q->GroupIt < uint32(Q->Output->Groups.Num()));
					FStrandsPositionOutput::FGroup& Group = Q->Output->Groups[Q->GroupIt];
					Group.Reserve(Q->CurveCount);
		
					FStrandsPositionOutput::FStrand Strand;
					const FVector4f* Positions = (const FVector4f*)Q->GPUPositions->Lock(sizeof(FVector4f) * Q->PointCount);
					for (uint32 PointIt = 0; PointIt < Q->PointCount; ++PointIt)
					{
						const FVector4f P = Positions[PointIt];
						check(P.W != HAIR_CONTROLPOINT_START || Strand.Num() == 0);
						check(P.W != HAIR_CONTROLPOINT_INSIDE || Strand.Num() > 0);

						Strand.Add(FVector3f(P.X, P.Y, P.Z));
						if (P.W == HAIR_CONTROLPOINT_END)
						{
							Group.Add(Strand);
							Strand.SetNum(0);
						}
					}
					check(Q->CurveCount == Group.Num());
					Q->GPUPositions->Unlock();

					--Q->Output->Status;
					check(Q->Output->Status >= 0);
				}
				else
				{
					if (Q->ReadbackDelay > 0) { --Q->ReadbackDelay; }
					QueryNotReady.Add(Q);
				}
			}
			else
			{
				QueryNotReady.Add(Q);
			}
		}
		for (FStrandsPositionData* N : QueryNotReady)
		{
			GStrandsPositionReadbacks.Enqueue(N);
		}
	}

	// 1. Copy positions
	{
		TArray<FStrandsPositionData*> QueryNotReady;
		FStrandsPositionData* Q = nullptr;
		while (GStrandsPositionQueries.Dequeue(Q))
		{
			if(Q && Q->Scene && Q->GroupInstance &&
				SceneView && SceneView->Family && (Q->Scene == SceneView->Family->Scene))
			{
				if (const TRefCountPtr<FHairGroupInstance> Instance = Q->GroupInstance)
				{
					// Require RestResource (the original crash path dereferenced it via Guides.GetData()).
					const bool bGuidesOk  =  Q->bReadGuides && Instance->Guides.IsValid();
					const bool bStrandsOk = !Q->bReadGuides && Instance->HairGroupPublicData && Instance->Strands.IsValid();
					if (!bGuidesOk && !bStrandsOk)
					{
						--Q->Output->Status;
						check(Q->Output->Status >= 0);
						continue;
					}

					Q->CurveCount = Q->bReadGuides ? Instance->Guides.GetData().GetNumCurves() : Instance->HairGroupPublicData->GetActiveStrandsCurveCount();
					Q->PointCount = Q->bReadGuides ? Instance->Guides.GetData().GetNumPoints() : Instance->HairGroupPublicData->GetActiveStrandsPointCount();

					bool bHasValidResources = true;
					FRDGBufferRef Positions = ReadPositions(GraphBuilder, ShaderMap, Instance, Q->bReadGuides, bHasValidResources);
					if(bHasValidResources)
					{
						AddEnqueueCopyPass(GraphBuilder, Q->GPUPositions.Get(), Positions, Positions->Desc.GetSize());
						GStrandsPositionReadbacks.Enqueue(Q);
					}
					else
					{
						--Q->Output->Status;
						check(Q->Output->Status >= 0);
					}
				}
				else
				{
					// If the instance is missing, Q->Output->Groups[GroupIt] remains an empty array
					// in this case, skip the readback to prevent the query from remaining unfinished
					--Q->Output->Status;
					check(Q->Output->Status >= 0);
				}
			}
			else
			{
				QueryNotReady.Add(Q);
			}
		}
		for (FStrandsPositionData* N : QueryNotReady)
		{
			GStrandsPositionQueries.Enqueue(N);
		}
	}
}

// Forward declaration of the GroomComponent function that is retrieving a ref counted group instance ptr given an index
TRefCountPtr<FHairGroupInstance> GetRefCountedHairGroupInstance(const UGroomComponent* GroomComponent, const int32 GroupIndex);


#if WITH_EDITOR
bool RequestStrandsPosition(const UGroomComponent* Component, TSharedPtr<FStrandsPositionOutput> Output, const bool bReadGuides)
{
	if (Component == nullptr || Output == nullptr)
	{
		return false;
	}

	const uint32 GroupCount = Component->GetGroupCount();
	for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
	{
		FStrandsPositionData* Data = new FStrandsPositionData();
		Data->GroupIt = GroupIt;
		Data->GroupCount = GroupCount;
		Data->GroupInstance = GetRefCountedHairGroupInstance(Component, GroupIt);
		Data->Scene = (Component && Component->GetWorld()) ? Component->GetWorld()->Scene : nullptr;
		Data->ReadbackDelay = 3;
		Data->CurveCount = 0;
		Data->PointCount = 0;
		Data->GPUPositions = MakeUnique<FRHIGPUBufferReadback>(TEXT("Readback.Positions"));
		Data->Output = Output;
		Data->bReadGuides = bReadGuides;
		GStrandsPositionQueries.Enqueue(Data);
	}
	
	*Output = FStrandsPositionOutput();
	Output->Component = Component;
	Output->Status = GroupCount;
	Output->Groups.SetNum(GroupCount);
	return true;
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Experimental curve rendering

namespace HairStrandsBuilder
{

static uint32 PackR10G10B10F(const FVector3f& rgb)
{
	uint32 r = (FFloat16(rgb.X).Encoded << 15) & 0x3FF00000;	// 0011 1111 1111 0000 0000 0000 0000 0000 
	uint32 g = (FFloat16(rgb.Y).Encoded << 5)  & 0x000FFC00;	// 0000 0000 0000 1111 1111 1100 0000 0000 
	uint32 b = (FFloat16(rgb.Z).Encoded >> 5)  & 0x000003FF;	// 0000 0000 0000 0000 0000 0011 1111 1111
	return r | g | b;
}

static FVector3f UnpackR10G10B10F(uint32 rgb)
{
	FFloat16 r, g, b;
	r.Encoded = (rgb >> 15) & 0x7FE0;
	g.Encoded = (rgb >> 5) & 0x7FE0;
	b.Encoded = (rgb << 5) & 0x7FE0;
	return FVector3f(r, g, b);
}

// Packed cluster data (=GPU data before transcoding)
// * The numnber of curve is constant per cluster (CurveCount = 64)
// * The number of point per curve is constant within a cluster (PointPerCurve = 2/4/8/16/32/64)
struct FPackedCluster
{
	// 32bytes header
	struct FHeader 
	{
		// 16b
		FVector3f Center = FVector3f::ZeroVector;
		uint8 CurvePerCluster = 0;
		uint8 PointPerCurve = 0;
		uint8 CurveCount = 0;
		uint8 Pad0 = 0;

		// 16b 
		FFloat16 MaxLength = 0;
		FFloat16 MaxRadius = 0;
		FFloat16 LocalBoundCenterX = 0;
		FFloat16 LocalBoundCenterY = 0;
		FFloat16 LocalBoundCenterZ = 0;
		FFloat16 LocalBoundExtentX = 0;
		FFloat16 LocalBoundExtentY = 0;
		FFloat16 LocalBoundExtentZ = 0;

	} Header;

	// * Data  2pt x 64c x 8bppt = 1024b  = 1k
	// * Data 64pt x 64c x 8bppt = 65536b = 64k
	// * Stored in SOA | C0_P0 C1_P0 C2_P0 ... | C0_P1 C1_P1 C2_P1 ... | C0_P2 C1_P2 C2_P2 ... |
	struct FData
	{
		TArray<FHairStrandsPositionFormat::Type> PackedPoints;
	} Data;

	uint32 GetSizeInBytes() const
	{
		return sizeof(FPackedCluster::FHeader) + Data.PackedPoints.GetTypeSize() * Data.PackedPoints.Num();
	}
};

/** Build the bulk/packed datas for gpu rendering/simulation */
void InternalBuildRenderCurveResourceBulkData(const FHairStrandsDatas& In, FRenderCurveResourceData& Out)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(HairStrandsBuilder::InternalBuildRenderCurveResourceBulkData);

	const uint32 NumCurves = In.GetNumCurves();
	const uint32 NumPoints = In.GetNumPoints();
	if (!(NumCurves > 0 && NumPoints > 0))
	{
		return;
	}

	// Resample curves
	const uint32 CurvePerCluster = 64u;
	const uint32 PointPerCurve = 16u; // TODO: Use histogram for picking correct value? Pixel metric needs to take that into account
	const uint32 ClusterCount = FMath::DivideAndRoundUp(In.GetNumCurves(), CurvePerCluster);

	static_assert(CurvePerCluster == RENDERCURVE_CURVE_PER_CLUSTER);

	const FHairStrandsCurves& Curves = In.StrandsCurves;
	const FHairStrandsPoints& Points = In.StrandsPoints;

	uint32 MinPointPerCurve = HAIR_MAX_NUM_POINT_PER_CURVE;
	uint32 MaxPointPerCurve = 0;
	uint32 AccPointPerCurve = 0;

	struct FCurve
	{
		TArray<FVector3f> Points;
		TArray<float> UCoord;
		TArray<float> Radius;
		TArray<bool>  bValid;
	};

	struct FCluster
	{
		uint32 PointPerCurve = 0;
		uint32 CurvePerCluster = 0;
		uint32 CurveCount = 0;
		float MaxRadius = 0.f;
		float MaxLength = 0.f;
		FVector3f BoundCenter = FVector3f::ZeroVector;
		FVector3f BoundExtent = FVector3f::ZeroVector;
		TArray<FCurve> Curves;
	};

	TArray<FCluster> Clusters;
	Clusters.SetNum(ClusterCount);

	// Find clusters
	TArray<TArray<int32>> ClusterCurveIndices;
	{
		// Extract curves' root position
		TArray<FVector3f> RootPoints;
		RootPoints.Reserve(In.GetNumCurves());
		for (uint32 CurveIt = 0; CurveIt < In.GetNumCurves(); ++CurveIt)
		{
			const uint32 PointOffset = In.StrandsCurves.CurvesOffset[CurveIt];
			const FVector3f RootPosition = In.StrandsPoints.PointsPosition[PointOffset];
			RootPoints.Add(RootPosition);
		}

		// Kmeans for extracting clusters
		UE::Geometry::FClusterKMeans Cluster;
		int32 NumInitialPoints = RootPoints.Num();
		int32 NumClusters = ClusterCount;
		
		TArray<FVector3f> Centers;
		Cluster.GetUniformSpacedInitialCenters<FVector3f>(RootPoints, NumClusters, Centers);
		int32 FoundClusters = Cluster.ComputeClusters<FVector3f, true/*bParallel*/>(RootPoints, NumClusters, Centers);

		// Verify cluster counts
		check(Cluster.ClusterSizes.Num() == FoundClusters);
		TArray<TArray<int32>> ClusterIndices;
		Cluster.GetClusters(ClusterIndices);
		check(ClusterIndices.Num() == FoundClusters);

		// Verify that each point is assigned a cluster index
		TArray<int32> SeenCount;
		SeenCount.Init(0, NumInitialPoints);
		for (int32 ClusterIdx = 0; ClusterIdx < ClusterIndices.Num(); ++ClusterIdx)
		{
			check(ClusterIndices[ClusterIdx].Num() == Cluster.ClusterSizes[ClusterIdx]);
			for (int32 PointIdx : ClusterIndices[ClusterIdx])
			{
				SeenCount[PointIdx]++;
			}
		}
		if (NumClusters > 0)
		{
			for (int32 PointIdx = 0; PointIdx < NumInitialPoints; ++PointIdx)
			{
				check(SeenCount[PointIdx] == 1);
			}
		}

		ClusterCurveIndices = ClusterIndices;
	}

	// Build clusters' data
	for (uint32 ClusterIndex = 0; ClusterIndex < ClusterCount; ++ClusterIndex)
	{
		FCluster& Cluster = Clusters[ClusterIndex];
		Cluster.Curves.SetNum(CurvePerCluster);
		Cluster.CurveCount   	= 0;
		Cluster.PointPerCurve   = PointPerCurve;
		Cluster.CurvePerCluster = CurvePerCluster;
		Cluster.MaxRadius       = 0.f;
		Cluster.MaxLength       = 0.f;

		FBox3f ClusterBound(ForceInit);
		for (uint32 CurveIt = 0; CurveIt < CurvePerCluster; ++CurveIt)
		{
			const uint32 ClusterCurveCount = ClusterCurveIndices[ClusterIndex].Num();

			// If the cluster does not have enough curves, fill with invalid curve.
			// This can happens for the last cluster of a group as the number of curve is not a multiple of 64
			if (CurveIt >= ClusterCurveCount)
			{
				FCurve& Curve = Cluster.Curves[CurveIt];
				Curve.Points.Init(FVector3f::ZeroVector, PointPerCurve);
				Curve.UCoord.Init(0, PointPerCurve);
				Curve.Radius.Init(0, PointPerCurve);
				Curve.bValid.Init(false, PointPerCurve);
				continue;
			}

			// Valid curve count
			++Cluster.CurveCount;

			const uint32 ClusterCurveIndex = FMath::Min(CurveIt, ClusterCurveCount-1);
			const uint32 CurveIndex = ClusterCurveIndices[ClusterIndex][ClusterCurveIndex];

			const int32 IndexOffset = Curves.CurvesOffset[CurveIndex];
			const uint16 PointCount = Curves.CurvesCount[CurveIndex];

			MinPointPerCurve = FMath::Min(MinPointPerCurve, uint32(PointCount));
			MaxPointPerCurve = FMath::Max(MaxPointPerCurve, uint32(PointCount));
			AccPointPerCurve += PointCount;

			FCurve& Curve = Cluster.Curves[CurveIt];
			Curve.Points.SetNum(PointPerCurve);
			Curve.UCoord.SetNum(PointPerCurve);
			Curve.Radius.SetNum(PointPerCurve);
			Curve.bValid.Init(false, PointPerCurve);
			if (PointCount <= PointPerCurve)
			{
				for (int32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
				{
					Curve.Points[PointIndex] = Points.PointsPosition[PointIndex + IndexOffset];
					Curve.UCoord[PointIndex] = Points.PointsCoordU[PointIndex + IndexOffset];
					Curve.Radius[PointIndex] = Points.PointsRadius[PointIndex + IndexOffset];
					Curve.bValid[PointIndex] = true;
					Cluster.MaxRadius = FMath::Max(Curve.Radius[PointIndex], Cluster.MaxRadius);

					ClusterBound += Curve.Points[PointIndex];
				}
			}
			else
			{
				uint32 CurrIndex = 0;
				uint32 NextIndex = 1;
				float InterpU = 0;
				const float StepCoordU = 1.f / float(PointPerCurve-1);
				for (int32 PointIndex = 0; PointIndex < PointPerCurve; ++PointIndex)
				{
					while (NextIndex < PointCount && !(InterpU >= Points.PointsCoordU[CurrIndex + IndexOffset] && InterpU <= Points.PointsCoordU[NextIndex + IndexOffset]))
					{
						CurrIndex++;
						NextIndex++;
					}

					check(CurrIndex < PointCount);
					check(NextIndex < PointCount);

					const float CurrU = Points.PointsCoordU[CurrIndex + IndexOffset];
					const float NextU = Points.PointsCoordU[NextIndex + IndexOffset];

					check(InterpU >= CurrU);
					check(InterpU <= NextU);
					check(CurrU <= NextU);
					float S = (InterpU >= CurrU && NextU > CurrU) ? (InterpU - CurrU) / (NextU - CurrU) : 0.f;
					S = FMath::Clamp(S, 0.f, 1.f);
					Curve.Points[PointIndex] = FMath::Lerp(Points.PointsPosition[CurrIndex + IndexOffset], Points.PointsPosition[NextIndex + IndexOffset], S);
					Curve.UCoord[PointIndex] = FMath::Lerp(Points.PointsCoordU[CurrIndex + IndexOffset],   Points.PointsCoordU[NextIndex + IndexOffset],   S);
					Curve.Radius[PointIndex] = FMath::Lerp(Points.PointsRadius[CurrIndex + IndexOffset],   Points.PointsRadius[NextIndex + IndexOffset],   S);
					Curve.bValid[PointIndex] = true;

					InterpU = FMath::Clamp(InterpU + StepCoordU, 0.f, 1.f);

					Cluster.MaxRadius = FMath::Max(Curve.Radius[PointIndex], Cluster.MaxRadius);

					ClusterBound += Curve.Points[PointIndex];
				}
			}

			Cluster.MaxLength = FMath::Max(Curves.CurvesLength[CurveIndex], Cluster.MaxLength);
		}

		Cluster.BoundCenter = ClusterBound.GetCenter();
		Cluster.BoundExtent = ClusterBound.GetExtent();
	}

	// Pack clusters' data 
	// * Quantize data
	// * Reorder data in SOA
	uint32 MaxClusterStrideInBytes = 0;
	TArray<FPackedCluster> PackedClusters;
	PackedClusters.SetNum(ClusterCount);
	for (uint32 ClusterIndex = 0; ClusterIndex < ClusterCount; ++ClusterIndex)
	{
		const FCluster& Cluster = Clusters[ClusterIndex];
		FPackedCluster& PackedCluster = PackedClusters[ClusterIndex];
		PackedCluster.Header.Center           = Cluster.Curves[0].Points[0]; // First point for now
		PackedCluster.Header.CurveCount       = Cluster.CurveCount;
		PackedCluster.Header.CurvePerCluster  = Cluster.CurvePerCluster;
		PackedCluster.Header.PointPerCurve    = Cluster.PointPerCurve;
		PackedCluster.Header.MaxLength        = Cluster.MaxLength;
		PackedCluster.Header.MaxRadius        = Cluster.MaxRadius;

		// Need to revist this in order to make it more compact.
		// The issue is that the F10bits representation does not have a sign bit -> could use the 2x2bits remaining to store center sign bit. Extend needs to be round up
		//		PackedCluster.Header.LocalBoundCenter = PackR10G10B10F(Cluster.BoundCenter - PackedCluster.Header.Center);
		//		PackedCluster.Header.LocalBoundCenter = PackR10G10B10F(Cluster.BoundCenter);
		//		PackedCluster.Header.LocalBoundExtent = PackR10G10B10F(Cluster.BoundExtent);
		PackedCluster.Header.LocalBoundCenterX = FFloat16(Cluster.BoundCenter.X - PackedCluster.Header.Center.X);
		PackedCluster.Header.LocalBoundCenterY = FFloat16(Cluster.BoundCenter.Y - PackedCluster.Header.Center.Y);
		PackedCluster.Header.LocalBoundCenterZ = FFloat16(Cluster.BoundCenter.Z - PackedCluster.Header.Center.Z);

		PackedCluster.Header.LocalBoundExtentX = FFloat16(Cluster.BoundExtent.X);
		PackedCluster.Header.LocalBoundExtentY = FFloat16(Cluster.BoundExtent.Y);
		PackedCluster.Header.LocalBoundExtentZ = FFloat16(Cluster.BoundExtent.Z);

		PackedCluster.Data.PackedPoints.SetNum(PackedCluster.Header.CurvePerCluster * PackedCluster.Header.PointPerCurve);
		for (uint32 PointIt = 0; PointIt < Cluster.PointPerCurve; ++PointIt)
		{
			for (uint32 CurveIt = 0; CurveIt < Cluster.CurvePerCluster; ++CurveIt)
			{
				const uint32 WriteIndex = CurveIt + Cluster.CurvePerCluster * PointIt;  //PointIt + Cluster.PointPerCurve * CurveIt;

				// Packed Position|CoordU|NormalizedRadius|Type
				{
					const FVector3f Position = Cluster.Curves[CurveIt].Points[PointIt];
					const float CoordU = Cluster.Curves[CurveIt].UCoord[PointIt];
					const float NormalizedRadius = Cluster.Curves[CurveIt].Radius[PointIt] / PackedCluster.Header.MaxRadius;
					const bool bValid = Cluster.Curves[CurveIt].bValid[PointIt];

					FHairStrandsPositionFormat::Type& PackedPosition = PackedCluster.Data.PackedPoints[WriteIndex];
					const FVector3f RelativePosition = bValid ? (Position - PackedCluster.Header.Center) : FVector3f::ZeroVector;
					CopyVectorToPosition(RelativePosition, PackedPosition);
					PackedPosition.UCoord = uint8(FMath::Clamp(CoordU * 255.f, 0.f, 255.f));
					PackedPosition.Radius = uint8(FMath::Clamp(NormalizedRadius * 63.f, 0.f, 63.f));
					PackedPosition.Type   = bValid ? 1u : 0u;
				}
			}
		}

		MaxClusterStrideInBytes = FMath::Max(MaxClusterStrideInBytes, PackedCluster.GetSizeInBytes());
	}
	// Make sure MaxClusterStrideInBytes is 16bytes aligned
	MaxClusterStrideInBytes = FMath::DivideAndRoundUp(MaxClusterStrideInBytes, 16u) * 16u;

	// Finalize bulk data
	{
		static_assert(sizeof(FPackedCluster::FHeader) == 32u);

		Out.Header.Id = FGuid::NewGuid();
		Out.Header.ClusterCount = PackedClusters.Num();
		Out.Header.MaxClusterStrideInBytes = MaxClusterStrideInBytes;
		const uint32 DataSizeInByte = PackedClusters.Num() * MaxClusterStrideInBytes;

		// The buffer is then stored into bulk data
		Out.Data.BulkData.Lock(LOCK_READ_WRITE);
		uint8* BulkBuffer = Out.Data.BulkData.Realloc(DataSizeInByte);
		FMemory::Memzero(BulkBuffer, DataSizeInByte);

		uint8* OriginalBulkBuffer = BulkBuffer;

		// Clusters are placed with a stride of MaxClusterStrideInBytes for now. This is not optimable but allow easy access on the GPU. 
		// Once we have transcoding in place, this can be made more compact.
		for (uint32 ClusterIndex = 0; ClusterIndex < Out.Header.ClusterCount; ++ClusterIndex)
		{
			// DEBUG
			#if 0
			if (ClusterIndex < 10)
			{
				const FPackedCluster::FHeader H = PackedClusters[ClusterIndex].Header;
				const FVector3f LocalBoundCenter = UnpackR10G10B10F(H.LocalBoundCenter);
				const FVector3f LocalBoundExtent = UnpackR10G10B10F(H.LocalBoundExtent);

				UE_LOGF(LogGroomBuilder, Warning, "[Cluster] ClusterIndex:%d Center (%f %f %f) \n CurveCount:%d \n PointPerCurve:%d \n MaxLength:%f \n MaxRadius:%f \n UintLocalBoundCenter:%d UintLocalBoundExtent:%d LocalBoundCenter:(%f %f %f) LocalBoundExtent:(%f %f %f)", \
					   ClusterIndex,\
					   H.Center.X, H.Center.Y, H.Center.Z, \
					   H.CurveCount, \
					   H.PointPerCurve, \
					   H.MaxLength.GetFloat(), \
					   H.MaxRadius.GetFloat(),\
					   H.LocalBoundCenter,\
					   H.LocalBoundExtent,\
					   LocalBoundCenter.X,\
					   LocalBoundCenter.Y,\
					   LocalBoundCenter.Z,\
					   LocalBoundExtent.X,\
					   LocalBoundExtent.Y,\
					   LocalBoundExtent.Z);
			}          
			#endif

			FMemory::Memcpy(BulkBuffer,                                  &PackedClusters[ClusterIndex].Header,                      sizeof(FPackedCluster::FHeader));
			FMemory::Memcpy(BulkBuffer + sizeof(FPackedCluster::FHeader), PackedClusters[ClusterIndex].Data.PackedPoints.GetData(), sizeof(FHairStrandsPositionFormat::Type) * PackedClusters[ClusterIndex].Data.PackedPoints.Num());
			BulkBuffer += MaxClusterStrideInBytes;
		}

		// DEBUG
		#if 0
		{
			uint32 ByteIndex = 0;
			const FVector3f Center = *(FVector3f*)&(OriginalBulkBuffer[ByteIndex]); ByteIndex += sizeof(FVector3f);
			const uint32 CurveCount = OriginalBulkBuffer[ByteIndex]; ByteIndex += 1;
			const uint32 PointPerCurve0 = OriginalBulkBuffer[ByteIndex]; ByteIndex += 1;

			// Pad0 Pad1
			ByteIndex += 1;
			ByteIndex += 1;

			const FFloat16 MaxLength = *(FFloat16*)&(OriginalBulkBuffer[ByteIndex]); ByteIndex += sizeof(FFloat16);
			const FFloat16 MaxRadius = *(FFloat16*)&(OriginalBulkBuffer[ByteIndex]); ByteIndex += sizeof(FFloat16);

			UE_LOGF(LogGroomBuilder, Warning, "[WRITE1] Center %f%f%f CurveCount%d PointPerCurve%d MaxLength%f MaxRadius%f.", Center.X, Center.Y, Center.Z, CurveCount, PointPerCurve0, MaxLength.GetFloat(), MaxRadius.GetFloat());
		}
		#endif

		Out.Data.BulkData.Unlock();
	}
}

} // namespace HairStrandsBuilder

void FGroomBuilder::BuildRenderCurveResourceBulkData(const FHairStrandsDatas& In, FRenderCurveResourceData& Out)
{
	Out = FRenderCurveResourceData();
	if (IsRenderCurveEnabled())
	{
		HairStrandsBuilder::InternalBuildRenderCurveResourceBulkData(In, Out);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
