// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCache.h"

#include "EditorFramework/AssetImportData.h"
#include "Engine/AssetUserData.h"
#include "GroomAsset.h"
#include "GroomBuilder.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomCache)

void UGroomCache::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		ArchiveVersion = Ar.UEVer();
	}

	int32 NumChunks = Chunks.Num();
	Ar << NumChunks;

	if (Ar.IsLoading())
	{
		Chunks.SetNum(NumChunks);
	}

	for (int32 ChunkId = 0; ChunkId < NumChunks; ++ChunkId)
	{
		Chunks[ChunkId].Serialize(Ar, this, ChunkId);
	}
}

void UGroomCache::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UGroomCache::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif

	Super::GetAssetRegistryTags(Context);
}

void UGroomCache::Initialize(EGroomCacheType Type)
{
	GroomCacheInfo.Type = Type;
}

int32 UGroomCache::GetStartFrame() const
{
	return GroomCacheInfo.AnimationInfo.StartFrame;
}

int32 UGroomCache::GetEndFrame() const
{
	return GroomCacheInfo.AnimationInfo.EndFrame;
}

float UGroomCache::GetDuration() const
{
	return GroomCacheInfo.AnimationInfo.Duration;
}

int32 UGroomCache::GetFrameNumberAtTime(const float Time, bool bLooping) const
{
	return GetStartFrame() + GetFrameIndexAtTime(Time, bLooping); 
}

int32 UGroomCache::GetFrameIndexAtTime(const float Time, bool bLooping) const
{
	const float FrameTime = GroomCacheInfo.AnimationInfo.SecondsPerFrame;
	if (FrameTime == 0.0f)
	{
		return 0;
	}

	const int32 NumFrames = GroomCacheInfo.AnimationInfo.NumFrames;

	// Include a small fudge factor to Duration to account for possible computation discrepancies in Time.
	// For example when sequencer computes the time for the end of a section, it might not exactly match the duration
	const float Duration = GroomCacheInfo.AnimationInfo.Duration - KINDA_SMALL_NUMBER;
	float AdjustedTime = Time;
	if (bLooping)
	{
		AdjustedTime = Time - Duration * FMath::FloorToFloat(Time / Duration);
	}
	else
	{
		AdjustedTime = FMath::Clamp(Time, 0.0f, GroomCacheInfo.AnimationInfo.EndTime - GroomCacheInfo.AnimationInfo.StartTime);
	}

	const int32 NormalizedFrame = FMath::Clamp(FMath::FloorToInt(AdjustedTime / FrameTime), 0, NumFrames - 1);
	return NormalizedFrame; 
}

void UGroomCache::GetFrameIndicesAtTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32 &OutFrameIndex, int32 &OutNextFrameIndex, float &InterpolationFactor)
{
	const int32 NumFrames = GroomCacheInfo.AnimationInfo.NumFrames;
	const float Duration = GroomCacheInfo.AnimationInfo.Duration - KINDA_SMALL_NUMBER;

	// No index possible
	if (NumFrames == 0 || Duration == 0.0f)
	{
		OutFrameIndex = 0;
		OutNextFrameIndex = 0;
		InterpolationFactor = 0.0f;
		return;
	}

	OutFrameIndex = GetFrameIndexAtTime(Time, bLooping);
	OutNextFrameIndex = FMath::Min(OutFrameIndex + 1, NumFrames - 1); // -1 since index is 0-based

	const float FrameDuration = GroomCacheInfo.AnimationInfo.SecondsPerFrame;
	if (FMath::IsNearlyZero(FrameDuration))
	{
		InterpolationFactor = 0.0f;
	}
	else
	{
		float AdjustedTime;

		if (bLooping)
		{
			AdjustedTime = Time - Duration * FMath::FloorToFloat(Time / Duration);
		}
		else
		{
			AdjustedTime = FMath::Clamp(Time, 0.0f, GroomCacheInfo.AnimationInfo.EndTime - GroomCacheInfo.AnimationInfo.StartTime);
		}

		float Delta = AdjustedTime - FrameDuration * OutFrameIndex;
		InterpolationFactor = Delta / FrameDuration;
	}

	// If playing backwards the logical order of previous and next is reversed
	if (bIsPlayingBackwards)
	{
		Swap(OutFrameIndex, OutNextFrameIndex);
		InterpolationFactor = 1.0f - InterpolationFactor;
	}
}

void UGroomCache::GetFrameIndicesForTimeRange(float StartTime, float EndTime, bool bLooping, TArray<int32>& OutFrameIndices)
{
	// Sanity check
	if (StartTime > EndTime)
	{
		EndTime = StartTime;
	}

	// Ensure the time range covers at least one frame
	if (EndTime - StartTime < GroomCacheInfo.AnimationInfo.SecondsPerFrame)
	{
		EndTime = StartTime + GroomCacheInfo.AnimationInfo.SecondsPerFrame;
	}

	int32 StartIndex = GetFrameIndexAtTime(StartTime, bLooping);
	int32 EndIndex = GetFrameIndexAtTime(EndTime, bLooping);

	if (bLooping)
	{
		// Special cases to handle with looping enabled
		if (((EndTime - StartTime) >= GroomCacheInfo.AnimationInfo.Duration) || 
			(StartIndex == EndIndex))
		{
			// Requested time range is longer than the animation or exactly matches the duration so include all the frames
			for (int32 FrameIndex = 0; FrameIndex < Chunks.Num(); ++FrameIndex)
			{
				OutFrameIndices.Add(FrameIndex);
			}
			return;
		}

		if (EndIndex < StartIndex)
		{
			// The requested time range is wrapping so add as two intervals
			// From start index to end of animation
			for (int32 FrameIndex = StartIndex; FrameIndex < (int32)GroomCacheInfo.AnimationInfo.NumFrames; ++FrameIndex)
			{
				OutFrameIndices.Add(FrameIndex);
			}
			// From 0 to end index
			for (int32 FrameIndex = 0; FrameIndex <= EndIndex; ++FrameIndex)
			{
				OutFrameIndices.Add(FrameIndex);
			}
			return;
		}
	}

	// Time range is a simple interval within the animation
	for (int32 FrameIndex = StartIndex; FrameIndex <= EndIndex; ++FrameIndex)
	{
		OutFrameIndices.Add(FrameIndex);
	}
}

bool UGroomCache::GetGroomDataAtTime(float Time, bool bLooping, FGroomCacheAnimationData& AnimData)
{
	const int32 FrameIndex = GetFrameIndexAtTime(Time, bLooping);
	return GetGroomDataAtFrameIndex(FrameIndex, AnimData);
}

bool UGroomCache::GetGroomDataAtFrameIndex(int32 FrameIndex, FGroomCacheAnimationData& AnimData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGroomCache::GetGroomDataAtFrameIndex);

	if (!Chunks.IsValidIndex(FrameIndex))
	{
		return false;
	}

	// This is the reverse operation of how the GroomCacheAnimationData is processed into a GroomCacheChunk
	FGroomCacheChunk& Chunk = Chunks[FrameIndex];

	TArray<uint8> TempBytes;
	TempBytes.SetNum(Chunk.DataSize);

	// This is where the bulk data is loaded from disk
	void* Buffer = TempBytes.GetData();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UGroomCache::GetGroomDataAtFrameIndex_BulkData);
		Chunk.BulkData.GetCopy(&Buffer, true);
	}

	// The bulk data buffer is then serialized into GroomCacheAnimationData
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UGroomCache::GetGroomDataAtFrameIndex_Serialize);
		FMemoryReader Ar(TempBytes, true);
		// Propagate the GroomCache archive version to the memory archive for proper serialization
		if (ArchiveVersion.IsSet())
		{
			Ar.SetUEVer(ArchiveVersion.GetValue());
		}
		AnimData.Serialize(Ar);

		// Reset the pointers in order to trigger a read next time
		for( auto& GroupData : AnimData.GroupsData)
		{
			GroupData.VertexData.PositionBuffer = nullptr;
			GroupData.VertexData.RadiusBuffer = nullptr;
		}
	}

	return true;
}

void UGroomCache::SetGroomAnimationInfo(const FGroomAnimationInfo& AnimInfo)
{
	GroomCacheInfo.AnimationInfo = AnimInfo;

	// Ensure that the guides groom cache serialize only positions
	if (GroomCacheInfo.Type == EGroomCacheType::Guides)
	{
		GroomCacheInfo.AnimationInfo.Attributes &= EGroomCacheAttributes::Position;
	}
}

EGroomCacheType UGroomCache::GetType() const
{
	return GroomCacheInfo.Type;
}

void UGroomCache::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UGroomCache::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void UGroomCache::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UGroomCache::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

void FGroomCacheChunk::Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex)
{
	Ar << DataSize;
	Ar << FrameIndex;

	// Forced not inline means the bulk data won't automatically be loaded when we deserialize
	// but only when we explicitly take action to load it
	BulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
	BulkData.Serialize(Ar, Owner, ChunkIndex, false);
}

FGroomCacheProcessor::FGroomCacheProcessor(EGroomCacheType InType, EGroomCacheAttributes InAttributes)
: Attributes(InAttributes)
, Type(InType)
{
}

void FGroomCacheProcessor::AddGroomSample(TArray<FGroomCacheInputData>&& InputDatas)
{
	TArray<uint8> TempBytes;
	FMemoryWriter Ar(TempBytes, true);

	// The HairGroupData is converted into GroomCacheAnimationData and serialized to a buffer
	FGroomCacheAnimationData AnimData(MoveTemp(InputDatas), FGroomCacheInfo::GetCurrentVersion(), Type, Attributes);
	AnimData.Serialize(Ar);

	FGroomCacheChunk& Chunk = Chunks.AddDefaulted_GetRef();

	// The buffer is then stored into bulk data
	const int32 DataSize = TempBytes.Num();
	Chunk.DataSize = DataSize;
	Chunk.FrameIndex = Chunks.Num() - 1;
	Chunk.BulkData.Lock(LOCK_READ_WRITE);
	void* ChunkBuffer = Chunk.BulkData.Realloc(DataSize);
	FMemory::Memcpy(ChunkBuffer, TempBytes.GetData(), DataSize);
	Chunk.BulkData.Unlock();
}

void FGroomCacheProcessor::TransferChunks(UGroomCache* GroomCache)
{
	GroomCache->Chunks = MoveTemp(Chunks);
}

namespace UE::Groom
{
	void BuildGroomCache(FGroomCacheProcessor& Processor, const FGroomAnimationInfo& AnimInfo, UGroomCache* GroomCache)
	{
		if (GroomCache)
		{
			Processor.TransferChunks(GroomCache);
			GroomCache->SetGroomAnimationInfo(AnimInfo);

			GroomCache->MarkPackageDirty();
#if WITH_EDITOR
			GroomCache->PostEditChange();
#endif
		}
	}

	bool BuildGroupsData(const FHairDescription& HairDescription, const TArray<FHairGroupPlatformData>& PlatformData,
		TArray<FHairGroupInfoWithVisibility>& VisibilityData, const TArray<FHairGroupsInterpolation>& InterpolationData, TArray<FGroomCacheInputData>& GroupsData)
	{
		bool bSuccess = true;
		FHairDescriptionGroups DescriptionGroups;
		// Do not add extra control points at the end of curve when hair strip geometry is enabled. This is because groom cache data are 
		// serialize within the uasset (i.e. do not used intermediate cached/build data), and need the asset to be compatible with or without hair strip geometry.
		if (FGroomBuilder::BuildHairDescriptionGroups(HairDescription, DescriptionGroups, false /*bAllowAddEndControlPoint*/))
		{
			const uint32 GroupCount = DescriptionGroups.HairGroups.Num();
			if(GroupCount == PlatformData.Num() && GroupCount == VisibilityData.Num() && GroupCount == InterpolationData.Num())
			{
				GroupsData.SetNum(GroupCount);
				for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
				{
					FGroomBuilder::BuildData(DescriptionGroups.HairGroups[GroupIndex], InterpolationData[GroupIndex], VisibilityData[GroupIndex], GroupsData[GroupIndex].Strands, GroupsData[GroupIndex].Guides);
				}
				return true;
			}
		}
		return false;
	}
}

