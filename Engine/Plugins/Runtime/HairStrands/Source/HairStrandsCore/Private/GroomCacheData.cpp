// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheData.h"
#include "GroomAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomCacheData)

float GetHairStrandsMaxRadius(const TArray<float>& InPointsRadius);

FGroomAnimationInfo::FGroomAnimationInfo()
	: NumFrames(0)
	, SecondsPerFrame(0.0f)
	, Duration(0.0f)
	, StartTime(MAX_flt)
	, EndTime(MIN_flt)
	, StartFrame(MAX_int32)
	, EndFrame(MIN_int32)
	, Attributes(EGroomCacheAttributes::None)
{
}

bool FGroomAnimationInfo::IsValid() const
{
	return NumFrames > 1 && SecondsPerFrame > 0.f && StartFrame < EndFrame && Attributes != EGroomCacheAttributes::None;
}

int32 FGroomCacheInfo::GetCurrentVersion()
{
	// Bump version number when there's a serialization change
	return 0;
}

FGroomCacheVertexData::FGroomCacheVertexData(FHairStrandsPoints&& PointsData)
: PointsPosition(MoveTemp(PointsData.PointsPosition))
, PointsRadius(MoveTemp(PointsData.PointsRadius))
, PointsCoordU(MoveTemp(PointsData.PointsCoordU))
, PointsBaseColor(MoveTemp(PointsData.PointsBaseColor))
{
}

void FGroomCacheVertexData::Serialize(FArchive& Ar, int32 Version, EGroomCacheAttributes Attributes)
{
	if (EnumHasAnyFlags(Attributes, EGroomCacheAttributes::Position))
	{
		Ar << PointsPosition;
		//Ar << PointsCoordU; // currently not used
	}

	if (EnumHasAnyFlags(Attributes, EGroomCacheAttributes::Width))
	{
		Ar << PointsRadius;
	}

	if (EnumHasAnyFlags(Attributes, EGroomCacheAttributes::Color))
	{
		Ar << PointsBaseColor;
	}
}

FGroomCacheStrandData::FGroomCacheStrandData(FHairStrandsCurves&& CurvesData, float InMaxLength, float InMaxRadius)
: CurvesLength(MoveTemp(CurvesData.CurvesLength))
, MaxLength(InMaxLength)
, MaxRadius(InMaxRadius)
{
}

void FGroomCacheStrandData::Serialize(FArchive& Ar, int32 Version, EGroomCacheAttributes Attributes)
{
	Ar << MaxLength;
	Ar << MaxRadius;
	Ar << CurvesLength;
}

// Use VertexData.PointsRadius, rather than GroupData, because GroupData.StrandsPoints has been 'moved' to 
// VertexData by the time we reach the GetHairStrandsMaxRadius function
FGroomCacheGroupData::FGroomCacheGroupData(FHairStrandsDatas&& GroupData)
: VertexData(MoveTemp(GroupData.StrandsPoints))
, StrandData(MoveTemp(GroupData.StrandsCurves), GetHairStrandsMaxLength(GroupData), GetHairStrandsMaxRadius(VertexData.PointsRadius)) 
, BoundingBox(GroupData.BoundingBox)
{
}

void FGroomCacheGroupData::Serialize(FArchive& Ar, int32 Version, EGroomCacheAttributes Attributes)
{
	BoundingBox.Serialize(Ar);
	VertexData.Serialize(Ar, Version, Attributes);
	StrandData.Serialize(Ar, Version, Attributes);
}

FGroomCacheAnimationData::FGroomCacheAnimationData(TArray<FGroomCacheInputData>&& InputDatas, int32 InVersion, EGroomCacheType Type, EGroomCacheAttributes InAttributes)
: Version(InVersion)
{
	Attributes = Type == EGroomCacheType::Strands ? InAttributes : InAttributes & EGroomCacheAttributes::Position;

	for (FGroomCacheInputData& InputData : InputDatas)
	{
		GroupsData.Add(FGroomCacheGroupData(MoveTemp(Type == EGroomCacheType::Strands ? InputData.Strands : InputData.Guides)));
	}
}

void FGroomCacheAnimationData::Serialize(FArchive& Ar)
{
	Ar << Version;
	Ar << Attributes;

	int32 NumGroups = GroupsData.Num();
	Ar << NumGroups;

	if (Ar.IsLoading())
	{
		GroupsData.SetNum(NumGroups);
	}

	for (int32 Index = 0; Index < NumGroups; ++Index)
	{
		GroupsData[Index].Serialize(Ar, Version, Attributes);
	}
}

