// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheStreamingData.h"

#include "Async/Async.h"
#include "GroomCache.h"
#include "GroomComponent.h"
#include "GroomPluginSettings.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryReader.h"
#include "ProfilingDebugging/IoStoreTrace.h"

DEFINE_LOG_CATEGORY(LogGroomCacheStreaming);

FGroomCacheStreamingData::FGroomCacheStreamingData(UGroomCache* InGroomCache)
: GroomCache(InGroomCache)
{
}

FGroomCacheStreamingData::~FGroomCacheStreamingData()
{
	// Make sure that all read requests are deleted including those in-flight, so block until they are finished
	BlockTillAllRequestsFinished();

	// Delete the read requests in the background as it can be time-consuming
	TSet<IBulkDataIORequest*> ReadyForDeletion(DelayedDeleteReadRequests);
	for (IBulkDataIORequest* ReadRequest : ReadyForDeletion)
	{
		delete ReadRequest;
	}

	// Delete AnimData buffers that are resident in memory
	for (TMap<int32, FResidentGroomCacheChunk>::TIterator Iter = Chunks.CreateIterator(); Iter; ++Iter)
	{
		delete Iter.Value().AnimDataPtr;
	}
}

void FGroomCacheStreamingData::ResetNeededChunks()
{
	ChunksNeeded.Empty();
}

void FGroomCacheStreamingData::AddNeededChunk(uint32 ChunkIndex)
{
	ChunksNeeded.AddUnique(ChunkIndex);
}

FResidentGroomCacheChunk& FGroomCacheStreamingData::AddResidentChunk(int32 ChunkId, const FGroomCacheChunk &ChunkInfo)
{
	FScopeLock Lock(&CriticalSection);

	FResidentGroomCacheChunk& Chunk = Chunks.FindOrAdd(ChunkId);

	Chunk.DataSize = ChunkInfo.DataSize;
	if (!Chunk.AnimDataPtr)
	{
		// The resident chunks are re-used so allocate the AnimData buffer the first time only or when RemoveResidentChunk has been called
		Chunk.AnimDataPtr = new FGroomCacheAnimationData();
	}
	return Chunk;
}

void FGroomCacheStreamingData::RemoveResidentChunk(FResidentGroomCacheChunk& LoadedChunk)
{
	checkf(LoadedChunk.Refcount == 0, TEXT("Tried to remove a chunk wich was still mapped. Make sure there is an unmap for every map."));

	if (LoadedChunk.IORequest == nullptr)
	{
		// There's no read request in-flight for this chunk so it's safe to delete the AnimData buffer
		// Otherwise, let ProcessCompletedChunks delete it once the read is completed
		delete LoadedChunk.AnimDataPtr;
	}
	LoadedChunk.AnimDataPtr = nullptr;
	LoadedChunk.IORequest = nullptr;
	LoadedChunk.DataSize = 0;
	LoadedChunk.Refcount = 0;
}

void FGroomCacheStreamingData::PrefetchData(UGroomComponent *Component)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::PrefetchData);

	float RequestStartTime = Component->GetAnimationTime();
	float RequestEndTime = RequestStartTime + GetDefault<UGroomPluginSettings>()->GroomCacheLookAheadBuffer;

	TArray<int32> NewChunksNeeded;
	GroomCache->GetFrameIndicesForTimeRange(RequestStartTime, RequestEndTime, Component->IsLooping(), NewChunksNeeded);

	for (int32 ChunkId : NewChunksNeeded)
	{
		ChunksNeeded.AddUnique(ChunkId);
	}

	// Synchronously load the first 2 frames, but it's possible there's only one frame
	// when scrubbing past a non-looping animation
	const int32 MaxNumPrefetches = FMath::Min(NewChunksNeeded.Num(), 2);

	// This ensures we have something to display initially
	for (int32 ChunkIndex = 0; ChunkIndex < MaxNumPrefetches; ++ChunkIndex)
	{
		// We just check here in case anything got loaded asynchronously last minute
		// to avoid unnecessary loading it synchronously again
		ProcessCompletedChunks();

		int32 ChunkId = NewChunksNeeded[ChunkIndex];

		// Already got it
		if (ChunksAvailable.Contains(ChunkId))
		{
			continue;
		}

		// Still waiting for eviction, revive it
		if (ChunksEvicted.Contains(ChunkId))
		{
			ChunksEvicted.Remove(ChunkId);
			ChunksAvailable.Add(ChunkId);
			continue;
		}

		const FGroomCacheChunk& Chunk = GroomCache->GetChunks()[ChunkId];
		FResidentGroomCacheChunk &ResidentChunk = AddResidentChunk(ChunkId, Chunk);

		// Already requested an async load but not complete yet 
		if (ChunksRequested.Contains(ChunkId))
		{
			// Chunk will be removed from ChunksRequested once the read request is completed in ProcessCompletedChunks
			// This prevents potentially requesting another read for this Chunk

			// Let the in-flight read request complete with the initial AnimDataPtr to avoid threading issues
			// The initial buffer will be orphaned but will be deleted once the completed read request is processed
			// This new buffer is used immediately to load the data synchronously
			ResidentChunk.AnimDataPtr = new FGroomCacheAnimationData();
		}

		// Load chunk from bulk data
		GroomCache->GetGroomDataAtFrameIndex(ChunkId, *ResidentChunk.AnimDataPtr);
		ChunksAvailable.Add(ChunkId);
	}

	// Schedule the rest of the needed frames to be loaded asynchronously
	UpdateStreamingStatus(true);
}

void FGroomCacheStreamingData::UpdateStreamingStatus(bool bAsyncDeletionAllowed)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::UpdateStreamingStatus);

	// Allow updating the StreamingData only one per frame since PrefetchData can happen
	// many times, like when the time jumps around with thumbnail generation in Sequencer.
	// This will prevent adding and removing chunks unnecessarily.
	if (LastUpdate == GFrameCounter)
	{
		return;
	}

	// Find any chunks that aren't available yet
	for (int32 NeededIndex : ChunksNeeded)
	{
		if (!ChunksAvailable.Contains(NeededIndex))
		{
			// Revive it if it was still referenced
			if (ChunksEvicted.Contains(NeededIndex))
			{
				ChunksEvicted.Remove(NeededIndex);
				ChunksAvailable.Add(NeededIndex);
				continue;
			}

			// If not requested yet, then request a load
			if (!ChunksRequested.Contains(NeededIndex) && GroomCache->GetChunks().IsValidIndex(NeededIndex))
			{
				const FGroomCacheChunk& Chunk = GroomCache->GetChunks()[NeededIndex];

				// This can happen in the editor if the asset hasn't been saved yet
				if (Chunk.BulkData.IsBulkDataLoaded())
				{
					FResidentGroomCacheChunk &ResidentChunk = AddResidentChunk(NeededIndex, Chunk);
					GroomCache->GetGroomDataAtFrameIndex(NeededIndex, *ResidentChunk.AnimDataPtr);
					ChunksAvailable.Add(NeededIndex);
					continue;
				}

				checkf(Chunk.BulkData.CanLoadFromDisk(), TEXT("Bulk data is not loaded and cannot be loaded from disk!"));
				check(!Chunk.BulkData.IsStoredCompressedOnDisk()); // We do not support compressed Bulkdata for this system. Limitation of the streaming request/bulk data

				FResidentGroomCacheChunk& ResidentChunk = AddResidentChunk(NeededIndex, Chunk);
				FGroomCacheAnimationData* AnimData = ResidentChunk.AnimDataPtr;
				int32 DataSize = ResidentChunk.DataSize;

				checkf(ResidentChunk.IORequest == nullptr, TEXT("Groom cache read request for chunk %d is already in progress. Make sure its bookkeeping is up to date."), NeededIndex);
				if (ResidentChunk.IORequest != nullptr)
				{
					continue;
				}

				FBulkDataIORequestCallBack AsyncFileCallBack = [this, NeededIndex, DataSize, AnimData](bool bWasCancelled, IBulkDataIORequest* ReadRequest)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::CompletedCallBack);

					// Now that the read request has completed, we can process the read buffer
					FCompletedGroomCacheChunk CompletedChunk(NeededIndex, AnimData, ReadRequest);

					if (uint8* ReadBuffer = ReadRequest->GetReadResults())
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::CompletedCallBack_BufferSerialize);

						// The bulk data buffer is then serialized into GroomCacheAnimationData
						TArrayView<uint8> TempView(ReadBuffer, DataSize);
						FMemoryReaderView Ar(TempView, true);
						// Propagate the GroomCache archive version to the memory archive for proper serialization
						if (GroomCache->ArchiveVersion.IsSet())
						{
							Ar.SetUEVer(GroomCache->ArchiveVersion.GetValue());
						}
						AnimData->Serialize(Ar);

						// We became the owner of ReadBuffer when GetReadResults was called so free it now
						FMemory::Free(ReadBuffer);

						CompletedChunk.bReadSuccessful = true;
					}

					CompletedChunks.Enqueue(MoveTemp(CompletedChunk));
				};
				
				// Start the read request
				EAsyncIOPriorityAndFlags AsyncIOPriority = AIOP_BelowNormal;
				TRACE_IOSTORE_METADATA_SCOPE_TAG("GroomCache");
				ResidentChunk.IORequest = Chunk.BulkData.CreateStreamingRequest(AsyncIOPriority, &AsyncFileCallBack, nullptr);
				if (!ResidentChunk.IORequest)
				{
					UE_LOGF(LogGroomCacheStreaming, Error, "Groom cache streaming read request couldn't be created.");
					return;
				}

				// Add it to the list for bookkeeping
				ChunksRequested.Add(NeededIndex);
			}

			// Read request is in-flight, just let it complete
		}
	}

	// Update bookkeeping with any recently completed chunks
	ProcessCompletedChunks();

	// Find chunks that are currently loaded but aren't needed anymore and add them to the list of chunks to evict
	TSet<int32> ChunksToRemove;
	for (int32 ChunkId : ChunksAvailable)
	{
		if (!ChunksNeeded.Contains(ChunkId))
		{
			ChunksEvicted.Add(ChunkId);
			ChunksToRemove.Add(ChunkId);
		}
	}
	ChunksAvailable = ChunksAvailable.Difference(ChunksToRemove);

	// Try to evict a bunch of chunks. Chunks that are still mapped (ie. in use) can't be evicted but others are free to go
	TSet<int32> EvictedChunks;
	{
		FScopeLock Lock(&CriticalSection);

		for (int32 ChunkId : ChunksEvicted)
		{
			FResidentGroomCacheChunk& ResidentChunk = Chunks[ChunkId];
			if (ResidentChunk.Refcount == 0)
			{
				RemoveResidentChunk(ResidentChunk);
				EvictedChunks.Add(ChunkId);
			}
		}
	}
	ChunksEvicted = ChunksEvicted.Difference(EvictedChunks);

	// Delete the read requests that were delayed
	TSet<IBulkDataIORequest*> ReadyForDeletion;
	for (IBulkDataIORequest* ReadRequest : DelayedDeleteReadRequests)
	{
		if (ReadRequest->PollCompletion())
		{
			ReadyForDeletion.Add(ReadRequest);
		}
	}
	DelayedDeleteReadRequests = DelayedDeleteReadRequests.Difference(ReadyForDeletion);
	
	if (ReadyForDeletion.Num() > 0)
	{
		if (bAsyncDeletionAllowed)
		{
			// Delete the read requests in the background as it can be time-consuming
			Async(EAsyncExecution::ThreadPool, [ReadyForDeletion]()
			{
				for (IBulkDataIORequest* ReadRequest : ReadyForDeletion)
				{
					delete ReadRequest;
				}
			});
		}
		else
		{
			for (IBulkDataIORequest* ReadRequest : ReadyForDeletion)
			{
				delete ReadRequest;
			}
		}
	}

	LastUpdate = GFrameCounter;
}

bool FGroomCacheStreamingData::BlockTillAllRequestsFinished(float TimeLimit)
{
	FScopeLock Lock(&CriticalSection);

	if (TimeLimit == 0.0f)
	{
		for (TMap<int32, FResidentGroomCacheChunk>::TIterator Iter = Chunks.CreateIterator(); Iter; ++Iter)
		{
			if (Iter.Value().IORequest)
			{
				Iter.Value().IORequest->WaitCompletion();
				ProcessCompletedChunks();
			}
		}
	}
	else
	{
		double EndTime = FPlatformTime::Seconds() + TimeLimit;
		for (TMap<int32, FResidentGroomCacheChunk>::TIterator Iter = Chunks.CreateIterator(); Iter; ++Iter)
		{
			if (Iter.Value().IORequest)
			{
				float ThisTimeLimit = EndTime - FPlatformTime::Seconds();
				if (ThisTimeLimit < .001f || // one ms is the granularity of the platform event system
					!Iter.Value().IORequest->WaitCompletion(ThisTimeLimit))
				{
					return false;
				}
				ProcessCompletedChunks();
			}
		}
	}

	return true;
}

void FGroomCacheStreamingData::ProcessCompletedChunks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::ProcessCompletedChunks);
	FScopeLock Lock(&CriticalSection);

	FCompletedGroomCacheChunk CompletedChunk;
	while (CompletedChunks.Dequeue(CompletedChunk))
	{
		FResidentGroomCacheChunk* Chunk = Chunks.Find(CompletedChunk.LoadedChunkIndex);
		if (!Chunk)
		{
			return;
		}

		// Check to see if we successfully managed to load anything
		// If the chunk's IORequest is null, it means RemoveResidentChunk has been called on it during the read request
		// in which case, we don't care about this chunk anymore 
		if (Chunk->IORequest != nullptr)
		{
			// There could be more than one read requests in-flight for the same Chunk so make sure that the completed
			// read request is still the one we're waiting for
			if (Chunk->IORequest == CompletedChunk.ReadRequest)
			{
				if (CompletedChunk.bReadSuccessful)
				{
					ChunksAvailable.Add(CompletedChunk.LoadedChunkIndex);
				}
				else
				{
					UE_LOGF(LogGroomCacheStreaming, Log, "Async loading request failed!");
				}

				Chunk->IORequest = nullptr;
				ChunksRequested.Remove(CompletedChunk.LoadedChunkIndex);
			}
			// If the read request doesn't match, it means we're still waiting for IORequest to complete so it must remain in ChunksRequested
		}
		else
		{
			ChunksRequested.Remove(CompletedChunk.LoadedChunkIndex);
		}

		// The AnimData buffer can change while the read request was in-flight because the associated chunk was prefetched synchronously instead
		// or it's been removed from resident chunks, so just delete the orphaned buffer
		if (CompletedChunk.AnimDataPtr != Chunk->AnimDataPtr)
		{
			delete CompletedChunk.AnimDataPtr;
		}

		// Process the read request deletion later as in can be quite time-consuming
		DelayedDeleteReadRequests.Add(CompletedChunk.ReadRequest);
	}
}

const FGroomCacheAnimationData* FGroomCacheStreamingData::MapAnimationData(uint32 ChunkIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::MapAnimationData);

	// Quickly check before mapping if maybe something new arrived we haven't done bookkeeping for yet
	ProcessCompletedChunks();

	if (!ChunksAvailable.Contains(ChunkIndex))
	{
		if (!ChunksRequested.Contains(ChunkIndex))
		{
			if (ChunksEvicted.Contains(ChunkIndex))
			{
				UE_LOGF(LogGroomCacheStreaming, Log, "Tried to map an evicted chunk: %i.", ChunkIndex);
			}
			else
			{
				UE_LOGF(LogGroomCacheStreaming, Log, "Tried to map an unavailabe non-requested chunk: %i.", ChunkIndex);
			}
		}
		else
		{
			UE_LOGF(LogGroomCacheStreaming, Log, "Tried to map a chunk (%i) that is still being streamed in.", ChunkIndex);
		}
		return nullptr;
	}
	else
	{
		FScopeLock Lock(&CriticalSection);

		FResidentGroomCacheChunk *ResidentChunk = Chunks.Find(ChunkIndex);
		if (ResidentChunk)
		{
			checkf(ResidentChunk->AnimDataPtr, TEXT("Tried to map a chunk without data. Make sure its bookkeeping is up to date."));
			++ResidentChunk->Refcount;
			return ResidentChunk->AnimDataPtr;
		}
		UE_LOGF(LogGroomCacheStreaming, Log, "Tried to map a chunk (%i) that is not resident in-memory.", ChunkIndex);
		return nullptr;
	}
}

void FGroomCacheStreamingData::UnmapAnimationData(uint32 ChunkIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingData::UnmapAnimationData);

	// The unmap could happen from the render thread when it releases the GroomCache buffers
	// so access to StreamingGroomCaches has to be protected
	FScopeLock Lock(&CriticalSection);

	FResidentGroomCacheChunk* ResidentChunk = Chunks.Find(ChunkIndex);
	// If the unmap is done from RT and GT at the same time (init resource...) even if the critical section is preventing a race condition
	// we could end up with already a refcount = 0 in that function
	if (ResidentChunk != nullptr && ResidentChunk->Refcount > 0)
	{
		checkf(ChunksAvailable.Contains(ChunkIndex) || ChunksEvicted.Contains(ChunkIndex), TEXT("Tried to unmap a chunk in an invalid state."));
		--ResidentChunk->Refcount;
	}
}

bool FGroomCacheStreamingData::IsStreamingInProgress()
{
	return ChunksRequested.Num() > 0;
}
