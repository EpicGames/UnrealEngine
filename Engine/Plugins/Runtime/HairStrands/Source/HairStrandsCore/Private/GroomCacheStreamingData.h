// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGroomCacheStreaming, Verbose, All);

struct FGroomCacheAnimationData;
class IBulkDataIORequest;
class UGroomCache;
class UGroomComponent;

/** Info about a chunk, loaded or not */
struct FResidentGroomCacheChunk
{
	FGroomCacheAnimationData* AnimDataPtr = nullptr;
	IBulkDataIORequest* IORequest = nullptr; // null when resident, non-null when in flight
	int32 Refcount = 0;
	int32 DataSize = 0;
};

/** Info about read request status of a chunk */
struct FCompletedGroomCacheChunk
{
	FGroomCacheAnimationData* AnimDataPtr = nullptr;
	IBulkDataIORequest* ReadRequest;
	int32 LoadedChunkIndex;
	bool bReadSuccessful;

	FCompletedGroomCacheChunk() : AnimDataPtr(nullptr), ReadRequest(nullptr), LoadedChunkIndex(0), bReadSuccessful(false) {}
	FCompletedGroomCacheChunk(int32 SetLoadedChunkIndex, FGroomCacheAnimationData* AnimBuffer, IBulkDataIORequest* SetReadRequest)
	: AnimDataPtr(AnimBuffer), ReadRequest(SetReadRequest), LoadedChunkIndex(SetLoadedChunkIndex), bReadSuccessful(false) {}
};

/** Class that manages the streaming and cached data for a GroomCache */
class FGroomCacheStreamingData
{
public:
	FGroomCacheStreamingData(UGroomCache* GroomCache);
	~FGroomCacheStreamingData();

	/** Updates the internal state of the streaming data. */
	void UpdateStreamingStatus(bool bAsyncDeletionAllowed);

	/** Returns a pointer to the cache data for ChunkIndex if it's loaded; nullptr otherwise. */
	const FGroomCacheAnimationData* MapAnimationData(uint32 ChunkIndex);

	/** Informs the streaming data that the ChunkIndex data is not used anymore.  */
	void UnmapAnimationData(uint32 ChunkIndex);

	/** Returns true if there are read requests in flight; false otherwise. */
	bool IsStreamingInProgress();

	/** Blocks until all read requests are completed. Returns true if they were all completed before TimeLimit (0 means no limit); false otherwise. */
	bool BlockTillAllRequestsFinished(float TimeLimit = 0.0f);

	/** Synchronously loads the initial frames needed to display the given component */
	void PrefetchData(UGroomComponent* Component);

	/** Resets the list of chunks needed by users of this GroomCache */
	void ResetNeededChunks();

	/** Adds ChunkIndex to the list of chunks needed to be loaded */
	void AddNeededChunk(uint32 ChunkIndex);

	UGroomCache* GetGroomCache() const { return GroomCache; }

private:

	FResidentGroomCacheChunk& AddResidentChunk(int32 ChunkId, const struct FGroomCacheChunk& ChunkInfo);
	void RemoveResidentChunk(FResidentGroomCacheChunk& LoadedChunk);
	void ProcessCompletedChunks();

	/** The GroomCache being streamed */
	UGroomCache* GroomCache;

	/** List of chunks that need to be loaded */
	TArray<int32> ChunksNeeded;

	/** List of chunks currently resident in memory */
	TSet<int32> ChunksAvailable;

	/** List of chunks being streamed, ie. their read requests are in-flight */
	TSet<int32> ChunksRequested;

	/** List of chunks to be evicted, not needed anymore so they can be unloaded */
	TSet<int32> ChunksEvicted;

	/** ChunkIndex to chunk info */
	TMap<int32, FResidentGroomCacheChunk> Chunks;

	/** List of read requests that are completed that need to be processed for bookkeeping (to be marked as available) */
	TQueue<FCompletedGroomCacheChunk, EQueueMode::Mpsc> CompletedChunks;

	/** List of read requests to be deleted after completion */
	TSet<IBulkDataIORequest*> DelayedDeleteReadRequests;

	/** Timestamp of the last update */
	uint64 LastUpdate = 0;

	FCriticalSection CriticalSection;
};

