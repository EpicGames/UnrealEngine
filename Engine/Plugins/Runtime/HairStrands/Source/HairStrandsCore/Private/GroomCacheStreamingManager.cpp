// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheStreamingManager.h"

#include "Async/Async.h"
#include "GroomCache.h"
#include "GroomCacheStreamingData.h"
#include "GroomComponent.h"
#include "GroomPluginSettings.h"
#include "Misc/ScopeLock.h"

struct FGroomCacheStreamingManager : public IGroomCacheStreamingManager
{
	//~ Begin IStreamingManager interface
	virtual void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) override;
	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) override;
	virtual void CancelForcedResources() override {}
	virtual void NotifyLevelChange() override {}
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) override {}
	virtual void AddLevel(class ULevel* Level) override {}
	virtual void RemoveLevel(class ULevel* Level) override {}
	virtual void NotifyLevelOffset(class ULevel* Level, const FVector& Offset) override {}
	//~ End IStreamingManager interface

	//~ Begin IGeometryCacheStreamingManager interface
	virtual void RegisterComponent(UGroomComponent* GroomComponent) override;
	virtual void UnregisterComponent(UGroomComponent* GroomComponent) override;
	virtual void PrefetchData(UGroomComponent* GroomComponent) override;
	virtual const FGroomCacheAnimationData* MapAnimationData(const UGroomCache* GroomCache, uint32 ChunkIndex) override;
	virtual void UnmapAnimationData(const UGroomCache* GroomCache, uint32 ChunkIndex) override;
	virtual void Shutdown() override;
	//~ End IGeometryCacheStreamingManager interface

private:

	void RegisterGroomCache(UGroomCache* GroomCache, UGroomComponent* GroomComponent);
	void UnregisterGroomCache(UGroomCache* GroomCache, UGroomComponent* GroomComponent);
	void PrefetchDataInternal(UGroomComponent* GroomComponent);
	void DeleteStreamingData(bool bAsyncDeletionAllowed);

	// StreamingData for registered GroomCaches
	TMap<UGroomCache*, FGroomCacheStreamingData*> StreamingGroomCaches;

	// Set of GroomComponent using the registered GroomCaches
	TMap<UGroomCache*, TSet<UGroomComponent*>> GroomCacheUsers;

	// Set of StreamingData to delete when ready
	TSet<FGroomCacheStreamingData*> StreamingGroomCachesToDelete;

	// GroomComponent that are registered to stream GroomCache data
	TArray<UGroomComponent*> StreamingComponents;

	FCriticalSection CriticalSection;
};

IGroomCacheStreamingManager &IGroomCacheStreamingManager::Get()
{
	static FGroomCacheStreamingManager Instance;
	return Instance;
}

void IGroomCacheStreamingManager::Register()
{
	IStreamingManager::Get().AddStreamingManager(&IGroomCacheStreamingManager::Get());
}

void IGroomCacheStreamingManager::Unregister()
{
	IStreamingManager::Get().RemoveStreamingManager(&IGroomCacheStreamingManager::Get());
	IGroomCacheStreamingManager::Get().Shutdown();
}

void FGroomCacheStreamingManager::Shutdown()
{
	// Delete the StreamingData that were already queued for deletion
	while (StreamingGroomCachesToDelete.Num() > 0)
	{
		DeleteStreamingData(false);
	}

	FScopeLock Lock(&CriticalSection);

	// At this point, make sure that the StreamingData have finished their read requests before deleting them
	for (TMap<UGroomCache*, FGroomCacheStreamingData*>::TIterator Iter = StreamingGroomCaches.CreateIterator(); Iter; ++Iter)
	{
		Iter.Value()->BlockTillAllRequestsFinished();
		delete Iter.Value();
	}

	StreamingGroomCaches.Reset();
	GroomCacheUsers.Reset();
	StreamingComponents.Reset();
}

void FGroomCacheStreamingManager::UpdateResourceStreaming(float DeltaTime, bool bProcessEverything)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingManager::UpdateResourceStreaming);

	const UGroomPluginSettings* Settings = GetDefault<UGroomPluginSettings>();
	{
		FScopeLock Lock(&CriticalSection);

		// Clear needed chunks, they will be recomputed in the next step
		for (TMap<UGroomCache*, FGroomCacheStreamingData*>::TIterator Iter = StreamingGroomCaches.CreateIterator(); Iter; ++Iter)
		{
			Iter.Value()->ResetNeededChunks();
		}

		// Gather all the chunks that need to be streamed from all playing instances based on their current animation time
		for (UGroomComponent* Component : StreamingComponents)
		{
			if (UGroomCache* GroomCache = Component->GetGroomCache())
			{
				FGroomCacheStreamingData** StreamingData = StreamingGroomCaches.Find(GroomCache);
				if (StreamingData)
				{
					float RequestStartTime = Component->GetAnimationTime();
					float RequestEndTime = RequestStartTime + Settings->GroomCacheLookAheadBuffer;

					TArray<int32> ChunksNeeded;
					GroomCache->GetFrameIndicesForTimeRange(RequestStartTime, RequestEndTime, Component->IsLooping(), ChunksNeeded);

					for (int32 ChunkId : ChunksNeeded)
					{
						(*StreamingData)->AddNeededChunk(ChunkId);
					}
				}
			}
		}

		// Update the internal state of the StreamingData
		for (TMap<UGroomCache*, FGroomCacheStreamingData*>::TIterator Iter = StreamingGroomCaches.CreateIterator(); Iter; ++Iter)
		{
			Iter.Value()->UpdateStreamingStatus(true);
		}
	}

	DeleteStreamingData(true);
}

void FGroomCacheStreamingManager::DeleteStreamingData(bool bAsyncDeletionAllowed)
{
	TSet<FGroomCacheStreamingData*> ReadyForDeletion;
	for (FGroomCacheStreamingData* StreamingData : StreamingGroomCachesToDelete)
	{
		StreamingData->UpdateStreamingStatus(bAsyncDeletionAllowed);
		if (!StreamingData->IsStreamingInProgress())
		{
			ReadyForDeletion.Add(StreamingData);
		}
	}

	StreamingGroomCachesToDelete = StreamingGroomCachesToDelete.Difference(ReadyForDeletion);

	for (FGroomCacheStreamingData* StreamingData : ReadyForDeletion)
	{
		delete StreamingData;
	}
}

int32 FGroomCacheStreamingManager::BlockTillAllRequestsFinished(float TimeLimit, bool)
{
	FScopeLock Lock(&CriticalSection);

	int32 Result = 0;

	if (TimeLimit == 0.0f)
	{
		for (TMap<UGroomCache*, FGroomCacheStreamingData*>::TIterator Iter = StreamingGroomCaches.CreateIterator(); Iter; ++Iter)
		{
			Iter.Value()->BlockTillAllRequestsFinished();
		}
	}
	else
	{
		double EndTime = FPlatformTime::Seconds() + TimeLimit;
		for (TMap<UGroomCache*, FGroomCacheStreamingData*>::TIterator Iter = StreamingGroomCaches.CreateIterator(); Iter; ++Iter)
		{
			float ThisTimeLimit = EndTime - FPlatformTime::Seconds();
			if (ThisTimeLimit < .001f || // one ms is the granularity of the platform event system
				!Iter.Value()->BlockTillAllRequestsFinished(ThisTimeLimit))
			{
				Result = 1; // we don't report the actual number, just 1 for any number of outstanding requests
				break;
			}
		}
	}

	return Result;
}

void FGroomCacheStreamingManager::RegisterGroomCache(UGroomCache* GroomCache, UGroomComponent* GroomComponent)
{
	FScopeLock Lock(&CriticalSection);

	if (!StreamingGroomCaches.Contains(GroomCache))
	{
		// Try to recover a StreamingData previously marked for deletion.
		// This helps performance by re-using data that is already loaded
		// and helps maintain accurate ref counts of the Chunks since a
		// delayed unmap could happen after an Unregister+RegisterGroomCache.
		FGroomCacheStreamingData* StreamingData = nullptr;
		for (FGroomCacheStreamingData* StreamingDataForDeletion : StreamingGroomCachesToDelete)
		{
			if (StreamingDataForDeletion->GetGroomCache() == GroomCache)
			{
				StreamingData = StreamingDataForDeletion;
				StreamingGroomCachesToDelete.Remove(StreamingDataForDeletion);
				break;
			}
		}

		if (!StreamingData)
		{
			StreamingData = new FGroomCacheStreamingData(GroomCache);
		}

		StreamingGroomCaches.Add(GroomCache, StreamingData);

		GroomCacheUsers.Add(GroomCache, {});
	}

	GroomCacheUsers.FindChecked(GroomCache).Add(GroomComponent);
}

void FGroomCacheStreamingManager::UnregisterGroomCache(UGroomCache* GroomCache, UGroomComponent* GroomComponent)
{
	FScopeLock Lock(&CriticalSection);

	TSet<UGroomComponent*>* Users = GroomCacheUsers.Find(GroomCache);
	if (!Users)
	{
		return;
	}

	Users->Remove(GroomComponent);

	if (Users->Num() == 0)
	{
		FGroomCacheStreamingData** StreamingData = StreamingGroomCaches.Find(GroomCache);
		if (StreamingData)
		{
			// The needed chunks are reset to prevent starting new read requests
			(*StreamingData)->ResetNeededChunks();

			// The StreamingData is deleted later once all the in-flight read requests are processed
			StreamingGroomCachesToDelete.Add(*StreamingData);

			StreamingGroomCaches.Remove(GroomCache);
		}

		GroomCacheUsers.Remove(GroomCache);
	}
}

void FGroomCacheStreamingManager::RegisterComponent(UGroomComponent* GroomComponent)
{
	if (GroomComponent && GroomComponent->GetGroomCache())
	{
		StreamingComponents.AddUnique(GroomComponent);

		RegisterGroomCache(GroomComponent->GetGroomCache(), GroomComponent);

		PrefetchData(GroomComponent);
	}
}

void FGroomCacheStreamingManager::UnregisterComponent(UGroomComponent* GroomComponent)
{
	if (GroomComponent)
	{
		StreamingComponents.Remove(GroomComponent);

		if (UGroomCache* GroomCache = GroomComponent->GetGroomCache())
		{
			UnregisterGroomCache(GroomCache, GroomComponent);
		}
	}
}

void FGroomCacheStreamingManager::PrefetchDataInternal(UGroomComponent* GroomComponent)
{
	if (UGroomCache* GroomCache = GroomComponent->GetGroomCache())
	{
		FScopeLock Lock(&CriticalSection);

		FGroomCacheStreamingData** StreamingData = StreamingGroomCaches.Find(GroomCache);
		if (StreamingData)
		{
			(*StreamingData)->PrefetchData(GroomComponent);
		}
	}
}

void FGroomCacheStreamingManager::PrefetchData(UGroomComponent* GroomComponent)
{
	if (IsInGameThread())
	{
		PrefetchDataInternal(GroomComponent);
	}
	else
	{
		// The prefetch doesn't need to be executed right now, so schedule it for the game thread
		FWeakObjectPtr WeakComponent(GroomComponent);
		AsyncTask(ENamedThreads::GameThread, [this, WeakComponent]()
		{
			UGroomComponent* Component = Cast<UGroomComponent>(WeakComponent.Get());
			if (Component)
			{
				PrefetchDataInternal(Component);
			}
		});
	}
}

const FGroomCacheAnimationData* FGroomCacheStreamingManager::MapAnimationData(const UGroomCache* GroomCache, uint32 ChunkIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingManager::MapAnimationData);

	FScopeLock Lock(&CriticalSection);

	FGroomCacheStreamingData** StreamingData = StreamingGroomCaches.Find(GroomCache);
	if (StreamingData)
	{
		return (*StreamingData)->MapAnimationData(ChunkIndex);
	}

	return nullptr;
}

void FGroomCacheStreamingManager::UnmapAnimationData(const UGroomCache* GroomCache, uint32 ChunkIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamingManager::UnmapAnimationData);

	// The unmap could happen from the render thread when it releases the GroomCache buffers
	// so access to StreamingGroomCaches has to be protected
	FScopeLock Lock(&CriticalSection);

	FGroomCacheStreamingData** StreamingData = StreamingGroomCaches.Find(GroomCache);
	if (StreamingData)
	{
		(*StreamingData)->UnmapAnimationData(ChunkIndex);
	}
}
