// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomComponent.h"
#include "GeometryCacheComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Engine.h"
#include "NiagaraSystem.h"
#include "PrimitiveSceneProxy.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "HairCardsVertexFactory.h"
#include "HairStrandsVertexFactory.h"
#include "RayTracingInstance.h"
#include "RayTracingDefinitions.h"
#include "HAL/LowLevelMemTracker.h"
#include "HairStrandsInterface.h"
#include "UObject/UObjectIterator.h"
#include "GlobalShader.h"
#include "Components/SkeletalMeshComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Engine/RendererSettings.h"
#include "Animation/AnimationSettings.h"
#include "Animation/MeshDeformerInstance.h"
#include "Animation/MeshDeformer.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "NiagaraComponent.h"
#include "UObject/ConstructorHelpers.h"
#include "GroomBindingBuilder.h"
#include "RenderTargetPool.h"
#include "GroomManager.h"
#include "GroomInstance.h"
#include "GroomCache.h"
#include "GroomCacheStreamingManager.h"
#include "GroomPluginSettings.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "PrimitiveSceneInfo.h"
#include "PSOPrecache.h"
#include "SceneInterface.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "GroomVisualizationData.h"
#include "NaniteGroomAsset.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomComponent)
LLM_DECLARE_TAG(Groom);

#define GROOMBINDING_SKELMESH_VALIDATION 0

static int32 GHairEnableAdaptiveSubsteps = 0;  
static FAutoConsoleVariableRef CVarHairEnableAdaptiveSubsteps(TEXT("r.HairStrands.EnableAdaptiveSubsteps"), GHairEnableAdaptiveSubsteps, TEXT("Enable adaptive solver substeps"));
bool IsHairAdaptiveSubstepsEnabled() { return (GHairEnableAdaptiveSubsteps == 1); }

static int32 GHairBindingValidationEnable = 0;
static FAutoConsoleVariableRef CVarHairBindingValidationEnable(TEXT("r.HairStrands.BindingValidation"), GHairBindingValidationEnable, TEXT("Enable groom binding validation, which report error/warnings with details about the cause."));

static bool GUseGroomCacheStreaming = true;
static FAutoConsoleVariableRef CVarGroomCacheStreamingEnable(TEXT("GroomCache.EnableStreaming"), GUseGroomCacheStreaming, TEXT("Enable groom cache streaming and prebuffering. Do not switch while groom caches are in use."));

static bool GUseProxyLocalToWorld = true;
static FAutoConsoleVariableRef CVarUseProxyLocalToWorld(TEXT("r.HairStrands.UseProxyLocalToWorld"), GUseProxyLocalToWorld, TEXT("Enable the use of the groom proxy local to world instead of extracting it from the game thread."));

static bool GHairStrands_Streaming_Prediction = false;
static FAutoConsoleVariableRef CVarHairStrands_Streaming_Prediction(TEXT("r.HairStrands.Streaming.Prediction"), GHairStrands_Streaming_Prediction, TEXT("Enable LOD streaming prediction."));

static int32 GHairStrands_BoundsMode = 0;
static FAutoConsoleVariableRef CVarHairStrands_BoundsMode(TEXT("r.HairStrands.BoundMode"), GHairStrands_BoundsMode, TEXT("Define how hair bound are computed at runtime when attached to a skel. mesh.\n 0: Use skel.mesh extented with grooms bounds.\n 1: Use skel.mesh bounds.\n 2: Use skel.mesh extented with grooms bounds (conservative)"));

static int32 GHairStrands_UseAttachedSimulationComponents = 0;
static FAutoConsoleVariableRef CVarHairStrands_UseAttachedSimulationComponents(TEXT("r.HairStrands.UseAttachedSimulationComponents"), GHairStrands_UseAttachedSimulationComponents, TEXT("Boolean to check if we are using already attached niagara components for simulation (WIP)"));

static int32 GHairStrands_ViewModeClumpIndex = 0;
static FAutoConsoleVariableRef CVarHairStrands_ViewModeClumpIndex(TEXT("r.HairStrands.ViewMode.ClumpIndex"), GHairStrands_ViewModeClumpIndex, TEXT("Define the ClumpID index (0, 1, or 2) which should be visualized"));

static int32 GHairStrands_ForceVelocityOutput = 0;
static FAutoConsoleVariableRef CVarHairStrands_ForceVelocityOutput(TEXT("r.HairStrands.ForceVelocityOutput"), GHairStrands_ForceVelocityOutput, TEXT("When enabled, force the cards/meshes to write velocity vectors."));

// By default GDME is disabled as it causess a (begnin) race condition (in CreateMeshBatch > BuildUniformShaderParameters) when reading velocity data, as the LastFrameIndex is mutable and can be read/writen at the same time.
// This can happens when view context (e.g., main view + scene capture) are updated at the same time.
static int32 GHairStrands_UseParallelGDME = 0;
static FAutoConsoleVariableRef CVarHairStrands_UseParallelGDME(TEXT("r.HairStrands.UseParallelGDME"), GHairStrands_UseParallelGDME, TEXT("When enabled, FHairStrandsSceneProxy uses parallel GDME (using async task gathering) for collecting instances to render."));

#define LOCTEXT_NAMESPACE "GroomComponent"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Forward declarations

bool IsHairManualSkinCacheEnabled();
bool GetSupportHairStrandsLinearSweptSpheres(EShaderPlatform InShaderPlatform);

#if RHI_RAYTRACING
void AllocateRaytracingResources(FHairGroupInstance* Instance, bool bNeedDynamicResources, FName OwnerName);
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool IsGroomBindingValidationEnabled()
{
	return GHairBindingValidationEnable > 0;
}

template<typename T>
void InternalResourceRelease(T*& In)
{
	if (In)
	{
		In->ReleaseResource();
		delete In;
		In = nullptr;
	}
}

FHairGroupInstance::~FHairGroupInstance()
{
	// Guides
	if (Guides.IsValid())
	{
		InternalResourceRelease(Guides.DeformedRootResource);
		InternalResourceRelease(Guides.DeformedResource);
	}

	// Strands
	if (Strands.IsValid())
	{
		InternalResourceRelease(Strands.DeformedRootResource);
		InternalResourceRelease(Strands.DeformedResource);
		InternalResourceRelease(Strands.CullingResource);

#if RHI_RAYTRACING
		if (Strands.RenRaytracingResourceOwned)
		{
			InternalResourceRelease(Strands.RenRaytracingResource);
		}
#endif
		Strands.DebugCurveAttributeBuffer.Release();

		InternalResourceRelease(Strands.LinearSweptSphereVertexFactory);
		InternalResourceRelease(Strands.VertexFactory);
	}

	// Cards
	{
		const uint32 CardLODCount = Cards.LODs.Num();
		for (uint32 CardLODIt = 0; CardLODIt < CardLODCount; ++CardLODIt)
		{
			if (Cards.IsValid(CardLODIt))
			{
				InternalResourceRelease(Cards.LODs[CardLODIt].Guides.DeformedRootResource);
				InternalResourceRelease(Cards.LODs[CardLODIt].Guides.DeformedResource);
				InternalResourceRelease(Cards.LODs[CardLODIt].DeformedResource);
#if RHI_RAYTRACING
				if (Cards.LODs[CardLODIt].RaytracingResourceOwned)
				{
					InternalResourceRelease(Cards.LODs[CardLODIt].RaytracingResource);
				}
#endif

				InternalResourceRelease(Cards.LODs[CardLODIt].VertexFactory);
			}
		}
	}

	// Meshes
	{
		const uint32 MeshesLODCount = Meshes.LODs.Num();
		for (uint32 MeshesLODIt = 0; MeshesLODIt < MeshesLODCount; ++MeshesLODIt)
		{
			if (Meshes.IsValid(MeshesLODIt))
			{
				InternalResourceRelease(Meshes.LODs[MeshesLODIt].DeformedResource);
#if RHI_RAYTRACING
				if (Meshes.LODs[MeshesLODIt].RaytracingResourceOwned)
				{
					InternalResourceRelease(Meshes.LODs[MeshesLODIt].RaytracingResource);
				}
#endif

				InternalResourceRelease(Meshes.LODs[MeshesLODIt].VertexFactory);
			}
		}
	}

	delete HairGroupPublicData;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const FLinearColor GetHairGroupDebugColor(int32 GroupIt)
{
	static TArray<FLinearColor> IndexedColor;
	if (IndexedColor.Num() == 0)
	{
		IndexedColor.Add(FLinearColor(FColor::Cyan));
		IndexedColor.Add(FLinearColor(FColor::Magenta));
		IndexedColor.Add(FLinearColor(FColor::Orange));
		IndexedColor.Add(FLinearColor(FColor::Silver));
		IndexedColor.Add(FLinearColor(FColor::Emerald));
		IndexedColor.Add(FLinearColor(FColor::Red));
		IndexedColor.Add(FLinearColor(FColor::Green));
		IndexedColor.Add(FLinearColor(FColor::Blue));
		IndexedColor.Add(FLinearColor(FColor::Yellow));
		IndexedColor.Add(FLinearColor(FColor::Purple));
		IndexedColor.Add(FLinearColor(FColor::Turquoise));
	
	}
	if (GroupIt >= IndexedColor.Num())
	{
		IndexedColor.Add(FLinearColor::MakeRandomColor());
	}
	check(GroupIt < IndexedColor.Num());
	return IndexedColor[GroupIt];	
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static EHairBindingType ToHairBindingType(EGroomBindingType In)
{
	switch (In)
	{
	case EGroomBindingType::Rigid		: return EHairBindingType::Rigid;
	case EGroomBindingType::Skinning	: return EHairBindingType::Skinning;
	case EGroomBindingType::NoneBinding	: return EHairBindingType::NoneBinding;
	}
	return EHairBindingType::NoneBinding;
}

static EHairInterpolationType ToHairInterpolationType(EGroomInterpolationType In)
{
	switch (In)
	{
	case EGroomInterpolationType::None				: return EHairInterpolationType::NoneSkinning; 
	case EGroomInterpolationType::RigidTransform	: return EHairInterpolationType::RigidSkinning;
	case EGroomInterpolationType::OffsetTransform	: return EHairInterpolationType::OffsetSkinning;
	case EGroomInterpolationType::SmoothTransform	: return EHairInterpolationType::SmoothSkinning;
	}
	return EHairInterpolationType::NoneSkinning;
}
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static FHairGroupDesc GetGroomGroupsDesc(const UGroomAsset* Asset, const UGroomComponent* Component, uint32 GroupIndex)
{
	if (!Asset || GroupIndex >= uint32(Component->GroomGroupsDesc.Num()))
	{
		return FHairGroupDesc();
	}

	const FHairGroupsRendering& LocalRendering = Asset->GetHairGroupsRendering()[GroupIndex];
	const FHairGroupPlatformData& LocalPlatformData = Asset->GetHairGroupsPlatformData()[GroupIndex];

	FHairGroupDesc O = Component->GroomGroupsDesc[GroupIndex];
	O.HairLength = LocalPlatformData.Strands.BulkData.Header.MaxLength;
	O.LODBias 	 = Asset->GetEffectiveLODBias()[GroupIndex] > 0 ? FMath::Max(O.LODBias, Asset->GetEffectiveLODBias()[GroupIndex]) : O.LODBias;

	// Width can be overridden at 3 levels:
	// * Instance width override
	// * Asset width override
	// * Imported width
	if (!O.HairWidth_Override)
	{ 
		O.HairWidth = 
			LocalRendering.GeometrySettings.HairWidth_Override ? 
			LocalRendering.GeometrySettings.HairWidth : 
			LocalPlatformData.Strands.BulkData.Header.MaxRadius * 2.0f;
	}

	if (!O.HairRootScale_Override)				{ O.HairRootScale				= LocalRendering.GeometrySettings.HairRootScale;				}
	if (!O.HairTipScale_Override)				{ O.HairTipScale				= LocalRendering.GeometrySettings.HairTipScale;					}
	if (!O.bSupportVoxelization_Override)		{ O.bSupportVoxelization		= LocalRendering.ShadowSettings.bVoxelize;						}
	if (!O.HairShadowDensity_Override)			{ O.HairShadowDensity			= LocalRendering.ShadowSettings.HairShadowDensity;				}
	if (!O.HairRaytracingRadiusScale_Override)	{ O.HairRaytracingRadiusScale	= LocalRendering.ShadowSettings.HairRaytracingRadiusScale;		}
	if (!O.bUseHairRaytracingGeometry_Override) { O.bUseHairRaytracingGeometry  = LocalRendering.ShadowSettings.bUseHairRaytracingGeometry;		}
	if (!O.bUseStableRasterization_Override)	{ O.bUseStableRasterization		= LocalRendering.AdvancedSettings.bUseStableRasterization;		}
	if (!O.bScatterSceneLighting_Override)		{ O.bScatterSceneLighting		= LocalRendering.AdvancedSettings.bScatterSceneLighting;		}

	return O;
}

static EPrimitiveType GetPrimitiveType(EHairGeometryType In)
{
	return (In == EHairGeometryType::Strands && GetHairStrandsUsesTriangleStrips()) ? PT_TriangleStrip : PT_TriangleList;
}

static uint32 GetPointToVertexCount()
{
	return GetHairStrandsUsesTriangleStrips() ? HAIR_POINT_TO_VERTEX_FOR_TRISTRP : HAIR_POINT_TO_VERTEX_FOR_TRILIST;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * An material render proxy which overrides the debug mode parameter.
 */
class FHairDebugModeMaterialRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const float DebugMode;
	const float HairMinRadius;
	const float HairMaxRadius;
	const FVector HairGroupColor;

	FName DebugModeParamName;
	FName MinHairRadiusParamName;
	FName MaxHairRadiusParamName;
	FName HairGroupHairColorParamName;

	/** Initialization constructor. */
	FHairDebugModeMaterialRenderProxy(const FMaterialRenderProxy* InParent, float InMode, float InMinRadius, float InMaxRadius, FVector InGroupColor) :
		FMaterialRenderProxy(InParent->GetMaterialName()),
		Parent(InParent),
		DebugMode(InMode),
		HairMinRadius(InMinRadius),
		HairMaxRadius(InMaxRadius),
		HairGroupColor(InGroupColor),
		DebugModeParamName(NAME_FloatProperty),
		MinHairRadiusParamName(NAME_ByteProperty),
		MaxHairRadiusParamName(NAME_IntProperty),
		HairGroupHairColorParamName(NAME_VectorProperty)
	{}

	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		switch (Type)
		{
		case EMaterialParameterType::Vector:
			if (ParameterInfo.Name == HairGroupHairColorParamName)
			{
				OutValue = FVector3f(HairGroupColor);
				return true;
			}
			break;
		case EMaterialParameterType::Scalar:
			if (ParameterInfo.Name == DebugModeParamName)
			{
				OutValue = DebugMode;
				return true;
			}
			else if (ParameterInfo.Name == MinHairRadiusParamName)
			{
				OutValue = HairMinRadius;
				return true;
			}
			else if (ParameterInfo.Name == MaxHairRadiusParamName)
			{
				OutValue = HairMaxRadius;
				return true;
			}
			break;
		default:
			break;
		}
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
};

enum class EHairMaterialCompatibility : uint8
{
	Valid,
	Invalid_UsedWithHairStrands,
	Invalid_ShadingModel,
	Invalid_BlendMode,
	Invalid_IsNull
};

static EHairMaterialCompatibility IsHairMaterialCompatible(UMaterialInterface* MaterialInterface, EShaderPlatform InShaderPlatform, EHairGeometryType GeometryType)
{
	// Hair material and opaque material are enforced for strands material as the strands system is tailored for this type of shading
	// (custom packing of material attributes). However this is not needed/required for cards/meshes, when the relaxation for these type
	// of goemetry
	if (MaterialInterface)
	{
		if (GeometryType != Strands)
		{
			return EHairMaterialCompatibility::Valid;
		}
		const FMaterialRelevance Relevance = MaterialInterface->GetRelevance_Concurrent(InShaderPlatform);
		const bool bIsRelevanceInitialized = Relevance.Raw != 0;
		if (bIsRelevanceInitialized && !Relevance.bHairStrands)
		{
			return EHairMaterialCompatibility::Invalid_UsedWithHairStrands;
		}
		if (!MaterialInterface->GetShadingModels().HasShadingModel(MSM_Hair) && GeometryType == EHairGeometryType::Strands)
		{
			return EHairMaterialCompatibility::Invalid_ShadingModel;
		}
		if (!IsOpaqueOrMaskedBlendMode(*MaterialInterface) && GeometryType == EHairGeometryType::Strands)
		{
			return EHairMaterialCompatibility::Invalid_BlendMode;
		}
	}
	else
	{
		return EHairMaterialCompatibility::Invalid_IsNull;
	}

	return EHairMaterialCompatibility::Valid;
}

enum class EHairMeshBatchType
{
	Raster,
	Raytracing
};

/////////////////////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////////////////////
//  FStrandHairSceneProxy
/////////////////////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////////////////////

inline int32 GetMaterialIndexWithFallback(int32 SlotIndex)
{
	// Default policy: if no slot index has been bound, fallback on slot 0. If there is no
	// slot, the material will fallback on the default material.
	return SlotIndex != INDEX_NONE ? SlotIndex : 0;
}

static EHairGeometryType ToHairGeometryType(EGroomGeometryType Type)
{
	switch (Type)
	{
	case EGroomGeometryType::Strands: return EHairGeometryType::Strands;
	case EGroomGeometryType::Cards:   return EHairGeometryType::Cards;
	case EGroomGeometryType::Meshes:  return EHairGeometryType::Meshes;
	}
	return EHairGeometryType::NoneGeometry;
}

class FHairStrandsSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FRenderCurveResourceData* CurveResourceData = nullptr;

	virtual FRenderCurveResourceData* GetRenderCurveResourceData() override
	{
		return CurveResourceData;
	}

	FHairStrandsSceneProxy(UGroomComponent* Component)
		: FPrimitiveSceneProxy(Component)
		, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
	{
		// Forcing primitive uniform as we don't support robustly GPU scene data
		bVFRequiresPrimitiveUniformBuffer = true;
		bCastDeepShadow = true;
		bSupportsParallelGDME = GHairStrands_UseParallelGDME>0;

		HairGroupMaterialProxies.SetNum(Component->HairGroupInstances.Num());
		HairGroupInstances = Component->HairGroupInstances;
		check(Component);
		check(Component->GroomAsset);
		check(Component->GroomAsset->GetNumHairGroups() > 0);
		ComponentId = Component->GetPrimitiveSceneId().PrimIDValue;
		Strands_DebugMaterial = Component->Strands_DebugMaterial;
		bAlwaysHasVelocity = false;
		if ((IsHairStrandsBindingEnable() && Component->RegisteredMeshComponent) || GHairStrands_ForceVelocityOutput > 0)
		{
			bAlwaysHasVelocity = true;
		}

		check(HairGroupInstances.Num());

		const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();
		const EShaderPlatform ShaderPlatform = GetScene().GetShaderPlatform();

		const int32 GroupCount = Component->GroomAsset->GetNumHairGroups();
		check(Component->GroomAsset->GetHairGroupsPlatformData().Num() == HairGroupInstances.Num());
		for (int32 GroupIt=0; GroupIt<GroupCount; GroupIt++)
		{
			const bool bIsVisible = Component->GroomAsset->GetHairGroupsInfo()[GroupIt].bIsVisible;

			FHairGroupPlatformData& InGroupDataNonConst = Component->GroomAsset->GetHairGroupsPlatformData()[GroupIt];
			const FHairGroupPlatformData& InGroupData = Component->GroomAsset->GetHairGroupsPlatformData()[GroupIt];
			FHairGroupInstance* HairInstance = HairGroupInstances[GroupIt];
			check(HairInstance->HairGroupPublicData);
			HairInstance->bForceCards = Component->bUseCards;
			HairInstance->bUpdatePositionOffset = Component->RegisteredMeshComponent != nullptr;

			if (HairInstance->Strands.IsValid())
			{
				if (GetSupportHairStrandsLinearSweptSpheres(ShaderPlatform) && IsHairStrandsLinearSweptSpheresEnabled())
				{
					if (HairInstance->Strands.LinearSweptSphereVertexFactory == nullptr)
					{
						HairInstance->Strands.LinearSweptSphereVertexFactory = new FLinearSweptSpheresHairStrandsVertexFactory(HairInstance, FeatureLevel, "FLinearSweptSphereStrandsHairSceneProxy");
					}
				}
				else 
				{
					if (HairInstance->Strands.VertexFactory == nullptr)
					{
						HairInstance->Strands.VertexFactory = new FHairStrandsVertexFactory(HairInstance, FeatureLevel, "FStrandsHairSceneProxy");
					}
				}
			}

			// (Experimental) For now only use the resource of the first group
			CurveResourceData = nullptr;
			if (IsRenderCurveEnabled() && HairInstance->Strands.IsValid() && GroupIt == 0)
			{
				CurveResourceData = (&InGroupDataNonConst.Strands.CurveResourceData);
			}

			if (HairInstance->Cards.IsValid())
			{
				const uint32 LODCount = HairInstance->Cards.LODs.Num();
				for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
				{
					if (HairInstance->Cards.IsValid(LODIt))
					{
						if (HairInstance->Cards.LODs[LODIt].VertexFactory == nullptr) HairInstance->Cards.LODs[LODIt].VertexFactory = new FHairCardsVertexFactory(HairInstance, LODIt, EHairGeometryType::Cards, ShaderPlatform, FeatureLevel, "HairCardsVertexFactory");
					}
				}
			}

			if (HairInstance->Meshes.IsValid())
			{
				const uint32 LODCount = HairInstance->Meshes.LODs.Num();
				for (uint32 LODIt = 0; LODIt < LODCount; ++LODIt)
				{
					if (HairInstance->Meshes.IsValid(LODIt))
					{
						if (HairInstance->Meshes.LODs[LODIt].VertexFactory == nullptr) HairInstance->Meshes.LODs[LODIt].VertexFactory = new FHairCardsVertexFactory(HairInstance, LODIt, EHairGeometryType::Meshes, ShaderPlatform, FeatureLevel, "HairMeshesVertexFactory");
					}
				}
			}

			{
				// If one of the group has simulation enable, then we enable velocity rendering for meshes/cards
				if (IsHairStrandsSimulationEnable() && HairInstance->Guides.IsValid() && (HairInstance->Guides.bIsSimulationEnable || HairInstance->Guides.bIsDeformationEnable || HairInstance->Guides.bIsSimulationCacheEnable))
				{
					bAlwaysHasVelocity = true;
				}
			}

			// Material - Strands
			if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands, ShaderPlatform))
			{
				const int32 SlotIndex = Component->GroomAsset->GetMaterialIndex(Component->GroomAsset->GetHairGroupsRendering()[GroupIt].MaterialSlotName);
				Component->CheckHairStrandsUsage(GetMaterialIndexWithFallback(SlotIndex));
				const UMaterialInterface* Material = Component->GetMaterial(GetMaterialIndexWithFallback(SlotIndex), EHairGeometryType::Strands, true);
				HairGroupMaterialProxies[GroupIt].Strands = Material ? Material->GetRenderProxy() : nullptr;
			}

			// Material - Cards
			HairGroupMaterialProxies[GroupIt].Cards.Init(nullptr, InGroupData.Cards.LODs.Num());
			if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards, ShaderPlatform))
			{
				uint32 CardsLODIndex = 0;
				for (const FHairGroupPlatformData::FCards::FLOD& LOD : InGroupData.Cards.LODs)
				{
					if (LOD.IsValid())
					{
						// Material
						int32 SlotIndex = INDEX_NONE;
						for (const FHairGroupsCardsSourceDescription& Desc : Component->GroomAsset->GetHairGroupsCards())
						{
							if (Desc.GroupIndex == GroupIt && Desc.LODIndex == CardsLODIndex)
							{
								SlotIndex = Component->GroomAsset->GetMaterialIndex(Desc.MaterialSlotName);
								break;
							}
						}
						Component->CheckHairStrandsUsage(GetMaterialIndexWithFallback(SlotIndex));
						const UMaterialInterface* Material = Component->GetMaterial(GetMaterialIndexWithFallback(SlotIndex), EHairGeometryType::Cards, true);
						HairGroupMaterialProxies[GroupIt].Cards[CardsLODIndex] = Material ? Material->GetRenderProxy() : nullptr;
					}
					++CardsLODIndex;
				}
			}

			// Material - Meshes
			HairGroupMaterialProxies[GroupIt].Meshes.Init(nullptr, InGroupData.Meshes.LODs.Num());
			if (IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, ShaderPlatform))
			{
				uint32 MeshesLODIndex = 0;
				for (const FHairGroupPlatformData::FMeshes::FLOD& LOD : InGroupData.Meshes.LODs)
				{
					if (LOD.IsValid())
					{
						// Material
						int32 SlotIndex = INDEX_NONE;
						for (const FHairGroupsMeshesSourceDescription& Desc : Component->GroomAsset->GetHairGroupsMeshes())
						{
							if (Desc.GroupIndex == GroupIt && Desc.LODIndex == MeshesLODIndex)
							{
								SlotIndex = Component->GroomAsset->GetMaterialIndex(Desc.MaterialSlotName);
								break;
							}
						}
						Component->CheckHairStrandsUsage(GetMaterialIndexWithFallback(SlotIndex));
						const UMaterialInterface* Material = Component->GetMaterial(GetMaterialIndexWithFallback(SlotIndex), EHairGeometryType::Meshes, true);
						HairGroupMaterialProxies[GroupIt].Meshes[MeshesLODIndex] = Material ? Material->GetRenderProxy() : nullptr;
					}
					++MeshesLODIndex;
				}
			}
		}
	}

	virtual ~FHairStrandsSceneProxy()
	{
	}
	
	/**
	 *	Called when the rendering thread adds the proxy to the scene.
	 *	This function allows for generating renderer-side resources.
	 *	Called in the rendering thread.
	 */
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) 
	{
		FPrimitiveSceneProxy::CreateRenderThreadResources(RHICmdList);

		// Register the data to the scene. AddHairStrands takes a raw pointer 
		// but internally registers it as TRefCountPtr which keeps the instance 
		// alive while it is registered in the scene.
		FSceneInterface& LocalScene = GetScene();
		for (TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			if (Instance.IsValid() && (Instance->IsValid() || Instance->Strands.ClusterResource))
			{
				check(Instance->HairGroupPublicData != nullptr);
				Instance->Debug.Proxy = this;
				LocalScene.AddHairStrands(Instance);
			}
		}
	}

	/**
	 *	Called when the rendering thread removes the proxy from the scene.
	 *	This function allows for removing renderer-side resources.
	 *	Called in the rendering thread.
	 */
	virtual void DestroyRenderThreadResources() override
	{
		FPrimitiveSceneProxy::DestroyRenderThreadResources();

		// Unregister the data from the scene. RemoveHairStrands releases the 
		// scene's TRefCountPtr, which may destroy the instance if this was 
		// the last reference.
		FSceneInterface& LocalScene = GetScene();
		for (TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			if (Instance.IsValid() && (Instance->IsValid() || Instance->Strands.ClusterResource))
			{
				LocalScene.RemoveHairStrands(Instance);
				Instance->Debug.Proxy = nullptr;
			}
		}
	}

	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override
	{
		const FTransform RigidLocalToWorld = FTransform(GetLocalToWorld());
		for (TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			Instance->Debug.RigidPreviousLocalToWorld = Instance->Debug.RigidCurrentLocalToWorld;
			Instance->Debug.RigidCurrentLocalToWorld = RigidLocalToWorld;
		}
	}

	FORCEINLINE bool UseProxyLocalToWorld(const FHairGroupInstance* Instance) const
	{
		return (GUseProxyLocalToWorld && (Instance->BindingType != EHairBindingType::Skinning));
	}

#if RHI_RAYTRACING
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool IsRayTracingStaticRelevant() const override { return false; }
	virtual bool HasRayTracingRepresentation() const override { return true; }

	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override
	{
		if (!IsHairRayTracingEnabled() || HairGroupInstances.Num() == 0)
			return;

		TConstArrayView<const FSceneView*> Views = Collector.GetViews();
		const uint32 VisibilityMap = Collector.GetVisibilityMap();

		// RT geometry will be generated based on first active view and then reused for all other views
		// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
		const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
		checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

		const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

		const EShaderPlatform Platform = FirstActiveView->GetShaderPlatform();
		if (!IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform) &&
			!IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Platform) &&
			!IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, Platform))
		{
			return;
		}

		const bool bWireframe = AllowDebugViewmodes() && FirstActiveView->Family->EngineShowFlags.Wireframe;
		const EHairViewRayTracingMask ViewRayTracingMask = FirstActiveView->Family->EngineShowFlags.PathTracing ? EHairViewRayTracingMask::PathTracing : EHairViewRayTracingMask::RayTracing;
		if (bWireframe)
			return;

		for (uint32 GroupIt = 0, GroupCount = HairGroupInstances.Num(); GroupIt < GroupCount; ++GroupIt)
		{
			FHairGroupInstance* Instance = HairGroupInstances[GroupIt];

			FMatrix OverrideLocalToWorld = UseProxyLocalToWorld(Instance) ? GetLocalToWorld() : Instance->GetCurrentLocalToWorld().ToMatrixWithScale();

			const EHairGeometryType GeometryType = Instance->HairGroupPublicData->VFInput.GeometryType;
			const uint32 LODIndex = Instance->HairGroupPublicData->GetIntLODIndex();

			FHairStrandsRaytracingResource* RTGeometry = nullptr;
			bool bIsHairStrands = false;
			EHairViewRayTracingMask InstanceViewRayTracingMask = EHairViewRayTracingMask::PathTracing | EHairViewRayTracingMask::RayTracing;
			switch (GeometryType)
			{
				case EHairGeometryType::Strands:
				{
					RTGeometry = Instance->Strands.RenRaytracingResource;
					InstanceViewRayTracingMask = Instance->Strands.ViewRayTracingMask;
					bIsHairStrands = true;
					break;
				}
				case EHairGeometryType::Cards:
				{
					RTGeometry = Instance->Cards.LODs[LODIndex].RaytracingResource;
					break;
				}
				case EHairGeometryType::Meshes:
				{
					RTGeometry = Instance->Meshes.LODs[LODIndex].RaytracingResource;
					break;
				}
			}

			// If the view and the instance raytracing mask don't match skip this instance.
			if (!EnumHasAnyFlags(InstanceViewRayTracingMask, ViewRayTracingMask))
			{
				continue;
			}

			if (RTGeometry && RTGeometry->RayTracingGeometry.IsValid())
			{
				const FRayTracingGeometryInitializer& RayTracingGeometryInitializer = RTGeometry->RayTracingGeometry.GetInitializer();

				for (const FRayTracingGeometrySegment& Segment : RayTracingGeometryInitializer.Segments)
				{
					check(Segment.VertexBuffer.IsValid());
				}

				// ViewFamily.EngineShowFlags.PathTracing
				if (FMeshBatch* MeshBatch = CreateMeshBatch(*FirstActiveView->Family, Collector, EHairMeshBatchType::Raytracing, Instance, GroupIt, nullptr))
				{
					FRayTracingInstance RayTracingInstance;
					RayTracingInstance.Geometry = &RTGeometry->RayTracingGeometry;
					RayTracingInstance.Materials.Add(*MeshBatch);
					RayTracingInstance.InstanceTransforms.Add(OverrideLocalToWorld);
					RayTracingInstance.bThinGeometry = bIsHairStrands;

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						if ((VisibilityMap & (1 << ViewIndex)) == 0)
						{
							continue;
						}

						Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);

						if (RTGeometry->PositionBuffer.Buffer)
						{
							Collector.AddRDGPooledBuffer(ViewIndex, RTGeometry->PositionBuffer.Buffer);
						}
						if (RTGeometry->IndexBuffer.Buffer)
						{
							Collector.AddRDGPooledBuffer(ViewIndex, RTGeometry->IndexBuffer.Buffer);
						}
					}
				}
			}
		}
	}
#endif

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		const EShaderPlatform Platform = ViewFamily.GetShaderPlatform();
		if (!IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform) &&
			!IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Platform) &&
			!IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, Platform))
		{
			return;
		}

		if (HairGroupInstances.Num() == 0)
		{
			return;
		}

		const uint32 GroupCount = HairGroupInstances.Num();

		QUICK_SCOPE_CYCLE_COUNTER(STAT_HairStrandsSceneProxy_GetDynamicMeshElements);

		// Need information back from the rendering thread to knwo which representation to use (strands/cards/mesh)
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FSceneView* View = Views[ViewIndex];
			if (View->bIsReflectionCapture)
			{
				continue;
			}

			if ((IsShadowCast(View) || IsShown(View)) && (VisibilityMap & (1 << ViewIndex)))
			{
				for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
				{
					FMaterialRenderProxy* Debug_MaterialProxy = nullptr;
					const EGroomViewMode ViewMode = GetGroomViewMode(*View);
					bool bNeedDebugMaterial = false;
					switch(ViewMode)
					{
					case EGroomViewMode::SimHairStrands	:
					case EGroomViewMode::Cluster		:
					case EGroomViewMode::ClusterAABB	:
					case EGroomViewMode::RenderHairStrands:
						bNeedDebugMaterial = HairGroupInstances[GroupIt]->GeometryType == EHairGeometryType::Strands; break;
					case EGroomViewMode::RootUV			:
					case EGroomViewMode::UV				:
					case EGroomViewMode::Seed			:
					case EGroomViewMode::ClumpID		:
					case EGroomViewMode::Dimension		:
					case EGroomViewMode::RadiusVariation:
					case EGroomViewMode::RootUDIM		:
					case EGroomViewMode::Color			:
					case EGroomViewMode::Roughness		:
					case EGroomViewMode::AO				:
					case EGroomViewMode::Group			:
					case EGroomViewMode::LODColoration	:
						bNeedDebugMaterial = true; break;
					};

					if (bNeedDebugMaterial)
					{
						float DebugModeScalar = 0;
						switch(ViewMode)
						{
						case EGroomViewMode::None				: DebugModeScalar =99.f; break;
						case EGroomViewMode::SimHairStrands		: DebugModeScalar = 0.f; break;
						case EGroomViewMode::RenderHairStrands	: DebugModeScalar = 0.f; break;
						case EGroomViewMode::RootUV				: DebugModeScalar = 1.f; break;
						case EGroomViewMode::UV					: DebugModeScalar = 2.f; break;
						case EGroomViewMode::Seed				: DebugModeScalar = 3.f; break;
						case EGroomViewMode::Dimension			: DebugModeScalar = 4.f; break;
						case EGroomViewMode::RadiusVariation	: DebugModeScalar = 5.f; break;
						case EGroomViewMode::RootUDIM			: DebugModeScalar = 6.f; break;
						case EGroomViewMode::Color				: DebugModeScalar = 7.f; break;
						case EGroomViewMode::Roughness			: DebugModeScalar = 8.f; break;
						case EGroomViewMode::Cluster			: DebugModeScalar = 0.f; break;
						case EGroomViewMode::ClusterAABB		: DebugModeScalar = 0.f; break;
						case EGroomViewMode::Group				: DebugModeScalar = 9.f; break;
						case EGroomViewMode::LODColoration		: DebugModeScalar = 10.f; break;
						case EGroomViewMode::ClumpID			: DebugModeScalar = 11.f; break;
						case EGroomViewMode::AO					: DebugModeScalar = 12.f; break;
						};

						// TODO: fix this as the radius is incorrect. This code run before the interpolation code, which is where HairRadius is updated.
						float HairMaxRadius = 0;
						for (FHairGroupInstance* Instance : HairGroupInstances)
						{
							HairMaxRadius = FMath::Max(HairMaxRadius, Instance->Strands.Modifier.HairWidth * 0.5f);
						}
						
						// Reuse the HairMaxRadius field to send the LOD index instead of adding yet another variable
						if (ViewMode == EGroomViewMode::LODColoration)
						{
							HairMaxRadius = HairGroupInstances[GroupIt]->HairGroupPublicData ? HairGroupInstances[GroupIt]->HairGroupPublicData->LODIndex : 0;
						}

						// Reuse the HairMaxRadius field to send the clumpID index selection
						if (ViewMode == EGroomViewMode::ClumpID)
						{
							HairMaxRadius = FMath::Clamp(GHairStrands_ViewModeClumpIndex, 0, 2);
						}

						FVector HairColor = FVector::ZeroVector;
						if (ViewMode == EGroomViewMode::Group)
						{
							HairColor = FVector(GetHairGroupDebugColor(GroupIt));
						}
						else if (ViewMode == EGroomViewMode::LODColoration)
						{
							int32 LODIndex = HairGroupInstances[GroupIt]->HairGroupPublicData ? HairGroupInstances[GroupIt]->HairGroupPublicData->LODIndex : 0;
							LODIndex = FMath::Clamp(LODIndex, 0, GEngine->LODColorationColors.Num() - 1);
							const FLinearColor LODColor = GEngine->LODColorationColors[LODIndex];
							HairColor = FVector(LODColor.R, LODColor.G, LODColor.B);
						}
						auto DebugMaterial = new FHairDebugModeMaterialRenderProxy(Strands_DebugMaterial ? Strands_DebugMaterial->GetRenderProxy() : nullptr, DebugModeScalar, 0, HairMaxRadius, HairColor);
						Collector.RegisterOneFrameMaterialProxy(DebugMaterial);
						Debug_MaterialProxy = DebugMaterial;
					}

					if (FMeshBatch* MeshBatch = CreateMeshBatch(ViewFamily, Collector, EHairMeshBatchType::Raster, HairGroupInstances[GroupIt], GroupIt, Debug_MaterialProxy))
					{
						Collector.AddMesh(ViewIndex, *MeshBatch);
					}
					else
					{
						continue;
					}

				#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					// Render bounds
					if (ViewMode == EGroomViewMode::Bounds)
					{
						FEngineShowFlags LocalShowFlags = ViewFamily.EngineShowFlags;
						LocalShowFlags.Bounds = true;
						RenderBounds(Collector.GetPDI(ViewIndex), LocalShowFlags, GetBounds(), true);
					}
					else
					{
						RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
					}
				#endif
				}
			}
		}
	}

	FMeshBatch* CreateMeshBatch(
		const FSceneViewFamily& ViewFamily,
		FMeshElementCollector& Collector,
		const EHairMeshBatchType MeshBatchType,
		const FHairGroupInstance* Instance,
		uint32 GroupIndex,
		FMaterialRenderProxy* Debug_MaterialProxy) const
	{
		const EHairGeometryType GeometryType = Instance->GeometryType;
		if (GeometryType == EHairGeometryType::NoneGeometry)
		{
			return nullptr;
		}

		const int32 IntLODIndex = Instance->HairGroupPublicData->GetIntLODIndex();
		const bool bIsVisible = Instance->HairGroupPublicData->GetLODVisibility();

		const FVertexFactory* VertexFactory = nullptr;
		FIndexBuffer* IndexBuffer = nullptr;
		const FMaterialRenderProxy* MaterialRenderProxy = Debug_MaterialProxy;
		const ERHIFeatureLevel::Type FeatureLevel = ViewFamily.GetFeatureLevel();

		uint32 NumPrimitive = 0;
		uint32 HairVertexCount = 0;
		uint32 MaxVertexIndex = 0;
		bool bUseCulling = false;
		bool bWireframe = false;
		EPrimitiveIdMode PrimitiveIdMode = PrimID_Num;
		if (GeometryType == EHairGeometryType::Meshes)
		{
			if (!Instance->Meshes.IsValid(IntLODIndex))
			{
				return nullptr;
			}
			VertexFactory = (FVertexFactory*)Instance->Meshes.LODs[IntLODIndex].GetVertexFactory();
			check(VertexFactory);
			PrimitiveIdMode = Instance->Meshes.LODs[IntLODIndex].GetVertexFactory()->GetPrimitiveIdMode(FeatureLevel);
			HairVertexCount = Instance->Meshes.LODs[IntLODIndex].RestResource->GetPrimitiveCount() * 3;
			MaxVertexIndex = HairVertexCount;
			NumPrimitive = HairVertexCount / 3;
			IndexBuffer = &Instance->Meshes.LODs[IntLODIndex].RestResource->IndexBuffer;
			bUseCulling = false;
			if (MaterialRenderProxy == nullptr)
			{
				MaterialRenderProxy = HairGroupMaterialProxies[GroupIndex].Meshes[IntLODIndex];
			}
			bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;
		}
		else if (GeometryType == EHairGeometryType::Cards)
		{
			if (!Instance->Cards.IsValid(IntLODIndex))
			{
				return nullptr;
			}

			VertexFactory = (FVertexFactory*)Instance->Cards.LODs[IntLODIndex].GetVertexFactory();
			check(VertexFactory);
			PrimitiveIdMode = Instance->Cards.LODs[IntLODIndex].GetVertexFactory()->GetPrimitiveIdMode(FeatureLevel);
			HairVertexCount = Instance->Cards.LODs[IntLODIndex].RestResource->GetPrimitiveCount() * 3;
			MaxVertexIndex = HairVertexCount;
			NumPrimitive = HairVertexCount / 3;
			IndexBuffer = &Instance->Cards.LODs[IntLODIndex].RestResource->RestIndexBuffer;
			bUseCulling = false;
			if (MaterialRenderProxy == nullptr)
			{
				MaterialRenderProxy = HairGroupMaterialProxies[GroupIndex].Cards[IntLODIndex];
			}
			bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;
		}
		else // if (GeometryType == EHairGeometryType::Strands)
		{
			check(Instance->Strands.LinearSweptSphereVertexFactory != nullptr || Instance->Strands.VertexFactory != nullptr);
			VertexFactory = Instance->Strands.LinearSweptSphereVertexFactory != nullptr ? static_cast<const FVertexFactory*>(Instance->Strands.LinearSweptSphereVertexFactory) : static_cast<const FVertexFactory*>(Instance->Strands.VertexFactory);
			PrimitiveIdMode = Instance->Strands.LinearSweptSphereVertexFactory != nullptr ? Instance->Strands.LinearSweptSphereVertexFactory->GetPrimitiveIdMode(FeatureLevel) : Instance->Strands.VertexFactory->GetPrimitiveIdMode(FeatureLevel);
			HairVertexCount = Instance->HairGroupPublicData->GetActiveStrandsPointCount();
			MaxVertexIndex = HairVertexCount * GetPointToVertexCount();
			bUseCulling = Instance->Strands.bCullingEnable;
			NumPrimitive = bUseCulling ? 0 : HairVertexCount * HAIR_POINT_TO_TRIANGLE;
			if (MaterialRenderProxy == nullptr)
			{
				MaterialRenderProxy = HairGroupMaterialProxies[GroupIndex].Strands;
			}
			bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;
		}

		if (MaterialRenderProxy == nullptr || !bIsVisible)
		{
			return nullptr;
		}

		// Invalid primitive setup. This can happens when the (procedural) resources are not ready.
		if (NumPrimitive == 0 && !bUseCulling)
		{
			return nullptr;
		}

		if (bWireframe)
		{
			FMaterialRenderProxy* ColoredMaterialRenderProxy = new FColoredMaterialRenderProxy( GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : NULL, FLinearColor(1.f, 0.5f, 0.f));
			Collector.RegisterOneFrameMaterialProxy(ColoredMaterialRenderProxy);
			MaterialRenderProxy = ColoredMaterialRenderProxy;
		}

		// Draw the mesh.
		FMeshBatch& Mesh = Collector.AllocateMesh();

		const bool bUseCardsOrMeshes = GeometryType == EHairGeometryType::Cards || GeometryType == EHairGeometryType::Meshes;
		Mesh.CastShadow = bUseCardsOrMeshes;
#if RHI_RAYTRACING
		Mesh.CastRayTracedShadow = (MeshBatchType == EHairMeshBatchType::Raytracing || bUseCardsOrMeshes) && bCastDynamicShadow;
#endif
		Mesh.bUseForMaterial = MeshBatchType == EHairMeshBatchType::Raytracing || bUseCardsOrMeshes;
		Mesh.bUseForDepthPass = bUseCardsOrMeshes;
		Mesh.SegmentIndex = 0;
		#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		Mesh.VisualizeLODIndex = IntLODIndex;
		#endif

		FMeshBatchElement& BatchElement = Mesh.Elements[0];
		BatchElement.IndexBuffer = IndexBuffer;
		Mesh.bWireframe = bWireframe;
		Mesh.VertexFactory = VertexFactory;
		Mesh.MaterialRenderProxy = MaterialRenderProxy;
		bool bHasPrecomputedVolumetricLightmap;
		FMatrix PreviousLocalToWorld;
		int32 SingleCaptureIndex;
		bool bOutputVelocity = GeometryType == EHairGeometryType::Cards || GeometryType == EHairGeometryType::Meshes;

		FPrimitiveSceneInfo* PrimSceneInfo = GetPrimitiveSceneInfo();
		GetScene().GetPrimitiveUniformShaderParameters_RenderThread(PrimSceneInfo, bHasPrecomputedVolumetricLightmap, PreviousLocalToWorld, SingleCaptureIndex, bOutputVelocity);
		bOutputVelocity |= AlwaysHasVelocity();

		const bool bUseProxy = UseProxyLocalToWorld(Instance);

		FMatrix CurrentLocalToWorld = bUseProxy ? GetLocalToWorld() : Instance->GetCurrentLocalToWorld().ToMatrixWithScale();
		PreviousLocalToWorld = bUseProxy ? PreviousLocalToWorld : Instance->GetPreviousLocalToWorld().ToMatrixWithScale();

		// Band-aid to avoid invalid velociy vector when switching LOD skinned <-> rigid
		if (Instance->HairGroupPublicData->VFInput.bHasLODSwitch && Instance->HairGroupPublicData->VFInput.bHasLODSwitchBindingType)
		{
			PreviousLocalToWorld = CurrentLocalToWorld;
		}

		// Update primitive uniform buffer
		{
			// Use default SceneProxy builder values
			FPrimitiveUniformShaderParametersBuilder Builder;
			BuildUniformShaderParameters(Builder);

			// Override transforms and the local bound.
			// The original local bound relative to the component local to world transform. If we override the local to world transform, 
			// we need to recompute the local bound relative to this new transform. It is important that the new local bound is correct 
			// as otherwise the GPUScene (which use bot the local to world transform and the local bound for culling purpose) will issue 
			// incorrect visibility test.
			FBoxSphereBounds NewLocalBound = GetLocalBounds();
			if (!bUseProxy)
			{
				const FTransform InvLocalToWorld = Instance->GetCurrentLocalToWorld().Inverse();
				const FBoxSphereBounds OriginalWorldBound = GetBounds();
				NewLocalBound = OriginalWorldBound.TransformBy(InvLocalToWorld);
			}
			Builder
				.LocalToWorld(CurrentLocalToWorld)
				.PreviousLocalToWorld(PreviousLocalToWorld)
				.LocalBounds(NewLocalBound)
				.OutputVelocity(bOutputVelocity)
				.UseVolumetricLightmap(false);

			// Create primitive uniform buffer
			FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();
			FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
			DynamicPrimitiveUniformBuffer.UniformBuffer.BufferUsage = UniformBuffer_SingleFrame;
			DynamicPrimitiveUniformBuffer.UniformBuffer.SetContents(RHICmdList, Builder.Build());
			DynamicPrimitiveUniformBuffer.UniformBuffer.InitResource(RHICmdList);
			BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer; // automatic copy to the gpu scene buffer
		}

		//primtiveid is set to 0
		BatchElement.FirstIndex = 0;
		BatchElement.NumInstances = 1;
		BatchElement.PrimitiveIdMode = PrimitiveIdMode;
		if (bUseCulling)
		{
			BatchElement.NumPrimitives = 0;
			BatchElement.IndirectArgsBuffer = bUseCulling ? Instance->HairGroupPublicData->GetDrawIndirectBuffer().Buffer->GetRHI() : nullptr;
			BatchElement.IndirectArgsOffset = 0;
		}
		else
		{
			BatchElement.NumPrimitives = NumPrimitive;
			BatchElement.IndirectArgsBuffer = nullptr;
			BatchElement.IndirectArgsOffset = 0;
		}

		// Setup our vertex factor custom data
		BatchElement.VertexFactoryUserData = const_cast<void*>(reinterpret_cast<const void*>(Instance->HairGroupPublicData));

		BatchElement.MinVertexIndex = 0;
		BatchElement.MaxVertexIndex = MaxVertexIndex;
		BatchElement.UserData = reinterpret_cast<void*>(uint64(ComponentId));
		Mesh.ReverseCulling = bUseCardsOrMeshes ? IsLocalToWorldDeterminantNegative() : false;
		Mesh.bDisableBackfaceCulling = GeometryType == EHairGeometryType::Strands;
		Mesh.Type = GetPrimitiveType(GeometryType);
		Mesh.DepthPriorityGroup = SDPG_World;
		Mesh.bCanApplyViewModeOverrides = false;
		Mesh.BatchHitProxyId = PrimSceneInfo->DefaultDynamicHitProxyId;

		return &Mesh;
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		// When path tracing is enabled force DrawRelevance if not visible in main view ('hidden in game'), 
		// but visible in shadow 'hidden shadow') so that raytracing geometry is created/updated correctly
		const bool bPathtracing = View->Family->EngineShowFlags.PathTracing;
		const bool bIsShown = IsShown(View);
		const bool bForceDrawRelevance = bPathtracing && (!bIsShown && (IsShadowCast(View) || bAffectIndirectLightingWhileHidden));
		const bool bVisible = View->Family->EngineShowFlags.Hair;

		bool bUseCardsOrMesh = false;
		for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			const EHairGeometryType GeometryType = Instance->GeometryType;
			bUseCardsOrMesh = bUseCardsOrMesh || GeometryType == EHairGeometryType::Cards || GeometryType == EHairGeometryType::Meshes;
		}

		FPrimitiveViewRelevance Result;

		// Special pass for hair strands geometry (not part of the base pass, and shadowing is handlded in a custom fashion). When cards rendering is enabled we reusethe base pass
		Result.bDrawRelevance		= bVisible && (bIsShown || bForceDrawRelevance);
		Result.bRenderInMainPass	= bUseCardsOrMesh && ShouldRenderInMainPass();
		Result.bShadowRelevance		= IsShadowCast(View);
		Result.bDynamicRelevance	= bUseCardsOrMesh;
		Result.bRenderCustomDepth	= ShouldRenderCustomDepth();
		Result.bVelocityRelevance	= Result.bRenderInMainPass && bUseCardsOrMesh;
		Result.bUsesLightingChannels= GetLightingChannelMask() != GetDefaultLightingChannelMask();

		// Selection only
		#if WITH_EDITOR
		{
			Result.bEditorStaticSelectionRelevance = true;
		}
		#endif
		MaterialRelevance.SetPrimitiveViewRelevance(Result);

		// Override the MaterialRelevance output
		Result.bHairStrands = bIsShown || bForceDrawRelevance;
		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }

	uint32 GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }

	TArray<TRefCountPtr<FHairGroupInstance>> HairGroupInstances;

	// Cache the material proxy to avoid race condition, when groom component's proxy is recreated, 
	// while another one is currently in flight for drawing.
	struct FHairGroupMaterialProxy
	{
		const FMaterialRenderProxy* Strands = nullptr;
		TArray<const FMaterialRenderProxy*> Cards;
		TArray<const FMaterialRenderProxy*> Meshes;
	};
	TArray<FHairGroupMaterialProxy> HairGroupMaterialProxies;
	
private:
	uint32 ComponentId = 0;
	FMaterialRelevance MaterialRelevance;
	UMaterialInterface* Strands_DebugMaterial = nullptr;
};

// Return a ref counted hair group instance. Will be removed/modified when we will switch to use PrimitiveId instead of components
TRefCountPtr<FHairGroupInstance> GetRefCountedHairGroupInstance(const UGroomComponent* GroomComponent, const int32 GroupIndex)
{
	const FHairStrandsSceneProxy* GroomProxy = GroomComponent ? static_cast<FHairStrandsSceneProxy*>(GroomComponent->GetSceneProxy()) : nullptr;
	if (GroomProxy && GroomProxy->HairGroupInstances.IsValidIndex(GroupIndex))
	{
		return GroomProxy->HairGroupInstances[GroupIndex];
	}
	return nullptr;
}

// @todo: deprecate this function in 5.6 to use GetRefCountedHairGroupInstance
FHairGroupInstance* GetHairGroupInstance(UGroomComponent* GroomComponent, int32 GroupIndex)
{
	return GetRefCountedHairGroupInstance(GroomComponent, GroupIndex).GetReference();
}

/** GroomCacheBuffers implementation that hold copies of the GroomCacheAnimationData needed for playback */
class FGroomCacheBuffers : public IGroomCacheBuffers
{
public:
	FGroomCacheBuffers(UGroomCache* InGroomCache)
	: GroomCache(InGroomCache)
	{
	}

	virtual ~FGroomCacheBuffers()
	{
	}

	virtual void Reset()
	{
	}

	virtual const FGroomCacheAnimationData& GetCurrentFrameBuffer() override
	{
		return CurrentFrame;
	}

	virtual const FGroomCacheAnimationData& GetNextFrameBuffer() override
	{
		return NextFrame;
	}

	virtual int32 GetCurrentFrameIndex() const override
	{
		return CurrentFrameIndex;
	}

	virtual int32 GetNextFrameIndex() const override
	{
		return NextFrameIndex;
	}

	virtual float GetInterpolationFactor() const override
	{
		return InterpolationFactor;
	}

	virtual void UpdateBuffersAtTime(float Time, bool bIsLooping)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheBuffers::UpdateBuffersAtTime);

		// Find the frame indices and interpolation factor to interpolate between
		int32 FrameIndexA = 0;
		int32 FrameIndexB = 0;
		GroomCache->GetFrameIndicesAtTime(Time, bIsLooping, false, FrameIndexA, FrameIndexB, InterpolationFactor);

		// Update and cache the frame data as needed
		if (FrameIndexA != CurrentFrameIndex)
		{
			if (FrameIndexA == NextFrameIndex)
			{
				Swap(CurrentFrame, NextFrame);
				CurrentFrameIndex = NextFrameIndex;
			}
			else
			{
				GroomCache->GetGroomDataAtFrameIndex(FrameIndexA, CurrentFrame);
				CurrentFrameIndex = FrameIndexA;
			}
		}

		if (FrameIndexB != NextFrameIndex)
		{
			GroomCache->GetGroomDataAtFrameIndex(FrameIndexB, NextFrame);
			NextFrameIndex = FrameIndexB;
		}
	}

	FBox GetBoundingBox()
	{
		// Approximate bounding box used for visibility culling
		FBox BBox(EForceInit::ForceInitToZero);
		for (const FGroomCacheGroupData& GroupData : GetCurrentFrameBuffer().GroupsData)
		{
			BBox += GroupData.BoundingBox;
		}

		for (const FGroomCacheGroupData& GroupData : GetNextFrameBuffer().GroupsData)
		{
			BBox += GroupData.BoundingBox;
		}

		return BBox;
	}


protected:
	static FGroomCacheAnimationData EmptyFrame;

	UGroomCache* GroomCache;

	/** Used with synchronous loading */
	FGroomCacheAnimationData CurrentFrame;
	FGroomCacheAnimationData NextFrame;

	int32 CurrentFrameIndex = -1;
	int32 NextFrameIndex = -1;
	float InterpolationFactor = 0.0f;
};

FGroomCacheAnimationData FGroomCacheBuffers::EmptyFrame;

class FGroomCacheStreamedBuffers : public FGroomCacheBuffers
{
public:
	FGroomCacheStreamedBuffers(UGroomCache* InGroomCache)
		: FGroomCacheBuffers(InGroomCache)
		, CurrentFramePtr(nullptr)
		, NextFramePtr(nullptr)
	{
	}

	virtual ~FGroomCacheStreamedBuffers()
	{
		ResetInternal();
	}

	virtual void Reset() override
	{
		ResetInternal();
	}

private:
	void ResetInternal()
	{
		FScopeLock Lock(GetCriticalSection());

		// Unmap the frames that are currently mapped
		if (CurrentFrameIndex != -1)
		{
			IGroomCacheStreamingManager::Get().UnmapAnimationData(GroomCache, CurrentFrameIndex);
			CurrentFrameIndex = -1;
			CurrentFramePtr = nullptr;
		}

		if (NextFrameIndex != -1)
		{
			IGroomCacheStreamingManager::Get().UnmapAnimationData(GroomCache, NextFrameIndex);
			NextFrameIndex = -1;
			NextFramePtr = nullptr;
		}
	}

public:
	virtual const FGroomCacheAnimationData& GetCurrentFrameBuffer() override
	{
		if (CurrentFramePtr)
		{
			return *CurrentFramePtr;
		}
		return EmptyFrame;
	}

	virtual const FGroomCacheAnimationData& GetNextFrameBuffer() override
	{
		if (NextFramePtr)
		{
			return *NextFramePtr;
		}
		return EmptyFrame;
	}

	virtual void UpdateBuffersAtTime(float Time, bool bIsLooping) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheStreamedBuffers::UpdateBuffersAtTime);

		FScopeLock Lock(GetCriticalSection());

		// Find the frame indices and interpolation factor to interpolate between
		int32 FrameIndexA = 0;
		int32 FrameIndexB = 0;
		GroomCache->GetFrameIndicesAtTime(Time, bIsLooping, false, FrameIndexA, FrameIndexB, InterpolationFactor);

		// Update and cache the frame data as needed
		if (FrameIndexA != CurrentFrameIndex)
		{
			if (FrameIndexA == NextFrameIndex)
			{
				// NextFrame is mapped to increment its ref count, but this could fail if the frame was evicted
				// This can happen if the time jumps around as is the case when Sequencer generates thumbnails
				const FGroomCacheAnimationData* DataPtr = IGroomCacheStreamingManager::Get().MapAnimationData(GroomCache, NextFrameIndex);
				if (DataPtr)
				{
					IGroomCacheStreamingManager::Get().UnmapAnimationData(GroomCache, CurrentFrameIndex);
					CurrentFramePtr = NextFramePtr;
					CurrentFrameIndex = NextFrameIndex;
				} 
			}
			else
			{
				const FGroomCacheAnimationData* DataPtr = IGroomCacheStreamingManager::Get().MapAnimationData(GroomCache, FrameIndexA);
				if (DataPtr)
				{
					IGroomCacheStreamingManager::Get().UnmapAnimationData(GroomCache, CurrentFrameIndex);
					CurrentFramePtr = DataPtr;
					CurrentFrameIndex = FrameIndexA;
				}
			}
		}

		if (FrameIndexB != NextFrameIndex)
		{
			const FGroomCacheAnimationData* DataPtr = IGroomCacheStreamingManager::Get().MapAnimationData(GroomCache, FrameIndexB);
			if (DataPtr)
			{
				IGroomCacheStreamingManager::Get().UnmapAnimationData(GroomCache, NextFrameIndex);
				NextFramePtr = DataPtr;
				NextFrameIndex = FrameIndexB;
			}
		}

		if (CurrentFramePtr == nullptr || NextFramePtr == nullptr)
		{
			return;
		}
	}

private:
	/** Used with GroomCache streaming. Point to cached data in the manager */
	const FGroomCacheAnimationData* CurrentFramePtr;
	const FGroomCacheAnimationData* NextFramePtr;
};

/////////////////////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////////////////////
// UComponent
//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////

UGroomComponent::UGroomComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;
	bAutoActivate = true;
	bSelectable = true;
	RegisteredMeshComponent = nullptr;
	InitializedResources = nullptr;
	Mobility = EComponentMobility::Movable;
	bIsGroomAssetCallbackRegistered = false;
	bIsGroomBindingAssetCallbackRegistered = false;
	SourceSkeletalMesh = nullptr;
	NiagaraComponents.Empty();
	PhysicsAsset = nullptr;
	bCanEverAffectNavigation = false;
	bValidationEnable = IsGroomBindingValidationEnabled();
	bRunning = true;
	bLooping = true;
	bManualTick = false;
	ElapsedTime = 0.0f;

	// Overlap events are expensive and not needed (at least at the moment) as we don't need to collide against other component.
	SetGenerateOverlapEvents(false);
	SetCollisionProfileName(UCollisionProfile::PhysicsActor_ProfileName);

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> Strands_DebugMaterialRef(TEXT("/HairStrands/Materials/HairDebugMaterial.HairDebugMaterial"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> Strands_DefaultMaterialRef(TEXT("/HairStrands/Materials/HairDefaultMaterial.HairDefaultMaterial"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> Cards_DefaultMaterialRef(TEXT("/HairStrands/Materials/HairCardsDefaultMaterial.HairCardsDefaultMaterial"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> Meshes_DefaultMaterialRef(TEXT("/HairStrands/Materials/HairMeshesDefaultMaterial.HairMeshesDefaultMaterial"));

	Strands_DebugMaterial   = Strands_DebugMaterialRef.Object;
	Strands_DefaultMaterial = Strands_DefaultMaterialRef.Object;
	Cards_DefaultMaterial = Cards_DefaultMaterialRef.Object;
	Meshes_DefaultMaterial = Meshes_DefaultMaterialRef.Object;

	AngularSpringsSystem = nullptr;
	CosseratRodsSystem = nullptr;

#if WITH_EDITORONLY_DATA
	GroomAssetBeingLoaded = nullptr;
	BindingAssetBeingLoaded = nullptr;
#endif
}

void UGroomComponent::UpdateHairGroupsDesc()
{
	if (!GroomAsset)
	{
		GroomGroupsDesc.Empty();
		return;
	}

	const uint32 GroupCount = GroomAsset->GetNumHairGroups();
	const bool bNeedResize = GroupCount != GroomGroupsDesc.Num();
	if (bNeedResize)
	{
		GroomGroupsDesc.Init(FHairGroupDesc(), GroupCount);
	}
}

bool UGroomComponent::IsSimulationEnable(int32 GroupIndex, int32 LODIndex) const
{
	bool bIsSimulationEnable = GroomAsset ? GroomAsset->IsSimulationEnable(GroupIndex, LODIndex) : false;
	if (SimulationSettings.bOverrideSettings)
	{
		bIsSimulationEnable = bIsSimulationEnable && SimulationSettings.SolverSettings.bEnableSimulation;
	}

	return bIsSimulationEnable;
}

bool UGroomComponent::IsDeformationEnable(int32 GroupIndex) const
{
	// To enable deformation groom asset should allow it and we must have a mesh deformer locally or being register to a solver component
	return GroomAsset->IsDeformationEnable(GroupIndex) && (GroomSolver || MeshDeformer || GroomAsset->GetGuideType(GroupIndex) == EGroomGuideType::Rigged);
}

void UGroomComponent::ReleaseHairSimulation()
{
	for (int32 CompIndex = 0; CompIndex < NiagaraComponents.Num(); ++CompIndex)
	{
		ReleaseHairSimulation(CompIndex);
	}
	NiagaraComponents.Empty();
}

void UGroomComponent::ReleaseHairSimulation(const int32 GroupIndex)
{
	if (GroupIndex < NiagaraComponents.Num())
	{
		TObjectPtr<UNiagaraComponent>& NiagaraComponent = NiagaraComponents[GroupIndex];
		if (NiagaraComponent && !NiagaraComponent->IsBeingDestroyed())
		{
			if (GetWorld() && NiagaraComponent->IsRegistered())
			{
				NiagaraComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
				NiagaraComponent->UnregisterComponent();
			}
			NiagaraComponent->DestroyComponent();
			NiagaraComponent->MarkAsGarbage();
			NiagaraComponent->Rename(nullptr, GetTransientPackage(), REN_NonTransactional | REN_DoNotDirty | REN_DontCreateRedirectors);
			NiagaraComponent = nullptr;
		}
	}
}

void UGroomComponent::CreateHairSimulation(const int32 GroupIndex, const int32 LODIndex)
{
	if (GroupIndex < NiagaraComponents.Num())
	{
		TObjectPtr<UNiagaraComponent>& NiagaraComponent = NiagaraComponents[GroupIndex];
		if (GroomAsset && (GroupIndex < GroomAsset->GetHairGroupsPhysics().Num()) && (LODIndex < GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num()) && IsSimulationEnable(GroupIndex, LODIndex))
		{
			if (!NiagaraComponent)
			{
				FNameBuilder NSName;
				NSName.Append(TEXT("NiagaraComponent_"));
				GetFName().AppendString(NSName);
				NSName.Appendf(TEXT("_%d_%d"), GroupIndex, LODIndex);

				NiagaraComponent = NewObject<UNiagaraComponent>(this, NSName.ToString(), RF_Transient);
				NiagaraComponent->bUseAttachParentBound = true;
			}
			if (GetWorld() && GetWorld()->bIsWorldInitialized)
			{
				if (!NiagaraComponent->IsRegistered())
				{
					NiagaraComponent->AttachToComponent(this, FAttachmentTransformRules::KeepRelativeTransform);
					NiagaraComponent->RegisterComponentWithWorld(GetWorld());
				}

				if (!AngularSpringsSystem) AngularSpringsSystem = LoadObject<UNiagaraSystem>(nullptr, TEXT("/HairStrands/Emitters/StableSpringsSystem.StableSpringsSystem"));
				if (!CosseratRodsSystem) CosseratRodsSystem = LoadObject<UNiagaraSystem>(nullptr, TEXT("/HairStrands/Emitters/StableRodsSystem.StableRodsSystem"));

				UNiagaraSystem* NiagaraAsset =
					(GroomAsset->GetHairGroupsPhysics()[GroupIndex].SolverSettings.NiagaraSolver == EGroomNiagaraSolvers::AngularSprings) ? ToRawPtr(AngularSpringsSystem) :
					(GroomAsset->GetHairGroupsPhysics()[GroupIndex].SolverSettings.NiagaraSolver == EGroomNiagaraSolvers::CosseratRods) ? ToRawPtr(CosseratRodsSystem) :
					GroomAsset->GetHairGroupsPhysics()[GroupIndex].SolverSettings.CustomSystem.LoadSynchronous();
				NiagaraComponent->SetVisibility(SimulationSettings.SimulationSetup.bDebugSimulation || GroomAsset->GetHairGroupsPhysics()[GroupIndex].SolverSettings.bForceVisible, false);
				NiagaraComponent->SetAsset(NiagaraAsset);
				NiagaraComponent->ReinitializeSystem();
			}
		}
		else if (NiagaraComponent && !NiagaraComponent->IsBeingDestroyed())
		{
			NiagaraComponent->SetAsset(nullptr);
			NiagaraComponent->DeactivateImmediate();
		}
	}
}

void UGroomComponent::UpdateHairSimulation()
{
	const int32 NumGroups = GroomAsset ? GroomAsset->GetHairGroupsPhysics().Num() : 0;
	const int32 NumComponents = FMath::Max(NumGroups, NiagaraComponents.Num());

	NiagaraComponents.SetNumZeroed(NumComponents);
	if( GHairStrands_UseAttachedSimulationComponents == 1)
	{
		int32 CompIndex = 0;
		// Fill the niagara components with the already attached children if any
		for(const TObjectPtr<USceneComponent>& GroomChild : GetAttachChildren())
		{
			if(GroomChild->IsA<UNiagaraComponent>() && (CompIndex < NumComponents) && !NiagaraComponents[CompIndex])
			{
				NiagaraComponents[CompIndex++] = StaticCast<UNiagaraComponent*>(GroomChild);
			}
		}
	}
	const int32 LODInit = (LODForcedIndex != -1.f) ? LODForcedIndex : (LODPredictedIndex != -1.f) ? LODPredictedIndex : 0;
	for (int32 CompIndex = 0; CompIndex < NumComponents; ++CompIndex)
	{
		CreateHairSimulation(CompIndex, LODInit);
	}
	UpdateSimulatedGroups();
}

void UGroomComponent::SwitchSimulationLOD(const int32 PreviousLOD, const int32 CurrentLOD, const EHairLODSelectionType InLODSelectionType)
{
	if (GroomAsset && PreviousLOD != CurrentLOD)
	{
		bool bRequiresSimulationUpdate = false;
		for (int32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIt < GroupCount; ++GroupIt)
		{
			const bool bPrevSimEnable = IsSimulationEnable(GroupIt, PreviousLOD);
			const bool bCurrSimEnable = IsSimulationEnable(GroupIt, CurrentLOD);
			if (bPrevSimEnable != bCurrSimEnable)
			{
				// Sanity check. Simulation does not support immediate mode.
				check(InLODSelectionType != EHairLODSelectionType::Immediate);

				CreateHairSimulation(GroupIt, CurrentLOD);
				bRequiresSimulationUpdate = true;
			}
		}
		if (bRequiresSimulationUpdate)
		{
			UpdateSimulatedGroups();
		}
	}
}

bool UGroomComponent::IsCompiling() const
{
	return BindingAsset && BindingAsset->IsCompiling();
}

void UGroomComponent::PostCompilation()
{
#if WITH_EDITOR
	if (GroomAssetBeingLoaded && GroomAssetBeingLoaded->IsValid())
	{
		// Re-set the assets on the component now that they are loaded
		SetGroomAsset(GroomAssetBeingLoaded, BindingAssetBeingLoaded);
	}
#endif
	InitIfDependenciesReady();
}

void UGroomComponent::SetGroomAsset(UGroomAsset* Asset)
{
	SetGroomAsset(Asset, BindingAsset);
}

void UGroomComponent::SetGroomAsset(UGroomAsset* Asset, UGroomBindingAsset* InBinding, const bool bUpdateSimulation)
{
	ReleaseResources();
	if (Asset && Asset->IsValid())
	{
		GroomAsset = Asset;
		LODForcedIndex = FMath::Clamp(LODForcedIndex, -1, GroomAsset->GetLODCount() - 1);

#if WITH_EDITORONLY_DATA
		if (InBinding && !InBinding->IsCompiling() && !InBinding->IsValid())
		{
			// The binding could be invalid if the groom asset was previously invalid.
			// This will re-fetch the binding data from the DDC to make it valid
			InBinding->InvalidateBinding();
		}
		GroomAssetBeingLoaded = nullptr;
		BindingAssetBeingLoaded = nullptr;
#endif
	}
#if WITH_EDITORONLY_DATA
	else if (Asset)
	{
		// The asset is still being loaded. This will allow the assets to be re-set once the groom is finished loading
		GroomAssetBeingLoaded = Asset;
		BindingAssetBeingLoaded = InBinding;
	}
#endif
	else
	{
		GroomAsset = nullptr;
	}
	if (BindingAsset != InBinding
#if WITH_EDITORONLY_DATA
		// With the groom still being loaded, the binding is still invalid
		&& !BindingAssetBeingLoaded
#endif
		)
	{
		BindingAsset = InBinding;
	}

	InitIfDependenciesReady(bUpdateSimulation);
}

void UGroomComponent::InitIfDependenciesReady(const bool bUpdateSimulation)
{
#if WITH_EDITOR
	if (BindingAsset && BindingAsset->IsCompiling())
	{
		return;
	}
#endif

	UpdateHairGroupsDesc();
	if (!GroomAsset || !GroomAsset->IsValid())
	{
		return;
	}
	InitResources();

	if (bUpdateSimulation)
	{
		UpdateHairSimulation();
	}

	// Registration can be skipped when BindingAsset is compiling, we need to take care of it here
	// if we're already registered.
	if (IsRegistered())
	{
		if (GUseGroomCacheStreaming)
		{
			IGroomCacheStreamingManager::Get().RegisterComponent(this);
		}

		MeshDeformerInstance = (MeshDeformer != nullptr) ? MeshDeformer->CreateInstance(this, MeshDeformerInstanceSettings) : nullptr;
	}
}

void UGroomComponent::SetStableRasterization(bool bEnable)
{
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		HairDesc.bUseStableRasterization = bEnable;
		HairDesc.bUseStableRasterization_Override = true;
	}
	UpdateHairGroupsDescAndInvalidateRenderState();
}

void UGroomComponent::SetHairRootScale(float Scale)
{
	Scale = FMath::Clamp(Scale, 0.f, 10.f);
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		HairDesc.HairRootScale = Scale;
		HairDesc.HairRootScale_Override = true;
	}
	UpdateHairGroupsDescAndInvalidateRenderState();
}

void UGroomComponent::SetHairWidth(float HairWidth)
{
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		HairDesc.HairWidth = HairWidth;
		HairDesc.HairWidth_Override = true;
	}
	UpdateHairGroupsDescAndInvalidateRenderState();
}

void UGroomComponent::SetScatterSceneLighting(bool Enable)
{
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		HairDesc.bScatterSceneLighting = Enable;
		HairDesc.bScatterSceneLighting_Override = true;
	}
	UpdateHairGroupsDescAndInvalidateRenderState();
}

void UGroomComponent::SetUseCards(bool InbUseCards)
{
	bUseCards = InbUseCards;
	ResetSimulation();
	UpdateHairGroupsDescAndInvalidateRenderState();
}

void UGroomComponent::SetHairLengthScale(float InLengthScale)
{
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		HairDesc.HairLengthScale = InLengthScale;
	}
	UpdateHairGroupsDescAndInvalidateRenderState(false);
}

void UGroomComponent::SetHairLengthScaleEnable(bool bEnable)
{
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		HairDesc.HairLengthScale_Override = bEnable;
	}
	UpdateHairGroupsDescAndInvalidateRenderState(true);
	InitResources();
}

bool UGroomComponent::GetIsHairLengthScaleEnabled()
{
	bool IsEnabled = true;
	for (FHairGroupDesc& HairDesc : GroomGroupsDesc)
	{
		if (!HairDesc.HairLengthScale_Override)
		{
			IsEnabled = false;
		}
	}
	return IsEnabled;
}

void AddHairStreamingRequest(FHairGroupInstance* InInstance, int32 InLODIndex);

void UGroomComponent::SetForcedLOD(int32 CurrLODIndex)
{
	// Previous LOD state
	EHairLODSelectionType PrevLODSelectionType = LODSelectionType;
	float PrevLODIndex = -1;
	if (PrevLODSelectionType == EHairLODSelectionType::Forced)			{ PrevLODIndex = LODForcedIndex; }
	else if (PrevLODSelectionType == EHairLODSelectionType::Predicted)	{ PrevLODIndex = LODPredictedIndex; }

	// Current LOD state
	EHairLODSelectionType CurrLODSelectionType = EHairLODSelectionType::Forced;
	if (GroomAsset)
	{
		CurrLODIndex = FMath::Clamp(CurrLODIndex, -1, GroomAsset->GetLODCount() - 1);
	}

	// Reset to non-forced LOD, and change LOD selection type to Predicted/Immediate
	if (CurrLODIndex < 0)
	{
		// If one of the Group/LOD needs simulation, then we switch to predicted LOD to ensure LOD switch happen 
		// on the game-thread so that the simulation component can do work at transition-time
		bool bNeedSimulationNeed = false;
		if (GroomAsset)
		{
			for (int32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIt < GroupCount; ++GroupIt)
			{
				for (uint32 LODIt = 0, LODCount = GroomAsset->GetLODCount(); LODIt < LODCount; ++LODIt)
				{
					if (IsSimulationEnable(GroupIt, LODIt))
					{
						bNeedSimulationNeed = true;
						break;
					}
				}
			}
		}
		CurrLODSelectionType = bNeedSimulationNeed ? EHairLODSelectionType::Predicted : EHairLODSelectionType::Immediate;
	}

	// If not setting a valid forced LOD, just reset the state and let the auto system handle the rest	
	if (CurrLODSelectionType != EHairLODSelectionType::Forced)
	{
		LODForcedIndex = CurrLODIndex;
		LODSelectionType = CurrLODSelectionType;
		return;
	}

	// Inform simulation about LOD switch.
	SwitchSimulationLOD(PrevLODIndex, CurrLODIndex, CurrLODSelectionType);

	// Finally, update the forced LOD value and LOD selection type
	const bool bHasLODSwitch = CurrLODIndex != PrevLODIndex;
	LODForcedIndex	 = CurrLODIndex;
	LODSelectionType = CurrLODSelectionType;

	// Add streaming request for the subsequent LOD as prediction, only when going to a finer LOD
	const bool bPredictionLoad = GHairStrands_Streaming_Prediction && bHasLODSwitch && CurrLODIndex > 0 && CurrLODIndex < PrevLODIndex;

	// Do not invalidate completly the proxy, but just update LOD index on the rendering thread	
	if (GroomAsset && !GroomAsset->UsesNanite())
	{
		if (FHairStrandsSceneProxy* GroomSceneProxy = (FHairStrandsSceneProxy*)SceneProxy)
		{
			const EHairLODSelectionType LocalLODSelectionType = LODSelectionType;
			ENQUEUE_RENDER_COMMAND(FHairComponentSendLODIndex)(/*UE::RenderCommandPipe::Groom,*/
			[GroomSceneProxy, CurrLODIndex, LocalLODSelectionType, bHasLODSwitch, bPredictionLoad](FRHICommandListImmediate& RHICmdList)
			{
				for (const TRefCountPtr<FHairGroupInstance>& Instance : GroomSceneProxy->HairGroupInstances)
				{
					Instance->Debug.LODForcedIndex = CurrLODIndex;
					Instance->Debug.LODSelectionTypeForDebug = LocalLODSelectionType;

					if (bHasLODSwitch)
					{
						AddHairStreamingRequest(Instance, CurrLODIndex);
					}

					if (bPredictionLoad)
					{
						AddHairStreamingRequest(Instance, CurrLODIndex-1);
					}
				}
			});
		}
	}
}

int32 UGroomComponent::GetNumLODs() const
{
	return GroomAsset ? GroomAsset->GetLODCount() : 0;
}

int32 UGroomComponent::GetForcedLOD() const
{
	return int32(LODForcedIndex);
}

int32 UGroomComponent::GetDesiredSyncLOD() const
{
	return LODPredictedIndex;
}

int32 UGroomComponent::GetBestAvailableLOD() const
{
	// For now we assume all LODs are available. This could be made more accurate in future.
	return 0;
}

void UGroomComponent::SetForceStreamedLOD(int32 LODIndex)
{
	// Force streaming is not supported yet
}

void UGroomComponent::SetForceRenderedLOD(int32 InLODIndex)
{
	SetForcedLOD(InLODIndex);
}

int32 UGroomComponent::GetNumSyncLODs() const
{
	return GetNumLODs();
}

int32 UGroomComponent::GetForceStreamedLOD() const
{
	// Force streaming is not supported yet
	return INDEX_NONE;
}
	
int32 UGroomComponent::GetForceRenderedLOD() const
{
	return GetForcedLOD();
}

void UGroomComponent::SetBinding(UGroomBindingAsset* InBinding)
{
	SetBindingAsset(InBinding);
}

void UGroomComponent::SetBindingAsset(UGroomBindingAsset* InBinding)
{
	SetGroomAsset(GroomAsset, InBinding);
}

void UGroomComponent::SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset)
{
	if (InPhysicsAsset && PhysicsAsset != InPhysicsAsset)
	{
		PhysicsAsset = InPhysicsAsset;
			
		ReleaseResources();
		UpdateHairGroupsDesc();
		InitResources();
		UpdateHairSimulation();
	}
}

void UGroomComponent::AddCollisionComponent(USkeletalMeshComponent* SkeletalMeshComponent)
{
	if(SkeletalMeshComponent && (CollisionComponents.Find(SkeletalMeshComponent) == INDEX_NONE))
	{
		CollisionComponents.Add(SkeletalMeshComponent);
	}
}

void UGroomComponent::ResetCollisionComponents()
{
	CollisionComponents.Reset();
}

void UGroomComponent::SetEnableSimulation(bool bInEnableSimulation)
{
	if (SimulationSettings.bOverrideSettings)
	{
		if (SimulationSettings.SolverSettings.bEnableSimulation != bInEnableSimulation)
		{
			SimulationSettings.SolverSettings.bEnableSimulation = bInEnableSimulation;

			ReleaseResources();
			UpdateHairGroupsDesc();
			InitResources();
			UpdateHairSimulation();
		}
	}
	else
	{
		UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - To be able to enable/disable the simulation from component you must enable the overide settings boolean in the simulatin settings.", *GetPathName());
	}
}

void UGroomComponent::ResetSimulation()
{
	bResetSimulation = true;
	bInitSimulation = true;
}

void UGroomComponent::UpdateHairGroupsDescAndInvalidateRenderState(bool bInvalid)
{
	UpdateHairGroupsDesc();

	uint32 GroupIndex = 0;
	for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
	{
		Instance->Strands.Modifier = GetGroomGroupsDesc(GroomAsset, this, GroupIndex);
		++GroupIndex;
	}
	if (bInvalid)
	{
		MarkRenderStateDirty();
	}
}

FPrimitiveSceneProxy* UGroomComponent::CreateSceneProxy()
{
	if (!GroomAsset || GroomAsset->GetNumHairGroups() == 0 || HairGroupInstances.Num() == 0)
	{
		return nullptr;
	}

	// This will be recreated when the binding asset compilation finishes
	if (BindingAsset && BindingAsset->IsCompiling())
	{
		return nullptr;
	}

	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogHairStrands, Verbose, "Skipping CreateSceneProxy for UGroomComponent %ls (UGroomComponent PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	if (GroomAsset->GetHairGroupsPlatformData().Num() != HairGroupInstances.Num())
	{
		return nullptr;
	}

	bool bIsValid = false;
	for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
	{
		bIsValid |= Instance.IsValid() && Instance->IsValid();
	}
	if (!bIsValid)
	{
		return nullptr;
	}

	if (FGroomAssetRenderData* LocalRenderData = GroomAsset->GetRenderData())
	{
		if (LocalRenderData->GetMaterialAudit())
		{
			return ::new Nanite::FGroomSceneProxy(*LocalRenderData->GetMaterialAudit(), this, GroomAsset, LocalRenderData);
		}
		else
		{
			return nullptr;
		}
	}
	else
	{
		return new FHairStrandsSceneProxy(this);
	}
}


FBoxSphereBounds UGroomComponent::CalcBounds(const FTransform& InLocalToWorld) const
{
	FBoxSphereBounds Out;

	if (GroomAsset && GroomAsset->GetNumHairGroups() > 0)
	{
		FBox LocalHairBound(EForceInit::ForceInitToZero);
		if (!GroomCacheBuffers.IsValid())
		{
			for (uint32 GroupIndex = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIndex<GroupCount; ++GroupIndex)
			{
				const FHairGroupPlatformData& GroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIndex];
				if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands) && GroupData.Strands.HasValidData())
				{
					// The hair width is not scaled by the transform scale. Since we increase the local bound by the hair width, 
					// and that amount is then scaled by the local to world matrix which conatins a scale factor, we apply a 
					// inverse scale to the radius computation
					const float InvScale = 1.f / InLocalToWorld.GetMaximumAxisScale();

					// Take into account the strands size and add it as a bound 'border'
					const FHairGroupDesc Desc = GetGroomGroupsDesc(GroomAsset, this, GroupIndex);
					const FVector HairBorder(0.5f * Desc.HairWidth * FMath::Max3(1.0f, Desc.HairRootScale, Desc.HairTipScale) * InvScale);

					LocalHairBound += GroupData.Strands.GetBounds();

					// If the strands data only contain a single curve, it likely means that the strands data have been trimmed 
					// and their bounds incorrectly represent the groom component. In such a case, we optionnally evaluate/include 
					// the cards/meshes bounds
					const bool bNeedCardsOrMeshesBound = GroupData.Strands.BulkData.Header.CurveCount <= 1;
					if (bNeedCardsOrMeshesBound)
					{
						if (GroupData.Cards.HasValidData())  { LocalHairBound += GroupData.Cards.GetBounds(); }
						if (GroupData.Meshes.HasValidData()) { LocalHairBound += GroupData.Meshes.GetBounds(); }
					}

					LocalHairBound.Min -= HairBorder;
					LocalHairBound.Max += HairBorder;
				}
				else if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards) && GroupData.Cards.HasValidData())
				{
					LocalHairBound += GroupData.Cards.GetBounds();
				}
				else if (IsHairStrandsEnabled(EHairStrandsShaderType::Meshes) && GroupData.Meshes.HasValidData())
				{
					LocalHairBound += GroupData.Meshes.GetBounds();
				}
				else if (GroupData.Guides.HasValidData())
				{
					LocalHairBound += GroupData.Guides.GetBounds();
				}
			}
		}
		else
		{
			FGroomCacheBuffers* Buffers = static_cast<FGroomCacheBuffers*>(GroomCacheBuffers.Get());
			LocalHairBound = Buffers->GetBoundingBox();
		}

		// If the attachment is done onto a skel. mesh, by default (i.e., GHairStrands_BoundsMode == 0):
		// * If the attachment is simple, use the skel. mesh expanded by the groom bounds.
		// * If the attachment is 'relative' to the skel mesh (using a bone anchor, which is provided with AttachmentName), 
		//   we use the simple groom bound.
		// Otherwise:
		// * GHairStrands_BoundsMode=1 : use the skel mesh bounds
		// * GHairStrands_BoundsMode=2 : use the skel mesh bounds + groom bounds. This is more conservative compares to GHairStrands_BoundsMode=0.
		if (RegisteredMeshComponent && GHairStrands_BoundsMode == 1)
		{
			const FVector Scale = GetRelativeScale3D();
			const FVector InvScale(1.f / Scale.X, 1.f / Scale.Y, 1.f / Scale.Z);

			FBox EffectiveBound = RegisteredMeshComponent->CalcBounds(InLocalToWorld).GetBox();
			EffectiveBound.Min *= InvScale;
			EffectiveBound.Max *= InvScale;

			Out = FBoxSphereBounds(EffectiveBound);
		}
		else if (RegisteredMeshComponent && GHairStrands_BoundsMode == 2)
		{
			const FVector Scale = GetRelativeScale3D();
			const FVector InvScale(1.f / Scale.X, 1.f / Scale.Y, 1.f / Scale.Z);

			const FVector3d GroomExtends = LocalHairBound.GetExtent();
			const float BoundExtraRadius = static_cast<float>(FMath::Max(0.0, FMath::Max3(GroomExtends.X, GroomExtends.Y, GroomExtends.Z)));
			FBox EffectiveBound = RegisteredMeshComponent->CalcBounds(FTransform::Identity).GetBox();
			EffectiveBound.Min *= InvScale;
			EffectiveBound.Max *= InvScale;			
			EffectiveBound.Min -= FVector3d(BoundExtraRadius);
			EffectiveBound.Max += FVector3d(BoundExtraRadius);
			Out = FBoxSphereBounds(EffectiveBound.TransformBy(InLocalToWorld));
		}
		else if (RegisteredMeshComponent && AttachmentName.IsEmpty())
		{
			// Compute an extra 'radius' which will be added to the skel. mesh bound. 
			// It is based on the different of the groom bound and the skel. bound at rest position
			float BoundExtraRadius = 0.f;
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RegisteredMeshComponent))
			{
				FBoxSphereBounds SkelRestBounds;
				SkeletalMeshComponent->GetPreSkinnedLocalBounds(SkelRestBounds);
				FBoxSphereBounds ExtentedBound = Union(SkelRestBounds, FBoxSphereBounds(LocalHairBound));
				const FVector MinDiff = SkelRestBounds.GetBox().Min - ExtentedBound.GetBox().Min;
				const FVector MaxDiff = ExtentedBound.GetBox().Max  - SkelRestBounds.GetBox().Max;
				BoundExtraRadius = static_cast<float>(FMath::Max3(0.0, FMath::Max3(MinDiff.X, MinDiff.Y, MinDiff.Z), FMath::Max3(MaxDiff.X, MaxDiff.Y, MaxDiff.Z)));
			}
			
			const FVector Scale = GetRelativeScale3D();
			const FVector InvScale(1.f / Scale.X, 1.f / Scale.Y, 1.f / Scale.Z);

			const FBox LocalSkeletalBound = RegisteredMeshComponent->CalcBounds(FTransform::Identity).GetBox();
			FBox LocalBound(EForceInit::ForceInitToZero);
			LocalBound += InvScale * LocalSkeletalBound.Min - BoundExtraRadius;
			LocalBound += InvScale * LocalSkeletalBound.Max + BoundExtraRadius;
			Out = FBoxSphereBounds(LocalBound.TransformBy(InLocalToWorld));
		}
		else
		{
			Out = FBoxSphereBounds(LocalHairBound.TransformBy(InLocalToWorld));
		}
	}
	else
	{
		FBoxSphereBounds LocalBounds(EForceInit::ForceInitToZero);
		Out = FBoxSphereBounds(LocalBounds.TransformBy(InLocalToWorld));
	}

	// Apply bounds scale
	Out.BoxExtent *= BoundsScale;
	Out.SphereRadius *= BoundsScale;

	return Out;
}

void UGroomComponent::CheckHairStrandsUsage(int32 ElementIndex) const
{
	UMaterialInterface* OverrideMaterial = Super::GetMaterial(ElementIndex);
	if (!OverrideMaterial && GroomAsset && ElementIndex < GroomAsset->GetHairGroupsMaterials().Num())
	{
		if (UMaterialInterface* Material = GroomAsset->GetHairGroupsMaterials()[ElementIndex].Material)
		{
			OverrideMaterial = Material;
		}
	}
	const bool bValidUsage = OverrideMaterial && OverrideMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_HairStrands);
	if(!OverrideMaterial || !bValidUsage)
	{		
		UE_CLOGF(OverrideMaterial && !bValidUsage, LogHairStrands, Warning, "Material with missing usage flag was applied to hair strands %ls", *OverrideMaterial->GetPathName());
	}
}

/* Return the material slot index corresponding to the material name */
int32 UGroomComponent::GetMaterialIndex(FName MaterialSlotName) const
{
	if (GroomAsset)
	{
		return GroomAsset->GetMaterialIndex(MaterialSlotName);
	}

	return INDEX_NONE;
}

bool UGroomComponent::IsMaterialSlotNameValid(FName MaterialSlotName) const
{
	return GetMaterialIndex(MaterialSlotName) != INDEX_NONE;
}

TArray<FName> UGroomComponent::GetMaterialSlotNames() const
{
	TArray<FName> MaterialNames;
	if (GroomAsset)
	{
		MaterialNames = GroomAsset->GetMaterialSlotNames();
	}

	return MaterialNames;
}

int32 UGroomComponent::GetNumMaterials() const
{
	if (GroomAsset)
	{
		return FMath::Max(GroomAsset->GetHairGroupsMaterials().Num(), 1);
	}
	return 1;
}

UMaterialInterface* UGroomComponent::GetMaterial(int32 ElementIndex, EHairGeometryType GeometryType, bool bUseDefaultIfIncompatible) const
{
	UMaterialInterface* OverrideMaterial = Super::GetMaterial(ElementIndex);

	bool bUseHairDefaultMaterial = false;

	if (!OverrideMaterial && GroomAsset && ElementIndex < GroomAsset->GetHairGroupsMaterials().Num())
	{
		if (UMaterialInterface* Material = GroomAsset->GetHairGroupsMaterials()[ElementIndex].Material)
		{
			OverrideMaterial = Material;
		}
		else if (bUseDefaultIfIncompatible)
		{
			bUseHairDefaultMaterial = true;
		}
	}

	if (bUseDefaultIfIncompatible)
	{
		const EShaderPlatform ShaderPlatform = GetScene() ? GetScene()->GetShaderPlatform() : EShaderPlatform::SP_NumPlatforms;
		if (ShaderPlatform != EShaderPlatform::SP_NumPlatforms && IsHairMaterialCompatible(OverrideMaterial, ShaderPlatform, GeometryType) != EHairMaterialCompatibility::Valid)
		{
			bUseHairDefaultMaterial = true;
		}
	}

	if (bUseHairDefaultMaterial)
	{
		if (GeometryType == EHairGeometryType::Strands)
		{
			OverrideMaterial = Strands_DefaultMaterial;
		}
		else if (GeometryType == EHairGeometryType::Cards)
		{
			OverrideMaterial = Cards_DefaultMaterial;
		}
		else if (GeometryType == EHairGeometryType::Meshes)
		{
			OverrideMaterial = Meshes_DefaultMaterial;
		}
	}

	return OverrideMaterial;
}

EHairGeometryType UGroomComponent::GetMaterialGeometryType(int32 ElementIndex) const
{
	if (!GroomAsset)
	{
		// If we don't know, enforce strands, as it has the most requirement.
		return EHairGeometryType::Strands;
	}

	const EShaderPlatform Platform = GetScene() ? GetScene()->GetShaderPlatform() : EShaderPlatform::SP_NumPlatforms;
	for (uint32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsRendering().Num(); GroupIt < GroupCount; ++GroupIt)
	{
		// Material - Strands
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform))
		{
			const FHairGroupPlatformData& InGroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIt];
			const int32 SlotIndex = GroomAsset->GetMaterialIndex(GroomAsset->GetHairGroupsRendering()[GroupIt].MaterialSlotName);
			if (SlotIndex == ElementIndex)
			{
				return EHairGeometryType::Strands;
			}
		}

		// Material - Cards
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Platform))
		{
			const FHairGroupPlatformData& InGroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIt];
			uint32 CardsLODIndex = 0;
			for (const FHairGroupPlatformData::FCards::FLOD& LOD : InGroupData.Cards.LODs)
			{
				if (LOD.IsValid())
				{
					// Material
					int32 SlotIndex = INDEX_NONE;
					for (const FHairGroupsCardsSourceDescription& Desc : GroomAsset->GetHairGroupsCards())
					{
						if (Desc.GroupIndex == GroupIt && Desc.LODIndex == CardsLODIndex)
						{
							SlotIndex = GroomAsset->GetMaterialIndex(Desc.MaterialSlotName);
							break;
						}
					}
					if (SlotIndex == ElementIndex)
					{
						return EHairGeometryType::Cards;
					}
				}
				++CardsLODIndex;
			}
		}

		// Material - Meshes
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, Platform))
		{
			const FHairGroupPlatformData& InGroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIt];
			uint32 MeshesLODIndex = 0;
			for (const FHairGroupPlatformData::FMeshes::FLOD& LOD : InGroupData.Meshes.LODs)
			{
				if (LOD.IsValid())
				{
					// Material
					int32 SlotIndex = INDEX_NONE;
					for (const FHairGroupsMeshesSourceDescription& Desc : GroomAsset->GetHairGroupsMeshes())
					{
						if (Desc.GroupIndex == GroupIt && Desc.LODIndex == MeshesLODIndex)
						{
							SlotIndex = GroomAsset->GetMaterialIndex(Desc.MaterialSlotName);
							break;
						}
					}
					if (SlotIndex == ElementIndex)
					{
						return EHairGeometryType::Meshes;
					}
				}
				++MeshesLODIndex;
			}
		}
	}
	// If we don't know, enforce strands, as it has the most requirement.
	return EHairGeometryType::Strands;
}

UMaterialInterface* UGroomComponent::GetMaterial(int32 ElementIndex) const
{
	const EHairGeometryType GeometryType = GetMaterialGeometryType(ElementIndex);
	return GetMaterial(ElementIndex, GeometryType, true);
}

FHairStrandsRestResource* UGroomComponent::GetGuideStrandsRestResource(uint32 GroupIndex)
{
	if (!GroomAsset || !GroomAsset->GetHairGroupsResources().IsValidIndex(GroupIndex))
	{
		return nullptr;
	}

	if (!GroomAsset->GetHairGroupsResources()[GroupIndex].Guides.IsValid())
	{
		return nullptr;
	}
	return GroomAsset->GetHairGroupsResources()[GroupIndex].Guides.RestResource;
}

FHairStrandsDeformedResource* UGroomComponent::GetGuideStrandsDeformedResource(uint32 GroupIndex)
{
	if (GroupIndex >= uint32(HairGroupInstances.Num()))
	{
		return nullptr;
	}

	if (!HairGroupInstances[GroupIndex]->Guides.IsValid())
	{
		return nullptr;
	}
	return HairGroupInstances[GroupIndex]->Guides.DeformedResource;
}

FHairStrandsRestRootResource* UGroomComponent::GetGuideStrandsRestRootResource(uint32 GroupIndex)
{
	if (GroupIndex >= uint32(HairGroupInstances.Num()))
	{
		return nullptr;
	}

	if (!HairGroupInstances[GroupIndex]->Guides.IsValid())
	{
		return nullptr;
	}
	return HairGroupInstances[GroupIndex]->Guides.RestRootResource;
}

FHairStrandsDeformedRootResource* UGroomComponent::GetGuideStrandsDeformedRootResource(uint32 GroupIndex)
{
	if (GroupIndex >= uint32(HairGroupInstances.Num()))
	{
		return nullptr;
	}

	if (!HairGroupInstances[GroupIndex]->Guides.IsValid())
	{
		return nullptr;
	}
	return HairGroupInstances[GroupIndex]->Guides.DeformedRootResource;
}

void UGroomComponent::UpdateSimulatedGroups()
{
	if (HairGroupInstances.Num()>0)
	{
		const uint32 Id = GetPrimitiveSceneId().PrimIDValue;

		const bool bIsStrandsEnabled = IsHairStrandsEnabled(EHairStrandsShaderType::Strands);

		// For now use the force LOD to drive enabling/disabling simulation & RBF
		const int32 LODIndex = GetForcedLOD();

		TArray<TRefCountPtr<FHairGroupInstance>> LocalInstances = HairGroupInstances;
		UGroomAsset* LocalGroomAsset = GroomAsset;
		UGroomBindingAsset* LocalBindingAsset = BindingAsset;
		ENQUEUE_RENDER_COMMAND(FHairStrandsTick_UEnableSimulatedGroups)(/*UE::RenderCommandPipe::Groom,*/
			[LocalInstances, LocalGroomAsset, LocalBindingAsset, Id, bIsStrandsEnabled, LODIndex](FRHICommandListImmediate& RHICmdList)
		{
			int32 GroupIt = 0;
			for (const TRefCountPtr<FHairGroupInstance>& Instance : LocalInstances)
			{
				Instance->Strands.HairInterpolationType = EHairInterpolationType::NoneSkinning;
				if (bIsStrandsEnabled && LocalGroomAsset)
				{
					Instance->Strands.HairInterpolationType = ToHairInterpolationType(LocalGroomAsset->GetHairInterpolationType());
				}
				if (Instance->Guides.IsValid())
				{
					check(Instance->HairGroupPublicData);
					Instance->Guides.bIsSimulationEnable	 = Instance->HairGroupPublicData->IsSimulationEnable(LODIndex);
					Instance->Guides.bHasGlobalInterpolation = Instance->HairGroupPublicData->IsGlobalInterpolationEnable(LODIndex);
					Instance->Guides.bIsDeformationEnable	 = Instance->HairGroupPublicData->bIsDeformationEnable;
					Instance->Guides.bIsSimulationCacheEnable= Instance->Debug.GroomCacheType == EGroomCacheType::Guides;
				}
				++GroupIt;
			}
		});
	}
}

static UGeometryCacheComponent* ValidateBindingAsset_GeometryCache(
	UGroomAsset* GroomAsset, 
	UGroomBindingAsset* BindingAsset, 
	UGeometryCacheComponent* GeometryCacheComponent, 
	bool bIsBindingReloading, 
	const USceneComponent* Component)
{
	if (!GroomAsset || !BindingAsset || !GeometryCacheComponent)
	{
		return nullptr;
	}

	bool bHasValidSectionCount = GeometryCacheComponent && GeometryCacheComponent->GetNumMaterials() < int32(GetHairStrandsMaxSectionCount());

	// Report warning if the section count is larger than the supported count
	if (GeometryCacheComponent && !bHasValidSectionCount)
	{
		FString Name = "";
		if (Component->GetOwner())
		{
			Name += Component->GetOwner()->GetName() + "/";
		}
		Name += Component->GetName() + "/" + GroomAsset->GetName();

		UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - Groom is bound to a GeometryCache which has too many sections (%d), which is higher than the maximum supported for hair binding (%d). The groom binding will be disbled on this component.", *Name, GeometryCacheComponent->GetNumMaterials(), GetHairStrandsMaxSectionCount());
	}

	const bool bIsBindingCompatible =
		UGroomBindingAsset::IsCompatible(GeometryCacheComponent ? GeometryCacheComponent->GeometryCache : nullptr, BindingAsset, true /*bValidationEnable*/) &&
		UGroomBindingAsset::IsCompatible(GroomAsset, BindingAsset, true /*bValidationEnable*/) &&
		UGroomBindingAsset::IsBindingAssetValid(BindingAsset, bIsBindingReloading, true /*bValidationEnable*/);

	return bIsBindingCompatible ? GeometryCacheComponent : nullptr;
}

// Return a non-null skeletal mesh Component if the binding asset is compatible with the current component
static USkeletalMeshComponent* ValidateBindingAsset_SkeletalMesh(
	UGroomAsset* GroomAsset,
	UGroomBindingAsset* BindingAsset,
	USkeletalMeshComponent* SkeletalMeshComponent,
	bool bIsBindingReloading,
	const USceneComponent* Component)
{
	if (!GroomAsset || !BindingAsset || !SkeletalMeshComponent)
	{
		return nullptr;
	}

	// Optional advanced check
	bool bHasValidSectionCount = SkeletalMeshComponent && SkeletalMeshComponent->GetNumMaterials() < int32(GetHairStrandsMaxSectionCount());
	if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset() && SkeletalMeshComponent->GetSkeletalMeshAsset()->GetResourceForRendering())
	{
		const FSkeletalMeshRenderData* RenderData = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetResourceForRendering();
		const int32 MaxSectionCount = GetHairStrandsMaxSectionCount();
		// Check that all LOD are below the number sections
		const uint32 MeshLODCount = RenderData->LODRenderData.Num();
		for (uint32 LODIt = 0; LODIt < MeshLODCount; ++LODIt)
		{
			if (RenderData->LODRenderData[LODIt].RenderSections.Num() >= MaxSectionCount)
			{
				bHasValidSectionCount = false;
			}
		}
	}

	// Report warning if the skeletal section count is larger than the supported count
	if (SkeletalMeshComponent && !bHasValidSectionCount)
	{
		FString Name = "";
		if (Component->GetOwner())
		{
			Name += Component->GetOwner()->GetName() + "/";
		}
		Name += Component->GetName() + "/" + GroomAsset->GetName();

		UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - Groom is bound to a skeletal mesh which has too many sections (%d), which is higher than the maximum supported for hair binding (%d). The groom binding will be disbled on this component.", *Name, SkeletalMeshComponent->GetNumMaterials(), GetHairStrandsMaxSectionCount());
	}

	const bool bIsBindingCompatible =
		UGroomBindingAsset::IsCompatible(SkeletalMeshComponent ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr, BindingAsset, true /*bValidationEnable*/) &&
		UGroomBindingAsset::IsCompatible(GroomAsset, BindingAsset, true /*bValidationEnable*/) &&
		UGroomBindingAsset::IsBindingAssetValid(BindingAsset, bIsBindingReloading, true /*bValidationEnable*/);

	if (!bIsBindingCompatible)
	{
		return nullptr;
	}

	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr;

	// Validate against cards data
	for (int32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIt < GroupCount; ++GroupIt)
	{
		FHairGroupPlatformData& GroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIt];
		uint32 CardsLODIndex = 0;
		for (FHairGroupPlatformData::FCards::FLOD& LOD : GroupData.Cards.LODs)
		{
			if (LOD.IsValid())
			{
				const bool bIsCardsBindingCompatible =
					BindingAsset != nullptr &&
					SkeletalMesh != nullptr && 
					GroupIt < BindingAsset->GetHairGroupResources().Num() &&
					CardsLODIndex < static_cast<uint32>(BindingAsset->GetHairGroupResources()[GroupIt].CardsRootResources.Num()) &&
					BindingAsset->GetHairGroupResources()[GroupIt].CardsRootResources[CardsLODIndex] != nullptr &&
					SkeletalMesh->GetLODNum() == BindingAsset->GetHairGroupResources()[GroupIt].CardsRootResources[CardsLODIndex]->GetLODCount();

				if (!bIsCardsBindingCompatible)
				{
					return nullptr;
				}
			}
			CardsLODIndex++;
		}
	}

	// Validate that if we are bound to a skel. mesh, and we have binding, and the binding type is set to skinning, that the skin cache is enabled for these
	if (BindingAsset)
	{
		// Extract if skin cache or mesh. deformer is enabled on at least on LOD.
		// Since there is 1:1 mapping between groom LOD and mesh LOD, only use this has a hint.
		bool bSupportSkinCache = SkeletalMeshComponent->HasMeshDeformer();
		if (!bSupportSkinCache)
		{
			for (uint32 SkelLODIt = 0, SkelLODCount = SkeletalMeshComponent->GetNumLODs(); SkelLODIt < SkelLODCount; ++SkelLODIt)
			{
				bSupportSkinCache = bSupportSkinCache || SkeletalMeshComponent->IsSkinCacheAllowed(SkelLODIt);
			}
		}

		for (int32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIt < GroupCount; ++GroupIt)
		{
			for (uint32 LODIt = 0, LODCount = GroomAsset->GetLODCount(); LODIt < LODCount; ++LODIt)
			{
				const EGroomBindingType BindingType = GroomAsset->GetBindingType(GroupIt, LODIt);
				const bool bIsVisible = GroomAsset->IsVisible(GroupIt, LODIt);

				if (BindingType == EGroomBindingType::Skinning && (!bSupportSkinCache && !IsHairManualSkinCacheEnabled()) && bIsVisible)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] Groom asset (Group:%d/%d) is set to use Skinning at LOD %d/%d while the parent skel. mesh does not support skin. cache at this LOD - Groom:%ls - Skel.Mesh:%ls",
						GroupIt,
						GroupCount,
						LODIt,
						LODCount,
						*GroomAsset->GetPathName(),
						*SkeletalMesh->GetPathName());
				}
			}
		}
	}

	return SkeletalMeshComponent;
}

static UMeshComponent* ValidateBindingAsset(
	UGroomAsset* GroomAsset,
	UGroomBindingAsset* BindingAsset,
	UMeshComponent* MeshComponent,
	bool bIsBindingReloading,
	bool bValidationEnable,
	const USceneComponent* Component)
{
	if (!GroomAsset || !BindingAsset || !MeshComponent)
	{
		return nullptr;
	}

	if (BindingAsset->GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		USkeletalMeshComponent* SkelMeshComponent = Cast<USkeletalMeshComponent>(MeshComponent);
		if (bValidationEnable)
		{
			SkelMeshComponent = ValidateBindingAsset_SkeletalMesh(GroomAsset, BindingAsset, SkelMeshComponent, bIsBindingReloading, Component);
		}
		return SkelMeshComponent;
	}
	else if (BindingAsset->GetGroomBindingType() == EGroomBindingMeshType::GeometryCache)
	{
		UGeometryCacheComponent* GeometryCacheComponent = Cast<UGeometryCacheComponent>(MeshComponent);
		if (bValidationEnable)
		{
			GeometryCacheComponent = ValidateBindingAsset_GeometryCache(GroomAsset, BindingAsset, GeometryCacheComponent, bIsBindingReloading, Component);
		}
		return GeometryCacheComponent;
	}
	else
	{
		return nullptr;
	}
}

static EGroomGeometryType GetEffectiveGeometryType(EGroomGeometryType Type, bool bUseCards, EShaderPlatform InPlatform)
{
	return Type == EGroomGeometryType::Strands && (!IsHairStrandsEnabled(EHairStrandsShaderType::Strands, InPlatform) || bUseCards) ? EGroomGeometryType::Cards : Type;
}

static EGroomCacheType GetEffectiveGroomCacheType(const UGroomCache* InCache, const UGroomAsset* InGroom)
{
	if (InCache && InGroom)
	{
		if (InCache->GetType() == EGroomCacheType::Guides)
		{
			// Ensure simulation cache is enable only if the groom asset as interpolation data enabled (i.e., GetEnableSimulationCache()=true)
			return InGroom->GetEnableSimulationCache() ? EGroomCacheType::Guides : EGroomCacheType::None;
		}
		else
		{
			return InCache->GetType();
		}
	}
	return EGroomCacheType::None;
}

void UGroomComponent::InitResources(bool bIsBindingReloading)
{
	LLM_SCOPE_BYTAG(Groom);

	ReleaseResources();
	bInitSimulation = true;
	bResetSimulation = true;

	UpdateHairGroupsDesc();

	if (!GroomAsset || GroomAsset->GetNumHairGroups() == 0 || !GroomAsset->IsValid())
	{
		return;
	}

	InitializedResources = GroomAsset;

	const EShaderPlatform ShaderPlatform = GetWorld() && GetWorld()->Scene ? GetWorld()->Scene->GetShaderPlatform() : EShaderPlatform::SP_NumPlatforms;

	// 1. Check if we need any kind of binding data, simulation data, or RBF data
	//
	//					  Requires    	| Requires     | Requires
	//         Features | Skeletal mesh	| Binding data | Guides
	// -----------------|---------------|--------------|-------------
	//    Rigid binding | No			| No		   | No
	// Skinning binding | Yes			| Yes		   | No
	//       Simulation | No			| No		   | Yes
	//              RBF | Yes		    | Yes		   | Yes
	//
	bool bHasNeedSkinningBinding = false;			   
	bool bHasNeedSkeletalMesh = false;
	TArray<bool> bHasNeedGlobalDeformation;
	TArray<bool> bHasNeedSimulation;
	TArray<bool> bHasNeedDeformation;
	bHasNeedGlobalDeformation.Init(false, GroomAsset->GetHairGroupsPlatformData().Num());
	bHasNeedSimulation.Init(false, GroomAsset->GetHairGroupsPlatformData().Num());
	bHasNeedDeformation.Init(false, GroomAsset->GetHairGroupsPlatformData().Num());
	
	bool bHasAnyNeedSimulation = false;
	bool bHasAnyNeedDeformation = false;
	bool bHasAnyNeedGlobalDeformation = false;
	for (int32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIt < GroupCount; ++GroupIt)
	{
		for (uint32 LODIt = 0, LODCount = GroomAsset->GetLODCount(); LODIt < LODCount; ++LODIt)
		{
			const EGroomGeometryType GeometryType = GetEffectiveGeometryType(GroomAsset->GetGeometryType(GroupIt, LODIt), bUseCards, ShaderPlatform);
			const EGroomBindingType BindingType = GroomAsset->GetBindingType(GroupIt, LODIt);

			// Note on Global Deformation:
			// * Global deformation require to have skinning binding
			// * Force global interpolation to be enable for meshes with skinning binding as we use RBF defomation for 'sticking' meshes onto skel. mesh surface
			bHasNeedSkeletalMesh				= bHasNeedSkeletalMesh || BindingType == EGroomBindingType::Rigid;
			bHasNeedSkinningBinding				= bHasNeedSkinningBinding || BindingType == EGroomBindingType::Skinning;
			bHasNeedSimulation[GroupIt]			= bHasNeedSimulation[GroupIt] || IsSimulationEnable(GroupIt, LODIt);
			bHasNeedGlobalDeformation[GroupIt]	= bHasNeedGlobalDeformation[GroupIt] || (BindingType == EGroomBindingType::Skinning && GroomAsset->IsGlobalInterpolationEnable(GroupIt, LODIt));

			bHasAnyNeedSimulation				= bHasAnyNeedSimulation			|| bHasNeedSimulation[GroupIt];
			bHasAnyNeedGlobalDeformation		= bHasAnyNeedGlobalDeformation	|| bHasNeedGlobalDeformation[GroupIt];
		}
		bHasNeedDeformation[GroupIt] = IsDeformationEnable(GroupIt);
		bHasAnyNeedDeformation = bHasAnyNeedDeformation || bHasNeedDeformation[GroupIt];
	}
	const bool bHasNeedBindingData = bHasNeedSkinningBinding || bHasAnyNeedGlobalDeformation;

	// LOD Selection type
	// By default set to immediate, meaning the LOD selection will be done on the rendering thread based on the screen coverage of the groom
	// If simulation is enabled, then the LOD selection will be done on the game thread based on feedback from the rendering thread.
	// If LODForcedIndex is >= 0 it means it has been set by a previous SetForcedLOD call. LODSelectionType is set to Forced in such a case, to applied this 'cached' value
	if (LODForcedIndex >= 0)
	{
		LODSelectionType = EHairLODSelectionType::Forced;
	}
	else
	{
		LODSelectionType = (bHasAnyNeedSimulation || bHasAnyNeedDeformation) ? EHairLODSelectionType::Predicted : EHairLODSelectionType::Immediate;
	}

	// 2. Insure that the binding asset is compatible, otherwise no binding
	UMeshComponent* ParentMeshComponent = GetAttachParent() ? Cast<UMeshComponent>(GetAttachParent()) : nullptr;
	UMeshComponent* ValidatedMeshComponent = nullptr;
	if (bHasNeedBindingData && ParentMeshComponent)
	{
		ValidatedMeshComponent = ValidateBindingAsset(GroomAsset, BindingAsset, ParentMeshComponent, bIsBindingReloading, bValidationEnable, this);
		if (ValidatedMeshComponent)
		{
			if (BindingAsset && 
				((BindingAsset->GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh && Cast<USkeletalMeshComponent>(ValidatedMeshComponent)->GetSkeletalMeshAsset() == nullptr) ||
				(BindingAsset->GetGroomBindingType() == EGroomBindingMeshType::GeometryCache && Cast<UGeometryCacheComponent>(ValidatedMeshComponent)->GeometryCache == nullptr)))
			{
				ValidatedMeshComponent = nullptr;
			}
		}
	}
	else if (bHasNeedSkeletalMesh && ParentMeshComponent)
	{
		if (USkeletalMeshComponent* ParentSkelMeshComponent = Cast<USkeletalMeshComponent>(ParentMeshComponent))
		{
			ValidatedMeshComponent = ParentSkelMeshComponent->GetSkeletalMeshAsset() ? ParentMeshComponent : nullptr;
		}
	}

	// Grab deformed mesh if it exists 
	if(GroomAsset && GroomAsset->GetRiggedSkeletalMesh())
	{
		const USkeleton* TargetSkeleton = GroomAsset->GetRiggedSkeletalMesh()->GetSkeleton();
		
		DeformedMeshComponent = nullptr;
		// Try to find the first component by walking the attachment hierarchy
		for (USceneComponent* Curr = this; Curr; Curr = Curr->GetAttachParent())
		{
			USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Curr);
			if (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset() && SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton() == TargetSkeleton)
			{
				DeformedMeshComponent = SkelMeshComp;
				break;
			}
		}
		if (DeformedMeshComponent)
		{
			AddTickPrerequisiteComponent(DeformedMeshComponent);
		}
	}

	// Insure the ticking of the Groom component always happens after the skeletalMeshComponent.
	UGroomBindingAsset* LocalBindingAsset = nullptr;
	if (ValidatedMeshComponent)
	{
		RegisteredMeshComponent = ValidatedMeshComponent;
		AddTickPrerequisiteComponent(ValidatedMeshComponent);

		if (bHasNeedBindingData)
		{
			LocalBindingAsset = BindingAsset;
			if (LocalBindingAsset && LocalBindingAsset->GetHairGroupResources().Num() != GroomAsset->GetHairGroupsPlatformData().Num())
			{
				LocalBindingAsset = nullptr;
			}

			if (LocalBindingAsset)
			{
				// Ensure the groom binding asset's resources match the groom asset's resources.
				// They can mismatch when the groom asset has been edited and the groom binding hasn't been updated/finished to be built yet.
				const uint32 NumGroup = LocalBindingAsset->GetHairGroupResources().Num();
				check(NumGroup == GroomAsset->GetHairGroupsResources().Num());
				for (uint32 GroupIt = 0; GroupIt < NumGroup; ++GroupIt)
				{
					const UGroomBindingAsset::FHairGroupResource& BindingResource = LocalBindingAsset->GetHairGroupResources()[GroupIt];
					const FHairGroupResources& GroomResources = GroomAsset->GetHairGroupsResources()[GroupIt];

					const bool bHasGuides = GroomAsset->NeedsInterpolationData(GroupIt);
					const bool bHasStrands = GroomAsset->HasGeometryType(GroupIt, EGroomGeometryType::Strands);
					const bool bHasCards = GroomAsset->HasGeometryType(GroupIt, EGroomGeometryType::Cards);

					bool bValid = true;
					// Check guides
					if (GroomResources.Guides.IsValid() && bHasGuides)
					{
						bValid = bValid && BindingResource.SimRootResources && BindingResource.SimRootResources->GetLODCount() > 0;
					}
					// Check strands
					if (bValid && GroomResources.Strands.IsValid() && bHasStrands)
					{
						bValid = bValid && BindingResource.RenRootResources && BindingResource.RenRootResources->GetLODCount() > 0;
					}
					// Check cards
					if (bValid && GroomResources.Cards.LODs.Num() > 0 && bHasCards)
					{
						bValid = bValid && GroomResources.Cards.LODs.Num() == BindingResource.CardsRootResources.Num();
						if (bValid)
						{
							for (uint32 CardIt = 0, CardCount = GroomResources.Cards.LODs.Num(); CardIt < CardCount; ++CardIt)
							{
								if (GroomResources.Cards.LODs[CardIt].IsValid())
								{
									bValid = bValid && BindingResource.CardsRootResources[CardIt] && BindingResource.CardsRootResources[CardIt]->GetLODCount() > 0;
								}
							}
						}
					}

					if (!bValid)
					{
						LocalBindingAsset = nullptr;
						break;
					}
				}
			}
		}
	}

	// Update requirement based on binding asset availability
	if (LocalBindingAsset == nullptr)
	{
		bHasNeedSkinningBinding = false;
		bHasAnyNeedGlobalDeformation = false;
		bHasNeedGlobalDeformation.Init(false, GroomAsset->GetHairGroupsPlatformData().Num());
	}

	if (GroomCache)
	{
		GroomCacheBuffers = GUseGroomCacheStreaming ? MakeShared<FGroomCacheStreamedBuffers, ESPMode::ThreadSafe>(GroomCache) : MakeShared<FGroomCacheBuffers, ESPMode::ThreadSafe>(GroomCache);
		
		UpdateGroomCache(ElapsedTime);
	}
	else
	{
		GroomCacheBuffers.Reset();
	}

	for (int32 GroupIt = 0, GroupCount = GroomAsset->GetHairGroupsPlatformData().Num(); GroupIt < GroupCount; ++GroupIt)
	{
		FHairGroupInstance* HairGroupInstance = new FHairGroupInstance();
		HairGroupInstances.Add(HairGroupInstance);
		HairGroupInstance->Debug.GroupIndex = GroupIt;
		HairGroupInstance->Debug.GroupCount = GroupCount;
		HairGroupInstance->Debug.GroomAssetHash = GroomAsset->GetAssetHash();
		HairGroupInstance->Debug.GroomAssetName = GroomAsset->GetName();
		HairGroupInstance->Debug.MeshComponentForDebug = RegisteredMeshComponent;
		HairGroupInstance->Debug.GroomComponentForDebug = this;
		HairGroupInstance->Debug.GroomBindingType = BindingAsset ? BindingAsset->GetGroomBindingType() : EGroomBindingMeshType::SkeletalMesh;
		HairGroupInstance->Debug.GroomCacheType = GetEffectiveGroomCacheType(GroomCache, GroomAsset);
		HairGroupInstance->Debug.GroomBindingAssetHash = BindingAsset ? BindingAsset->GetAssetHash() : 0u;
		HairGroupInstance->Debug.GroomCacheBuffers = GroomCacheBuffers;
		HairGroupInstance->Debug.LODForcedIndex = LODForcedIndex;
		HairGroupInstance->Debug.LODPredictedIndex = LODPredictedIndex;
		HairGroupInstance->Debug.LODSelectionTypeForDebug = LODSelectionType;
		HairGroupInstance->DeformedComponent = DeformedMeshComponent;
		HairGroupInstance->DeformedSection = GroomAsset->GetDeformedGroupSections().IsValidIndex(GroupIt) ? GroomAsset->GetDeformedGroupSections()[GroupIt] : INDEX_NONE; 
		HairGroupInstance->Debug.MeshComponentId = ValidatedMeshComponent ? ValidatedMeshComponent->GetPrimitiveSceneId() : FPrimitiveComponentId();
		HairGroupInstance->Debug.CachedMeshPersistentPrimitiveIndex = FPersistentPrimitiveIndex();

		if (RegisteredMeshComponent)
		{
			HairGroupInstance->Debug.MeshComponentName = RegisteredMeshComponent->GetPathName();
		}
		HairGroupInstance->GeometryType = EHairGeometryType::NoneGeometry;
		HairGroupInstance->BindingType = EHairBindingType::NoneBinding;

		HairGroupInstance->Debug.SkinningCurrentLocalToWorld	= RegisteredMeshComponent ? RegisteredMeshComponent->GetComponentTransform() : FTransform::Identity;
		HairGroupInstance->Debug.SkinningRestLocalToWorld		= HairGroupInstance->Debug.SkinningCurrentLocalToWorld;
		HairGroupInstance->Debug.SkinningPreviousLocalToWorld	= HairGroupInstance->Debug.SkinningCurrentLocalToWorld;
		HairGroupInstance->Debug.RigidCurrentLocalToWorld		= GetComponentTransform();
		HairGroupInstance->Debug.RigidRestLocalToWorld			= HairGroupInstance->Debug.RigidCurrentLocalToWorld;
		HairGroupInstance->Debug.RigidPreviousLocalToWorld		= HairGroupInstance->Debug.RigidCurrentLocalToWorld;
		HairGroupInstance->LocalToWorld							= HairGroupInstance->GetCurrentLocalToWorld();

		FHairGroupPlatformData& GroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIt];
		FHairGroupResources& GroupResources = GroomAsset->GetHairGroupsResources()[GroupIt];

		const EHairInterpolationType HairInterpolationType = ToHairInterpolationType(GroomAsset->GetHairInterpolationType());

		const FName OwnerName = GroomAsset->GetAssetPathName();

		// Initialize LOD screen size & visibility
		bool bNeedStrandsData = false;
		{
			HairGroupInstance->HairGroupPublicData = new FHairGroupPublicData(GroupIt, OwnerName);
			HairGroupInstance->HairGroupPublicData->Instance = HairGroupInstance;
			HairGroupInstance->HairGroupPublicData->bDebugDrawLODInfo = bPreviewMode;
			HairGroupInstance->HairGroupPublicData->DebugScreenSize = 0.f;
			HairGroupInstance->HairGroupPublicData->DebugGroupColor = GetHairGroupDebugColor(GroupIt);
			TArray<float> CPULODScreenSize;
			TArray<bool> LODVisibility;
			TArray<EHairGeometryType> LODGeometryTypes;
			const FHairGroupsLOD& GroupLOD = GroomAsset->GetHairGroupsLOD()[GroupIt];
			for (uint32 LODIt = 0, LODCount = GroupLOD.LODs.Num(); LODIt < LODCount; ++LODIt)
			{
				const FHairLODSettings& LODSettings = GroupLOD.LODs[LODIt];
				EHairBindingType BindingType = ToHairBindingType(GroomAsset->GetBindingType(GroupIt, LODIt));
				if (BindingType == EHairBindingType::Skinning && !LocalBindingAsset)
				{
					BindingType = EHairBindingType::NoneBinding;
				}
				else if (BindingType == EHairBindingType::Rigid && !RegisteredMeshComponent)
				{
					BindingType = EHairBindingType::NoneBinding;
				}

				// * Force global interpolation to be enable for meshes with skinning binding as we use RBF defomation for 'sticking' meshes onto skel. mesh surface
				// * Global deformation are allowed only with 'Skinning' binding type
				const EHairGeometryType GeometryType = ToHairGeometryType(GetEffectiveGeometryType(GroomAsset->GetGeometryType(GroupIt, LODIt), bUseCards, ShaderPlatform));
				const bool LODSimulation = IsSimulationEnable(GroupIt, LODIt);
				const bool LODGlobalInterpolation = LocalBindingAsset && BindingType == EHairBindingType::Skinning && GroomAsset->IsGlobalInterpolationEnable(GroupIt, LODIt);
				bNeedStrandsData = bNeedStrandsData || GeometryType == EHairGeometryType::Strands;

				CPULODScreenSize.Add(LODSettings.ScreenSize);
				LODVisibility.Add(LODSettings.bVisible);
				LODGeometryTypes.Add(GeometryType);
				HairGroupInstance->HairGroupPublicData->BindingTypes.Add(BindingType);
				HairGroupInstance->HairGroupPublicData->LODSimulations.Add(LODSimulation);
				HairGroupInstance->HairGroupPublicData->LODGlobalInterpolations.Add(LODGlobalInterpolation);
			}
			HairGroupInstance->HairGroupPublicData->bIsSimulationCacheEnable = HairGroupInstance->Debug.GroomCacheType == EGroomCacheType::Guides;
			HairGroupInstance->HairGroupPublicData->bIsDeformationEnable = IsDeformationEnable(GroupIt);
			HairGroupInstance->HairGroupPublicData->SetLODScreenSizes(CPULODScreenSize);
			HairGroupInstance->HairGroupPublicData->SetLODVisibilities(LODVisibility);
			HairGroupInstance->HairGroupPublicData->SetLODGeometryTypes(LODGeometryTypes);
			HairGroupInstance->HairGroupPublicData->bAutoLOD = GroomAsset->GetLODMode() == EGroomLODMode::Auto;
			HairGroupInstance->HairGroupPublicData->AutoLODBias = GroomAsset->GetAutoLODBias(GroupIt);

			HairGroupInstance->HairGroupPublicData->BindingMinMeshLOD = INDEX_NONE;
			if (LocalBindingAsset)
			{
				const TArray<UGroomBindingAsset::FHairGroupPlatformData>& BindingPlatformData = LocalBindingAsset->GetHairGroupsPlatformData();
				if (BindingPlatformData.IsValidIndex(GroupIt))
				{
					HairGroupInstance->HairGroupPublicData->BindingMinMeshLOD = BindingPlatformData[GroupIt].TargetMeshMinLOD;
				}
			}
		}

		FHairResourceName ResourceName(GetFName(), GroupIt);

		// Not needed anymore
		//const bool bDynamicResources = IsHairStrandsSimulationEnable() || IsHairStrandsBindingEnable();

		// Sim data
		// Simulation guides are used for two purposes:
		// * Physics simulation
		// * RBF deformation.
		// Therefore, even if simulation is disabled, we need to run partially the update if the binding system is enabled (skin deformation + RBF correction)
		const bool bNeedGuides = (GroupData.Guides.HasValidData() && (bHasNeedSimulation[GroupIt] || bHasNeedGlobalDeformation[GroupIt] || bHasNeedDeformation[GroupIt])) || HairGroupInstance->HairGroupPublicData->bIsSimulationCacheEnable;
		if (bNeedGuides)
		{
			if (LocalBindingAsset)
			{
				check(RegisteredMeshComponent);
				check(LocalBindingAsset);
				check(GroupIt < LocalBindingAsset->GetHairGroupResources().Num());
				#if GROOMBINDING_SKELMESH_VALIDATION
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RegisteredMeshComponent))
				{
					check(SkeletalMeshComponent->GetSkeletalMeshAsset() ? SkeletalMeshComponent->GetSkeletalMeshAsset()->GetLODNum() == LocalBindingAsset->GetHairGroupResources()[GroupIt].SimRootResources->GetLODCount() : false);
				}
				#endif

				HairGroupInstance->Guides.RestRootResource = LocalBindingAsset->GetHairGroupResources()[GroupIt].SimRootResources;
				HairGroupInstance->Guides.DeformedRootResource = new FHairStrandsDeformedRootResource(HairGroupInstance->Guides.RestRootResource, EHairStrandsResourcesType::Guides, ResourceName, OwnerName);
			}

			// Lazy allocation of the guide resources
			HairGroupInstance->Guides.RestResource = GroomAsset->AllocateGuidesResources(GroupIt);
			check(GroupResources.Guides.RestResource);

			// If guides are allocated, deformed resources are always needs since they are either used with simulation, or RBF deformation. Both are dynamics, and require deformed positions
			HairGroupInstance->Guides.DeformedResource = new FHairStrandsDeformedResource(GroupData.Guides.BulkData, EHairStrandsResourcesType::Guides, ResourceName, OwnerName);

			// Initialize the simulation and the global deformation to its default behavior by setting it with LODIndex = -1
			const int32 LODIndex = -1;
			HairGroupInstance->Guides.bIsSimulationEnable = IsSimulationEnable(GroupIt,LODIndex);
			HairGroupInstance->Guides.bHasGlobalInterpolation = LocalBindingAsset && GroomAsset->IsGlobalInterpolationEnable(GroupIt,LODIndex);
			HairGroupInstance->Guides.bIsDeformationEnable = IsDeformationEnable(GroupIt);
			HairGroupInstance->Guides.bIsSimulationCacheEnable = HairGroupInstance->HairGroupPublicData->bIsSimulationCacheEnable;
		}

		// LODBias is in the Modifier which is needed for LOD selection regardless if the strands are there or not
		HairGroupInstance->Strands.Modifier = GetGroomGroupsDesc(GroomAsset, this, GroupIt);

		// Strands data/resources
		if (bNeedStrandsData && GroupData.Strands.HasValidData())
		{
			check(GroupIt < GroomGroupsDesc.Num());

			HairGroupInstance->HairGroupPublicData->RestPointCount = GroupData.Strands.BulkData.GetNumPoints();
			HairGroupInstance->HairGroupPublicData->RestCurveCount = GroupData.Strands.BulkData.GetNumCurves();
			HairGroupInstance->HairGroupPublicData->ClusterCount = 0u;
			HairGroupInstance->HairGroupPublicData->ClusterScale = 0.f;

			// (Lazy) Allocate interpolation resources, only if guides are required
			if (bNeedGuides)
			{
				HairGroupInstance->Strands.InterpolationResource = GroomAsset->AllocateInterpolationResources(GroupIt);
				check(HairGroupInstance->Strands.InterpolationResource);
			}

			// Check if strands needs to load/allocate root resources
			bool bNeedRootResources = false;
			bool bNeedDynamicResources = false;
			{
				const FHairGroupsLOD& GroupLOD = GroomAsset->GetHairGroupsLOD()[GroupIt];
				for (uint32 LODIt = 0, LODCount = GroupLOD.LODs.Num(); LODIt < LODCount; ++LODIt)
				{
					if (HairGroupInstance->HairGroupPublicData->GetGeometryType(LODIt) == EHairGeometryType::Strands)
					{
						if (LocalBindingAsset &&
							(HairGroupInstance->HairGroupPublicData->GetBindingType(LODIt) == EHairBindingType::Skinning ||
							 HairGroupInstance->HairGroupPublicData->IsGlobalInterpolationEnable(LODIt)))
						{
							bNeedRootResources = true;
						}

						if ((LocalBindingAsset && HairGroupInstance->HairGroupPublicData->GetBindingType(LODIt) == EHairBindingType::Skinning) ||
							HairGroupInstance->HairGroupPublicData->IsSimulationEnable(LODIt))
						{
							bNeedDynamicResources = true;
						}
						
					}
				}

				// If length scale override is enabled, the group will potentially be scaled, which will modified the strands position. 
				// For this reason we need to allocate deformed resources
				if (GroomGroupsDesc[GroupIt].HairLengthScale_Override)
				{
					bNeedDynamicResources = true;
				}

				// The strands GroomCache needs the Strands.DeformedResource, but the guides GroomCache also needs it for the HairTangentPass
				if (HairGroupInstance->Debug.GroomCacheType != EGroomCacheType::None)
				{
					bNeedDynamicResources = true;
				}

				// the bones deformation needs to setup the deformed resources
				if(HairGroupInstance->HairGroupPublicData->bIsDeformationEnable)
				{
					bNeedDynamicResources = true;
				}

				// Mesh deformer requires to dynamic resources
				if (MeshDeformer || GroomSolver)
				{
					bNeedDynamicResources = true;

					// When an instance has a groom deformer, we disable streaming in order to load all its data, as they 
					// can be accessed/modified by a groom deformer
					HairGroupInstance->bSupportStreaming = false;
				}
			}

			if (bNeedRootResources)
			{
				check(LocalBindingAsset);
				check(RegisteredMeshComponent);
				check(GroupIt < LocalBindingAsset->GetHairGroupResources().Num());
				#if GROOMBINDING_SKELMESH_VALIDATION
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RegisteredMeshComponent))
				{
					check(SkeletalMeshComponent->GetSkeletalMeshAsset() ? SkeletalMeshComponent->GetSkeletalMeshAsset()->GetLODNum() == LocalBindingAsset->GetHairGroupResources()[GroupIt].RenRootResources->GetLODCount() : false);
				}
				#endif

				HairGroupInstance->Strands.RestRootResource = LocalBindingAsset->GetHairGroupResources()[GroupIt].RenRootResources;
				HairGroupInstance->Strands.DeformedRootResource = new FHairStrandsDeformedRootResource(HairGroupInstance->Strands.RestRootResource, EHairStrandsResourcesType::Strands, ResourceName, OwnerName);
			}

			HairGroupInstance->Strands.RestResource = GroupResources.Strands.RestResource;
			if (bNeedDynamicResources)
			{
				HairGroupInstance->Strands.DeformedResource = new FHairStrandsDeformedResource(GroupData.Strands.BulkData, EHairStrandsResourcesType::Strands, ResourceName, OwnerName);
			} 

			#if RHI_RAYTRACING
			HairGroupInstance->Strands.ViewRayTracingMask = EHairViewRayTracingMask::None;
			HairGroupInstance->Strands.RenRaytracingResourceShared = GroomAsset->GetStrandsRaytracingResourcesPtr(GroupIt);
			if (IsHairRayTracingEnabled() && bVisibleInRayTracing && HairGroupInstance->Strands.IsValid())
			{
				// Visible to ray tracing if the user has opted in
				// If the project has path tracing, resources for pathtracing will be lazyly allocated on demand
				if (HairGroupInstance->Strands.Modifier.bUseHairRaytracingGeometry)
				{
					HairGroupInstance->Strands.ViewRayTracingMask |= EHairViewRayTracingMask::RayTracing;
				}
				if (HairGroupInstance->Strands.ViewRayTracingMask != EHairViewRayTracingMask::None)
				{
					AllocateRaytracingResources(HairGroupInstance, bNeedDynamicResources, OwnerName);
				}
			}
			#endif

			// An empty groom doesn't have a ClusterResource
			HairGroupInstance->Strands.ClusterResource = GroupResources.Strands.ClusterResource;
			if (HairGroupInstance->Strands.ClusterResource)
			{
				// This codes assumes strands LOD are contigus and the highest (i.e., 0...x). Change this code to something more robust
				check(HairGroupInstance->HairGroupPublicData);
				const int32 StrandsLODCount = GroupResources.Strands.ClusterResource->BulkData.Header.LODInfos.Num();
				const TArray<float>& LODScreenSizes = HairGroupInstance->HairGroupPublicData->GetLODScreenSizes();
				const TArray<bool>& LODVisibilities = HairGroupInstance->HairGroupPublicData->GetLODVisibilities();
				check(StrandsLODCount <= LODScreenSizes.Num());
				check(StrandsLODCount <= LODVisibilities.Num());
				HairGroupInstance->HairGroupPublicData->ClusterCount = HairGroupInstance->Strands.ClusterResource->BulkData.Header.ClusterCount;
				HairGroupInstance->HairGroupPublicData->ClusterScale = HairGroupInstance->Strands.ClusterResource->BulkData.Header.ClusterScale;
			}

			HairGroupInstance->Strands.CullingResource = new FHairStrandsCullingResource(
				HairGroupInstance->HairGroupPublicData->RestPointCount, 
				HairGroupInstance->HairGroupPublicData->RestCurveCount, 
				HairGroupInstance->HairGroupPublicData->ClusterCount, 
				ResourceName, OwnerName);
			HairGroupInstance->HairGroupPublicData->Culling = &HairGroupInstance->Strands.CullingResource->Resources;

			HairGroupInstance->Strands.HairInterpolationType = HairInterpolationType;
		}

		// Cards resources
		const uint32 CardLODCount = GroupData.Cards.LODs.Num();
		for (uint32 CardLODIt=0;CardLODIt<CardLODCount;++CardLODIt)
		{
			FHairGroupPlatformData::FCards::FLOD& LODData = GroupData.Cards.LODs[CardLODIt];
			FHairGroupResources::FCards::FLOD& LODResource = GroupResources.Cards.LODs[CardLODIt];

			const uint32 CardsLODIndex = HairGroupInstance->Cards.LODs.Num();
			FHairGroupInstance::FCards::FLOD& InstanceLOD = HairGroupInstance->Cards.LODs.AddDefaulted_GetRef();
			if (LODData.HasValidData())
			{
				FHairResourceName LODResourceName(GetFName(), GroupIt, CardsLODIndex);
				const FName LODOwnerName = GroomAsset->GetAssetPathName(CardsLODIndex);

				const EHairBindingType BindingType	= HairGroupInstance->HairGroupPublicData->GetBindingType(CardsLODIndex);
				const bool bHasSimulation			= HairGroupInstance->HairGroupPublicData->IsSimulationEnable(CardsLODIndex);
				const bool bHasDeformation			= HairGroupInstance->HairGroupPublicData->bIsDeformationEnable;
				const bool bHasGlobalDeformation	= HairGroupInstance->HairGroupPublicData->IsGlobalInterpolationEnable(CardsLODIndex);
				const bool bNeedDeformedPositions	= bHasSimulation || bHasDeformation || bHasGlobalDeformation || BindingType == EHairBindingType::Skinning;
				const bool bNeedRootData			= bHasGlobalDeformation || BindingType == EHairBindingType::Skinning;

				// Sanity check
				if (bHasGlobalDeformation)
				{
					check(BindingType == EHairBindingType::Skinning);
				}

				InstanceLOD.RestResource = LODResource.RestResource;
				InstanceLOD.InterpolationResource = LODResource.InterpolationResource;

				if (bNeedDeformedPositions)
				{
					InstanceLOD.DeformedResource = new FHairCardsDeformedResource(LODData.BulkData, false, LODResourceName, LODOwnerName);
				}

				#if RHI_RAYTRACING
				if (IsHairRayTracingEnabled() && IsRayTracingAllowed() && bVisibleInRayTracing)
				{
					if (bNeedDeformedPositions)
					{
						// Allocate dynamic raytracing resources (owned by the groom component/instance)
						InstanceLOD.RaytracingResource = new FHairStrandsRaytracingResource(InstanceLOD.GetData(), LODResourceName, LODOwnerName);
						InstanceLOD.RaytracingResourceOwned = true;
					}
					else
					{
						// (Lazy) Allocate static raytracing resources (owned by the grooom asset)
						InstanceLOD.RaytracingResourceOwned = false;
						InstanceLOD.RaytracingResource = GroomAsset->AllocateCardsRaytracingResources(GroupIt, CardsLODIndex);
						check(InstanceLOD.RaytracingResource);
					}
				}
				#endif

				// Strands data/resources
				if (bNeedDeformedPositions)
				{
					InstanceLOD.Guides.InterpolationResource = LODResource.GuideInterpolationResource;

					if (bNeedRootData)
					{
						check(LocalBindingAsset);
						check(GroupIt < LocalBindingAsset->GetHairGroupResources().Num());
						#if GROOMBINDING_SKELMESH_VALIDATION
						if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RegisteredMeshComponent))
						{
							check(SkeletalMeshComponent->GetSkeletalMeshAsset() ? SkeletalMeshComponent->GetSkeletalMeshAsset()->GetLODNum() == LocalBindingAsset->GetHairGroupResources()[GroupIt].CardsRootResources[CardsLODIndex]->GetLODCount() : false);
						}
						#endif
						InstanceLOD.Guides.RestRootResource = LocalBindingAsset->GetHairGroupResources()[GroupIt].CardsRootResources[CardsLODIndex];
						InstanceLOD.Guides.DeformedRootResource = new FHairStrandsDeformedRootResource(InstanceLOD.Guides.RestRootResource, EHairStrandsResourcesType::Cards, LODResourceName, LODOwnerName);
					}

					InstanceLOD.Guides.RestResource = LODResource.GuideRestResource;
					{
						InstanceLOD.Guides.DeformedResource = new FHairStrandsDeformedResource(LODData.GuideBulkData, EHairStrandsResourcesType::Cards, LODResourceName, LODOwnerName);
					}

					InstanceLOD.Guides.HairInterpolationType = HairInterpolationType;
				}
			}
		}

		// Meshes resources
		const uint32 MeshLODCount = GroupData.Meshes.LODs.Num();
		for (uint32 MeshLODIt=0;MeshLODIt<MeshLODCount;++MeshLODIt)
		{
			FHairGroupPlatformData::FMeshes::FLOD& LODData = GroupData.Meshes.LODs[MeshLODIt];
			FHairGroupResources::FMeshes::FLOD& LODResource = GroupResources.Meshes.LODs[MeshLODIt];

			const int32 MeshLODIndex = HairGroupInstance->Meshes.LODs.Num();
			FHairGroupInstance::FMeshes::FLOD& InstanceLOD = HairGroupInstance->Meshes.LODs.AddDefaulted_GetRef();
			if (LODData.IsValid())
			{
				FHairResourceName LODResourceName(GetFName(), GroupIt, MeshLODIndex);
				const FName LODOwnerName = GroomAsset->GetAssetPathName(MeshLODIndex);

				const EHairBindingType BindingType = HairGroupInstance->HairGroupPublicData->GetBindingType(MeshLODIndex);
				const bool bHasGlobalDeformation   = HairGroupInstance->HairGroupPublicData->IsGlobalInterpolationEnable(MeshLODIndex);
				const bool bNeedDeformedPositions  = bHasGlobalDeformation && BindingType == EHairBindingType::Skinning;

				InstanceLOD.RestResource = LODResource.RestResource;
				if (bNeedDeformedPositions)
				{
					InstanceLOD.DeformedResource = new FHairMeshesDeformedResource(LODData.BulkData, true, LODResourceName, LODOwnerName);
				}

				#if RHI_RAYTRACING
				if (IsHairRayTracingEnabled() && IsRayTracingAllowed() && bVisibleInRayTracing)
				{
					if (bNeedDeformedPositions)
					{
						// Allocate dynamic raytracing resources (owned by the groom component/instance)
						InstanceLOD.RaytracingResource = new FHairStrandsRaytracingResource(LODData.BulkData, LODResourceName, LODOwnerName);
						InstanceLOD.RaytracingResourceOwned = true;
					}
					else
					{
						// (Lazy) Allocate static raytracing resources (owned by the grooom asset)
						InstanceLOD.RaytracingResourceOwned = false;
						InstanceLOD.RaytracingResource = GroomAsset->AllocateMeshesRaytracingResources(GroupIt, MeshLODIndex);
					}
				}
				#endif
			}
		}
	}

	// Does not run projection code when running with null RHI as this is not needed, and will crash as the skeletal GPU resources are not created
	if (GUsingNullRHI)
	{
		return;
	}
}

void UGroomComponent::ReleaseResources()
{
	InitializedResources = nullptr;

	HairGroupInstances.Empty();

	// Insure the ticking of the Groom component always happens after the skeletalMeshComponent.
	if (RegisteredMeshComponent && !RegisteredMeshComponent->IsBeingDestroyed())
	{
		RemoveTickPrerequisiteComponent(RegisteredMeshComponent);
	}
	RegisteredMeshComponent = nullptr;

	GroomCacheBuffers.Reset();

	MarkRenderStateDirty();
}

void UGroomComponent::PostLoad()
{
	LLM_SCOPE_BYTAG(Groom);

	Super::PostLoad();

	if (GroomAsset)
	{
		// Make sure that the asset initialized its resources first since the component needs them to initialize its own resources
		GroomAsset->ConditionalPostLoad();
	}

	if (BindingAsset)
	{
		// Make sure that the asset initialized its resources first since the component needs them to initialize its own resources
		BindingAsset->ConditionalPostLoad();
	}

	// This call will handle the GroomAsset properly if it's still being loaded
	SetGroomAsset(GroomAsset, BindingAsset, false);

	PrecachePSOs();

#if WITH_EDITOR
	if (GroomAsset && !bIsGroomAssetCallbackRegistered)
	{
		// Delegate used for notifying groom data invalidation
		GroomAsset->GetOnGroomAssetChanged().AddUObject(this, &UGroomComponent::Invalidate);

		// Delegate used for notifying groom data & groom resoures invalidation
		GroomAsset->GetOnGroomAssetResourcesChanged().AddUObject(this, &UGroomComponent::InvalidateAndRecreate);

		bIsGroomAssetCallbackRegistered = true;
	}

	if (BindingAsset && !bIsGroomBindingAssetCallbackRegistered)
	{
		BindingAsset->GetOnGroomBindingAssetChanged().AddUObject(this, &UGroomComponent::InvalidateAndRecreate);
		bIsGroomBindingAssetCallbackRegistered = true;
	}

	// Do not validate the groom yet as the component count be loaded, but material/binding & co will be set later on
	// ValidateMaterials(false);
#endif

	// Warning in case the groom is using a guide cache, but the GetEnableSimulationCache() is disabled.
	if (GroomAsset && GroomCache && GroomCache->GetType() == EGroomCacheType::Guides && !GroomAsset->GetEnableSimulationCache())
	{
		UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - The groom instance tries to use a guide-cache, but 'Enable Guide-Cache Support' option is not enabled on the groom asset. Because of this, the Guide-Cache won't be used.", *GetPathName());
	}
}

void UGroomComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (GroomAsset == nullptr)
	{
		return;
	}

	TArray<FHairVertexFactoryTypesPerMaterialData> VFsPerMaterials = GroomAsset->CollectVertexFactoryTypesPerMaterialData(GMaxRHIShaderPlatform);

	FPSOPrecacheParams PrecachePSOParams = BasePrecachePSOParams;

	for (FHairVertexFactoryTypesPerMaterialData& VFsPerMaterial : VFsPerMaterials)
	{
		UMaterialInterface* MaterialInterface = GetMaterial(GetMaterialIndexWithFallback(VFsPerMaterial.MaterialIndex), VFsPerMaterial.HairGeometryType, true);
		if (MaterialInterface)
		{
			PrecachePSOParams.PrimitiveType = GetPrimitiveType(VFsPerMaterial.HairGeometryType);

			FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
			ComponentParams.MaterialInterface = MaterialInterface;
			ComponentParams.VertexFactoryDataList = VFsPerMaterial.VertexFactoryDataList;
			ComponentParams.PSOPrecacheParams = PrecachePSOParams;
			ComponentParams.Priority = EPSOPrecachePriority::High;
		}
	}

}

#if WITH_EDITOR
void UGroomComponent::Invalidate()
{
	UpdateHairGroupsDescAndInvalidateRenderState();
	UpdateHairSimulation();
	ValidateMaterials(false);
}

void UGroomComponent::InvalidateAndRecreate()
{
	InitResources(true);
	UpdateHairSimulation();
	MarkRenderInstancesDirty();
}
#endif

void UGroomComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	if (BindingAsset && BindingAsset->IsCompiling())
	{
		return;
	}
#endif

	UpdateHairGroupsDesc();

	if (GUseGroomCacheStreaming)
	{
		IGroomCacheStreamingManager::Get().RegisterComponent(this);
	}

	// Insure the parent skeletal mesh is the same than the registered skeletal mesh, and if not reinitialized resources
	// This can happens when the OnAttachment callback is not called, but the skeletal mesh change (e.g., the skeletal mesh get recompiled within a blueprint)
	UMeshComponent* MeshComponent = GetAttachParent() ? Cast<UMeshComponent>(GetAttachParent()) : nullptr;

	const bool bNeedInitialization = !InitializedResources || InitializedResources != GroomAsset || MeshComponent != RegisteredMeshComponent;
	if (bNeedInitialization)
	{
		InitResources();
	}
	else if (GroomCache)
	{
		// Buffers already initialized, just need to update them
		UpdateGroomCache(ElapsedTime);
	}
	UpdateHairSimulation();

	MeshDeformerInstance = (MeshDeformer != nullptr) ? MeshDeformer->CreateInstance(this, MeshDeformerInstanceSettings) : nullptr;
}

void UGroomComponent::OnUnregister()
{
	Super::OnUnregister();
	ReleaseHairSimulation();

	if (GUseGroomCacheStreaming)
	{
		if (GroomCacheBuffers.IsValid())
		{
			// Reset the buffers so they can be updated at OnRegister
			FGroomCacheBuffers* Buffers = static_cast<FGroomCacheBuffers*>(GroomCacheBuffers.Get());
			Buffers->Reset();
		}
		IGroomCacheStreamingManager::Get().UnregisterComponent(this);
	}

	MeshDeformerInstance = nullptr;
}

void UGroomComponent::BeginDestroy()
{
	ReleaseResources();
	Super::BeginDestroy();
}

void UGroomComponent::FinishDestroy()
{
	Super::FinishDestroy();
}

void UGroomComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	ReleaseResources();

#if WITH_EDITOR
	if (bIsGroomAssetCallbackRegistered && GroomAsset)
	{
		GroomAsset->GetOnGroomAssetChanged().RemoveAll(this);
		GroomAsset->GetOnGroomAssetResourcesChanged().RemoveAll(this);
		bIsGroomAssetCallbackRegistered = false;
	}
	if (bIsGroomBindingAssetCallbackRegistered && BindingAsset)
	{
		BindingAsset->GetOnGroomBindingAssetChanged().RemoveAll(this);
		bIsGroomBindingAssetCallbackRegistered = false;
	}
#endif

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UGroomComponent::OnAttachmentChanged()
{
	Super::OnAttachmentChanged();

#if WITH_EDITOR
	// InitResources will be called when the binding finishes compiling
	if (BindingAsset && BindingAsset->IsCompiling())
	{
		return;
	}
#endif

	if (GroomAsset && !IsBeingDestroyed() && HasBeenCreated() && IsValidChecked(this))
	{
		UMeshComponent* NewMeshComponent = Cast<UMeshComponent>(GetAttachParent());
		const bool bHasAttachmentChanged = RegisteredMeshComponent != NewMeshComponent;
		if (bHasAttachmentChanged)
		{
			InitResources();
		}
	}
}

// Override the DetachFromComponent so that we do not mark the actor as modified/dirty when attaching to a specific bone
void UGroomComponent::DetachFromComponent(const FDetachmentTransformRules& In)
{
	FDetachmentTransformRules DetachmentRules = In;
	DetachmentRules.bCallModify = false;
	Super::DetachFromComponent(DetachmentRules);
}

void UGroomComponent::UpdateGroomCache(float Time)
{
	if (GroomCache && GroomCacheBuffers.IsValid() && bRunning)
	{
		FGroomCacheBuffers* Buffers = static_cast<FGroomCacheBuffers*>(GroomCacheBuffers.Get());
		Buffers->UpdateBuffersAtTime(Time, bLooping);

		// Trigger an update of the bounds so that it follows the GroomCache
		if (IsInGameThread())
		{
			MarkRenderTransformDirty();
		}
		else
		{
			FWeakObjectPtr WeakComponent(this);
			AsyncTask(ENamedThreads::GameThread, [WeakComponent]()
			{
				if (UGroomComponent* Component = Cast<UGroomComponent>(WeakComponent.Get()))
				{
					
					Component->MarkRenderTransformDirty();
				}
			});
		}
	}
}

void UGroomComponent::SetGroomCache(UGroomCache* InGroomCache)
{
	if (GroomCache != InGroomCache)
	{
		ReleaseResources();
		ResetAnimationTime();

		if (GUseGroomCacheStreaming)
		{
			IGroomCacheStreamingManager::Get().UnregisterComponent(this);
			GroomCache = InGroomCache;
			IGroomCacheStreamingManager::Get().RegisterComponent(this);
		}
		else
		{
			GroomCache = InGroomCache;
		}

		InitResources();
	}
}

float UGroomComponent::GetGroomCacheDuration() const
{
	return GroomCache ? GroomCache->GetDuration() : 0.0f;
}

void UGroomComponent::SetManualTick(bool bInManualTick)
{
	bManualTick = bInManualTick;
}

bool UGroomComponent::GetManualTick() const
{
	return bManualTick;
}

void UGroomComponent::ResetAnimationTime()
{
	ElapsedTime = 0.0f;
	if (GroomCache && bRunning && GUseGroomCacheStreaming)
	{
		IGroomCacheStreamingManager::Get().PrefetchData(this);
	}
	UpdateGroomCache(ElapsedTime);
}

float UGroomComponent::GetAnimationTime() const
{
	return ElapsedTime;
}

void UGroomComponent::TickAtThisTime(const float Time, bool bInIsRunning, bool bInBackwards, bool bInIsLooping)
{
	if (GroomCache && bRunning && bManualTick)
	{
		float DeltaTime = Time - ElapsedTime;
		if (GUseGroomCacheStreaming)
		{
			// Scrubbing forward (or backward) can induce large (or negative) delta time, so force a prefetch
			if ((DeltaTime > GetDefault<UGroomPluginSettings>()->GroomCacheLookAheadBuffer) ||
				(DeltaTime < 0))
			{
				ElapsedTime = Time;
				IGroomCacheStreamingManager::Get().PrefetchData(this);

				// Update the buffers with the prefetched data right away
				UpdateGroomCache(Time);
				return;
			}
		}

		// Queue the update for the render thread to sync it with GeometryCache rendering
		ENQUEUE_RENDER_COMMAND(FGroomCacheUpdate)(UE::RenderCommandPipe::Groom,
		[this, Time](FRHICommandList& RHICmdList)
		{
			ElapsedTime = Time;
			UpdateGroomCache(Time);
		});
	}
}

void UGroomComponent::BuildSimulationTransform(FTransform& SimulationTransform) const
{
	SimulationTransform = FTransform::Identity;
	if (SimulationSettings.SimulationSetup.bLocalSimulation)
	{
		const USkeletalMeshComponent* SkeletelMeshComponent = Cast<const USkeletalMeshComponent>(RegisteredMeshComponent);
		if (SkeletelMeshComponent && SkeletelMeshComponent->GetSkeletalMeshAsset() && !SimulationSettings.SimulationSetup.LocalBone.IsEmpty())
		{
			const FReferenceSkeleton& RefSkeleton = SkeletelMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton();
			const FName BoneName(SimulationSettings.SimulationSetup.LocalBone);
			const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);

			SimulationTransform = SkeletelMeshComponent->GetBoneTransform(BoneIndex);
		}
		else
		{
			SimulationTransform = GetComponentTransform();
		}
	}
}

UPhysicsAsset* UGroomComponent::BuildAndCollect(FTransform& BoneTransform, TArray<TWeakObjectPtr<USkeletalMeshComponent>>& SourceComponents, TArray<TWeakObjectPtr<UPhysicsAsset>>& PhysicsAssets) const
{
	BuildSimulationTransform(BoneTransform);

	for (auto& CollisionComponent : CollisionComponents)
	{
		if (CollisionComponent.IsValid() && CollisionComponent->GetPhysicsAsset())
		{
			SourceComponents.Add(CollisionComponent);
			PhysicsAssets.Add(CollisionComponent->GetPhysicsAsset());
		}
	}

	return PhysicsAsset;
}

void UGroomComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);	
	
	const uint32 Id = GetPrimitiveSceneId().PrimIDValue;
	const ERHIFeatureLevel::Type FeatureLevel = GetWorld() ? ERHIFeatureLevel::Type(GetWorld()->GetFeatureLevel()) : ERHIFeatureLevel::Num;

	// When a groom binding and simulation are disabled, and the groom component is parented with a skeletal mesh, we can optionally 
	// attach the groom to a particular socket/bone
	USkeletalMeshComponent* SkeletelMeshComponent = Cast<USkeletalMeshComponent>(RegisteredMeshComponent);
	if (SkeletelMeshComponent && SkeletelMeshComponent->GetSkeletalMeshAsset() && !AttachmentName.IsEmpty())
	{
		const FName BoneName(AttachmentName);
		if (GetAttachSocketName() != BoneName)
		{
			const int32 BoneIndex = SkeletelMeshComponent->GetBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				AttachToComponent(SkeletelMeshComponent, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false), BoneName);
				const FMatrix BoneTransformRaw = SkeletelMeshComponent->GetSkeletalMeshAsset()->GetComposedRefPoseMatrix(BoneIndex);
				const FVector BoneLocation = BoneTransformRaw.GetOrigin();
				const FQuat BoneRotation = BoneTransformRaw.ToQuat();

				FTransform BoneTransform = FTransform::Identity;
				BoneTransform.SetLocation(BoneLocation);
				BoneTransform.SetRotation(BoneRotation);

				FTransform InvBoneTransform = BoneTransform.Inverse();
				SetRelativeLocation(InvBoneTransform.GetLocation());
				SetRelativeRotation(InvBoneTransform.GetRotation());
			}
		} 
	}

	const AActor* Owner = GetOwner();
	bool bHidden = !IsVisible() || (Owner && Owner->IsHidden());

#if WITH_EDITOR
	bHidden |= (Owner && Owner->IsHiddenEd());
#endif

	if (bHidden && !bWasHiddenLastTick)
	{
		// Visible -> hidden: pause the Niagara sim so it doesn't drift while off-screen.
		for (UNiagaraComponent* NC : NiagaraComponents)
		{
			if (NC) { NC->DeactivateImmediate(); }
		}
		bWasHiddenLastTick = true;
	}
	else if (!bHidden && bWasHiddenLastTick)
	{
		// Hidden -> visible: re-activate with reset so we start from rest pose.
		for (UNiagaraComponent* NC : NiagaraComponents)
		{
			if (NC) { NC->Activate(/*bReset=*/true); }
		}
		bResetSimulation   = true;
		bInitSimulation    = true;
		bWasHiddenLastTick = false;
	}

	if (!bHidden)
	{
		bResetSimulation = bInitSimulation;
		if (!bInitSimulation)
		{
			const bool bLocalSimulation = SimulationSettings.SimulationSetup.bLocalSimulation &&
				(SimulationSettings.SimulationSetup.LinearVelocityScale == 0.0) &&
				(SimulationSettings.SimulationSetup.AngularVelocityScale == 0.0);
			if (!bLocalSimulation)
			{
				if (USkeletalMeshComponent* ParentComp = Cast<USkeletalMeshComponent>(GetAttachParent()))
				{
					if (ParentComp->GetNumBones() > 0)
					{
						const int32 BoneIndex = FMath::Min(1, ParentComp->GetNumBones() - 1);
						const FMatrix NextBoneMatrix = ParentComp->GetBoneMatrix(BoneIndex);

						const float BoneDistance = FVector::DistSquared(PrevBoneMatrix.GetOrigin(), NextBoneMatrix.GetOrigin());
						if (SimulationSettings.SimulationSetup.TeleportDistance > 0.0 && BoneDistance >
							SimulationSettings.SimulationSetup.TeleportDistance * SimulationSettings.SimulationSetup.TeleportDistance)
						{
							bResetSimulation = true;
						}
						PrevBoneMatrix = NextBoneMatrix;
					}
				}
			}
			bResetSimulation |= SimulationSettings.SimulationSetup.bResetSimulation;
		}
	}
	bInitSimulation = false;

	if (HairGroupInstances.Num() == 0)
	{
		return;
	}

	if (RegisteredMeshComponent)
	{
		// When a skeletal object with projection is enabled, activate the refresh of the bounding box to
		// insure the component/proxy bounding box always lies onto the actual skinned mesh
		MarkRenderTransformDirty();
	}

	// Tick GroomCache only when playing
	if (GroomCache && GetWorld()->AreActorsInitialized() && bRunning && !bManualTick)
	{
		ElapsedTime += DeltaTime;
		UpdateGroomCache(ElapsedTime);
	}

	// Based on LOD selection type pick the effect LOD index. 
	// When LOD prediction is used, we read from the rendering thread and use the smallest prediction accross all groups
	float EffectiveForceLOD = -1;
	if (LODSelectionType == EHairLODSelectionType::Predicted)
	{
		// 1. Compute the effective LOD (taking the min. LOD across all the groups)
		for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			/* /!\ Access rendering thread data at that point /!\ */
			const float LODPredictedIndex_Instance = Instance->Debug.LODPredictedIndex;
			if (LODPredictedIndex_Instance > 0)
			{
				EffectiveForceLOD = EffectiveForceLOD < 0 ? LODPredictedIndex_Instance : FMath::Min(LODPredictedIndex_Instance, EffectiveForceLOD);
			}
		}

		// 2. If there is a LOD change, update the simulation adequately 
		SwitchSimulationLOD(LODPredictedIndex, EffectiveForceLOD, LODSelectionType);

		// 3. Update global predicted index (used for SyncLOD API)
		LODPredictedIndex = EffectiveForceLOD;
	}
	else if (LODSelectionType == EHairLODSelectionType::Forced)
	{
		EffectiveForceLOD = LODForcedIndex;
	}

	const bool bSwapBuffer = GetHairSwapBufferType() == EHairBufferSwapType::Tick;
	FTransform SkelLocalToTransform = FTransform::Identity;	
	if (RegisteredMeshComponent)
	{
		SkelLocalToTransform = RegisteredMeshComponent->GetComponentTransform();
	}

	if (ERHIFeatureLevel::Num != FeatureLevel)
	{
		TArray<TRefCountPtr<FHairGroupInstance>> LocalHairGroupInstances = HairGroupInstances;
		const EHairLODSelectionType LocalLODSelectionType = LODSelectionType;
		ENQUEUE_RENDER_COMMAND(FHairStrandsTick_TransformUpdate)(/*UE::RenderCommandPipe::Groom,*/
			[Id, FeatureLevel, LocalHairGroupInstances, EffectiveForceLOD, LocalLODSelectionType, bSwapBuffer, SkelLocalToTransform](FRHICommandListImmediate& RHICmdList)
		{
			for (const TRefCountPtr<FHairGroupInstance>& Instance : LocalHairGroupInstances)
			{
				Instance->Debug.LODForcedIndex = EffectiveForceLOD;
				Instance->Debug.LODSelectionTypeForDebug = LocalLODSelectionType;
				if (LocalLODSelectionType == EHairLODSelectionType::Predicted && EffectiveForceLOD >= 0)
				{
					AddHairStreamingRequest(Instance, EffectiveForceLOD);
				}

				if (bSwapBuffer)
				{
					Instance->Debug.SkinningPreviousLocalToWorld = Instance->Debug.SkinningCurrentLocalToWorld;
					Instance->Debug.SkinningCurrentLocalToWorld = SkelLocalToTransform;

					if (Instance->Guides.DeformedResource) { Instance->Guides.DeformedResource->SwapBuffer(); }
					if (Instance->Strands.DeformedResource) { Instance->Strands.DeformedResource->SwapBuffer(); }
				}
			}
		});
	}
	
	// Need to run SendRenderDynamicData_Concurrent() 
	// * If SwapBufferType is based on RenderFrame
	// * If groom has a mesh deformer
	if(GetHairSwapBufferType() == EHairBufferSwapType::RenderFrame || MeshDeformerInstance != nullptr)
	{
		MarkRenderDynamicDataDirty();
	}
}

void UGroomComponent::SendRenderDynamicData_Concurrent()
{
	Super::SendRenderDynamicData_Concurrent();

	if(GetHairSwapBufferType() == EHairBufferSwapType::RenderFrame)
	{
		FTransform SkelLocalToTransform = FTransform::Identity;
		if (RegisteredMeshComponent)
		{
			SkelLocalToTransform = RegisteredMeshComponent->GetComponentTransform();
		}

		TArray<TRefCountPtr<FHairGroupInstance>> LocalHairGroupInstances = HairGroupInstances;
		ENQUEUE_RENDER_COMMAND(FHairStrandsTick_TransformUpdate)(/*UE::RenderCommandPipe::Groom,*/
			[LocalHairGroupInstances, SkelLocalToTransform](FRHICommandListImmediate& RHICmdList)
		{
			for (const TRefCountPtr<FHairGroupInstance>& Instance : LocalHairGroupInstances)
			{
				Instance->Debug.SkinningPreviousLocalToWorld = Instance->Debug.SkinningCurrentLocalToWorld;
				Instance->Debug.SkinningCurrentLocalToWorld  = SkelLocalToTransform;

				if (Instance->Guides.DeformedResource)  { Instance->Guides.DeformedResource->SwapBuffer(); }
				if (Instance->Strands.DeformedResource) { Instance->Strands.DeformedResource->SwapBuffer(); }
			}
		});
	}

	if (MeshDeformerInstance != nullptr && GroomAsset != nullptr)
	{
		UMeshDeformerInstance::FEnqueueWorkDesc Desc;
		Desc.Scene = GetScene();
		Desc.OwnerName = GroomAsset->GetFName();
		Desc.ExecutionGroup = UMeshDeformerInstance::ExecutionGroup_BeginInitViews;
		// TODO: Provide a FEnqueueWorkDesc::FallbackDelegate so that groom appears rest pose if the Enqueue fails.
		MeshDeformerInstance->EnqueueWork(Desc);
	}
}

void UGroomComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	if (MeshDeformerInstance)
	{
		MeshDeformerInstance->AllocateResources();
	}

	Super::CreateRenderState_Concurrent(Context);
}

void UGroomComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (MeshDeformerInstance)
	{
		MeshDeformerInstance->ReleaseResources();
	}
}

bool UGroomComponent::RequiresGameThreadEndOfFrameRecreate() const
{
	return Super::RequiresGameThreadEndOfFrameRecreate();
}

void UGroomComponent::SetMeshDeformer(UMeshDeformer* InMeshDeformer)
{
	MeshDeformer = InMeshDeformer;
	MeshDeformerInstanceSettings = MeshDeformer->CreateSettingsInstance(this);
	MeshDeformerInstance = (MeshDeformer != nullptr) ? MeshDeformer->CreateInstance(this, MeshDeformerInstanceSettings) : nullptr;
	MarkRenderDynamicDataDirty();
}

void UGroomComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	UMeshComponent::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
	if (bGetDebugMaterials)
	{
		if (IsHairStrandsEnabled(EHairStrandsShaderType::Tool))
		{
			OutMaterials.Add(Strands_DebugMaterial);
		}
	}

	if (IsHairStrandsEnabled(EHairStrandsShaderType::Strands))
	{
		OutMaterials.Add(Strands_DefaultMaterial);
	}

	if (IsHairStrandsEnabled(EHairStrandsShaderType::Cards))
	{
		OutMaterials.Add(Cards_DefaultMaterial);
	}
	if (IsHairStrandsEnabled(EHairStrandsShaderType::Meshes))
	{
		OutMaterials.Add(Meshes_DefaultMaterial);
	}
}

#if WITH_EDITOR
void UGroomComponent::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	const FName PropertyName = PropertyAboutToChange ? PropertyAboutToChange->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, GroomAsset))
	{
		// Remove the callback on the GroomAsset about to be replaced
		if (bIsGroomAssetCallbackRegistered && GroomAsset)
		{
			GroomAsset->GetOnGroomAssetChanged().RemoveAll(this);
			GroomAsset->GetOnGroomAssetResourcesChanged().RemoveAll(this);
		}
		bIsGroomAssetCallbackRegistered = false;
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, BindingAsset))
	{
		// Remove the callback on the GroomAsset about to be replaced
		if (bIsGroomBindingAssetCallbackRegistered && BindingAsset)
		{
			BindingAsset->GetOnGroomBindingAssetChanged().RemoveAll(this);
		}
		bIsGroomBindingAssetCallbackRegistered = false;
	}
}

void UGroomComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	const FName PropertyName = PropertyThatChanged ? PropertyThatChanged->GetFName() : NAME_None;

	//  Init/release resources when setting the GroomAsset (or undoing)
	const bool bAssetChanged = PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, GroomAsset);
	const bool bSourceSkeletalMeshChanged = PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, SourceSkeletalMesh);
	const bool bBindingAssetChanged = PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, BindingAsset);
	const bool bIsBindingCompatible = UGroomBindingAsset::IsCompatible(GroomAsset, BindingAsset, bValidationEnable);
	const bool bEnableSolverChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FHairSimulationSolver, bEnableSimulation);
	const bool bGroomCacheChanged = PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, GroomCache);
	const bool bMeshDeformerChanged = PropertyName == GET_MEMBER_NAME_CHECKED(UGroomComponent, MeshDeformer);
	const bool bEnableLengthScaleChanged = PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairLengthScale);
	if (!bIsBindingCompatible || !UGroomBindingAsset::IsBindingAssetValid(BindingAsset, false, bValidationEnable))
	{
		BindingAsset = nullptr;
	}

	if (GroomAsset && !GroomAsset->IsValid())
	{
		GroomAsset = nullptr;
	}

	// HairLengthScale_Override does not generate an edit event (only HairLengthScale does), so we manually go through 
	// all the groups to check if there is a need for reallocating the resources
	bool bEnableLengthScaleOverrideChanged = false;
	if (bEnableLengthScaleChanged && GroomAsset && !bAssetChanged)
	{
		for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			const FHairGroupDesc GroupDesc = GetGroomGroupsDesc(GroomAsset, this, Instance->Debug.GroupIndex);
			const bool bRecreate = Instance->Strands.DeformedResource == nullptr && GroupDesc.HairLengthScale_Override;
			if (bRecreate)
			{
				bEnableLengthScaleOverrideChanged = true;
				break;
			}
		}
	}

	bool bIsEventProcess = false;

	// If the raytracing flag change, then force to recreate resources
	// Update the raytracing resources only if the groom asset hasn't changed. Otherwise the group desc might be
	// invalid. If the groom asset has change, the raytracing resources will be correctly be rebuild witin the
	// InitResources function
	bool bRayTracingGeometryChanged = false;
	#if RHI_RAYTRACING
	if (GroomAsset && !bAssetChanged)
	{
		for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
		{
			const FHairGroupDesc GroupDesc = GetGroomGroupsDesc(GroomAsset, this, Instance->Debug.GroupIndex);
			const bool bWasUsingRayTracingGeometry = EnumHasAnyFlags(Instance->Strands.ViewRayTracingMask, EHairViewRayTracingMask::RayTracing);
			const bool bRecreate = bWasUsingRayTracingGeometry != GroupDesc.bUseHairRaytracingGeometry;
			if (bRecreate)
			{
				bRayTracingGeometryChanged = true;
				break;
			}
		}
	}
	#endif

	// If material is assigned to the groom from the viewport (i.e., drag&drop a material from the content brown onto the groom geometry, it results into a unknown property). There is other case 
	const bool bIsUnknown = PropertyThatChanged == nullptr;

	const bool bRecreateResources = bAssetChanged || bBindingAssetChanged || bGroomCacheChanged || bEnableLengthScaleOverrideChanged || bIsUnknown || bSourceSkeletalMeshChanged || bRayTracingGeometryChanged || bEnableSolverChanged|| bMeshDeformerChanged;
	if (bRecreateResources)
	{
		// Release the resources before Super::PostEditChangeProperty so that they get
		// re-initialized in OnRegister
		ReleaseResources();
		bIsEventProcess = true;
	}

#if WITH_EDITOR
	if (bGroomCacheChanged)
	{
		ResetAnimationTime();
	}

	// Check to see if simulation cache is enabled, which is needed when using with the guides groom cache,
	// and enable it on all groom groups if it's not already
	const EGroomCacheType GroomCacheType = GroomCache ? GroomCache->GetType() : EGroomCacheType::None;
	if (GroomCacheType == EGroomCacheType::Guides && GroomAsset)
	{
		for (int32 Index = 0; Index < GroomAsset->GetNumHairGroups(); ++Index)
		{
			if (!GroomAsset->GetEnableSimulationCache())
			{
				GroomAsset->ValidateSimulationCache();
				break;
			}
		}
	}

	if (bAssetChanged)
	{
		if (GroomAsset)
		{
			// Set the callback on the new GroomAsset being assigned
			GroomAsset->GetOnGroomAssetChanged().AddUObject(this, &UGroomComponent::Invalidate);
			GroomAsset->GetOnGroomAssetResourcesChanged().AddUObject(this, &UGroomComponent::InvalidateAndRecreate);
			bIsGroomAssetCallbackRegistered = true;
		}
	}

	if (bBindingAssetChanged)
	{
		if (BindingAsset)
		{
			// Set the callback on the new GroomAsset being assigned
			BindingAsset->GetOnGroomBindingAssetChanged().AddUObject(this, &UGroomComponent::InvalidateAndRecreate);
			bIsGroomBindingAssetCallbackRegistered = true;
		}
	}
#endif

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairWidth) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairRootScale) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairTipScale) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairShadowDensity) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairRaytracingRadiusScale) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, LODBias) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, bUseStableRasterization) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, bScatterSceneLighting) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, HairLengthScale))
	{
		UpdateHairGroupsDescAndInvalidateRenderState();
		bIsEventProcess = true;
	}

	if (bIsUnknown)
	{
		InitResources(false);
	}

	// Always call parent PostEditChangeProperty as parent expect such a call (would crash in certain case otherwise)
	//if (!bIsEventProcess)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);
	}

#if WITH_EDITOR
	ValidateMaterials(false);
#endif
}

bool UGroomComponent::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		FString PropertyName = InProperty->GetName();

		if (FCString::Strcmp(*PropertyName, TEXT("HairRaytracingRadiusScale")) == 0)
		{
			bool bIsEditable = false;
			#if RHI_RAYTRACING
			if (IsHairRayTracingEnabled())
			{
				bIsEditable = true;
			}
			#endif
			return bIsEditable;
		}
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UGroomComponent, MeshDeformer))
		{
			if(GroomAsset)
			{
				for(int32 GroupIndex = 0, NumGroups = GroomAsset->GetNumHairGroups(); GroupIndex < NumGroups; ++GroupIndex)
				{
					if(GroomAsset->IsDeformationEnable(GroupIndex))
					{
						return true;
					}
				}
			}
			return false;
		}
	}

	return Super::CanEditChange(InProperty);
}
#endif

#if WITH_EDITOR

void UGroomComponent::ValidateMaterials(bool bMapCheck) const
{
	if (!GroomAsset)
		return;

	FString Name = "";
	if (GetOwner())
	{
		Name += GetOwner()->GetName() + "/";
	}
	Name += GetName() + "/" + GroomAsset->GetName();

	const EShaderPlatform ShaderPlatform = GetScene() ? GetScene()->GetShaderPlatform() : EShaderPlatform::SP_NumPlatforms;
	for (uint32 MaterialIt = 0, MaterialCount = GetNumMaterials(); MaterialIt < MaterialCount; ++MaterialIt)
	{
		// Do not fallback on default material, so that we can detect that a material is not valid, and we can emit warning/validation error for this material
		const EHairGeometryType GeometryType = GetMaterialGeometryType(MaterialIt);
		UMaterialInterface* OverrideMaterial = GetMaterial(MaterialIt, GeometryType, false);

		const EHairMaterialCompatibility Result = IsHairMaterialCompatible(OverrideMaterial, ShaderPlatform, GeometryType);
		switch (Result)
		{
			case EHairMaterialCompatibility::Invalid_UsedWithHairStrands:
			{
				if (bMapCheck)
				{
					FMessageLog("MapCheck").Warning()
						->AddToken(FUObjectToken::Create(GroomAsset))
						->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_HairStrandsMissingUseHairStrands", "Groom's material needs to enable the UseHairStrands option. Groom's material will be replaced with default hair strands shader in editor.")))
						->AddToken(FMapErrorToken::Create(FMapErrors::InvalidHairStrandsMaterial));
				}
				else
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - Groom's material needs to enable the UseHairStrands option. Groom's material will be replaced with default hair strands shader in editor.", *Name);
				}
			} break;
			case EHairMaterialCompatibility::Invalid_ShadingModel:
			{
				if (bMapCheck)
				{
					FMessageLog("MapCheck").Warning()
						->AddToken(FUObjectToken::Create(GroomAsset))
						->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_HairStrandsInvalidShadingModel", "Groom's material needs to have Hair shading model. Groom's material will be replaced with default hair strands shader in editor.")))
						->AddToken(FMapErrorToken::Create(FMapErrors::InvalidHairStrandsMaterial));
				}
				else
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - Groom's material needs to have Hair shading model. Groom's material will be replaced with default hair strands shader in editor.", *Name);
				}
			}break;
			case EHairMaterialCompatibility::Invalid_BlendMode:
			{
				if (bMapCheck)
				{
					FMessageLog("MapCheck").Warning()
						->AddToken(FUObjectToken::Create(GroomAsset))
						->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_HairStrandsInvalidBlendMode", "Groom's material needs to have Opaque blend mode. Groom's material will be replaced with default hair strands shader in editor.")))
						->AddToken(FMapErrorToken::Create(FMapErrors::InvalidHairStrandsMaterial));
				}
				else
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - Groom's material needs to have Opaque blend mode. Groom's material will be replaced with default hair strands shader in editor.", *Name);
				}
			}break;
			// Disable this warning as it is not really useful, and spams the log when editing parameters, since it is invoke on Invalidation & on PostEditChange
			#if 0
			case EHairMaterialCompatibility::Invalid_IsNull:
			{
				if (bMapCheck)
				{
					FMessageLog("MapCheck").Info()
						->AddToken(FUObjectToken::Create(GroomAsset))
						->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_HairStrandsMissingMaterial", "Groom's material is not set and will fallback on default hair strands shader in editor.")))
						->AddToken(FMapErrorToken::Create(FMapErrors::InvalidHairStrandsMaterial));
				}
				else
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] %ls - Groom's material is not set and will fallback on default hair strands shader in editor.", *Name);
				}
			}break;
			#endif
		}
	}
}

void UGroomComponent::CheckForErrors()
{
	Super::CheckForErrors();

	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	// Get the mesh owner's name.
	AActor* Owner = GetOwner();
	FString OwnerName(*(CoreTexts.None.ToString()));
	if (Owner)
	{
		OwnerName = Owner->GetName();
	}

	ValidateMaterials(true);
}

void UGroomComponent::PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel)
{

}

void UGroomComponent::HandlePlatformPreviewChanged(ERHIFeatureLevel::Type InFeatureLevel)
{
	HandlePlatformPreviewChanged(GShaderPlatformForFeatureLevel[InFeatureLevel]);
}

void UGroomComponent::HandleFeatureLevelChanged(ERHIFeatureLevel::Type InFeatureLevel)
{
	HandlePlatformPreviewChanged(GShaderPlatformForFeatureLevel[InFeatureLevel]);
}

void UGroomComponent::HandlePlatformPreviewChanged(EShaderPlatform InShaderPlatform)
{
	if (BindingAsset)
	{
		BindingAsset->ChangeShaderPlatform(InShaderPlatform);
	}
	if (GroomAsset)
	{
		GroomAsset->ChangeShaderPlatform(InShaderPlatform);
	}
	InvalidateAndRecreate();
}
#endif // WITH_EDITOR

template<typename T>
void InternalAddDedicatedVideoMemoryBytes(FResourceSizeEx& CumulativeResourceSize, T Resource)
{
	if (Resource)
	{
		CumulativeResourceSize.AddDedicatedVideoMemoryBytes(Resource->GetResourcesSize());
	}
}

void UGroomComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
	{
		InternalAddDedicatedVideoMemoryBytes(CumulativeResourceSize, Instance->Guides.DeformedResource);
		InternalAddDedicatedVideoMemoryBytes(CumulativeResourceSize, Instance->Guides.DeformedRootResource);

		InternalAddDedicatedVideoMemoryBytes(CumulativeResourceSize, Instance->Strands.DeformedResource);
		InternalAddDedicatedVideoMemoryBytes(CumulativeResourceSize, Instance->Strands.DeformedRootResource);

		for (const FHairGroupInstance::FCards::FLOD& LOD : Instance->Cards.LODs)
		{
			InternalAddDedicatedVideoMemoryBytes(CumulativeResourceSize, LOD.DeformedResource);
		}

		for (const FHairGroupInstance::FMeshes::FLOD& LOD : Instance->Meshes.LODs)
		{
			InternalAddDedicatedVideoMemoryBytes(CumulativeResourceSize, LOD.DeformedResource);
		}
	}
}

uint32 UGroomComponent::GetResourcesSize() const
{
	FGroomComponentMemoryStats Total;
	for (const TRefCountPtr<FHairGroupInstance>& Instance : HairGroupInstances)
	{
		Total.Accumulate(FGroomComponentMemoryStats::Get(Instance));
	}
	return Total.GetTotalSize();
}

/////////////////////////////////////////////////////////////////////////////////////////
// Asset dump function

void DumpLoadedGroomComponent(IConsoleVariable* InCVarPakTesterEnabled);

int32 GHairStrandsDump_GroomComponent = 0;
static FAutoConsoleVariableRef CVarHairStrandsDump_GroomComponent(
	TEXT("r.HairStrands.Dump.GroomComponent"),
	GHairStrandsDump_GroomComponent,
	TEXT("Dump information of all active groom components."),
	FConsoleVariableDelegate::CreateStatic(DumpLoadedGroomComponent));

void DumpLoadedGroomComponent(IConsoleVariable* InCVarPakTesterEnabled)
{
	const bool bDetails = true;
	const float ToMb = 1.f / 1000000.f;

	uint32 Total_Component = 0;
	uint32 Total_Group = 0;

	FGroomComponentMemoryStats TotalMemory;

	UE_LOGF(LogHairStrands, Log, "[Groom] ##### UGroomComponent #####");
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogHairStrands, Log, "--  No.  - LOD -          GPU (     Common|     Guides|    Strands|      Cards|     Meshes) - Asset Name ");
	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	FString OutputData;
	for (TObjectIterator<UGroomComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt)
		{
			const uint32 GroupCount = ComponentIt->GetGroupCount();
			for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
			{
				const FHairGroupInstance* Instance = ComponentIt->GetGroupInstance(GroupIt);
				const FGroomComponentMemoryStats InstanceMemory = FGroomComponentMemoryStats::Get(ComponentIt->GetGroupInstance(GroupIt));
				TotalMemory.Accumulate(InstanceMemory);
				
				if (bDetails)
				{
//					UE_LOGF(LogHairStrands, Log, "--  No.  - LOD -    GPU Total (    Common|     Guides|    Strands|      Cards|     Meshes) - Asset Name ");
//					UE_LOGF(LogHairStrands, Log, "-- 00/00 -  00 -  0000.0Mb (0000.0Mb |0000.0Mb|0000.0Mb |0000.0Mb) -  0000.0Mb (0000.0Mb|0000.0Mb|0000.0Mb|0000.0Mb) - AssetName");
					UE_LOGF(LogHairStrands, Log, "-- %2d/%2d -  %2d -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb) - %ls", 
						GroupIt,
						GroupCount,
                        Instance->Cards.LODs.Num(), 
                        InstanceMemory.GetTotalSize()* ToMb,
                        0.0f,
						InstanceMemory.Guides * ToMb,
						InstanceMemory.Strands* ToMb,
						InstanceMemory.Cards  * ToMb,
						InstanceMemory.Meshes * ToMb,
						GroupIt == 0 && ComponentIt->GroomAsset ? *ComponentIt->GroomAsset->GetPathName() : TEXT("."));
				}
			}

			Total_Component++;
			Total_Group += GroupCount;
		}
	}

	UE_LOGF(LogHairStrands, Log, "----------------------------------------------------------------------------------------------------------------------------------------------------------------");
	UE_LOGF(LogHairStrands, Log, "-- C:%3d|G:%3d -  %9.3fMb (%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb|%9.3fMb)",
		Total_Component,
		Total_Group,
        TotalMemory.GetTotalSize() * ToMb,
		0.0f,
		TotalMemory.Guides * ToMb,
		TotalMemory.Strands * ToMb,
		TotalMemory.Cards * ToMb,
		TotalMemory.Meshes * ToMb);
}

/////////////////////////////////////////////////////////////////////////////////////////

int32 GHairStrandsResetSimulationOnAllGroomComponents = 0;
void ResetSimulationOnAllGroomComponents(IConsoleVariable* InCVarPakTesterEnabled)
{
	if (GHairStrandsResetSimulationOnAllGroomComponents > 0)
	{
		for (TObjectIterator<UGroomComponent> HairStrandsComponentIt; HairStrandsComponentIt; ++HairStrandsComponentIt)
		{
			HairStrandsComponentIt->ResetSimulation();
		}
	}
	GHairStrandsResetSimulationOnAllGroomComponents = 0;
}

static FAutoConsoleVariableRef CVarHairStrandsResetSimulationOnAllGroomComponents(
	TEXT("r.HairStrands.Simulation.ResetAll"),
	GHairStrandsResetSimulationOnAllGroomComponents,
	TEXT("Reset hair strands simulation on all groom components."),
	FConsoleVariableDelegate::CreateStatic(ResetSimulationOnAllGroomComponents));

#if WITH_EDITORONLY_DATA
FGroomComponentRecreateRenderStateContext::FGroomComponentRecreateRenderStateContext(UGroomAsset* GroomAsset)
{
	if (!GroomAsset)
	{
		return;
	}

	for (TObjectIterator<UGroomComponent> HairStrandsComponentIt; HairStrandsComponentIt; ++HairStrandsComponentIt)
	{
		if (HairStrandsComponentIt->GroomAsset == GroomAsset ||
			HairStrandsComponentIt->GroomAssetBeingLoaded == GroomAsset) // A GroomAsset was set on the component while it was still loading
		{
			if (HairStrandsComponentIt->IsRenderStateCreated())
			{
				HairStrandsComponentIt->DestroyRenderState_Concurrent();
				GroomComponents.Add(*HairStrandsComponentIt);
			}
		}
	}

	if (GroomComponents.Num())
	{
		// Flush the rendering commands generated by the detachments.
		FlushRenderingCommands();
	}
}

FGroomComponentRecreateRenderStateContext::~FGroomComponentRecreateRenderStateContext()
{
	const int32 ComponentCount = GroomComponents.Num();
	for (int32 ComponentIndex = 0; ComponentIndex < ComponentCount; ++ComponentIndex)
	{
		UGroomComponent* GroomComponent = GroomComponents[ComponentIndex];

		if (GroomComponent->IsRegistered() && !GroomComponent->IsRenderStateCreated())
		{
			if (GroomComponent->GroomAssetBeingLoaded && GroomComponent->GroomAssetBeingLoaded->IsValid())
			{
				// Re-set the assets on the component now that they are loaded
				GroomComponent->SetGroomAsset(GroomComponent->GroomAssetBeingLoaded, GroomComponent->BindingAssetBeingLoaded);
			}
			else
			{
				GroomComponent->InitResources();
			}

			if (!GroomComponent->IsRenderStateCreated())
			{
				GroomComponent->CreateRenderState_Concurrent(nullptr);
			}
		}
	}
}
#endif

FGroomComponentMemoryStats FGroomComponentMemoryStats::Get(const FHairGroupInstance* In)
{
	FGroomComponentMemoryStats Out;
	Out.Guides  += In->Guides.DeformedResource ? In->Guides.DeformedResource->GetResourcesSize() : 0;
	Out.Guides  += In->Guides.DeformedRootResource ? In->Guides.DeformedRootResource->GetResourcesSize() : 0;
	Out.Strands += In->Strands.DeformedResource ? In->Strands.DeformedResource->GetResourcesSize() : 0;
	Out.Strands += In->Strands.DeformedRootResource ? In->Strands.DeformedRootResource->GetResourcesSize() : 0;
	Out.Strands += In->Strands.CullingResource ? In->Strands.CullingResource->GetResourcesSize() : 0;
#if RHI_RAYTRACING
	Out.Strands += (In->Strands.RenRaytracingResourceOwned && In->Strands.RenRaytracingResource) ? In->Strands.RenRaytracingResource->GetResourcesSize() : 0;
#endif
	for (const FHairGroupInstance::FCards::FLOD& LOD : In->Cards.LODs)
	{
		Out.Cards += LOD.DeformedResource ? LOD.DeformedResource->GetResourcesSize() : 0;
	}
	for (const FHairGroupInstance::FMeshes::FLOD& LOD : In->Meshes.LODs)
	{
		Out.Meshes += LOD.DeformedResource ? LOD.DeformedResource->GetResourcesSize() : 0;
	}
	return Out;
}

void FGroomComponentMemoryStats::Accumulate(const FGroomComponentMemoryStats& In)
{
	Guides += In.Guides ;
	Strands+= In.Strands;
	Cards  += In.Cards  ;
	Meshes += In.Meshes ;
}
uint32 FGroomComponentMemoryStats::GetTotalSize() const
{
	uint32 Out = 0;
	Out += Guides ;
	Out += Strands;
	Out += Cards  ;
	Out += Meshes ;
	return Out;
}
#undef LOCTEXT_NAMESPACE

