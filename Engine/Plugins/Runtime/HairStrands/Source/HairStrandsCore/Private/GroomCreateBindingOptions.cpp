// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCreateBindingOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomCreateBindingOptions)

UGroomCreateBindingOptions::UGroomCreateBindingOptions(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	GroomBindingType = EGroomBindingMeshType::SkeletalMesh;
	SourceSkeletalMesh = nullptr;
	TargetSkeletalMesh = nullptr;
	SourceGeometryCache = nullptr;
	TargetGeometryCache = nullptr;
	NumInterpolationPoints = 100;
	MatchingSection = 0;
	TargetBindingAttribute = NAME_None;
}