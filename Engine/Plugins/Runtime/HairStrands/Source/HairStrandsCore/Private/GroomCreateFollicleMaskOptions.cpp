// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCreateFollicleMaskOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomCreateFollicleMaskOptions)

UGroomCreateFollicleMaskOptions::UGroomCreateFollicleMaskOptions(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	
}

