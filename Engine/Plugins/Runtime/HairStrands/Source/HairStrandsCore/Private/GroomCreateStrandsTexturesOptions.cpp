// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCreateStrandsTexturesOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomCreateStrandsTexturesOptions)

UGroomCreateStrandsTexturesOptions::UGroomCreateStrandsTexturesOptions(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	
}

