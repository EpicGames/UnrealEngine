// Copyright Epic Games, Inc. All Rights Reserved. 

#include "GeometryCacheComponent.h"
#include "GroomInstance.h"
#include "GroomManager.h"
#include "HairStrandsMeshProjection.h"
#include "HairStrandsInterface.h"
#include "CommonRenderResources.h"
#include "CachedGeometry.h"
#include "GroomGeometryCache.h"
#include "Components/SkeletalMeshComponent.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "SkeletalRenderPublic.h"
#include "UnrealEngine.h"
#include "SystemTextures.h"
#include "CanvasTypes.h"
#include "ShaderCompilerCore.h"
#include "GroomVisualizationData.h"
#include "GroomCacheData.h"
#include "HairStrandsClusterCulling.h"
#include "ShaderPrintParameters.h"
#include "GroomComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HairStrandsDefinitions.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

static int32 GHairDebugMeshProjection_SkinCacheMeshInUVsSpace = 0;

static int32 GHairDebugMeshProjection_Override = 0;
static int32 GHairDebugMeshProjection_Sim_HairRestTriangles = 0;
static int32 GHairDebugMeshProjection_Sim_HairRestFrames = 0;
static int32 GHairDebugMeshProjection_Sim_HairRestSamples = 0;
static int32 GHairDebugMeshProjection_Sim_HairDeformedTriangles = 0;
static int32 GHairDebugMeshProjection_Sim_HairDeformedFrames = 0;
static int32 GHairDebugMeshProjection_Sim_HairDeformedSamples = 0;

static int32 GHairDebugMeshProjection_Render_HairRestTriangles = 0;
static int32 GHairDebugMeshProjection_Render_HairRestFrames = 0;
static int32 GHairDebugMeshProjection_Render_HairRestSamples = 0;
static int32 GHairDebugMeshProjection_Render_HairDeformedTriangles = 0;
static int32 GHairDebugMeshProjection_Render_HairDeformedFrames = 1;
static int32 GHairDebugMeshProjection_Render_HairDeformedSamples = 0;


static FAutoConsoleVariableRef CVarHairDebugMeshProjection_SkinCacheMeshInUVsSpace(TEXT("r.HairStrands.MeshProjection.DebugInUVsSpace"), GHairDebugMeshProjection_SkinCacheMeshInUVsSpace, TEXT("Render debug mes projection in UVs space"));

static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Override(TEXT("r.HairStrands.MeshProjection"), GHairDebugMeshProjection_Override, TEXT("Override in shader settings for displaying root debug data"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Render_HairRestTriangles(TEXT("r.HairStrands.MeshProjection.Render.Rest.Triangles"), GHairDebugMeshProjection_Render_HairRestTriangles, TEXT("Render strands rest triangles"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Render_HairRestFrames(TEXT("r.HairStrands.MeshProjection.Render.Rest.Frames"), GHairDebugMeshProjection_Render_HairRestFrames, TEXT("Render strands rest frames"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Render_HairDeformedTriangles(TEXT("r.HairStrands.MeshProjection.Render.Deformed.Triangles"), GHairDebugMeshProjection_Render_HairDeformedTriangles, TEXT("Render strands deformed triangles"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Render_HairDeformedFrames(TEXT("r.HairStrands.MeshProjection.Render.Deformed.Frames"), GHairDebugMeshProjection_Render_HairDeformedFrames, TEXT("Render strands deformed frames"));

static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Sim_HairRestTriangles(TEXT("r.HairStrands.MeshProjection.Sim.Rest.Triangles"), GHairDebugMeshProjection_Sim_HairRestTriangles, TEXT("Render guides rest triangles"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Sim_HairRestFrames(TEXT("r.HairStrands.MeshProjection.Sim.Rest.Frames"), GHairDebugMeshProjection_Sim_HairRestFrames, TEXT("Render guides rest frames"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Sim_HairRestSamples(TEXT("r.HairStrands.MeshProjection.Sim.Rest.Samples"), GHairDebugMeshProjection_Sim_HairRestSamples, TEXT("Render guides rest samples"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Sim_HairDeformedTriangles(TEXT("r.HairStrands.MeshProjection.Sim.Deformed.Triangles"), GHairDebugMeshProjection_Sim_HairDeformedTriangles, TEXT("Render guides deformed triangles"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Sim_HairDeformedFrames(TEXT("r.HairStrands.MeshProjection.Sim.Deformed.Frames"), GHairDebugMeshProjection_Sim_HairDeformedFrames, TEXT("Render guides deformed frames"));
static FAutoConsoleVariableRef CVarHairDebugMeshProjection_Sim_HairDeformedSamples(TEXT("r.HairStrands.MeshProjection.Sim.Deformed.Samples"), GHairDebugMeshProjection_Sim_HairDeformedSamples, TEXT("Render guides deformed samples"));

static int32 GHairCardsAtlasDebug = 0;
static FAutoConsoleVariableRef CVarHairCardsAtlasDebug(TEXT("r.HairStrands.Cards.DebugAtlas"), GHairCardsAtlasDebug, TEXT("Draw debug hair cards atlas."));

///////////////////////////////////////////////////////////////////////////////////////////////////

enum class HairStrandsTriangleType
{
	RestPose,
	DeformedPose,
};

static void AddShaderPrintDebugFlag(EShaderPlatform InPlatform, FShaderCompilerEnvironment& OutEnvironment)
{
	const bool bNeedDebugFlags = FDataDrivenShaderPlatformInfo::GetIsLanguageNintendo(InPlatform);
	if (bNeedDebugFlags)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairProjectionHairDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairProjectionHairDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FHairProjectionHairDebugCS, FGlobalShader);	

	class FInputType : SHADER_PERMUTATION_INT("PERMUTATION_INPUT_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FInputType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(uint32, MaxRootCount)
		SHADER_PARAMETER(uint32, DeformedFrameEnable)
		SHADER_PARAMETER(FMatrix44f, RootLocalToWorld)
		SHADER_PARAMETER(uint32, DrawIndex)
		SHADER_PARAMETER(uint32, bSimPass)
		SHADER_PARAMETER(uint32, bRestPass)

		SHADER_PARAMETER(uint32, bOverride)
		SHADER_PARAMETER(uint32, Render_HairRestTriangles)
		SHADER_PARAMETER(uint32, Render_HairRestFrames)
		SHADER_PARAMETER(uint32, Render_HairDeformedTriangles)
		SHADER_PARAMETER(uint32, Render_HairDeformedFrames)
		SHADER_PARAMETER(uint32, Sim_HairRestTriangles)
		SHADER_PARAMETER(uint32, Sim_HairRestFrames)
		SHADER_PARAMETER(uint32, Sim_HairRestSamples)
		SHADER_PARAMETER(uint32, Sim_HairDeformedTriangles)
		SHADER_PARAMETER(uint32, Sim_HairDeformedFrames)
		SHADER_PARAMETER(uint32, Sim_HairDeformedSamples)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, RestPositionOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, DeformedPositionOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, DeformedPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, RestSamplePositionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeformedSamplePositionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootBarycentricBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootToUniqueTriangleIndexBuffer)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_MESH_PROJECTION_HAIR"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairProjectionHairDebugCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

enum class EDebugProjectionHairType
{
	HairRoots,
	HairSamples,
};

static void AddDebugProjectionHairPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	FGlobalShaderMap* ShaderMap,
	const FShaderPrintData* ShaderPrintData,
	FIntRect Viewport,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
	const int32 MeshLODIndex,
	const FHairStrandsRestResource* RestResources,
	const FHairStrandsDeformedResource* DeformedResources,
	const FHairStrandsRestRootResource* RestRootResources,
	const FHairStrandsDeformedRootResource* DeformedRootResources,
	const FTransform& LocalToWorld,
	const uint32 DrawIndex,
	const bool bSim,
	const EDebugProjectionHairType GeometryType,
	const HairStrandsTriangleType PoseType)
{
	if (!View || !ShaderPrint::IsSupported(View->GetShaderPlatform()))
	{
		return;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	if (!RestRootResources->IsValid(MeshLODIndex) || !DeformedRootResources->IsValid(MeshLODIndex))
	{
		return;
	}
	
	const uint32 RootCount = EDebugProjectionHairType::HairSamples == GeometryType ? 3 * RestRootResources->GetLOD(MeshLODIndex)->SampleCount : RestRootResources->GetRootCount();
	const uint32 PrimitiveCount = RootCount;

	if (PrimitiveCount == 0)
	{
		return;
	}

	if (ShaderPrintData == nullptr || !ShaderPrint::IsEnabled(*ShaderPrintData))
	{
		return;
	}

	ShaderPrint::RequestSpaceForLines(PrimitiveCount * 10 /* 3+1 hair root + 3 triangles + 3 normals */);
	ShaderPrint::RequestSpaceForTriangles(PrimitiveCount * 3);

	if (EDebugProjectionHairType::HairRoots == GeometryType &&
		!RestRootResources->GetLOD(MeshLODIndex)->RootBarycentricBuffer.Buffer)
	{
		return;
	}

	if (EDebugProjectionHairType::HairSamples == GeometryType &&
		!RestRootResources->GetLOD(MeshLODIndex)->RestSamplePositionsBuffer.Buffer)
	{
		return;
	}

	const FHairStrandsLODRestRootResource& RestLODDatas = *RestRootResources->GetLOD(MeshLODIndex);
	const FHairStrandsLODDeformedRootResource& DeformedLODDatas = *DeformedRootResources->GetLOD(MeshLODIndex);

	if (!RestLODDatas.RestUniqueTrianglePositionBuffer.Buffer || !DeformedLODDatas.DeformedUniqueTrianglePositionBuffer[0].Buffer)
	{
		return;
	}

	// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
	if (IsHairStrandContinuousDecimationReorderingEnabled() && !DeformedLODDatas.DeformedUniqueTrianglePositionBuffer[1].Buffer)
	{
		return;
	}

	const FIntPoint Resolution(Viewport.Width(), Viewport.Height());

	FHairProjectionHairDebugCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairProjectionHairDebugCS::FParameters>();
	Parameters->OutputResolution 			= Resolution;
	Parameters->MaxRootCount 				= RootCount;
	Parameters->RootLocalToWorld 			= FMatrix44f(LocalToWorld.ToMatrixWithScale());	// LWC_TODO: Precision loss
	Parameters->DeformedFrameEnable 		= PoseType == HairStrandsTriangleType::DeformedPose;
	Parameters->DrawIndex					= DrawIndex;
	Parameters->bSimPass 					= bSim ? 1u : 0u;
	Parameters->bRestPass 					= PoseType == HairStrandsTriangleType::RestPose ? 1u : 0u;
	Parameters->bOverride 					= GHairDebugMeshProjection_Override > 0 ? 1u : 0u;
	Parameters->Render_HairRestTriangles	= GHairDebugMeshProjection_Render_HairRestTriangles > 0 ? 1u : 0u;
	Parameters->Render_HairRestFrames 		= GHairDebugMeshProjection_Render_HairRestFrames > 0 ? 1u : 0u;
	Parameters->Render_HairDeformedTriangles= GHairDebugMeshProjection_Render_HairDeformedTriangles > 0 ? 1u : 0u;
	Parameters->Render_HairDeformedFrames	= GHairDebugMeshProjection_Render_HairDeformedFrames > 0 ? 1u : 0u;
	Parameters->Sim_HairRestTriangles		= GHairDebugMeshProjection_Sim_HairRestTriangles > 0 ? 1u : 0u;
	Parameters->Sim_HairRestFrames			= GHairDebugMeshProjection_Sim_HairRestFrames > 0 ? 1u : 0u;
	Parameters->Sim_HairRestSamples			= GHairDebugMeshProjection_Sim_HairRestSamples > 0 ? 1u : 0u;
	Parameters->Sim_HairDeformedTriangles	= GHairDebugMeshProjection_Sim_HairDeformedTriangles > 0 ? 1u : 0u;
	Parameters->Sim_HairDeformedFrames		= GHairDebugMeshProjection_Sim_HairDeformedFrames > 0 ? 1u : 0u;
	Parameters->Sim_HairDeformedSamples		= GHairDebugMeshProjection_Sim_HairDeformedSamples > 0 ? 1u : 0u;

	if (EDebugProjectionHairType::HairRoots == GeometryType)
	{
		Parameters->RootBarycentricBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RootBarycentricBuffer);
	}

	Parameters->RootToUniqueTriangleIndexBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RootToUniqueTriangleIndexBuffer);
	
	Parameters->RestPositionOffset = RegisterAsSRV(GraphBuilder, RestResources->PositionOffsetBuffer);	
	Parameters->DeformedPositionOffset = RegisterAsSRV(GraphBuilder, DeformedResources->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current));

	Parameters->RestPositionBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RestUniqueTrianglePositionBuffer);
	Parameters->DeformedPositionBuffer = RegisterAsSRV(GraphBuilder, DeformedLODDatas.GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current));

	Parameters->RestSamplePositionsBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RestSamplePositionsBuffer);
	Parameters->DeformedSamplePositionsBuffer = RegisterAsSRV(GraphBuilder, DeformedLODDatas.GetDeformedSamplePositionsBuffer(FHairStrandsLODDeformedRootResource::Current));

	Parameters->ViewUniformBuffer = ViewUniformBuffer;
	ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintUniformBuffer);

	FHairProjectionHairDebugCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairProjectionHairDebugCS::FInputType>(GeometryType == EDebugProjectionHairType::HairRoots ? 0 : 1);

	TShaderMapRef<FHairProjectionHairDebugCS> ComputeShader(ShaderMap, PermutationVector);
	ClearUnusedGraphResources(ComputeShader, Parameters);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::MeshProjectionDebug(Hair, %s)", GeometryType == EDebugProjectionHairType::HairRoots ? TEXT("Root") : TEXT("Samples")),
		ComputeShader,
		Parameters,
		FIntVector(FMath::DivideAndRoundUp(PrimitiveCount, 256u), 1, 1));
}	

///////////////////////////////////////////////////////////////////////////////////////////////////

class FDrawDebugClusterAABBCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawDebugClusterAABBCS);
	SHADER_USE_PARAMETER_STRUCT(FDrawDebugClusterAABBCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ClusterAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, GroupAABBBuffer)
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, ClusterOffset)
		SHADER_PARAMETER(uint32, ClusterCount)
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(uint32, HairGroupId)
		SHADER_PARAMETER(uint32, bDrawAABB)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 64; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLUSTERAABB"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
		// Skip optimization for avoiding long compilation time due to large UAV writes
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDrawDebugClusterAABBCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainDrawDebugAABBCS", SF_Compute);

void AddDrawDebugClusterPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	FHairTransientResources& TransientResources,
	const FShaderPrintData* ShaderPrintData,
	EGroomViewMode ViewMode,
	FHairStrandClusterData& HairClusterData)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	const uint32 ClusterCount = TransientResources.ClusterAABBSRV ? TransientResources.ClusterAABBSRV->Desc.Buffer->Desc.NumElements / 6u : 16000u;
	const uint32 InstanceCount = TransientResources.bIsGroupAABBValid.Num();
	const uint32 LineCount = ClusterCount * 16 + InstanceCount * 16u;

	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForLines(LineCount);
	ShaderPrint::RequestSpaceForCharacters(32 * InstanceCount + 2048);
	if (!ShaderPrintData) { return; }

	const bool bDebugAABB = ViewMode == EGroomViewMode::ClusterAABB;

	// Sort cluster group by registered index to get stable listing
	TArray<const FHairStrandClusterData::FHairGroup*> Groups;
	Groups.Reserve(HairClusterData.HairGroups.Num());
	for (const FHairStrandClusterData::FHairGroup& HairGroupClusters : HairClusterData.HairGroups)
	{
		Groups.Add(&HairGroupClusters);
	}
	Groups.Sort([](const FHairStrandClusterData::FHairGroup& A, const FHairStrandClusterData::FHairGroup& B) 
	{ 
		return A.InstanceRegisteredIndex < B.InstanceRegisteredIndex;
	});

	uint32 DataIndex = 0;
	for (const FHairStrandClusterData::FHairGroup* Group : Groups)
	{
		TShaderMapRef<FDrawDebugClusterAABBCS> ComputeShader(ShaderMap);

		FDrawDebugClusterAABBCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawDebugClusterAABBCS::FParameters>();
		Parameters->InstanceRegisteredIndex = Group->InstanceRegisteredIndex;
		Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
		Parameters->ClusterOffset = TransientResources.GetClusterOffset(Group->InstanceRegisteredIndex);
		Parameters->ClusterCount  = TransientResources.GetClusterCount(Group->InstanceRegisteredIndex);
		Parameters->PointCount = Group->HairGroupPublicPtr->GetActiveStrandsPointCount();
		Parameters->CurveCount = Group->HairGroupPublicPtr->GetActiveStrandsCurveCount();
		Parameters->HairGroupId = DataIndex++;
		Parameters->bDrawAABB = ViewMode == EGroomViewMode::ClusterAABB ? 1 : 0;
		Parameters->ClusterAABBBuffer = TransientResources.ClusterAABBSRV;
		Parameters->GroupAABBBuffer = TransientResources.GroupAABBSRV;
		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintParameters);

		const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(Parameters->ClusterCount, 1, 1), FIntVector(FDrawDebugClusterAABBCS::GetGroupSize(), 1, 1));
		FComputeShaderUtils::AddPass(
			GraphBuilder, RDG_EVENT_NAME("HairStrands::DrawDebugClusterAABB"),
			ComputeShader, Parameters, DispatchCount);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FDrawDebugCardAtlasCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawDebugCardAtlasCS);
	SHADER_USE_PARAMETER_STRUCT(FDrawDebugCardAtlasCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_TEXTURE(Texture2D, AtlasTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		SHADER_PARAMETER(int32, DebugMode)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Cards, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_CARDS_ATLAS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDrawDebugCardAtlasCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddDrawDebugCardsAtlasPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	const FHairGroupInstance* Instance,
	const FShaderPrintData* ShaderPrintData,
	FRDGTextureRef SceneColorTexture)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	if (Instance->HairGroupPublicData->VFInput.GeometryType != EHairGeometryType::Cards || ShaderPrintData == nullptr)
	{
		return;
	}

	const int32 LODIndex = Instance->HairGroupPublicData->GetIntLODIndex();
	if (!Instance->Cards.IsValid(LODIndex))
	{
		return;
	}

	const int32 DebugMode = FMath::Clamp(GHairCardsAtlasDebug, 1, Instance->Cards.LODs[LODIndex].RestResource->Textures.Num()-1);
	FTextureReferenceRHIRef AtlasTexture = Instance->Cards.LODs[LODIndex].RestResource->Textures.IsValidIndex(DebugMode) ? Instance->Cards.LODs[LODIndex].RestResource->Textures[DebugMode] : nullptr;

	if (AtlasTexture != nullptr)
	{
		TShaderMapRef<FDrawDebugCardAtlasCS> ComputeShader(ShaderMap);

		FDrawDebugCardAtlasCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawDebugCardAtlasCS::FParameters>();
		Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
		Parameters->OutputResolution = SceneColorTexture->Desc.Extent;
		Parameters->AtlasResolution = FIntPoint(AtlasTexture->GetSizeXYZ().X, AtlasTexture->GetSizeXYZ().Y);
		Parameters->AtlasTexture = AtlasTexture;
		Parameters->DebugMode = DebugMode;
		Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters->OutputTexture = GraphBuilder.CreateUAV(SceneColorTexture);

		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintParameters);

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::DrawDebugCardsAtlas"), ComputeShader, Parameters,
		FIntVector::DivideAndRoundUp(FIntVector(Parameters->OutputResolution.X, Parameters->OutputResolution.Y, 1), FIntVector(8, 8, 1)));

	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FDrawDebugStrandsCVsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawDebugStrandsCVsCS);
	SHADER_USE_PARAMETER_STRUCT(FDrawDebugStrandsCVsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(uint32, MaxVertexCount)
		SHADER_PARAMETER(uint32, InstanceIt)
		SHADER_PARAMETER(FVector3f, GroupColor)
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ColorTexture)
		SHADER_PARAMETER_STRUCT_REF(FHairStrandsVertexFactoryUniformShaderParameters, HairStrandsVF)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CVS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDrawDebugStrandsCVsCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

const FLinearColor GetHairGroupDebugColor(int32 GroupIt);

static void AddDrawDebugStrandsCVsPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	const FHairGroupInstance* Instance,
	const FShaderPrintData* ShaderPrintData,
	const uint32 InIndex,
	FRDGTextureRef ColorTexture,
	FRDGTextureRef DepthTexture)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForLines(256);
	ShaderPrint::RequestSpaceForCharacters(256);

	if (Instance->HairGroupPublicData->VFInput.GeometryType != EHairGeometryType::Strands || ShaderPrintData == nullptr)
	{
		return;
	}

	if (!Instance->Strands.IsValid())
	{
		return;
	}

	TShaderMapRef<FDrawDebugStrandsCVsCS> ComputeShader(ShaderMap);
	FDrawDebugStrandsCVsCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawDebugStrandsCVsCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->HairStrandsVF = Instance->Strands.UniformBuffer;
	Parameters->LocalToWorld = FMatrix44f(Instance->LocalToWorld.ToMatrixWithScale());		// LWC_TODO: Precision loss // TODO change this to Uniform buffer parameters..
	Parameters->MaxVertexCount = Instance->Strands.GetData().GetNumPoints();
	Parameters->ColorTexture = GraphBuilder.CreateUAV(ColorTexture);
	Parameters->DepthTexture = DepthTexture;
	Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->InstanceIt = InIndex;
	Parameters->GroupColor = FVector3f(GetHairGroupDebugColor(Instance->Debug.GroupIndex));

	ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintParameters);

	const uint32 PointCount = Instance->HairGroupPublicData->VFInput.Strands.Common.PointCount;
	FComputeShaderUtils::AddPass(
		GraphBuilder, 
		RDG_EVENT_NAME("HairStrands::DrawCVs"), 
		ComputeShader, 
		Parameters,
		FIntVector::DivideAndRoundUp(FIntVector(PointCount, 1, 1), FIntVector(256, 1, 1)));
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FDrawDebugCardGuidesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawDebugCardGuidesCS);
	SHADER_USE_PARAMETER_STRUCT(FDrawDebugCardGuidesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, CardLODIndex)
		SHADER_PARAMETER(uint32, DebugMode)
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		
		SHADER_PARAMETER(uint32,  RenVertexCount)
		SHADER_PARAMETER(FVector3f, RenRestOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, RenDeformedOffset)
		
		SHADER_PARAMETER(uint32,  SimVertexCount)
		SHADER_PARAMETER(FVector3f, SimRestOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, SimDeformedOffset)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenRestPosition)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenDeformedPosition)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SimRestPosition)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SimDeformedPosition)

		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Cards, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_CARDS_GUIDE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDrawDebugCardGuidesCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddDrawDebugCardsGuidesPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	const FHairGroupInstance* Instance,
	const FShaderPrintData* ShaderPrintData,
	const bool bDeformed, 
	const bool bRen)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	if (ShaderPrintData == nullptr || !ShaderPrint::IsEnabled(*ShaderPrintData))
	{
		return;
	}

	if (Instance->HairGroupPublicData->VFInput.GeometryType != EHairGeometryType::Cards)
	{
		return;
	}

	const int32 HairLODIndex = Instance->HairGroupPublicData->GetIntLODIndex();
	if (!Instance->Cards.IsValid(HairLODIndex))
	{
		return;
	}

	const FHairGroupInstance::FCards::FLOD& LOD = Instance->Cards.LODs[HairLODIndex];
	if (!LOD.Guides.RestResource)
	{
		return;
	}
	
	const uint32 MaxCount = FMath::Max(
		Instance->Guides.RestResource ? Instance->Guides.RestResource->GetPointCount() * 2 : 0,
		LOD.Guides.RestResource ? LOD.Guides.RestResource->GetPointCount() * 2 : 0);
	ShaderPrint::RequestSpaceForLines(MaxCount);

	TShaderMapRef<FDrawDebugCardGuidesCS> ComputeShader(ShaderMap);
	const bool bGuideValid			= Instance->Guides.RestResource != nullptr && Instance->Guides.NeedsGuides();
	const bool bGuideDeformValid	= Instance->Guides.DeformedResource != nullptr;
	const bool bRenderValid			= LOD.Guides.RestResource != nullptr;
	const bool bRenderDeformValid	= LOD.Guides.DeformedResource != nullptr;
	if (bRen  && !bRenderValid)						{ return; }
	if (bRen  && bDeformed && !bRenderDeformValid)	{ return; }
	if (!bRen && !bGuideValid)						{ return; }
	if (!bRen && bDeformed && !bGuideDeformValid)	{ return; }

	FRDGBufferSRVRef DefaultByteAddreeBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 8u));
	FRDGBufferSRVRef DefaultBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 8, 0u), PF_R16G16B16A16_UINT);

	FDrawDebugCardGuidesCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawDebugCardGuidesCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;

	Parameters->InstanceRegisteredIndex = Instance->RegisteredIndex;
	Parameters->CardLODIndex = HairLODIndex;
	Parameters->RenVertexCount = 0;
	Parameters->RenRestOffset = FVector3f::ZeroVector;
	Parameters->RenRestPosition = DefaultByteAddreeBuffer;
	Parameters->RenDeformedOffset = DefaultBuffer;
	Parameters->RenDeformedPosition = DefaultByteAddreeBuffer;

	Parameters->SimVertexCount = 0;
	Parameters->SimRestOffset = FVector3f::ZeroVector;
	Parameters->SimRestPosition = DefaultByteAddreeBuffer;
	Parameters->SimDeformedOffset = DefaultBuffer;
	Parameters->SimDeformedPosition = DefaultByteAddreeBuffer;

	if (bRen)
	{
		Parameters->RenVertexCount = LOD.Guides.RestResource->GetPointCount();
		Parameters->RenRestOffset = (FVector3f)LOD.Guides.RestResource->GetPositionOffset();
		Parameters->RenRestPosition = RegisterAsSRV(GraphBuilder, LOD.Guides.RestResource->PositionBuffer);
		if (bDeformed)
		{
			Parameters->RenDeformedOffset = RegisterAsSRV(GraphBuilder, LOD.Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current));
			Parameters->RenDeformedPosition = RegisterAsSRV(GraphBuilder, LOD.Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current));
		}
	}

	if (!bRen)
	{
		Parameters->SimVertexCount = Instance->Guides.RestResource->GetPointCount();
		Parameters->SimRestOffset = (FVector3f)Instance->Guides.RestResource->GetPositionOffset();
		Parameters->SimRestPosition = RegisterAsSRV(GraphBuilder, Instance->Guides.RestResource->PositionBuffer);
		if (bDeformed)
		{
			Parameters->SimDeformedOffset = RegisterAsSRV(GraphBuilder, Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current));
			Parameters->SimDeformedPosition = RegisterAsSRV(GraphBuilder, Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current));
		}
	}

	Parameters->LocalToWorld = FMatrix44f(Instance->LocalToWorld.ToMatrixWithScale());		// LWC_TODO: Precision loss

	const TCHAR* DebugName = TEXT("Unknown");
	if (!bDeformed &&  bRen) { Parameters->DebugMode = 1; DebugName = TEXT("Ren, Rest"); } 
	if ( bDeformed &&  bRen) { Parameters->DebugMode = 2; DebugName = TEXT("Ren, Deformed"); } 
	if (!bDeformed && !bRen) { Parameters->DebugMode = 3; DebugName = TEXT("Sim, Rest"); } 
	if ( bDeformed && !bRen) { Parameters->DebugMode = 4; DebugName = TEXT("Sim, Deformed"); } 

	ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintParameters);

	const uint32 VertexCount = Parameters->DebugMode <= 2 ? Parameters->RenVertexCount : Parameters->SimVertexCount;
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::DebugCards(%s)", DebugName), ComputeShader, Parameters,
	FIntVector::DivideAndRoundUp(FIntVector(VertexCount, 1, 1), FIntVector(32, 1, 1)));
}

///////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SHADER_PARAMETER_STRUCT(FHairDebugCanvasParameter, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static const TCHAR* ToString(EHairGeometryType In)
{
	switch (In)
	{
	case EHairGeometryType::NoneGeometry:	return TEXT("None");
	case EHairGeometryType::Strands:		return TEXT("Strands");
	case EHairGeometryType::Cards:			return TEXT("Cards");
	case EHairGeometryType::Meshes:			return TEXT("Meshes");
	}
	return TEXT("None");
}

static const TCHAR* ToString(EHairBindingType In)
{
	switch (In)
	{
	case EHairBindingType::NoneBinding: return TEXT("None");
	case EHairBindingType::Rigid:		return TEXT("Rigid");
	case EHairBindingType::Skinning:	return TEXT("Skinning");
	}
	return TEXT("None");
}

static const TCHAR* ToString(EHairLODSelectionType In)
{
	switch (In)
	{
	case EHairLODSelectionType::Immediate:	return TEXT("Immed");
	case EHairLODSelectionType::Predicted:	return TEXT("Predic");
	case EHairLODSelectionType::Forced:		return TEXT("Forced");
	}
	return TEXT("None");
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairDebugPrintInstanceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDebugPrintInstanceCS);
	SHADER_USE_PARAMETER_STRUCT(FHairDebugPrintInstanceCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceCount)
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, InstanceCount_StrandsPrimaryView)
		SHADER_PARAMETER(uint32, InstanceCount_StrandsShadowView)
		SHADER_PARAMETER(uint32, InstanceCount_CardsOrMeshesPrimaryView)
		SHADER_PARAMETER(uint32, InstanceCount_CardsOrMeshesShadowView)
		SHADER_PARAMETER(FVector4f, InstanceScreenSphereBound)
		SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderParameters, InstanceNames)
		SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderParameters, AttributeNames)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, Infos)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, InstanceAABB)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		AddShaderPrintDebugFlag(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_PRINT_INSTANCE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDebugPrintInstanceCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

bool UseHairStrandsForceAutoLOD();
const TCHAR* GetHairAttributeText(EHairAttribute In, uint32 InFlags);
uint32 GetHairAttributeIndex(EHairAttribute In);
EGroomCacheType GetHairInstanceCacheType(const FHairGroupInstance* Instance);

static void AddHairDebugPrintInstancePass(
	FRDGBuilder& GraphBuilder, 
	FGlobalShaderMap* ShaderMap,
	const FSceneView& View,
	FHairTransientResources& TransientResources,
	const FShaderPrintData* ShaderPrintData,
	const FHairStrandsInstances& Instances,
	const FHairInstanceCullingResults& CullingResults)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	const uint32 InstanceCount = Instances.Num();

	// Compute instace count per visibility types
	TArray<uint32> InstanceCountPerType;
	InstanceCountPerType.Init(0u, uint32(EHairInstanceVisibilityType::Count));
	for (uint32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
	{
		const EHairInstanceVisibilityType VisibilityType = CullingResults.Is(View, *Instances[InstanceIndex]);
		InstanceCountPerType[uint32(VisibilityType)]++;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	// Request more drawing primitives & characters for printing if needed	
	ShaderPrint::RequestSpaceForLines(InstanceCount * 64u);
	ShaderPrint::RequestSpaceForCharacters(InstanceCount * 256 + 512);

	if (!ShaderPrintData || InstanceCount == 0) { return; }


	ShaderPrint::FStrings InstanceNames(InstanceCount);
	struct FInstanceInfos
	{
		FUintVector4 Data0 = { 0,0,0,0 };
		FUintVector4 Data1 = { 0,0,0,0 };
		FUintVector4 Data2 = { 0,0,0,0 };
		FUintVector4 Data3 = { 0,0,0,0 };
		FUintVector4 Data4 = { 0,0,0,0 };
	};
	TArray<FInstanceInfos> Infos;
	Infos.Reserve(InstanceCount);
	for (uint32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
	{
		const FHairGroupInstance* Instance = static_cast<const FHairGroupInstance*>(Instances[InstanceIndex].GetReference());

		// Collect InstanceNames
		InstanceNames.Add(*Instance->Debug.GroomAssetName, InstanceIndex);

		const float LODIndex = Instance->HairGroupPublicData->LODIndex;
		const uint32 IntLODIndex = Instance->HairGroupPublicData->LODIndex;
		const uint32 LODCount = Instance->HairGroupPublicData->GetLODScreenSizes().Num();
		const EGroomCacheType ActiveGroomCacheType = GetHairInstanceCacheType(Instance);
		const EHairInstanceVisibilityType VisibilityType = CullingResults.Is(View,*Instance);

		FInstanceInfos& D = Infos.AddDefaulted_GetRef();
		D.Data0.X =
			((Instance->Debug.GroupIndex & 0xFF)) |
			((Instance->Debug.GroupCount & 0xFF) << 8) |
			((LODCount & 0xFF) << 16) |
			((uint32(Instance->GeometryType) & 0x7)<<24) |
			((uint32(Instance->BindingType) & 0x7)<<27) |
			((Instance->Guides.bIsSimulationEnable ? 0x1 : 0x0) << 30) |
			((Instance->Guides.bHasGlobalInterpolation ? 0x1 : 0x0) << 31);

		D.Data0.Y =
			(FFloat16(LODIndex).Encoded) |
			(FFloat16(Instance->Strands.Modifier.HairLengthScale_Override ? Instance->Strands.Modifier.HairLengthScale : -1.f).Encoded << 16);
		
		bool bHasRaytracing = false;
		switch (Instance->GeometryType) 
		{
		case EHairGeometryType::Strands:
			if (Instance->Strands.IsValid())
			{
				D.Data0.Z = Instance->Strands.GetData().GetNumCurves(); // Change this later on for having dynamic value
				D.Data0.W = Instance->Strands.GetData().GetNumPoints(); // Change this later on for having dynamic value
				const int32 MeshLODIndex = Instance->HairGroupPublicData->MeshLODIndex;
				if (MeshLODIndex>=0 && Instance->Strands.RestRootResource && Instance->Strands.RestRootResource->IsValid(MeshLODIndex))
				{
					D.Data1.X = Instance->Strands.RestRootResource->GetLOD(MeshLODIndex)->GetBulkData().Header.UniqueSectionIndices.Num();
					D.Data1.Y = Instance->Strands.RestRootResource->GetLOD(MeshLODIndex)->GetBulkData().Header.UniqueTriangleCount;
					D.Data1.Z = Instance->Strands.RestRootResource->GetLOD(MeshLODIndex)->GetBulkData().Header.RootCount;
					D.Data1.W = Instance->Strands.RestRootResource->GetLOD(MeshLODIndex)->GetBulkData().Header.PointCount;
				}

				{
					D.Data2 = FUintVector4(0);
					D.Data2.X |= VisibilityType == EHairInstanceVisibilityType::StrandsPrimaryView           ? 0x1u : 0u;
					D.Data2.X |= VisibilityType == EHairInstanceVisibilityType::StrandsShadowView            ? 0x2u : 0u;
					D.Data2.X |= Instance->HairGroupPublicData->bSupportVoxelization                         ? 0x4u : 0u;
					D.Data2.X |= ActiveGroomCacheType == EGroomCacheType::Guides                             ? 0x8u : 0u;
					D.Data2.X |= ActiveGroomCacheType == EGroomCacheType::Strands                            ? 0x10u: 0u;
					D.Data2.X |= Instance->HairGroupPublicData->VFInput.Strands.Common.Flags << 8u;
					D.Data2.X |= uint32(FFloat16(Instance->HairGroupPublicData->ContinuousLODScreenSize).Encoded) << 16u;

					D.Data2.Y = Instance->HairGroupPublicData->GetActiveStrandsPointCount();
					D.Data2.Z = Instance->HairGroupPublicData->GetActiveStrandsCurveCount();

					D.Data2.W = Instance->Strands.GetData().Header.ImportedAttributes;
				}
				
				D.Data3 = FUintVector4(0);
				D.Data3.X |= FFloat16(Instance->HairGroupPublicData->VFInput.Strands.Common.Radius).Encoded;
				D.Data3.X |= FFloat16(Instance->HairGroupPublicData->VFInput.Strands.Common.Density).Encoded << 16u;

				D.Data3.Y |= FFloat16(Instance->HairGroupPublicData->VFInput.Strands.Common.RootScale).Encoded;
				D.Data3.Y |= FFloat16(Instance->HairGroupPublicData->VFInput.Strands.Common.TipScale).Encoded << 16u;

				D.Data3.Z |= FFloat16(Instance->HairGroupPublicData->VFInput.Strands.Common.Length).Encoded;
				D.Data3.Z |= FFloat16(Instance->HairGroupPublicData->VFInput.Strands.Common.LengthScale).Encoded << 16u;
				D.Data3.W |= FFloat16(Instance->HairGroupPublicData->GetActiveStrandsCoverageScale()).Encoded;
				D.Data3.W |= (Instance->HairGroupPublicData->bAutoLOD || UseHairStrandsForceAutoLOD()) ? (1u << 16u) : 0;
				D.Data3.W |= Instance->HairGroupPublicData->VFInput.Strands.Common.bCurveRemapping ? (1u << 17u) : 0;
				D.Data3.W |= Instance->HairGroupPublicData->VFInput.Strands.Common.bPointRemapping ? (1u << 18u) : 0;

				#if RHI_RAYTRACING
				bHasRaytracing = Instance->Strands.RenRaytracingResource != nullptr;
				#endif
			}
			break;
		case EHairGeometryType::Cards:
			if (Instance->Cards.IsValid(IntLODIndex))
			{
				D.Data0.Z = Instance->Cards.LODs[IntLODIndex].Guides.IsValid() ? Instance->Cards.LODs[IntLODIndex].Guides.GetData().GetNumCurves() : 0;
				D.Data0.W = Instance->Cards.LODs[IntLODIndex].GetData().GetNumVertices();

				D.Data2 = FUintVector4(0);
				D.Data2.X |= VisibilityType == EHairInstanceVisibilityType::CardsOrMeshesPrimaryView ? 0x1u : 0u;
				D.Data2.X |= VisibilityType == EHairInstanceVisibilityType::CardsOrMeshesShadowView  ? 0x2u : 0u;

				D.Data2.Y = D.Data0.W;
				D.Data2.Z = D.Data0.Z;

				#if RHI_RAYTRACING
				bHasRaytracing = Instance->Cards.LODs[IntLODIndex].RaytracingResource != nullptr;
				#endif
			}
			break;
		case EHairGeometryType::Meshes:
			if (Instance->Meshes.IsValid(IntLODIndex))
			{
				D.Data0.Z = 0;
				D.Data0.W = Instance->Meshes.LODs[IntLODIndex].GetData().GetNumVertices();

				D.Data2 = FUintVector4(0);
				D.Data2.X |= VisibilityType == EHairInstanceVisibilityType::CardsOrMeshesPrimaryView ? 0x1u : 0u;
				D.Data2.X |= VisibilityType == EHairInstanceVisibilityType::CardsOrMeshesShadowView  ? 0x2u : 0u;

				D.Data2.Y = D.Data0.W;
				D.Data2.Z = D.Data0.Z;

				#if RHI_RAYTRACING
				bHasRaytracing = Instance->Meshes.LODs[IntLODIndex].RaytracingResource != nullptr;
				#endif
			}
			break;
		}

		D.Data4.X = 0;
		D.Data4.X |= Instance->HairGroupPublicData->VFInput.bHasLODSwitch ? 1u : 0u;
		D.Data4.X |= bHasRaytracing ? 2u : 0u;
	}

	ShaderPrint::FStrings AttributeNames;
	for (uint32 AttributeIt = 0; AttributeIt < uint32(EHairAttribute::Count); ++AttributeIt)
	{
		// Only add valid optional attribute 
		// HAIR_ATTRIBUTE_XXX and EHairAttribute don't have a 1:1 mapping
		const EHairAttribute Attribute = (EHairAttribute)AttributeIt;
		AttributeNames.Add(GetHairAttributeText(Attribute, 0u), AttributeIt);
	}

	const uint32 InfoInBytes  = sizeof(FInstanceInfos);
	const uint32 InfoInUints = sizeof(FInstanceInfos) / sizeof(uint32);
	FRDGBufferRef InfoBuffer = CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.InstanceInfos"), FRDGBufferDesc::CreateBufferDesc(4, InfoInUints * Infos.Num()), Infos.GetData(), InfoInBytes * Infos.Num());

	// Draw general information for all instances (one pass for all instances)
	{
		FHairDebugPrintInstanceCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDebugPrintInstanceCS::FParameters>();
		Parameters->InstanceCount = InstanceCount;
		Parameters->InstanceCount_StrandsPrimaryView = InstanceCountPerType[uint32(EHairInstanceVisibilityType::StrandsPrimaryView)];
		Parameters->InstanceCount_StrandsShadowView = InstanceCountPerType[uint32(EHairInstanceVisibilityType::StrandsShadowView)];
		Parameters->InstanceCount_CardsOrMeshesPrimaryView = InstanceCountPerType[uint32(EHairInstanceVisibilityType::CardsOrMeshesPrimaryView)];
		Parameters->InstanceCount_CardsOrMeshesShadowView = InstanceCountPerType[uint32(EHairInstanceVisibilityType::CardsOrMeshesShadowView)];
		Parameters->InstanceNames = InstanceNames.GetParameters(GraphBuilder);
		Parameters->AttributeNames = AttributeNames.GetParameters(GraphBuilder);
		Parameters->Infos = GraphBuilder.CreateSRV(InfoBuffer, PF_R32_UINT);
		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
		FHairDebugPrintInstanceCS::FPermutationDomain PermutationVector;
		TShaderMapRef<FHairDebugPrintInstanceCS> ComputeShader(ShaderMap, PermutationVector);

		ClearUnusedGraphResources(ComputeShader, Parameters);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::DebugPrintInstance(Info,Instances:%d)", InstanceCount),
			ComputeShader,
			Parameters,
			FIntVector(1, 1, 1));
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairDebugPrintMemoryCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDebugPrintMemoryCS);
	SHADER_USE_PARAMETER_STRUCT(FHairDebugPrintMemoryCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, ComponentCount)
		SHADER_PARAMETER(uint32, GroomCount)
		SHADER_PARAMETER(uint32, BindingCount)
		SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderParameters, ComponentNames)
		SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderParameters, GroomNames)
		SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderParameters, BindingNames)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ComponentBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GroomBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, BindingBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		AddShaderPrintDebugFlag(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_PRINT_MEMORY"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDebugPrintMemoryCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

const TCHAR* GetHairAttributeText(EHairAttribute In, uint32 InFlags);
uint32 GetHairAttributeIndex(EHairAttribute In);
EGroomCacheType GetHairInstanceCacheType(const FHairGroupInstance* Instance);

static void AddHairDebugPrintMemoryPass(
	FRDGBuilder& GraphBuilder, 
	FGlobalShaderMap* ShaderMap,
	const FSceneView& View,
	const FShaderPrintData* ShaderPrintData)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	ShaderPrint::SetEnabled(true);
	
	if (!ShaderPrintData) { return; }

	struct FInfos
	{
		FUintVector4 Data0 = { 0,0,0,0 };
		FUintVector4 Data1 = { 0,0,0,0 };
		FUintVector4 Data2 = { 0,0,0,0 };
		FUintVector4 Data3 = { 0,0,0,0 };
		FUintVector4 Data4 = { 0,0,0,0 };
	};
	const uint32 InfoInBytes  = sizeof(FInfos);
	const uint32 InfoInUints = sizeof(FInfos) / sizeof(uint32);

	// Component
	uint32 ComponentCount = 0;
	for (TObjectIterator<UGroomComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt)
		{
			ComponentCount += ComponentIt->GetGroupCount();
		}
	}
	ShaderPrint::FStrings ComponentNames(ComponentCount);
	TArray<FInfos> ComponentBuffer;
	ComponentBuffer.Reserve(ComponentCount);
	for (TObjectIterator<UGroomComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		if (ComponentIt)
		{
			const uint32 GroupCount = ComponentIt->GetGroupCount();
			for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
			{
				const FHairGroupInstance* Instance = ComponentIt->GetGroupInstance(GroupIt);
				const FGroomComponentMemoryStats MemoryStats = FGroomComponentMemoryStats::Get(Instance);
		
				ComponentNames.Add(*Instance->Debug.GroomAssetName);
		
				FInfos& D = ComponentBuffer.AddDefaulted_GetRef();
				D.Data0.X = Instance->Debug.GroupIndex;
				D.Data0.Y = Instance->Debug.GroupCount;
				D.Data0.Z = Instance->HairGroupPublicData->GetActiveStrandsCurveCount();
				D.Data0.W = Instance->Strands.RestResource ? Instance->Strands.RestResource->BulkData.Header.CurveCount : 0;
		
				D.Data1.X = MemoryStats.Guides;
				D.Data1.Y = MemoryStats.Strands;
				D.Data1.Z = MemoryStats.Cards;
				D.Data1.W = MemoryStats.Meshes;
		
				D.Data2.X = MemoryStats.Guides;
				D.Data2.Y = MemoryStats.Strands;
				D.Data2.Z = MemoryStats.Cards;
				D.Data2.W = MemoryStats.Meshes;

				D.Data3.X = 0;
				D.Data3.Y = 0;
				D.Data3.Z = 0;
				D.Data3.W = 0;
			}
		}
	}

	// Groom
	uint32 GroomCount = 0;
	for (TObjectIterator<UGroomAsset> AssetIt; AssetIt; ++AssetIt)
	{
		if (AssetIt)
		{
			GroomCount += AssetIt->GetHairGroupsPlatformData().Num();
		}
	}
	ShaderPrint::FStrings GroomNames(GroomCount);
	TArray<FInfos> GroomBuffer;
	GroomBuffer.Reserve(GroomCount);
	for (uint32 It = 0; It < GroomCount; ++It)
	{
		for (TObjectIterator<UGroomAsset> AssetIt; AssetIt; ++AssetIt)
		{
			if (AssetIt)
			{
				const uint32 GroupCount = AssetIt->GetHairGroupsPlatformData().Num();
				for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
				{					
					const FHairGroupPlatformData& Data = AssetIt->GetHairGroupsPlatformData()[GroupIt];
					const FHairGroupResources& Resources = AssetIt->GetHairGroupsResources()[GroupIt];
					const FGroomAssetMemoryStats MemoryStats = FGroomAssetMemoryStats::Get(Data, Resources);

					GroomNames.Add(AssetIt->GetName());

					FInfos& D = GroomBuffer.AddDefaulted_GetRef();
					D.Data0.X = GroupIt;
					D.Data0.Y = GroupCount;
					D.Data0.Z = Resources.Strands.RestResource ? Resources.Strands.RestResource->MaxAvailableCurveCount : 0;
					D.Data0.W = Data.Strands.BulkData.IsValid() ? Data.Strands.BulkData.GetNumCurves() : 0;
			
					D.Data1.X = MemoryStats.CPU.Guides;
					D.Data1.Y = MemoryStats.CPU.Strands;
					D.Data1.Z = MemoryStats.CPU.Cards;
					D.Data1.W = MemoryStats.CPU.Meshes;
			
					D.Data2.X = MemoryStats.GPU.Guides;
					D.Data2.Y = MemoryStats.GPU.Strands;
					D.Data2.Z = MemoryStats.GPU.Cards;
					D.Data2.W = MemoryStats.GPU.Meshes;

					D.Data3.X = MemoryStats.Memory.Rest;
					D.Data3.Y = MemoryStats.Memory.Interpolation;
					D.Data3.Z = MemoryStats.Memory.Cluster;
					D.Data3.W = MemoryStats.Memory.Raytracing;

					D.Data4.X = MemoryStats.Curves.Rest;
					D.Data4.Y = MemoryStats.Curves.Interpolation;
					D.Data4.Z = MemoryStats.Curves.Cluster;
					D.Data4.W = MemoryStats.Curves.Raytracing;
				}
			}
		}
	}

	// Binding
	uint32 BindingCount = 0;
	for (TObjectIterator<UGroomBindingAsset> AssetIt; AssetIt; ++AssetIt)
	{
		if (AssetIt)
		{
			BindingCount += AssetIt->GetHairGroupsPlatformData().Num();
		}
	}
	ShaderPrint::FStrings BindingNames(BindingCount);
	TArray<FInfos> BindingBuffer;
	BindingBuffer.Reserve(BindingCount);
	for (uint32 It = 0; It < BindingCount; ++It)
	{
		for (TObjectIterator<UGroomBindingAsset> AssetIt; AssetIt; ++AssetIt)
		{
			if (AssetIt)
			{
				const uint32 GroupCount = AssetIt->GetHairGroupsPlatformData().Num();
				for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
				{
					const UGroomBindingAsset::FHairGroupPlatformData& Data = AssetIt->GetHairGroupsPlatformData()[GroupIt];
					const UGroomBindingAsset::FHairGroupResource& Resource = AssetIt->GetHairGroupResources()[GroupIt];
					const FGroomBindingAssetMemoryStats MemoryStats = FGroomBindingAssetMemoryStats::Get(Data, Resource);
			

					BindingNames.Add(*AssetIt->GetName(), It);
			
					FInfos& D = BindingBuffer.AddDefaulted_GetRef();
					D.Data0.X = GroupIt;
					D.Data0.Y = GroupCount;
					D.Data0.Z = Resource.RenRootResources ? 0u : 0u; // TODO: need a mesh index: we could take also the max amoung all mesh LODs. Resource.RenRootResources->MaxAvailableCurveCount : 0;
					D.Data0.W = Resource.RenRootResources ? Resource.RenRootResources->GetRootCount() : 0;
			
					D.Data1.X = MemoryStats.CPU.Guides;
					D.Data1.Y = MemoryStats.CPU.Strands;
					D.Data1.Z = MemoryStats.CPU.Cards;
					D.Data1.W = 0;
			
					D.Data2.X = MemoryStats.GPU.Guides;
					D.Data2.Y = MemoryStats.GPU.Strands;
					D.Data2.Z = MemoryStats.GPU.Cards;
					D.Data2.W = 0;
				}
			}
		}
	}

	// Empty cases
	if (ComponentCount == 0 && GroomCount == 0 && BindingCount == 0) return;
	if (ComponentBuffer.IsEmpty())	{ ComponentBuffer.AddDefaulted(); }
	if (GroomBuffer.IsEmpty())		{ GroomBuffer.AddDefaulted(); }
	if (BindingBuffer.IsEmpty())	{ BindingBuffer.AddDefaulted(); }

	// Force ShaderPrint on.
	ShaderPrint::RequestSpaceForCharacters((ComponentCount + GroomCount + BindingCount) * 256 + 512);

	FHairDebugPrintMemoryCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDebugPrintMemoryCS::FParameters>();
	Parameters->ComponentCount  = ComponentCount;
	Parameters->ComponentNames  = ComponentNames.GetParameters(GraphBuilder);
	Parameters->ComponentBuffer = GraphBuilder.CreateSRV(CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.Component"), FRDGBufferDesc::CreateBufferDesc(4, InfoInUints * ComponentBuffer.Num()), ComponentBuffer.GetData(), InfoInBytes * ComponentBuffer.Num()), PF_R32_UINT);

	Parameters->GroomCount		= GroomCount;
	Parameters->GroomNames  	= GroomNames.GetParameters(GraphBuilder);
	Parameters->GroomBuffer 	= GraphBuilder.CreateSRV(CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.Groom"), FRDGBufferDesc::CreateBufferDesc(4, InfoInUints * GroomBuffer.Num()), GroomBuffer.GetData(), InfoInBytes * GroomBuffer.Num()), PF_R32_UINT);

	Parameters->BindingCount  	= BindingCount;
	Parameters->BindingNames  	= BindingNames.GetParameters(GraphBuilder);
	Parameters->BindingBuffer 	= GraphBuilder.CreateSRV(CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.Binding"), FRDGBufferDesc::CreateBufferDesc(4, InfoInUints * BindingBuffer.Num()), BindingBuffer.GetData(), InfoInBytes * BindingBuffer.Num()), PF_R32_UINT);

	ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
	TShaderMapRef<FHairDebugPrintMemoryCS> ComputeShader(ShaderMap);

	ClearUnusedGraphResources(ComputeShader, Parameters);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::DebugMemory(Info,Instances:%d, Groom:%d, Binding:%d)", ComponentCount, GroomCount, BindingCount),
		ComputeShader,
		Parameters,
		FIntVector(1, 1, 1));
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void RunHairStrandsDebug(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FSceneInterface* Scene,
	const FSceneView& View,
	const FHairStrandsInstances& Instances,
	const FHairInstanceCullingResults& CullingResults,
	FHairTransientResources& TransientResources,
	const FShaderPrintData* ShaderPrintData,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FIntRect Viewport,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer)
{
	const EGroomViewMode ViewMode = GetGroomViewMode(View);

	if (ViewMode == EGroomViewMode::MacroGroups)
	{
		AddHairDebugPrintInstancePass(GraphBuilder, ShaderMap, View, TransientResources, ShaderPrintData, Instances, CullingResults);
	}

	if (ViewMode == EGroomViewMode::MeshProjection)
	{
		{
			const FSceneView* LocalView = &View;

			uint32 DrawIndex = 0;
			auto RenderProjectionData = [&GraphBuilder, LocalView, ShaderMap, Viewport, &ViewUniformBuffer, Instances, ShaderPrintData, &DrawIndex](bool bSim)
			{
				TArray<int32> HairLODIndices;
				for (FHairStrandsInstance* AbstractInstance : Instances)
				{
					FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
					if (!Instance->HairGroupPublicData || Instance->BindingType != EHairBindingType::Skinning)
						continue;

					FHairStrandsRestResource* RestResource = nullptr;
					FHairStrandsDeformedResource* DeformedResource = nullptr;

					FHairStrandsRestRootResource* RestRootResource = nullptr;
					FHairStrandsDeformedRootResource* DeformedRootResource = nullptr;
					if (bSim)
					{
						RestResource 	 = Instance->Guides.RestResource;
						DeformedResource = Instance->Guides.DeformedResource;

						RestRootResource 	 = Instance->Guides.RestRootResource;
						DeformedRootResource = Instance->Guides.DeformedRootResource;
					}
					else if (Instance->GetHairGeometry() == EHairGeometryType::Strands)
					{
						RestResource 	 = Instance->Strands.RestResource;
						DeformedResource = Instance->Strands.DeformedResource;

						RestRootResource 	 = Instance->Strands.RestRootResource;
						DeformedRootResource = Instance->Strands.DeformedRootResource;
					}
					else if (Instance->GetHairGeometry() == EHairGeometryType::Cards)
					{
						const int32 LODIndex = Instance->HairGroupPublicData->GetIntLODIndex();
						if (Instance->Cards.IsValid(LODIndex))
						{
							RestResource 	 = Instance->Cards.LODs[LODIndex].Guides.RestResource;
							DeformedResource = Instance->Cards.LODs[LODIndex].Guides.DeformedResource;

							RestRootResource 	 = Instance->Cards.LODs[LODIndex].Guides.RestRootResource;
							DeformedRootResource = Instance->Cards.LODs[LODIndex].Guides.DeformedRootResource;
						}
					}
					if (RestRootResource == nullptr || DeformedRootResource == nullptr || RestResource == nullptr || DeformedResource == nullptr)
						continue;

					const int32 MeshLODIndex = Instance->Debug.MeshLODIndex;

					AddDebugProjectionHairPass(GraphBuilder, LocalView, ShaderMap, ShaderPrintData, Viewport, ViewUniformBuffer, MeshLODIndex, RestResource, DeformedResource, RestRootResource, DeformedRootResource, Instance->HairGroupPublicData->VFInput.LocalToWorldTransform, DrawIndex++, bSim, EDebugProjectionHairType::HairRoots,    HairStrandsTriangleType::RestPose);
					AddDebugProjectionHairPass(GraphBuilder, LocalView, ShaderMap, ShaderPrintData, Viewport, ViewUniformBuffer, MeshLODIndex, RestResource, DeformedResource, RestRootResource, DeformedRootResource, Instance->HairGroupPublicData->VFInput.LocalToWorldTransform, DrawIndex++, bSim, EDebugProjectionHairType::HairSamples,  HairStrandsTriangleType::RestPose);
					AddDebugProjectionHairPass(GraphBuilder, LocalView, ShaderMap, ShaderPrintData, Viewport, ViewUniformBuffer, MeshLODIndex, RestResource, DeformedResource, RestRootResource, DeformedRootResource, Instance->HairGroupPublicData->VFInput.LocalToWorldTransform, DrawIndex++, bSim, EDebugProjectionHairType::HairRoots,    HairStrandsTriangleType::DeformedPose);
					AddDebugProjectionHairPass(GraphBuilder, LocalView, ShaderMap, ShaderPrintData, Viewport, ViewUniformBuffer, MeshLODIndex, RestResource, DeformedResource, RestRootResource, DeformedRootResource, Instance->HairGroupPublicData->VFInput.LocalToWorldTransform, DrawIndex++, bSim, EDebugProjectionHairType::HairSamples,  HairStrandsTriangleType::DeformedPose);
				}
			};

			RenderProjectionData(false /*bSim*/);
			RenderProjectionData(true  /*bSim*/);
		}
	}

	if (GHairCardsAtlasDebug > 0)
	{
		for (FHairStrandsInstance* AbstractInstance : Instances)
		{
			FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);

			AddDrawDebugCardsAtlasPass(GraphBuilder, View, ShaderMap, Instance, ShaderPrintData, SceneColorTexture);
		}
	}

	const bool bEnabled =
		ViewMode == EGroomViewMode::CardGuides ||
		ViewMode == EGroomViewMode::SimHairStrands ||
		ViewMode == EGroomViewMode::ControlPoints;

	if (bEnabled)
	{	
		uint32 Index = 0;
		for (FHairStrandsInstance* AbstractInstance : Instances)
		{
			FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
	
			if (ViewMode == EGroomViewMode::CardGuides)
			{
				AddDrawDebugCardsGuidesPass(GraphBuilder, View, ShaderMap, Instance, ShaderPrintData, true /*bDeformed*/, true);
			}
	
			if (ViewMode == EGroomViewMode::SimHairStrands)
			{
				AddDrawDebugCardsGuidesPass(GraphBuilder, View, ShaderMap, Instance, ShaderPrintData, true /*bDeformed*/, false);
			}
	
			if (ViewMode == EGroomViewMode::ControlPoints)
			{
				AddDrawDebugStrandsCVsPass(GraphBuilder, View, ShaderMap, Instance, ShaderPrintData, Index, SceneColorTexture, SceneDepthTexture);
			}
			++Index;
		}
	}

	if (ViewMode == EGroomViewMode::Memory)
	{
		AddHairDebugPrintMemoryPass(GraphBuilder, ShaderMap, View, ShaderPrintData);
	}
}
