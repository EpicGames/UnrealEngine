// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomDeformerBuilder.h"
#include "Engine/SkinnedAssetCommon.h"
#include "GroomBuilder.h"
#include "HairStrandsCore.h"
#include "Animation/Skeleton.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "GroomAsset.h"
#include "UObject/Package.h"

#if WITH_EDITOR

static void AddGuideBones(UGroomAsset* GroomAsset, USkeletalMesh* SkeletalMesh, const FString& NameSkeleton, FSkeletalMeshLODModel* LodRenderData,
	const FVector& RootPosition, TArray<FVector>& BonesPositions, TArray<int32>& BonesOffsets, TArray<int32>& GroupsOffsets, const TArray<FHairStrandsDatas>& GuidesDatas)
{
	BonesOffsets.Reset();
	GroupsOffsets.Reset();
	
	int32 BoneIndex = 1;
	for(int32 GroupIndex = 0; GroupIndex < GroomAsset->GetNumHairGroups(); ++GroupIndex)
	{
		GroupsOffsets.Add(BonesOffsets.Num());
		if(GroomAsset->IsDeformationEnable(GroupIndex))
		{
			const FHairStrandsDatas& GuidesData = GuidesDatas[GroupIndex];
			
			for(uint32 CurveIndex = 0; CurveIndex < GuidesData.StrandsCurves.Num(); ++CurveIndex)
			{
				// Compute the node positions and rotations
				FVector TangentPrev = FVector(0,0,1);
				FQuat PointRotation = FQuat::Identity;
				FTransform PointTransform = FTransform::Identity;
				FTransform ParentTransform = FTransform::Identity;
				ParentTransform.SetTranslation(RootPosition);

				BonesOffsets.Add(BoneIndex);
				const uint32 NumPoints = GuidesData.StrandsCurves.CurvesCount[CurveIndex];
				for(uint32 PointIndex = 0; PointIndex < GuidesData.StrandsCurves.CurvesCount[CurveIndex]; ++PointIndex, ++BoneIndex)
				{
					const int32 PositionIndex = PointIndex+GuidesData.StrandsCurves.CurvesOffset[CurveIndex];
					const FVector PointPosition(GuidesData.StrandsPoints.PointsPosition[PositionIndex]);

					FVector TangentNext = (PointIndex < (NumPoints-1)) ? (FVector(GuidesData.StrandsPoints.PointsPosition[PositionIndex+1]) - PointPosition) :
																		 (PointPosition - FVector(GuidesData.StrandsPoints.PointsPosition[PositionIndex-1]));
					TangentNext.Normalize();
						
					PointRotation = ( FQuat::FindBetweenVectors(TangentPrev,TangentNext) * PointRotation ).GetNormalized();
					TangentPrev = TangentNext;
					
					FString BoneString = NameSkeleton + TEXT("_Bone_") + FString::FromInt(BoneIndex);

					PointTransform.SetTranslation(PointPosition);
					PointTransform.SetRotation(PointRotation);

					BonesPositions.Add(PointPosition);

					const int32 ParentIndex = (PointIndex == 0) ? 0 : BoneIndex -1;
					FReferenceSkeletonModifier Modifier = FReferenceSkeletonModifier(SkeletalMesh->GetRefSkeleton(), nullptr);
					Modifier.Add(FMeshBoneInfo(FName(*BoneString), BoneString, ParentIndex), PointTransform * ParentTransform.Inverse());
					ParentTransform = PointTransform;

					LodRenderData->ActiveBoneIndices.Add(BoneIndex);
					LodRenderData->RequiredBones.Add(BoneIndex);
				}
			}
		}
	}
	GroupsOffsets.Add(BonesOffsets.Num());
	BonesOffsets.Add(BoneIndex);
}

static FVector AddRootBone(UGroomAsset* GroomAsset, USkeletalMesh* SkeletalMesh, const FString& NameSkeleton, FSkeletalMeshLODModel* LodRenderData, TArray<FVector>& BonesPositions, TArray<FHairStrandsDatas>& GuidesDatas)
{
	GuidesDatas.SetNum(GroomAsset->GetNumHairGroups());
	
	// Get the number of bones
	int32 NumBones = 1;
	int32 NumRoots = 0;
	
	FVector CenterBone(0,0,0);
	int32 GroupIndex = 0;
	for(auto& HairGroup : GroomAsset->GetHairDescriptionGroups().HairGroups)
	{
		FHairStrandsDatas StrandsData;
		FHairStrandsDatas& GuidesData = GuidesDatas[GroupIndex];
		FGroomBuilder::BuildData(HairGroup, GroomAsset->GetHairGroupsInterpolation()[GroupIndex], GroomAsset->GetHairGroupsInfo()[GroupIndex], StrandsData, GuidesData);
		
		if(GroomAsset->IsDeformationEnable(GroupIndex))
		{
			for(uint32 CurveIndex = 0; CurveIndex < GuidesData.StrandsCurves.Num(); ++CurveIndex)
			{
				CenterBone += FVector(GuidesData.StrandsPoints.PointsPosition[GuidesData.StrandsCurves.CurvesOffset[CurveIndex]]);
			}
			NumRoots += GuidesData.StrandsCurves.Num();
			NumBones += GuidesData.StrandsPoints.Num();
		}
		++GroupIndex;
	}
	if(NumRoots != 0)
	{
		CenterBone /= float(NumRoots);
	}

	// Add all the bones from the guides 
	SkeletalMesh->GetRefSkeleton().Empty(NumBones);
	FTransform Transform = FTransform::Identity;

	// Add root bone
	{
		FString BoneString = NameSkeleton + TEXT("_Bone_") + FString::FromInt(0);
		FReferenceSkeletonModifier Modifier = FReferenceSkeletonModifier(SkeletalMesh->GetRefSkeleton(), nullptr);
		Transform.SetTranslation(CenterBone);
		BonesPositions.Add(CenterBone);
			
		Modifier.Add(FMeshBoneInfo(FName(*BoneString), BoneString, INDEX_NONE), Transform);

		LodRenderData->ActiveBoneIndices.Add(0);
		LodRenderData->RequiredBones.Add(0); 
	}
	return CenterBone;
}

static void BuildSkeletonBones(UGroomAsset* GroomAsset, USkeletalMesh* SkeletalMesh, FSkeletalMeshLODModel* LodRenderData, TArray<FVector>& BonesPositions, TArray<int32>& BonesOffsets, TArray<int32>& GroupsOffsets)
{
	FString PackageNameSkeleton, NameSkeleton;
	FHairStrandsCore::AssetHelper().CreateFilename(GroomAsset->GetOutermost()->GetName(), TEXT("_Skeleton"), PackageNameSkeleton, NameSkeleton);
	
	// Create packages for skeleton and skelmesh
	UPackage* PackageSkeleton = CreatePackage(*PackageNameSkeleton);
	USkeleton* Skeleton = SkeletalMesh->GetSkeleton() ? SkeletalMesh->GetSkeleton() :
    			NewObject<USkeleton>( PackageSkeleton, *NameSkeleton, RF_Public | RF_Standalone | RF_Transactional );

	TArray<FName> SkeletonBones;
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
	{
		SkeletonBones.Add(RefSkeleton.GetBoneName(BoneIndex));
	}
	Skeleton->RemoveBonesFromSkeleton(SkeletonBones, true);
	
	// Add the bones
	TArray<FHairStrandsDatas> GuidesDatas;
	const FVector RootPosition = AddRootBone(GroomAsset, SkeletalMesh, NameSkeleton, LodRenderData, BonesPositions, GuidesDatas);
	AddGuideBones(GroomAsset, SkeletalMesh, NameSkeleton, LodRenderData, RootPosition, BonesPositions, BonesOffsets, GroupsOffsets, GuidesDatas);
	
	SkeletalMesh->SetSkeleton(Skeleton);
	Skeleton->SetPreviewMesh(SkeletalMesh);
	Skeleton->RecreateBoneTree(SkeletalMesh);
	
	SkeletalMesh->GetRefBasesInvMatrix().Reset();
	SkeletalMesh->CalculateInvRefMatrices();
}

static void BuildMeshSection(UGroomAsset* GroomAsset, USkeletalMesh* SkeletalMesh, FSkeletalMeshLODModel* LodRenderData, const TArray<FVector>& BonesPositions,
	const TArray<int32>& BonesOffsets, const TArray<int32>& GroupsOffsets, TArray<int32>& GroupSections)
{
	// Initialize mesh section
	LodRenderData->Sections.Reset();
	LodRenderData->IndexBuffer.Reset();
	LodRenderData->NumTexCoords = 1;
	LodRenderData->NumVertices = 0;
	
	const int32 NumGroups = GroupsOffsets.Num()-1;
	GroupSections.Init(INDEX_NONE, NumGroups);
	for(int32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
	{
		const int32 CurveStart = GroupsOffsets[GroupIndex], CurveStop = GroupsOffsets[GroupIndex+1];
		const int32 PointStart = BonesOffsets[CurveStart], PointStop = BonesOffsets[CurveStop];
		
		const int32 NumCurves = CurveStop - CurveStart;
		const int32 NumPoints = PointStop - PointStart;
		GroupSections[GroupIndex] = INDEX_NONE;
		if(NumCurves > 0 && NumPoints > 0)
		{
			const int32 SectionIndex = LodRenderData->Sections.Add(FSkelMeshSection());
			auto& MeshSection = LodRenderData->Sections[SectionIndex];
			GroupSections[GroupIndex] = SectionIndex;
		
			MeshSection.MaterialIndex = 0;
			MeshSection.BaseIndex = LodRenderData->IndexBuffer.Num();
	
			MeshSection.BaseVertexIndex = LodRenderData->NumVertices;
			MeshSection.MaxBoneInfluences = 1;

			MeshSection.SoftVertices.Reset();
			MeshSection.NumVertices = 1 + NumPoints;
			
			for (int32_t VertexIndex = 0; VertexIndex < MeshSection.NumVertices; VertexIndex++)
			{
				const int32 BoneIndex = (VertexIndex == 0) ? 0 : PointStart + VertexIndex - 1;
				
				FSoftSkinVertex SoftVertex;
				SoftVertex.Color = FColor::White;
				SoftVertex.Position = FVector3f(BonesPositions[BoneIndex]);
			
				FMemory::Memzero(SoftVertex.InfluenceBones);
				FMemory::Memzero(SoftVertex.InfluenceWeights);
		
				SoftVertex.InfluenceWeights[0] = 1.0f;
				SoftVertex.InfluenceBones[0] = MeshSection.BoneMap.Num();
		
				MeshSection.SoftVertices.Add(SoftVertex);
				MeshSection.BoneMap.Add(BoneIndex);
			}
			LodRenderData->NumVertices += MeshSection.NumVertices;
			
			int32 NumTriangles = 0;
			for(int32 CurveIndex = CurveStart; CurveIndex < CurveStop; ++CurveIndex)
			{
				for(int32 BoneIndex = BonesOffsets[CurveIndex]; BoneIndex < BonesOffsets[CurveIndex+1]-1; ++BoneIndex, ++NumTriangles)
				{
					LodRenderData->IndexBuffer.Add(MeshSection.BaseVertexIndex);
					LodRenderData->IndexBuffer.Add(MeshSection.BaseVertexIndex+1+BoneIndex-PointStart);
					LodRenderData->IndexBuffer.Add(MeshSection.BaseVertexIndex+1+BoneIndex+1-PointStart);
				}
			}
			MeshSection.NumTriangles = NumTriangles;
		}
	}
}

static FSkeletalMeshLODModel* BuildRenderData(UGroomAsset* GroomAsset, USkeletalMesh* SkeletalMesh)
{
	// Create Skeletal Mesh LOD Render Data and LOD Info
	FSkeletalMeshLODModel* LodRenderData = new FSkeletalMeshLODModel();
	SkeletalMesh->GetImportedModel()->LODModels.Add(LodRenderData);

	// Set default LOD Info
	FSkeletalMeshLODInfo& LodInfo = SkeletalMesh->AddLODInfo();
	LodInfo.ScreenSize = 0.3f;
	LodInfo.LODHysteresis = 0.2f;
	LodInfo.BuildSettings.bUseFullPrecisionUVs = true;
	LodInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0;
	LodInfo.ReductionSettings.NumOfVertPercentage = 1.0;
	
	//Add default material as backup
	LodInfo.LODMaterialMap.Add(0);
	UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	SkeletalMesh->GetMaterials().Add(DefaultMaterial);
	SkeletalMesh->GetMaterials()[0].UVChannelData.bInitialized = true;

	return LodRenderData;
}

static void BuildBodySetup(UPhysicsAsset* PhysicsAsset, USkeletalMesh* SkeletalMesh, const int32 BoneIndex, const int32 ParentIndex, const FName& BoneName)
{
	FString BodyString = BoneName.ToString() + TEXT("_Body_") + FString::FromInt(BoneIndex);
			
	int32 BodyIndex = PhysicsAsset->FindBodyIndex(FName(BodyString));
	if(BodyIndex == INDEX_NONE)
	{
		USkeletalBodySetup* BodySetup = NewObject<USkeletalBodySetup>(PhysicsAsset, NAME_None, RF_Transactional);
		// make default to be use complex as simple 
		BodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		BodySetup->PhysicsType = (ParentIndex == INDEX_NONE) ? EPhysicsType::PhysType_Kinematic : EPhysicsType::PhysType_Default;;
			
		BodyIndex = PhysicsAsset->SkeletalBodySetups.Add(BodySetup );
		BodySetup->BoneName = BoneName;

		PhysicsAsset->UpdateBodySetupIndexMap();
		PhysicsAsset->UpdateBoundsBodiesArray();
	}

	UBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
	
	// Empty any existing collision.
	BodySetup->RemoveSimpleCollision();
	
	FKSphereElem SphereElem;
	SphereElem.Center = FVector::ZeroVector;
	SphereElem.Radius = 1.0f;

	BodySetup->AggGeom.SphereElems.Add(SphereElem);
}

static void BuildConstraintSetup(UPhysicsAsset* PhysicsAsset, const FTransform& RelativeTransform, const int32 BoneIndex, const FName& BoneName, const FName& ParentName)
{
	FString ConstraintString = BoneName.ToString() + TEXT("_Constraint_") + FString::FromInt(BoneIndex);

	// constraintClass must be a subclass of UPhysicsConstraintTemplate
	int32 ConstraintIndex = PhysicsAsset->FindConstraintIndex(FName(ConstraintString));
	if(ConstraintIndex == INDEX_NONE)
	{
		UPhysicsConstraintTemplate* ConstraintSetup = NewObject<UPhysicsConstraintTemplate>(PhysicsAsset, NAME_None, RF_Transactional);
                
		ConstraintIndex = PhysicsAsset->ConstraintSetup.Add( ConstraintSetup );
		ConstraintSetup->DefaultInstance.JointName = BoneName;
	}

	UPhysicsConstraintTemplate* ConstraintSetup = PhysicsAsset->ConstraintSetup[ConstraintIndex];
					
	// set angular constraint mode
	ConstraintSetup->DefaultInstance.SetAngularSwing1Motion(EAngularConstraintMotion::ACM_Free);
	ConstraintSetup->DefaultInstance.SetAngularSwing2Motion(EAngularConstraintMotion::ACM_Free);
	ConstraintSetup->DefaultInstance.SetAngularTwistMotion(EAngularConstraintMotion::ACM_Free);

	// Place joint at origin of child
	ConstraintSetup->DefaultInstance.ConstraintBone1 = BoneName;
	ConstraintSetup->DefaultInstance.Pos1 = FVector::ZeroVector;
	ConstraintSetup->DefaultInstance.PriAxis1 = FVector(1, 0, 0);
	ConstraintSetup->DefaultInstance.SecAxis1 = FVector(0, 1, 0);

	ConstraintSetup->DefaultInstance.ConstraintBone2 = ParentName;
	ConstraintSetup->DefaultInstance.Pos2 = RelativeTransform.GetLocation();
	ConstraintSetup->DefaultInstance.PriAxis2 = RelativeTransform.GetUnitAxis(EAxis::X);
	ConstraintSetup->DefaultInstance.SecAxis2 = RelativeTransform.GetUnitAxis(EAxis::Y);

	ConstraintSetup->SetDefaultProfile(ConstraintSetup->DefaultInstance);
}

static void BuildPhysicsAsset(UGroomAsset* GroomAsset, USkeletalMesh* SkeletalMesh, FSkeletalMeshLODModel* LodRenderData)
{
	// Fill the package/asset names for the skelmesh and the skeleton
	FString PackageNamePhysicsAsset, NamePhysicsAsset;
	FHairStrandsCore::AssetHelper().CreateFilename(GroomAsset->GetOutermost()->GetName(), TEXT("_PhysicsAsset"), PackageNamePhysicsAsset, NamePhysicsAsset);
	
	// Create packages for skeleton and skelmesh
	UPackage* PackagePhysicsAsset = CreatePackage(*PackageNamePhysicsAsset);
	UPhysicsAsset* PhysicsAsset = SkeletalMesh->GetPhysicsAsset() ? SkeletalMesh->GetPhysicsAsset() :
			NewObject<UPhysicsAsset>( PackagePhysicsAsset, *NamePhysicsAsset, RF_Public | RF_Standalone | RF_Transactional );

	PhysicsAsset->SkeletalBodySetups.Empty();
	PhysicsAsset->ConstraintSetup.Empty();
	
	PhysicsAsset->UpdateBodySetupIndexMap();
	PhysicsAsset->UpdateBoundsBodiesArray();

	PhysicsAsset->CollisionDisableTable.Empty();
	for (int32 BoneIndex = 0; BoneIndex < SkeletalMesh->GetRefSkeleton().GetNum(); BoneIndex++)
	{
		const int32 ParentIndex = SkeletalMesh->GetRefSkeleton().GetParentIndex(BoneIndex);

		// Go ahead and make this bone physical.
		FName BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex);
		FTransform BoneTransform(SkeletalMesh->GetComposedRefPoseMatrix(BoneIndex));
		
		BuildBodySetup(PhysicsAsset, SkeletalMesh, BoneIndex, ParentIndex, BoneName);
		if(ParentIndex != INDEX_NONE)
		{
			FName ParentName = SkeletalMesh->GetRefSkeleton().GetBoneName(ParentIndex);
			FTransform ParentTransform(FTransform(SkeletalMesh->GetComposedRefPoseMatrix(ParentIndex)));
			BuildConstraintSetup(PhysicsAsset, BoneTransform * ParentTransform.Inverse(), BoneIndex, BoneName, ParentName);
		}
	}
	SkeletalMesh->SetPhysicsAsset(PhysicsAsset);
	PhysicsAsset->SetPreviewMesh(SkeletalMesh);
}

static USkeletalMesh* BuildSkeletalMesh(UGroomAsset* GroomAsset)
{
	// Get package name
	FString PackageNameSkeletalMesh, NameSkeletalMesh;
	FHairStrandsCore::AssetHelper().CreateFilename(GroomAsset->GetOutermost()->GetName(), TEXT("_SkeletalMesh"), PackageNameSkeletalMesh, NameSkeletalMesh);
	
	// Create package for the skeletal mesh
	UPackage* PackageSkeletalMesh = CreatePackage(*PackageNameSkeletalMesh);
	USkeletalMesh* SkeletalMesh = GroomAsset->GetRiggedSkeletalMesh() ? GroomAsset->GetRiggedSkeletalMesh() :
		NewObject<USkeletalMesh>(PackageSkeletalMesh, *NameSkeletalMesh, RF_Public | RF_Standalone | RF_Transactional);

	// Reset the skeletal mesh
	SkeletalMesh->GetImportedModel()->LODModels.Reset();
	SkeletalMesh->SetNumSourceModels(0);
	SkeletalMesh->ReleaseResources();

	// Build the LOD render data
	FSkeletalMeshLODModel* LodRenderData = BuildRenderData(GroomAsset, SkeletalMesh);
	TArray<FVector> BonesPositions;
	TArray<int32> BonesOffsets;
	TArray<int32> GroupsOffsets;

	// Build the skeleton bones
	BuildSkeletonBones(GroomAsset, SkeletalMesh, LodRenderData, BonesPositions, BonesOffsets, GroupsOffsets);

	// Build the mesh section
	BuildMeshSection(GroomAsset, SkeletalMesh, LodRenderData, BonesPositions, BonesOffsets, GroupsOffsets, GroomAsset->GetDeformedGroupSections());

	// Build the physics asset
	BuildPhysicsAsset(GroomAsset, SkeletalMesh, LodRenderData);
	
	SkeletalMesh->InvalidateDeriveDataCacheGUID();
	SkeletalMesh->PostEditChange();
	SkeletalMesh->InitResources();
	if(SkeletalMesh->MarkPackageDirty())
	{
		FHairStrandsCore::AssetHelper().RegisterAsset(SkeletalMesh);
		FHairStrandsCore::SaveAsset(SkeletalMesh);
	}
	
	SkeletalMesh->GetSkeleton()->PostEditChange();
	if(SkeletalMesh->GetSkeleton()->MarkPackageDirty())
	{
		FHairStrandsCore::AssetHelper().RegisterAsset(SkeletalMesh->GetSkeleton());
		FHairStrandsCore::SaveAsset(SkeletalMesh->GetSkeleton());
	}

	SkeletalMesh->GetPhysicsAsset()->PostEditChange();
	if(SkeletalMesh->GetPhysicsAsset()->MarkPackageDirty())
	{
		FHairStrandsCore::AssetHelper().RegisterAsset(SkeletalMesh->GetPhysicsAsset());
		FHairStrandsCore::SaveAsset(SkeletalMesh->GetPhysicsAsset());
	}
	
	return SkeletalMesh;
}

#endif

USkeletalMesh* FGroomDeformerBuilder::CreateSkeletalMesh(UGroomAsset* GroomAsset)
{
#if WITH_EDITOR
	
	if (!GroomAsset)
	{
		return nullptr;
	}
	return BuildSkeletalMesh(GroomAsset);
#else
	return nullptr;
#endif
}