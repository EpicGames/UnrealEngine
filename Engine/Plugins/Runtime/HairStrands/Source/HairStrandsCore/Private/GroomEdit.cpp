// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomEdit.h"
#include "GroomAsset.h"
#include "GroomBuilder.h"

// Notes:
// This is a work-in-progress intermediate representation for editable groom.
// At the moment we do a full in-out conversion. As the conversion refines, we 
// will optimize conversion toward minimum edits

#if WITH_EDITORONLY_DATA

static void HairStrandsDataToEditableHairGuides(const FHairStrandsDatas& In, TArray<FEditableHairGuide>& Out)
{
	const bool bHasValidStrandsIDs = In.StrandsCurves.StrandIDs.Num() > 0;
	const bool bHasValidRootUVs = In.StrandsCurves.CurvesRootUV.Num() > 0;
	
	const uint32 StrandsCount = In.GetNumCurves();
	Out.SetNum(StrandsCount);
	for (uint32 StrandsIt = 0; StrandsIt < StrandsCount; ++StrandsIt)
	{
		Out[StrandsIt].GuideID = bHasValidStrandsIDs ? In.StrandsCurves.StrandIDs[StrandsIt] : StrandsIt;
		Out[StrandsIt].RootUV = bHasValidRootUVs ? In.StrandsCurves.CurvesRootUV[StrandsIt] : FVector2f(0.0f,0.0f);

		Out[StrandsIt].bHasGuideID = bHasValidStrandsIDs;
		Out[StrandsIt].bHasRootUV = bHasValidRootUVs;

		const uint32 PointCount = In.StrandsCurves.CurvesCount[StrandsIt];
		const uint32 PointOffset = In.StrandsCurves.CurvesOffset[StrandsIt];

		Out[StrandsIt].ControlPoints.SetNum(PointCount);
		for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
		{
			const uint32 EffectiveIndex = PointIt + PointOffset;
			Out[StrandsIt].ControlPoints[PointIt].Position = In.StrandsPoints.PointsPosition[EffectiveIndex];
			Out[StrandsIt].ControlPoints[PointIt].U = In.StrandsPoints.PointsCoordU[EffectiveIndex];
		}
	}
}

static void HairStrandsDataToEditableHairStrands(const FHairStrandsDatas& In, TArray<FEditableHairStrand>& Out)
{
	const bool bHasValidStrandsIDs = In.StrandsCurves.StrandIDs.Num() > 0;
	const bool bHasValidClumpIDs = In.StrandsCurves.ClumpIDs.Num() > 0;
	const bool bHasValidClosestGuide = In.StrandsCurves.CurvesClosestGuideIDs.Num() > 0;
	const bool bHasValidRootUVs = In.StrandsCurves.CurvesRootUV.Num() > 0;
	
	const uint32 StrandsCount = In.GetNumCurves();
	Out.SetNum(StrandsCount);
	for (uint32 StrandsIt = 0; StrandsIt<StrandsCount; ++StrandsIt)
	{			
		const FIntVector GuideIDs	 = bHasValidClosestGuide ? In.StrandsCurves.CurvesClosestGuideIDs[StrandsIt] : FIntVector(-1,-1,-1);
		const FVector3f GuideWeights = bHasValidClosestGuide ? (FVector3f)In.StrandsCurves.CurvesClosestGuideIDs[StrandsIt] : FVector3f::ZeroVector;

		Out[StrandsIt].StrandID		= bHasValidStrandsIDs ? In.StrandsCurves.StrandIDs[StrandsIt] : StrandsIt;
		Out[StrandsIt].ClumpID		= bHasValidClumpIDs ? In.StrandsCurves.ClumpIDs[StrandsIt].X : 0;
		Out[StrandsIt].RootUV		= bHasValidRootUVs ? In.StrandsCurves.CurvesRootUV[StrandsIt] : FVector2f(0.0f,0.0f);
		Out[StrandsIt].GuideIDs[0]  = GuideIDs.X;
		Out[StrandsIt].GuideIDs[1]  = GuideIDs.Y;
		Out[StrandsIt].GuideIDs[2]  = GuideIDs.Z;
		Out[StrandsIt].GuideWeights[0] = GuideWeights.X;
		Out[StrandsIt].GuideWeights[1] = GuideWeights.Y;
		Out[StrandsIt].GuideWeights[2] = GuideWeights.Z;

		Out[StrandsIt].bHasStrandID = bHasValidStrandsIDs;
		Out[StrandsIt].bHasClumpID = bHasValidClumpIDs;
		Out[StrandsIt].bHasClosestGuide = bHasValidClosestGuide;
		Out[StrandsIt].bHasRootUV = bHasValidRootUVs;

		const uint32 PointCount  = In.StrandsCurves.CurvesCount[StrandsIt];
		const uint32 PointOffset = In.StrandsCurves.CurvesOffset[StrandsIt];
		const uint32 MaxPoint = PointOffset + PointCount - 1;

		const bool bHasValidColor = (In.StrandsPoints.PointsBaseColor.IsValidIndex(MaxPoint));
		const bool bHasValidRoughness = (In.StrandsPoints.PointsRoughness.IsValidIndex(MaxPoint));
		const bool bHasValidAO = (In.StrandsPoints.PointsAO.IsValidIndex(MaxPoint));

		Out[StrandsIt].ControlPoints.SetNum(PointCount);
		for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
		{				
			const uint32 EffectiveIndex = PointIt + PointOffset;
			Out[StrandsIt].ControlPoints[PointIt].Position	= In.StrandsPoints.PointsPosition[EffectiveIndex];
			Out[StrandsIt].ControlPoints[PointIt].U			= In.StrandsPoints.PointsCoordU[EffectiveIndex];
			Out[StrandsIt].ControlPoints[PointIt].Radius	= In.StrandsPoints.PointsRadius[EffectiveIndex];

			Out[StrandsIt].ControlPoints[PointIt].bHasColor = bHasValidColor;
			Out[StrandsIt].ControlPoints[PointIt].bHasRoughness = bHasValidRoughness;
			Out[StrandsIt].ControlPoints[PointIt].bHasAO = bHasValidAO;
			
			if(bHasValidColor)
			{
				Out[StrandsIt].ControlPoints[PointIt].BaseColor	= In.StrandsPoints.PointsBaseColor[EffectiveIndex];
			}
			if(bHasValidRoughness)
			{
				Out[StrandsIt].ControlPoints[PointIt].Roughness	= In.StrandsPoints.PointsRoughness[EffectiveIndex];
			}
			if(bHasValidAO)
			{
				Out[StrandsIt].ControlPoints[PointIt].AO		= In.StrandsPoints.PointsAO[EffectiveIndex];
			}
		}
	}
}

static void EditableGroomGroupToHairDescription(FHairDescription& OutHairDescription, const TArray<FEditableHairGuide>& In, uint32 GroupID, const FName& GroupName, uint32& CurveIndex, uint32& VertexIndex)
{
	for (const FEditableHairGuide& Guide : In)
	{
		FStrandID StrandID = (FStrandID)CurveIndex++;

		const uint32 PointCount = Guide.ControlPoints.Num();
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::VertexCount, (int32)PointCount);
		if(Guide.bHasRootUV)
		{ 
			SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::RootUV,      Guide.RootUV);
		}
		if (Guide.bHasGuideID)
		{
			SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::ID,          int(Guide.GuideID));
		}
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::GroupID,     int(GroupID));
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::GroupName,   GroupName);
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::Guide,       1);

		for (const FEditableHairGuideControlPoint& Point : Guide.ControlPoints)
		{
			//FVertexID VertexID = OutHairDescription.AddVertex();
			FVertexID VertexID = (FVertexID)VertexIndex++; // FVertexID VertexID = OutHairDescription.AddVertex();
			SetHairVertexAttribute(OutHairDescription, VertexID, HairAttribute::Vertex::Position,  Point.Position);
		}
	}
}

static void EditableGroomGroupToHairDescription(FHairDescription& OutHairDescription, const TArray<FEditableHairStrand>& In, uint32 GroupID, const FName& GroupName, uint32& CurveIndex, uint32& VertexIndex)
{
	// Check if the groom as precomputed weights
	bool bHasSingleWeights = false;
	bool bHasTripleWeights = false;
	for (const FEditableHairStrand& Strands : In)
	{
		bHasTripleWeights = bHasTripleWeights || (Strands.GuideIDs[0] != -1 && Strands.GuideIDs[1] != -1 && Strands.GuideIDs[2] != -1);
		bHasSingleWeights = bHasSingleWeights || (Strands.GuideIDs[0] != -1);
	}

	for (const FEditableHairStrand& Strands : In)
	{
		FStrandID StrandID = (FStrandID)CurveIndex++;

		const uint32 PointCount = Strands.ControlPoints.Num();
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::VertexCount, (int32)PointCount);
		if(Strands.bHasRootUV)
		{ 
			SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::RootUV,      Strands.RootUV);
		}
		if (Strands.bHasStrandID)
		{
			SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::ID,          int(Strands.StrandID));
		}
		if (Strands.bHasClumpID)
		{
			SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::ClumpID,     int(Strands.ClumpID));
		}
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::GroupID,     int(GroupID));
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::GroupName,   GroupName);
		SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::Guide,       0);

		if (Strands.bHasClosestGuide)
		{
			if (bHasTripleWeights)
			{
				const FVector3f ClosestGuides = FVector3f(Strands.GuideIDs[0], Strands.GuideIDs[1], Strands.GuideIDs[2]);
				const FVector3f GuideWeights = FVector3f(Strands.GuideWeights[0], Strands.GuideWeights[1], Strands.GuideWeights[2]);
				SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::ClosestGuides, ClosestGuides);
				SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::GuideWeights, GuideWeights);
			}
			else if (bHasSingleWeights)
			{
				int ClosestGuide = Strands.GuideIDs[0];
				float GuideWeight = Strands.GuideWeights[0];
				SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::ClosestGuides, ClosestGuide);
				SetHairStrandAttribute(OutHairDescription, StrandID, HairAttribute::Strand::GuideWeights, GuideWeight);
			}
		}

		for (const FEditableHairStrandControlPoint& Point : Strands.ControlPoints)
		{
			FVertexID VertexID = (FVertexID)VertexIndex++;
			SetHairVertexAttribute(OutHairDescription, VertexID, HairAttribute::Vertex::Position,  Point.Position);
			SetHairVertexAttribute(OutHairDescription, VertexID, HairAttribute::Vertex::Width,     Point.Radius * 2.f);
			if(Point.bHasColor)
			{ 
				SetHairVertexAttribute(OutHairDescription, VertexID, HairAttribute::Vertex::Color,     FVector3f(Point.BaseColor.R, Point.BaseColor.G, Point.BaseColor.B));
			}
			if (Point.bHasRoughness)
			{
				SetHairVertexAttribute(OutHairDescription, VertexID, HairAttribute::Vertex::Roughness, Point.Roughness);
			}
			if (Point.bHasAO)
			{
				SetHairVertexAttribute(OutHairDescription, VertexID, HairAttribute::Vertex::AO,        Point.AO);
			}
		}
	}
}
#endif // WITH_EDITORONLY_DATA

void ConvertFromGroomAsset(UGroomAsset* InGroom, FEditableGroom* OutGroom, const bool bAllowCurveReordering, const bool bApplyDecimation, const bool bAllowAddEndControlPoint)
{
#if WITH_EDITORONLY_DATA
	check(InGroom);
	check(OutGroom);
	UGroomAsset& In = *InGroom;
	FEditableGroom& Out = *OutGroom;

	FHairDescriptionGroups HairDescriptionGroups;
	FGroomBuilder::BuildHairDescriptionGroups(In.LoadHairDescription(EHairDescriptionType::Source), HairDescriptionGroups, bAllowAddEndControlPoint);

	const uint32 GroupCount = In.GetNumHairGroups();
	if(GroupCount == HairDescriptionGroups.HairGroups.Num() && GroupCount == In.GetHairGroupsInterpolation().Num())
	{ 
		Out.Groups.SetNum(GroupCount);

		for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
		{
			FHairGroupInfo GroupInfo;
			FHairStrandsDatas StrandsData;
			FHairStrandsDatas GuidesData;
			FGroomBuilder::BuildData(HairDescriptionGroups.HairGroups[GroupIndex], In.GetHairGroupsInterpolation()[GroupIndex], GroupInfo, StrandsData, GuidesData, bAllowCurveReordering, bApplyDecimation);

			Out.Groups[GroupIndex].GroupIndex = GroupIndex;
			Out.Groups[GroupIndex].GroupID = GroupInfo.GroupID;
			Out.Groups[GroupIndex].GroupName = GroupInfo.GroupName;
			HairStrandsDataToEditableHairGuides(GuidesData, Out.Groups[GroupIndex].Guides);

			HairStrandsDataToEditableHairStrands(StrandsData, Out.Groups[GroupIndex].Strands);
		}
	}
	else
	{
		Out.Groups.Reset();
	}
#endif
}

void ConvertToGroomAsset(UGroomAsset* OutGroom, const FEditableGroom* InGroom, uint32 Operations)
{
#if WITH_EDITORONLY_DATA
	check(InGroom);
	check(OutGroom);
	const FEditableGroom& In = *InGroom;
	const UGroomAsset& Out = *OutGroom;

	// If group count differs ensure, extent/shrink it accordingly to the edited groom
	const uint32 GroupCount = InGroom->Groups.Num();
	if (uint32(OutGroom->GetNumHairGroups()) != GroupCount)
	{
		OutGroom->SetNumGroup(GroupCount, false, false);
	}

	// Count strands/vertex for later allocation
	uint32 VertexCount = 0;
	uint32 StrandCount = 0;
	for (const FEditableGroomGroup& Group : In.Groups)
	{
		StrandCount += Group.Strands.Num();
		StrandCount += Group.Guides.Num();
		for (const FEditableHairStrand& Strand : Group.Strands)
		{
			VertexCount += Strand.ControlPoints.Num();
		}
		for (const FEditableHairGuide& Guide : Group.Guides)
		{
			VertexCount += Guide.ControlPoints.Num();
		}
	}

	// Create hair description and allocate vertex/strands
	FHairDescription HairDescription;
	HairDescription.InitializeStrands(StrandCount);
	HairDescription.InitializeVertices(VertexCount);

	// Convert guides/strands to the final
	uint32 CurveIndex = 0;
	uint32 VertexIndex = 0;
	for (const FEditableGroomGroup& Group : In.Groups)
	{
		EditableGroomGroupToHairDescription(HairDescription, Group.Guides,  Group.GroupID, Group.GroupName, CurveIndex, VertexIndex);
		EditableGroomGroupToHairDescription(HairDescription, Group.Strands, Group.GroupID, Group.GroupName, CurveIndex, VertexIndex);
	}

	// Commit & build data for the groom asset
	// Internally, this will derived the build data, recreate resources, and 
	// recreate all components using this groom assets
	OutGroom->CommitHairDescription(MoveTemp(HairDescription), EHairDescriptionType::Edit);
	OutGroom->CacheDerivedDatas();
	OutGroom->OnGroomAssetChanged.Broadcast();
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Examples / Tests (Temporary)

#if WITH_EDITORONLY_DATA
static int32 GHairStrandsEditTestCase = 0;
static FAutoConsoleVariableRef CVarHairStrandsEditTestCase(TEXT("r.HairStrands.Edit.TestCase"), GHairStrandsEditTestCase, TEXT("Test case for testing the groom edit API"));

// Test 1: Change radius of all strands
static void InternalEditTest1(UGroomAsset* InGroom)
{
	FEditableGroom EditGroom;
	ConvertFromGroomAsset(InGroom, &EditGroom);

	// Override the radius of all vertices
	for (FEditableGroomGroup& G : EditGroom.Groups)
	{
		for (FEditableHairStrand& S : G.Strands)
		{
			for (FEditableHairStrandControlPoint& C : S.ControlPoints)
			{
				C.Radius = 1.0f;
			}
		}
	}

	ConvertToGroomAsset(InGroom, &EditGroom, EEditableGroomOperations::ControlPoints_Modified);
}

// Test 2: Orient all strands based on their root direction
static void InternalEditTest2(UGroomAsset* InGroom)
{
	FEditableGroom EditGroom;
	ConvertFromGroomAsset(InGroom, &EditGroom);

	// Override the radius of all vertices
	for (FEditableGroomGroup& G : EditGroom.Groups)
	{
		for (FEditableHairStrand& S : G.Strands)
		{
			const uint32 PointCount = S.ControlPoints.Num();
			// Compute normal direction based
			FVector3f Direction = FVector3f(0, 0, 1);
			if (PointCount >= 2)
			{
				Direction = S.ControlPoints[1].Position - S.ControlPoints[0].Position;
				Direction.Normalize();
			}

			for (uint32 PointIt = 2; PointIt < PointCount; ++PointIt)
			{
				const float Distance = FVector3f::Distance(S.ControlPoints[PointIt].Position, S.ControlPoints[PointIt-1].Position);
				S.ControlPoints[PointIt].Position = S.ControlPoints[PointIt - 1].Position + Direction * Distance;
			}
		}
	}

	ConvertToGroomAsset(InGroom, &EditGroom, EEditableGroomOperations::ControlPoints_Modified);
}

// Test 3: Remove some strands
static void InternalEditTest3(UGroomAsset* InGroom)
{
	FEditableGroom EditGroom;
	ConvertFromGroomAsset(InGroom, &EditGroom);

	// Override the radius of all vertices
	for (FEditableGroomGroup& G : EditGroom.Groups)
	{
		G.Strands.SetNum(uint32(G.Strands.Num() * 0.1f));
	}

	ConvertToGroomAsset(InGroom, &EditGroom, EEditableGroomOperations::Strands_Deleted);
}

// Test 4: Add a new group
static void InternalEditTest4(UGroomAsset* InGroom)
{
	FEditableGroom EditGroom;
	ConvertFromGroomAsset(InGroom, &EditGroom);

	// Override the radius of all vertices
	if (EditGroom.Groups.Num() > 0)
	{
		const uint32 GroupIt = EditGroom.Groups.Num();
		EditGroom.Groups.SetNum(EditGroom.Groups.Num() + 1);

		const FEditableGroomGroup& SrcG = EditGroom.Groups[0];
		FEditableGroomGroup& DstG = EditGroom.Groups[GroupIt];

		DstG.GroupIndex = GroupIt;
		DstG.GroupID = GroupIt;
		DstG.GroupName = FName(TEXT("MyNewGroup"));

		const FVector3f Translation = FVector3f(100, 0, 0);
		for (const FEditableHairGuide& SrcS : SrcG.Guides)
		{
			DstG.Guides.Add(SrcS);
			for (FEditableHairGuideControlPoint& C : DstG.Guides[DstG.Guides.Num()-1].ControlPoints)
			{
				C.Position += Translation;
			}
		}
	
		for (const FEditableHairStrand& SrcS : SrcG.Strands)
		{
			DstG.Strands.Add(SrcS);
			for (FEditableHairStrandControlPoint& C : DstG.Strands[DstG.Strands.Num() - 1].ControlPoints)
			{
				C.Position += Translation;
			}
		}
	}

	ConvertToGroomAsset(InGroom, &EditGroom, EEditableGroomOperations::Group_Added);
}

// Test 5: Remove new group
static void InternalEditTest5(UGroomAsset* InGroom)
{
	FEditableGroom EditGroom;
	ConvertFromGroomAsset(InGroom, &EditGroom);

	// Override the radius of all vertices
	if (EditGroom.Groups.Num() > 1)
	{
		EditGroom.Groups.SetNum(EditGroom.Groups.Num() - 1);
	}

	ConvertToGroomAsset(InGroom, &EditGroom, EEditableGroomOperations::Group_Deleted);
}

void EditTest(UGroomAsset* InGroom)
{
	switch (GHairStrandsEditTestCase)
	{
		case 1: InternalEditTest1(InGroom); break;
		case 2: InternalEditTest2(InGroom); break;
		case 3: InternalEditTest3(InGroom); break;
		case 4: InternalEditTest4(InGroom); break;
		case 5: InternalEditTest5(InGroom); break;
	}
}

#endif // WITH_EDITORONLY_DATA