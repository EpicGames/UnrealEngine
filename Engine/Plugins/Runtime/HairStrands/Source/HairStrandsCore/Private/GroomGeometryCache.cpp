// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomGeometryCache.h"
#include "HairStrandsMeshProjection.h"

#include "GeometryCacheComponent.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheSceneProxy.h"
#include "CachedGeometry.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "CommonRenderResources.h"
#include "Components/SkeletalMeshComponent.h"
#include "SkeletalMeshSceneProxy.h"
#include "SkeletalRenderPublic.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "RenderGraphUtils.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "HairStrandsDatas.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

 void GetCachedGeometry(
	 FRDGBuilder& GraphBuilder,
	 FGlobalShaderMap* ShaderMap, 
	 const FGeometryCacheSceneProxy* SceneProxy,
	 const bool bOutputTriangleData,
	 FCachedGeometry& Out)
 {
	if (SceneProxy)
	{
		Out.LODIndex = 0;
		if (bOutputTriangleData)
		{
			// Prior to getting here, the GeometryCache has been validated to be flattened (ie. has only one track)
			check(SceneProxy->GetTracks().Num() > 0);
			const FGeomCacheTrackProxy* TrackProxy = SceneProxy->GetTracks()[0];

			check(TrackProxy->MeshData && TrackProxy->NextFrameMeshData);
			const bool bHasMotionVectors = (
				TrackProxy->MeshData->VertexInfo.bHasMotionVectors &&
				TrackProxy->NextFrameMeshData->VertexInfo.bHasMotionVectors &&
				TrackProxy->MeshData->Positions.Num() == TrackProxy->MeshData->MotionVectors.Num())
				&& (TrackProxy->NextFrameMeshData->Positions.Num() == TrackProxy->NextFrameMeshData->MotionVectors.Num());

			// PositionBuffer depends on CurrentPositionBufferIndex and on if the cache has motion vectors
			const uint32 PositionIndex = (TrackProxy->CurrentPositionBufferIndex == -1 || bHasMotionVectors) ? 0 : TrackProxy->CurrentPositionBufferIndex % 2;
			for (int32 SectionIdx = 0; SectionIdx < TrackProxy->MeshData->BatchesInfo.Num(); ++SectionIdx)
			{
				const FGeometryCacheMeshBatchInfo& BatchInfo = TrackProxy->MeshData->BatchesInfo[SectionIdx];
			
				FCachedGeometry::Section OutSection;
				OutSection.PositionBuffer = TrackProxy->PositionBuffers[PositionIndex].GetBufferSRV();
				OutSection.UVsBuffer = TrackProxy->TextureCoordinatesBuffer.GetBufferSRV();
				OutSection.TangentBuffer = TrackProxy->TangentZBuffer.GetBufferSRV();
				OutSection.TotalVertexCount = TrackProxy->MeshData->Positions.Num();
				OutSection.IndexBuffer = TrackProxy->IndexBuffer.GetBufferSRV();
				OutSection.TotalIndexCount = TrackProxy->IndexBuffer.NumValidIndices;
				OutSection.UVsChannelCount = 1;
				OutSection.NumPrimitives = BatchInfo.NumTriangles;
				OutSection.NumVertices = TrackProxy->MeshData->Positions.Num();
				OutSection.IndexBaseIndex = BatchInfo.StartIndex;
				OutSection.VertexBaseIndex = 0;
				OutSection.SectionIndex = SectionIdx;
				OutSection.LODIndex = 0;
				OutSection.UVsChannelOffset = 0;
				OutSection.TangentFormat = 1u;
			
				if (OutSection.PositionBuffer && OutSection.IndexBuffer)
				{
					Out.Sections.Add(OutSection);
				}
			}
		}
	}
 }
