// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HairStrandsMeshProjection.h"
#include "CachedGeometry.h"

struct FCachedGeometry;
class FGlobalShaderMap;
class FRDGBuilder;
class FGeometryCacheSceneProxy;

void BuildCacheGeometry(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap, 
	const FGeometryCacheSceneProxy* Proxy,
	const bool bOutputTriangleData,
	FCachedGeometry& OutCachedGeometry);