// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomImportOptions.h"
#include "GroomAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomImportOptions)

UGroomImportOptions::UGroomImportOptions(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UGroomHairGroupsPreview::UGroomHairGroupsPreview(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UGroomHairGroupsMapping::UGroomHairGroupsMapping(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

const TArray<TSharedPtr<FString>>& UGroomHairGroupsMapping::GetOldGroupNames()
{
	if (CachedOldGroupNames.Num() == 0)
	{
		for (uint32 GroupIt = 0, GroupCount=OldGroupNames.Num(); GroupIt < GroupCount; ++GroupIt)
		{
			CachedOldGroupNames.Add(MakeShared<FString>(OldGroupNames[GroupIt].ToString()));
		}
		CachedOldGroupNames.Add(MakeShared<FString>(TEXT("Default")));
	}
	return CachedOldGroupNames;
}

bool UGroomHairGroupsMapping::HasValidMapping() const
{
	for (int32 Index : NewToOldGroupIndexMapping)
	{
		if (Index != -1)
		{
			return true;
		}
	}

	return false;
}

void UGroomHairGroupsMapping::Map(const FHairDescriptionGroups& OldGroups, const FHairDescriptionGroups& NewGroups)
{
	// New group names
	for (const FHairDescriptionGroup& NewGroup : NewGroups.HairGroups)
	{
		NewGroupNames.Add(NewGroup.Info.GroupName);
	}

	// Old group names
	for (const FHairDescriptionGroup& OldGroup : OldGroups.HairGroups)
	{
		OldGroupNames.Add(OldGroup.Info.GroupName);
	}

	// * Old->New mapping
	RemapHairDescriptionGroups(OldGroups, NewGroups, OldToNewGroupIndexMapping);

	// * New->Old mapping
	RemapHairDescriptionGroups(NewGroups, OldGroups, NewToOldGroupIndexMapping);
}

// Src -> Dst mapping
void UGroomHairGroupsMapping::RemapHairDescriptionGroups(
	const FHairDescriptionGroups& SrcGroups, 
	const FHairDescriptionGroups& DstGroups, 
	TArray<int32>& Out)
{
	Out.Init(-1, SrcGroups.HairGroups.Num());

	// 2. Remap Src settings to the Dst asset (use GroupName/GroomID to do the remapping)
	for (const FHairDescriptionGroup& SrcGroup : SrcGroups.HairGroups)
	{
		int32 MatchGroupIndex = -1;
		for (const FHairDescriptionGroup& DstGroup : DstGroups.HairGroups)
		{
			// For now, only use GroupName, as if no group name is provided, the group name is built from the GroupID
			if (DstGroup.Info.GroupName == SrcGroup.Info.GroupName)
			{
				MatchGroupIndex = DstGroup.Info.GroupIndex;
				break;
			}
		}

		Out[SrcGroup.Info.GroupIndex] = MatchGroupIndex;
	}
}

void UGroomHairGroupsMapping::SetIndex(int32 InNewIndex, int32 InOldIndex)
{
	if (NewGroupNames.IsValidIndex(InNewIndex))
	{
		NewToOldGroupIndexMapping[InNewIndex] = OldGroupNames.IsValidIndex(InOldIndex) ? InOldIndex : -1;
	}

	if (OldGroupNames.IsValidIndex(InOldIndex))
	{
		OldToNewGroupIndexMapping[InOldIndex] = NewGroupNames.IsValidIndex(InNewIndex) ? InNewIndex : -1;
	}
}

UGroomImportOptions* CreateGroomImportOptions(const FHairDescriptionGroups& GroupsDescription, const TArray<FHairGroupsInterpolation>& BuildSettings)
{
	// Create a new groom import options and populate the interpolation settings based on the group count
	UGroomImportOptions* ImportOptions = NewObject<UGroomImportOptions>();
	const uint32 GroupCount = GroupsDescription.HairGroups.Num();
	if (GroupCount != uint32(ImportOptions->InterpolationSettings.Num()))
	{
		ImportOptions->InterpolationSettings.Init(FHairGroupsInterpolation(), GroupCount);
	}

	TOptional<FHairGroupsInterpolation> LastBuildSettings;
	if (BuildSettings.Num() > 0)
	{
		LastBuildSettings = BuildSettings.Last();
	}

	// If there are less build settings than groups, use the last one that was specified by the user
	for (uint32 Index = 0; Index < GroupCount; ++Index)
	{
		if (BuildSettings.IsValidIndex(Index))
		{
			ImportOptions->InterpolationSettings[Index] = BuildSettings[Index];
		}
		else if (LastBuildSettings.IsSet())
		{
			ImportOptions->InterpolationSettings[Index] = LastBuildSettings.GetValue();
		}
	}
	return ImportOptions;
}
