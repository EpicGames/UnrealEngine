// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomManager.h"
#include "HairStrandsMeshProjection.h"

#include "GeometryCacheComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "CommonRenderResources.h"
#include "Components/SkeletalMeshComponent.h"
#include "SkeletalMeshSceneProxy.h"
#include "SkeletalRenderPublic.h"
#include "GroomTextureBuilder.h"
#include "GroomResources.h"
#include "GroomInstance.h"
#include "GroomGeometryCache.h"
#include "GeometryCacheSceneProxy.h"
#include "HAL/IConsoleManager.h"
#include "SceneView.h"
#include "HairStrandsInterpolation.h"
#include "HairCardsVertexFactory.h"
#include "HairStrandsVertexFactory.h"
#include "RenderGraphEvent.h"
#include "RenderGraphUtils.h"
#include "CachedGeometry.h"
#include "GroomCacheData.h"
#include "SceneInterface.h"
#include "PrimitiveSceneInfo.h"
#include "HairStrandsClusterCulling.h"
#include "GroomVisualizationData.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "HairStrandsInterface.h"
#include "ShaderPlatformCachedIniValue.h"
#include "SkeletalMeshUpdater.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "ShaderCompilerCore.h"
#include "Shader.h"
#include "ShaderPrintParameters.h"

DECLARE_GPU_STAT(HairStrandsInterpolationCurve);
DECLARE_GPU_STAT(HairGuideInterpolation);
DECLARE_GPU_STAT(HairStrandsClusterCulling);
DECLARE_GPU_STAT(HairStrandsInterpolation);
DECLARE_GPU_STAT(HairCardsInterpolation);

static int32 GHairStrandsMinLOD = 0;
static FAutoConsoleVariableRef CVarGHairStrandsMinLOD(TEXT("r.HairStrands.MinLOD"), GHairStrandsMinLOD, TEXT("Clamp the min hair LOD to this value, preventing to reach lower/high-quality LOD."), ECVF_Scalability);

static int32 GHairStrands_UseCards = 0;
static FAutoConsoleVariableRef CVarHairStrands_UseCards(TEXT("r.HairStrands.UseCardsInsteadOfStrands"), GHairStrands_UseCards, TEXT("Force cards geometry on all groom elements. If no cards data is available, nothing will be displayed"), ECVF_Scalability);

static int32 GHairStrands_SwapBufferType = 2;
static FAutoConsoleVariableRef CVarGHairStrands_SwapBufferType(TEXT("r.HairStrands.SwapType"), GHairStrands_SwapBufferType, TEXT("Swap rendering buffer at the end of frame. This is an experimental toggle. Default:1"));

static int32 GHairStrands_ManualSkinCache = 1;
static FAutoConsoleVariableRef CVarGHairStrands_ManualSkinCache(TEXT("r.HairStrands.ManualSkinCache"), GHairStrands_ManualSkinCache, TEXT("If skin cache is not enabled, and grooms use skinning method, this enable a simple skin cache mechanisme for groom. Default:disable"));

static int32 GHairStrands_InterpolationFrustumCullingEnable = 1;
static FAutoConsoleVariableRef CVarHairStrands_InterpolationFrustumCullingEnable(TEXT("r.HairStrands.Interoplation.FrustumCulling"), GHairStrands_InterpolationFrustumCullingEnable, TEXT("Swap rendering buffer at the end of frame. This is an experimental toggle. Default:1"));

static int32 GHairStrands_Streaming = 0;
static FAutoConsoleVariableRef CVarHairStrands_Streaming(TEXT("r.HairStrands.Streaming"), GHairStrands_Streaming, TEXT("Hair strands streaming toggle."), ECVF_RenderThreadSafe | ECVF_ReadOnly);
static int32 GHairStrands_Streaming_CurvePage = 2048;
static FAutoConsoleVariableRef CVarHairStrands_StreamingCurvePage(TEXT("r.HairStrands.Streaming.CurvePage"), GHairStrands_Streaming_CurvePage, TEXT("Number of strands curve per streaming page"));
static int32 GHairStrands_Streaming_StreamOutThreshold = 2;
static FAutoConsoleVariableRef CVarHairStrands_Streaming_StreamOutThreshold(TEXT("r.HairStrands.Streaming.StreamOutThreshold"), GHairStrands_Streaming_StreamOutThreshold, TEXT("Threshold used for streaming out data. In curve page. Default:2."));

static int32 GHairStrands_AutoLOD_Force = 0;
static FAutoConsoleVariableRef CVarHairStrands_AutoLOD_Force(TEXT("r.HairStrands.AutoLOD.Force"), GHairStrands_AutoLOD_Force, TEXT("Force Auto LOD on all grooms. Used for debugging purpose."), ECVF_RenderThreadSafe);
static float GHairStrands_AutoLOD_Bias = 0.1f;
static FAutoConsoleVariableRef CVarHairStrands_AutoLOD_Bias(TEXT("r.HairStrands.AutoLOD.Bias"), GHairStrands_AutoLOD_Bias, TEXT("Global bias for Auto LOD on all grooms. Used for debugging purpose."), ECVF_RenderThreadSafe);

bool UseHairStrandsForceAutoLOD()
{
	return GHairStrands_AutoLOD_Force > 0;
}

float GetHairStrandsAutoLODBias()
{
	return FMath::Clamp(GHairStrands_AutoLOD_Bias, -1.f, 1.f);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Foward declaration

bool IsHairStrandsForceRebuildBVH();
bool IsHairStrandsTransferPositionOnLODChange();
uint32 GetHairRaytracingProceduralSplits();
void CreateHairStrandsDebugAttributeBuffer(FRDGBuilder& GraphBuilder, FRDGExternalBuffer* DebugAttributeBuffer, uint32 VertexCount, const FName& OwnerName);
FHairGroupPublicData::FVertexFactoryInput InternalComputeHairStrandsVertexInputData(FRDGBuilder* GraphBuilder, const FHairGroupInstance* Instance, EGroomViewMode ViewMode);
EGroomCacheType GetHairInstanceCacheType(const FHairGroupInstance* Instance);
bool GetSupportHairStrandsLinearSweptSpheres(EShaderPlatform InShaderPlatform);

void AddHairClusterAABBPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 ActiveCurveCount,
	const FVector& TranslatedWorldOffset,
	const FShaderPrintData* ShaderPrintData,
	const EHairAABBUpdateType UpdateType,
	FHairGroupInstance* Instance,
	FRDGBufferSRVRef RenderDeformedOffsetBuffer,
	FHairStrandClusterData::FHairGroup* ClusterData,
	const uint32 ClusterOffset,
	const uint32 CluesterCount,
	FRDGBufferSRVRef RenderPositionBufferSRV,
	FRDGBufferSRVRef& DrawIndirectRasterComputeBuffer,
	FRDGBufferUAVRef ClusterAABBUAV,
	FRDGBufferUAVRef GroupAABBBUAV);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Utils

bool IsHairManualSkinCacheEnabled()
{
	return GHairStrands_ManualSkinCache > 0;
}

EHairBufferSwapType GetHairSwapBufferType()
{
	switch (GHairStrands_SwapBufferType)
	{
	case 0: return EHairBufferSwapType::BeginOfFrame;
	case 1: return EHairBufferSwapType::EndOfFrame;
	case 2: return EHairBufferSwapType::Tick;
	case 3: return EHairBufferSwapType::RenderFrame;
	}
	return EHairBufferSwapType::EndOfFrame;
}

uint32 GetStreamingCurvePage()
{
	return FMath::Clamp(uint32(GHairStrands_Streaming_CurvePage), 1u, 32768u);
}

static uint32 GetRoundedCurveCount(uint32 InRequest, uint32 InMaxCurve)
{
	const uint32 Page = GetStreamingCurvePage();
	return FMath::Min(FMath::DivideAndRoundUp(InRequest, Page) * Page,  InMaxCurve);
}

bool NeedDeallocation(uint32 InRequest, uint32 InAvailable)
{
	bool bNeedDeallocate = false;
	if (GHairStrands_Streaming > 0)
	{

		const uint32 Page = GetStreamingCurvePage();
		const uint32 RequestedPageCount = FMath::DivideAndRoundUp(InRequest, Page);
		const uint32 AvailablePageCount = FMath::DivideAndRoundUp(InAvailable, Page);
	
		const uint32 DeallactionThreshold = FMath::Max(GHairStrands_Streaming_StreamOutThreshold, 2);
		bNeedDeallocate = AvailablePageCount - RequestedPageCount >= DeallactionThreshold;
	}
	return bNeedDeallocate;
}

DEFINE_LOG_CATEGORY_STATIC(LogGroomManager, Log, All);

///////////////////////////////////////////////////////////////////////////////////////////////////

void FHairGroupInstance::FCards::FLOD::InitVertexFactory()
{
	VertexFactory->InitResources(FRHICommandListImmediate::Get());
}

void FHairGroupInstance::FMeshes::FLOD::InitVertexFactory()
{
	VertexFactory->InitResources(FRHICommandListImmediate::Get());
}

static bool IsInstanceFrustumCullingEnable()
{
	return GHairStrands_InterpolationFrustumCullingEnable > 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair manual skin cache

// Retrive the skel. mesh scene info
static const FPrimitiveSceneInfo* GetMeshSceneInfo(FSceneInterface* Scene, FHairGroupInstance* Instance)
{
	if (!Instance->Debug.MeshComponentId.IsValid())
	{
		return nullptr;
	}

	// Try to use the cached persistent primitive index (fast)
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = nullptr;
	if (Instance->Debug.CachedMeshPersistentPrimitiveIndex.IsValid())
	{
		PrimitiveSceneInfo = Scene->GetPrimitiveSceneInfo(Instance->Debug.CachedMeshPersistentPrimitiveIndex);
		PrimitiveSceneInfo = (PrimitiveSceneInfo && PrimitiveSceneInfo->PrimitiveComponentId == Instance->Debug.MeshComponentId) ? PrimitiveSceneInfo : nullptr;
	}

	// If this fails, find the primitive index, and then the primitive scene info (slow)
	if (PrimitiveSceneInfo == nullptr)
	{
		const int32 PrimitiveIndex = Scene->GetScenePrimitiveComponentIds().Find(Instance->Debug.MeshComponentId);
		PrimitiveSceneInfo = Scene->GetPrimitiveSceneInfo(PrimitiveIndex);
		Instance->Debug.CachedMeshPersistentPrimitiveIndex = PrimitiveSceneInfo ? PrimitiveSceneInfo->GetPersistentIndex() : FPersistentPrimitiveIndex();
	}

	return PrimitiveSceneInfo;
}

struct FHairGeometryCacheKey
{
	uint32 ComponentId = 0;
	uint32 LODIndex = 0;
};
static FORCEINLINE bool operator!= (const FHairGeometryCacheKey& A, const FHairGeometryCacheKey& B) 
{ 
	return A.ComponentId != B.ComponentId || A.LODIndex != B.LODIndex; 
}
static FORCEINLINE uint32 GetHash(const FHairGeometryCacheKey& In) 
{ 
	return Murmur32({In.ComponentId, In.LODIndex}); 
}

static FRDGBufferRef UpdateBoneData(
	FRDGBuilder& GraphBuilder,
	const FSkeletalMeshObject* InMeshObject, 
	TConstArrayView<FMatrix44f> ReferenceToLocalMatrices,
	TConstArrayView<FBoneIndexType> BoneMap)
{
	const uint32 NumBones = BoneMap.Num();
	check(IsInParallelRenderingThread());
	check(NumBones > 0);
	
	TArray<FMatrix3x4> ChunkMatrices;
	ChunkMatrices.SetNum(NumBones);

	constexpr int32 PreFetchStride = 2; // FPlatformMisc::Prefetch stride
	for (uint32 BoneIdx = 0; BoneIdx < NumBones; BoneIdx++)
	{
		const FBoneIndexType RefToLocalIdx = BoneMap[BoneIdx];
		FPlatformMisc::Prefetch(ReferenceToLocalMatrices.GetData() + RefToLocalIdx + PreFetchStride);
		FPlatformMisc::Prefetch(ReferenceToLocalMatrices.GetData() + RefToLocalIdx + PreFetchStride, PLATFORM_CACHE_LINE_SIZE);

		FMatrix3x4& BoneMat = ChunkMatrices[BoneIdx];
		const FMatrix44f& RefToLocal = ReferenceToLocalMatrices[RefToLocalIdx];
		RefToLocal.To3x4MatrixTranspose((float*)BoneMat.M);
	}

	const uint32 BytesPerElement = sizeof(FMatrix3x4);
	FRDGBufferRef BoneMatricesBuffer = CreateVertexBuffer(GraphBuilder, TEXT("Hair.SkinnedBoneMatrices"), FRDGBufferDesc::CreateBufferDesc(BytesPerElement, ChunkMatrices.Num()), ChunkMatrices.GetData(), BytesPerElement * ChunkMatrices.Num());
	return BoneMatricesBuffer;
}

static FRDGBufferRef CreateBoneBuffer(FRDGBuilder& GraphBuilder, const FSkeletalMeshObject* InMeshObject, int32 LODIndex, int32 SectionIndex)
{
	const TArray<FSkelMeshRenderSection>& Sections = InMeshObject->GetRenderSections(LODIndex);
	if (Sections.IsValidIndex(SectionIndex))
	{
		const FSkelMeshRenderSection& Section = Sections[SectionIndex];
		if (Section.IsValid())
		{
			return UpdateBoneData(GraphBuilder, InMeshObject, InMeshObject->GetReferenceToLocalMatrices(), Section.BoneMap);
		}
	}

	return nullptr;
}

struct FHairGeometryCache
{
	enum class ECacheType
	{
		SkinCache,
		HairCache,
		GeomCache
	};

	struct FData
	{
		const FSkeletalMeshObject* MeshObject		= nullptr;
		TRefCountPtr<FSkeletalMeshLODRenderData> LODData; // Ref. count to ensure the data are not streamed out while being used
		FRDGBufferRef PositionBuffer				= nullptr;
		FRDGBufferRef PreviousPositionBuffer		= nullptr;
		FRDGBufferRef TangentBuffer					= nullptr;
		FRDGBufferSRVRef PositionSRV				= nullptr;
		FRDGBufferSRVRef PreviousPositionSRV		= nullptr;
		FRDGBufferSRVRef TangentSRV					= nullptr;
		TArray<TRefCountPtr<FRDGPooledBuffer>> BoneBuffers;

		FHairGeometryCacheKey Key;
		uint32 Hash 				= 0;
		uint32 TotalSectionCount 	= 0;
		TArray<uint32> RequestedSectionIndices;
		TArray<FSkinUpdateSection> RequestedSections;
		TBitArray<> RequestedSectionBits;
	};

	struct FDebugData
	{
		const FPrimitiveSceneProxy* Proxy = nullptr;
		FString MeshComponentName;
		int32  LODIndex = -1;
		uint32 InstanceCount = 0;
		uint32 CacheType = 0;
		uint32 GeometryType = 0;
		TBitArray<> SectionBits;
	};

	FHairGeometryCache(EGroomViewMode InViewMode)
	{
		bDebugEnable = InViewMode == EGroomViewMode::MeshProjection;
	}

	void GetOrAdd(
		FRDGBuilder& GraphBuilder, 
		const FSkeletalMeshObject* InMeshObject, 
		FSkeletalMeshLODRenderData* InLODData,
		uint32 InLODIndex, 
		const TArray<uint32>& UniqueSections,
		FRDGBufferSRVRef& Out, 
		FRDGBufferSRVRef& OutPrev,
		FRDGBufferSRVRef& OutTangent)
	{
		check(InLODData);
		check(InMeshObject);
		
		if (UniqueSections.Num() == 0)
		{
			// Binding data is valid but has no section indices
			static bool bWarningIssued = false;
			if (!bWarningIssued)
			{
				UE_LOGF(LogHairStrands, Warning, "GetOrAllocateCachedGeometry: RootResource LOD %d has no UniqueSectionIndices. Skipping.", InLODIndex);
				bWarningIssued = true;
			}
			return;
		}

		// Find existing entry
		const bool bNeedPreviousPosition = IsHairStrandContinuousDecimationReorderingEnabled();
		const FHairGeometryCacheKey Key = { InMeshObject->GetComponentId(), InLODIndex };
		const uint32 Hash = GetHash(Key);
		uint32 Index = HashTable.First(Hash);
		while (HashTable.IsValid(Index) && Datas[Index].Key != Key)
		{
			Index = HashTable.Next(Index);
		}

		// Add it if it does not exist
		if (!HashTable.IsValid(Index))
		{
			// Or add it
			Index = Datas.AddDefaulted();
			HashTable.Add(Hash, Index);

			FData& Data = Datas[Index];
			Data.MeshObject 			= InMeshObject;
			Data.LODData 				= InLODData;
			Data.Hash					= Hash;
			Data.Key 					= Key;
			Data.TotalSectionCount 		= InLODData->RenderSections.Num();
			Data.PositionBuffer 		= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float), InLODData->StaticVertexBuffers.PositionVertexBuffer.GetNumVertices() * 3), TEXT("Hair.SkinnedDeformedPositions"));;
			Data.PreviousPositionBuffer	= bNeedPreviousPosition ?  GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float), InLODData->StaticVertexBuffers.PositionVertexBuffer.GetNumVertices() * 3), TEXT("Hair.SkinnedDeformedPreviousPositions")) : nullptr;;
			Data.PositionSRV 			= GraphBuilder.CreateSRV(Data.PositionBuffer, PF_R32_FLOAT);
			Data.PreviousPositionSRV	= bNeedPreviousPosition ? GraphBuilder.CreateSRV(Data.PreviousPositionBuffer, PF_R32_FLOAT) : nullptr;

			// If available create and compute deformed tangents
			Data.TangentBuffer = nullptr;
			Data.TangentSRV = nullptr;
			if (const FRHIShaderResourceView* TangentSRV = InLODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV())
			{
				const uint32 TangentStride = InLODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis() ? 
					sizeof(TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::HighPrecision>::TangentTypeT>):
					sizeof(TStaticMeshVertexTangentDatum<typename TStaticMeshVertexTangentTypeSelector<EStaticMeshVertexTangentBasisType::Default>::TangentTypeT>);
				const EPixelFormat TangentFormat = TangentSRV->GetDesc().Buffer.SRV.Format;
				const uint32 TangentElementCount = InLODData->StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
				check(TangentSRV->GetDesc().Buffer.SRV.BufferType == FRHIViewDesc::EBufferType::Typed);
				Data.TangentBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(TangentStride, TangentElementCount), TEXT("Hair.SkinnedDeformedTangents"));
				Data.TangentSRV = GraphBuilder.CreateSRV(Data.TangentBuffer, TangentFormat);
			}
			Data.BoneBuffers.Init(nullptr, Data.TotalSectionCount);
			Data.RequestedSections.Reserve(Data.TotalSectionCount);
			Data.RequestedSectionBits.Init(false, Data.TotalSectionCount);
		}

		// Add requested sections
		FData& Data = Datas[Index];
		check(Data.PositionSRV);
		for (uint32 SectionIndex : UniqueSections)
		{
			if (!Data.RequestedSectionBits[SectionIndex])
			{
				const FSkelMeshRenderSection& Section = InLODData->RenderSections[SectionIndex];
				FSkinUpdateSection& UpdateSection 	= Data.RequestedSections.AddDefaulted_GetRef();
				UpdateSection.SectionIndex 			= SectionIndex;
				UpdateSection.NumVertexToProcess 	= Section.NumVertices;
				UpdateSection.SectionVertexBaseIndex= Section.BaseVertexIndex;

				const FSkeletalMeshObject::FBoneBufferSRV CurrentBones = InMeshObject->GetSectionBoneBuffer(InLODIndex, SectionIndex, false);
				const FSkeletalMeshObject::FBoneBufferSRV PrevBones    = InMeshObject->GetSectionBoneBuffer(InLODIndex, SectionIndex, true);
				UpdateSection.BoneBuffer 			  = CurrentBones.SRV;
				UpdateSection.BonePrevBuffer 		  = PrevBones.SRV;
				UpdateSection.BoneTransformOffset	  = CurrentBones.Offset;
				UpdateSection.PrevBoneTransformOffset = PrevBones.Offset;
				Data.RequestedSectionBits[SectionIndex] = true;

				if (UpdateSection.BoneBuffer == nullptr)
				{
					if (Data.BoneBuffers[SectionIndex] == nullptr)
					{
						FRDGBufferRef BoneBuffer = CreateBoneBuffer(GraphBuilder, InMeshObject, InLODIndex, SectionIndex);
						Data.BoneBuffers[SectionIndex] = GraphBuilder.ConvertToExternalBuffer(BoneBuffer);
					}
					UpdateSection.BoneBuffer = Data.BoneBuffers[SectionIndex]->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(PF_A32B32G32R32F));
					UpdateSection.BoneTransformOffset = 0;
					UpdateSection.PrevBoneTransformOffset = 0;
				}
				check(UpdateSection.BoneBuffer);

				UE_CLOGF(!UpdateSection.BoneBuffer, LogHairStrands, Error,
					"Failed to get bone buffer from mesh object for hair skin cache. MeshObject: %ls, LOD: %d, Section: %d, IsGPUSkinMesh %d, IsSectionValid %d",
					*InMeshObject->GetDebugName().ToString(), InLODIndex, SectionIndex, InMeshObject->IsGPUSkinMesh() ? 1 : 0, Section.IsValid() ? 1 : 0);
			}
		}

		// Initialized returned values
		Out 		= Data.PositionSRV;
		OutPrev 	= Data.PreviousPositionSRV;
		OutTangent	= Data.TangentSRV;
	}

	void AddDebug(const FHairGroupInstance* InInstance, const FPrimitiveSceneProxy* InProxy, const FCachedGeometry& InGeom, EHairPositionUpdateType InGeometryType, ECacheType InCacheType, uint32 InTotalSectionCount=0)
	{
		if (bDebugEnable)
		{
			// SkinCache or GeomCache returns the entire set of sections, while HairCache only returns the used sections
			if (InCacheType != ECacheType::HairCache)
			{
				InTotalSectionCount = InGeom.Sections.Num();
			}

			// Convert a section array into a bitfield
			auto ToBitArray = [InTotalSectionCount](const TArray<FCachedGeometry::Section>& In)
			{
				TBitArray<> Out;
				Out.Init(false, InTotalSectionCount);
				for (const FCachedGeometry::Section& S : In)
				{
					check(S.SectionIndex < InTotalSectionCount);
					Out[S.SectionIndex] = true;
				}
				return Out;
			};

			for (FDebugData& DebugData : DebugDatas)
			{
				if (DebugData.Proxy == InProxy && DebugData.LODIndex == InGeom.LODIndex)
				{
					++DebugData.InstanceCount;
					DebugData.CacheType    |= 1u << uint32(InCacheType);
					DebugData.GeometryType |= 1u << uint32(InGeometryType);
					DebugData.SectionBits = TBitArray<>::BitwiseOR(DebugData.SectionBits, ToBitArray(InGeom.Sections), EBitwiseOperatorFlags::MaxSize);
					return;
				}
			}
		
			FDebugData& DebugData = DebugDatas.AddDefaulted_GetRef();
			DebugData.MeshComponentName = InInstance->Debug.MeshComponentForDebug->GetName();
			DebugData.Proxy             = InProxy;
			DebugData.LODIndex          = InGeom.LODIndex;
			DebugData.InstanceCount     = 1;
			DebugData.CacheType        |= 1u << uint32(InCacheType);
			DebugData.GeometryType     |= 1u << uint32(InGeometryType);
			DebugData.SectionBits       = ToBitArray(InGeom.Sections);
		}
	}

	FHashTable HashTable;
	TArray<FData> Datas;

	bool bDebugEnable = false;
	TArray<FDebugData> DebugDatas;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairDebugPrintHairSkinCacheCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDebugPrintHairSkinCacheCS);
	SHADER_USE_PARAMETER_STRUCT(FHairDebugPrintHairSkinCacheCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, HairSkinCacheEnable)
		SHADER_PARAMETER(uint32, GPUSkinCacheEnable)
		SHADER_PARAMETER(uint32, InstanceCount)
		SHADER_PARAMETER(uint32, UniqueMeshCount)
		SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderParameters, UniqueMeshNames)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, Infos)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_PRINT_HAIR_SKIN_CACHE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDebugPrintHairSkinCacheCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddHairSkinCacheDebugPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FSceneView* View,
	const FShaderPrintData* ShaderPrintData,
	const FHairStrandsInstances& Instances,
	FHairGeometryCache& HairGeometryCache)
{
	if (!HairGeometryCache.bDebugEnable || !ShaderPrint::IsSupported(View->GetShaderPlatform()) || !ShaderPrintData || Instances.Num() == 0)
	{
		return;
	}

	bool bIsGPUSkinCacheEnable = false;
	{
		static FShaderPlatformCachedIniValue<int32> CVarSkinCacheCompileShader(TEXT("r.SkinCache.CompileShaders"));
		static FShaderPlatformCachedIniValue<int32> CVarSkinCacheMode(TEXT("r.SkinCache.Mode"));
		bIsGPUSkinCacheEnable = 
			CVarSkinCacheCompileShader.Get(View->GetShaderPlatform()) != 0 &&
			CVarSkinCacheMode.Get(View->GetShaderPlatform()) != 0;
	}

	const uint32 InstanceCount = Instances.Num();
	const uint32 UniqueMeshCount = HairGeometryCache.DebugDatas.Num();

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	// Request more drawing primitives & characters for printing if needed	
	ShaderPrint::RequestSpaceForLines(UniqueMeshCount * 64u);
	ShaderPrint::RequestSpaceForCharacters(UniqueMeshCount * 256 + 512);

	ShaderPrint::FStrings UniqueMeshNames(UniqueMeshCount);
	struct FUniqueMeshInfo
	{
		FUintVector4 Data0 = { 0,0,0,0 };
		FUintVector4 Data1 = { 0,0,0,0 };
	};
	TArray<FUniqueMeshInfo> Infos;
	Infos.Reserve(UniqueMeshCount);
	for (uint32 UniqueMeshIndex = 0; UniqueMeshIndex < UniqueMeshCount; ++UniqueMeshIndex)
	{
		const FHairGeometryCache::FDebugData& DebugData = HairGeometryCache.DebugDatas[UniqueMeshIndex];

		UniqueMeshNames.Add(DebugData.MeshComponentName, UniqueMeshIndex);

		FUniqueMeshInfo& D = Infos.AddDefaulted_GetRef();
		D.Data0.X = UniqueMeshIndex;
		D.Data0.Y = DebugData.GeometryType;
		D.Data0.Z = DebugData.CacheType;
		D.Data0.W = DebugData.InstanceCount;

		D.Data1.X = DebugData.SectionBits.CountSetBits();
		D.Data1.Y = DebugData.SectionBits.Num();
		D.Data1.Z = DebugData.LODIndex;
		D.Data1.W = 0;
	}

	// Add dummy values for creating non-empty buffer
	if (UniqueMeshCount == 0)
	{
		UniqueMeshNames.Add(FString(TEXT("DummyName")), 0u);
		Infos.Add(FUniqueMeshInfo());
	}

	const uint32 InfoInBytes  = sizeof(FUniqueMeshInfo);
	const uint32 InfoInUints = sizeof(FUniqueMeshInfo) / sizeof(uint32);
	FRDGBufferRef InfoBuffer = CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.UniqueMeshNames"), FRDGBufferDesc::CreateBufferDesc(4, InfoInUints * Infos.Num()), Infos.GetData(), InfoInBytes * Infos.Num());

	FHairDebugPrintHairSkinCacheCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDebugPrintHairSkinCacheCS::FParameters>();
	Parameters->HairSkinCacheEnable = IsHairManualSkinCacheEnabled() ? 1u : 0u;
	Parameters->GPUSkinCacheEnable	= bIsGPUSkinCacheEnable ? 1u : 0u;
	Parameters->InstanceCount 		= InstanceCount;
	Parameters->UniqueMeshCount 	= UniqueMeshCount;
	Parameters->UniqueMeshNames 	= UniqueMeshNames.GetParameters(GraphBuilder);
	Parameters->Infos 				= GraphBuilder.CreateSRV(InfoBuffer, PF_R32_UINT);
	ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
	TShaderMapRef<FHairDebugPrintHairSkinCacheCS> ComputeShader(ShaderMap);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::DebugPrintSkinCache"),
		ComputeShader,
		Parameters,
		FIntVector(1, 1, 1));
}

static void GetOrAllocateCachedGeometry(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap, 
	const FSkeletalMeshSceneProxy* Proxy,
	const FHairStrandsRestRootResource* RootResource,
	const bool bOutputTriangleData,
	FCachedGeometry& Out,
	FHairGeometryCache& OutHairGeometryCache,
	uint32& OutTotalSectionCount)
{
	if (Proxy == nullptr)
	{
		return;
	}
	
	const FSkeletalMeshObject* SkeletalMeshObject = Proxy->GetMeshObject();
	if (SkeletalMeshObject == nullptr)
	{
		return;
	}

	const int32 LODIndex = SkeletalMeshObject->GetLOD();
	Out.LODIndex = LODIndex;

	FSkeletalMeshLODRenderData& LODData = SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[LODIndex];
	OutTotalSectionCount = LODData.RenderSections.Num();
	if (!bOutputTriangleData || OutTotalSectionCount == 0)
	{
		return;
	}

	check(RootResource);
	if (!RootResource->IsDataValid(LODIndex))
	{
		return;
	}
	const TArray<uint32>& UniqueSections = RootResource->GetLOD(LODIndex)->GetBulkData().Header.UniqueSectionIndices;

	// Create deformed position buffer (output)
	FRDGBufferSRVRef DeformedPositionSRV = nullptr;
	FRDGBufferSRVRef DeformedPreviousPositionSRV = nullptr;
	FRDGBufferSRVRef DeformedTangentSRV = nullptr;
	OutHairGeometryCache.GetOrAdd(GraphBuilder, SkeletalMeshObject, &LODData, LODIndex, UniqueSections, DeformedPositionSRV, DeformedPreviousPositionSRV, DeformedTangentSRV);

	// Fill in result
	for (uint32 SectionIndex : UniqueSections)
	{
		FCachedGeometry::Section& OutSection= Out.Sections.AddDefaulted_GetRef();
		OutSection.RDGPositionBuffer 		= DeformedPositionSRV;
		OutSection.RDGPreviousPositionBuffer= DeformedPreviousPositionSRV;
		OutSection.RDGTangentBuffer			= DeformedTangentSRV;
		OutSection.PositionBuffer 			= nullptr; // Do not use the SRV slot, but instead use the RDG buffer created above (DeformedPositionSRV)
		OutSection.PreviousPositionBuffer 	= nullptr; // Do not use the SRV slot, but instead use the RDG buffer created above (DeformedPositionSRV)
		OutSection.UVsBuffer 				= LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();
		OutSection.TangentBuffer 			= LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
		OutSection.TotalVertexCount 		= LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
		OutSection.IndexBuffer 				= LODData.MultiSizeIndexContainer.GetIndexBuffer()->GetSRV();
		OutSection.TotalIndexCount 			= LODData.MultiSizeIndexContainer.GetIndexBuffer()->Num();
		OutSection.UVsChannelCount 			= LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		OutSection.NumPrimitives 			= LODData.RenderSections[SectionIndex].NumTriangles;
		OutSection.NumVertices 				= LODData.RenderSections[SectionIndex].NumVertices;
		OutSection.IndexBaseIndex 			= LODData.RenderSections[SectionIndex].BaseIndex;
		OutSection.VertexBaseIndex 			= LODData.RenderSections[SectionIndex].BaseVertexIndex;
		OutSection.SectionIndex 			= SectionIndex;
		OutSection.LODIndex 				= LODIndex;
		OutSection.UVsChannelOffset 		= 0; // Assume that we needs to pair meshes based on UVs 0
	}
}

static int32 GetMeshLODIndex(
	FRDGBuilder& GraphBuilder, 
	FGlobalShaderMap* ShaderMap,
	FSceneInterface* Scene,
	FHairGroupInstance* Instance,
	FTransform& OutMeshLODLocalToWorld)
{
	int32 OutLODIndex = -1;
	if (const FPrimitiveSceneInfo* PrimitiveSceneInfo = GetMeshSceneInfo(Scene, Instance))
	{
		if (Instance->Debug.GroomBindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			// For now, binding on Nanite mesh are not supported
			if (PrimitiveSceneInfo->Proxy && !PrimitiveSceneInfo->Proxy->IsNaniteMesh())
			{
				if (const FSkeletalMeshSceneProxy* SceneProxy = static_cast<const FSkeletalMeshSceneProxy*>(PrimitiveSceneInfo->Proxy))
				{
					OutLODIndex = SceneProxy->GetMeshObject() ? SceneProxy->GetMeshObject()->GetLOD() : -1;
				}
			}
		}
		else if (Instance->Debug.GroomBindingType == EGroomBindingMeshType::GeometryCache)
		{
			if (const FGeometryCacheSceneProxy* SceneProxy = static_cast<const FGeometryCacheSceneProxy*>(PrimitiveSceneInfo->Proxy))
			{
				OutLODIndex = 0;
			}
		}
		if (PrimitiveSceneInfo->Proxy)
		{
			OutMeshLODLocalToWorld = FTransform(PrimitiveSceneInfo->Proxy->GetLocalToWorld());
		}
	}

	return OutLODIndex;
}

void GetCachedGeometry(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap, 
	const FGeometryCacheSceneProxy* SceneProxy,
	const bool bOutputTriangleData,
	FCachedGeometry& Out);

// Returns the cached geometry of the underlying geometry on which a hair instance is attached to
static FCachedGeometry GetCacheGeometryForHair(
	FRDGBuilder& GraphBuilder, 
	FGlobalShaderMap* ShaderMap,
	FSceneInterface* Scene,
	FHairGroupInstance* Instance, 
	const FHairStrandsRestRootResource* RootResource,
	const bool bOutputTriangleData,
	const EHairPositionUpdateType PositionUpdateType,
	FHairGeometryCache& OutHairGeometryCache)
{
	FCachedGeometry Out;
	if (const FPrimitiveSceneInfo* PrimitiveSceneInfo = GetMeshSceneInfo(Scene, Instance))
	{
		if (Instance->Debug.GroomBindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			if (const FSkeletalMeshSceneProxy* SceneProxy = static_cast<const FSkeletalMeshSceneProxy*>(PrimitiveSceneInfo->Proxy))
			{
				// 1. Get cached geometry from GPU SkinCache, if enabled
				SceneProxy->GetCachedGeometry(GraphBuilder, Out);
				if (Out.Sections.Num() > 0)
				{
					OutHairGeometryCache.AddDebug(Instance, SceneProxy, Out, PositionUpdateType, FHairGeometryCache::ECacheType::SkinCache);
				}
				// 2. If no cached geometry is  extracted from the skel. mesh proxy, compute it using the manual hair skin cache
				else if (IsHairManualSkinCacheEnabled())
				{
					// Sync the async update tasks since we will be accessing bone buffers.
					FSkeletalMeshUpdater::WaitForStage(GraphBuilder, ESkeletalMeshUpdateStage::Inline);

					uint32 OutTotalSectionCount = 0;
					GetOrAllocateCachedGeometry(GraphBuilder, ShaderMap, SceneProxy, RootResource, bOutputTriangleData, Out, OutHairGeometryCache, OutTotalSectionCount);
					OutHairGeometryCache.AddDebug(Instance, SceneProxy, Out, PositionUpdateType, FHairGeometryCache::ECacheType::HairCache, OutTotalSectionCount);
				}
			}
		}
		else if (Instance->Debug.GroomBindingType == EGroomBindingMeshType::GeometryCache)
		{
			if (const FGeometryCacheSceneProxy* SceneProxy = static_cast<const FGeometryCacheSceneProxy*>(PrimitiveSceneInfo->Proxy))
			{
				// Get cached geometry from the geometry cache surface
				GetCachedGeometry(GraphBuilder, ShaderMap, SceneProxy, bOutputTriangleData, Out);
				OutHairGeometryCache.AddDebug(Instance, SceneProxy, Out, PositionUpdateType, FHairGeometryCache::ECacheType::GeomCache);
			}
		}
	}
	return Out;
}

// Binding surface parameters (skel.mesh/geom. cache)
static void RunHairBindingSurfaceUpdate(
	FRDGBuilder& GraphBuilder,
	FSceneInterface* Scene,
	const FSceneView* View,
	const uint32 ViewUniqueID,
	const FHairStrandsInstances& Instances,
	const FShaderPrintData* ShaderPrintData,
	FHairTransientResources& TransientResources,
	FGlobalShaderMap* ShaderMap)
{
	const uint32 TotalInstanceCount = Instances.Num();
	TransientResources.SimMeshDatas.SetNum(TotalInstanceCount);
	TransientResources.RenMeshDatas.SetNum(TotalInstanceCount);

	const bool bStrandSupported = IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Scene->GetShaderPlatform());
	const bool bCardSupported   = IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Scene->GetShaderPlatform());

 	FHairGeometryCache HairGeometryCache(GetGroomViewMode(*View));
	for (FHairStrandsInstance* AbstractInstance : Instances)
	{
		FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
		check(Instance);
		if (Instance->GeometryType == EHairGeometryType::NoneGeometry)
		{
			continue;
		}

		check(Instance->HairGroupPublicData);
		const uint32 HairLODIndex = Instance->HairGroupPublicData->LODIndex;
		int32 MeshLODIndex = -1;

		// 1. Guides
		{
			FHairStrandsRestRootResource* RootResource = nullptr;
			if (Instance->Guides.IsValid())
			{	
				const bool bNeedSurfaceUpdate = 
					Instance->BindingType == EHairBindingType::Skinning &&
					(Instance->HairGroupPublicData->IsGlobalInterpolationEnable(HairLODIndex) ||
					Instance->HairGroupPublicData->IsSimulationEnable(HairLODIndex) ||
					Instance->HairGroupPublicData->bIsDeformationEnable ||
					Instance->HairGroupPublicData->bIsSimulationCacheEnable);
				if (bNeedSurfaceUpdate)
				{
					// Extract MeshLODData and MeshLODIndex
					check(Instance->Guides.RestRootResource);
					RootResource = Instance->Guides.RestRootResource;
				}
			}
			TransientResources.SimMeshDatas[Instance->RegisteredIndex] = GetCacheGeometryForHair(GraphBuilder, ShaderMap, Scene, Instance, RootResource, RootResource != nullptr /*bOutputTriangleData*/, EHairPositionUpdateType::Guides, HairGeometryCache);
			MeshLODIndex = TransientResources.SimMeshDatas[Instance->RegisteredIndex].LODIndex;
		}

		// 2. Strands/Cards
		{
			// Strands
			if (Instance->GeometryType == EHairGeometryType::Strands && bStrandSupported && Instance->Strands.IsValid())
			{
				// Extract MeshLODData and compute MeshLODIndex
				const bool bNeedOutputTriangleData = Instance->Strands.RestRootResource != nullptr;
				FHairStrandsRestRootResource* RootResource = bNeedOutputTriangleData ? Instance->Strands.RestRootResource : nullptr;
				TransientResources.RenMeshDatas[Instance->RegisteredIndex] = GetCacheGeometryForHair(GraphBuilder, ShaderMap, Scene, Instance, RootResource, RootResource != nullptr /*bOutputTriangleData*/, EHairPositionUpdateType::Strands, HairGeometryCache);
				MeshLODIndex = TransientResources.RenMeshDatas[Instance->RegisteredIndex].LODIndex;
			}
			// Cards 
			// This is only needed for card geometry. Mesh geometry only uses RBF deformation, which are initialized by the guide pass.
			else if (Instance->GeometryType == EHairGeometryType::Cards && bCardSupported && Instance->Cards.IsValid(HairLODIndex))
			{
				FHairStrandsRestRootResource* RootResource = nullptr;
				const bool bNeedSurfaceUpdate = 
					Instance->BindingType == EHairBindingType::Skinning || 
					Instance->HairGroupPublicData->IsGlobalInterpolationEnable(HairLODIndex) || 
					Instance->HairGroupPublicData->bIsDeformationEnable;
				if (bNeedSurfaceUpdate)
				{	
					// Extract MeshLODData and MeshLODIndex
					check(Instance->Cards.LODs.IsValidIndex(HairLODIndex));
					RootResource = Instance->Cards.LODs[HairLODIndex].Guides.RestRootResource;
				}
				TransientResources.RenMeshDatas[Instance->RegisteredIndex] = GetCacheGeometryForHair(GraphBuilder, ShaderMap, Scene, Instance, RootResource, RootResource != nullptr /*bOutputTriangleData*/, EHairPositionUpdateType::Cards, HairGeometryCache);
				MeshLODIndex = TransientResources.RenMeshDatas[Instance->RegisteredIndex].LODIndex;
			}
		}

		// Update MeshLODIndex
		Instance->HairGroupPublicData->MeshLODIndex = MeshLODIndex;
		Instance->Debug.MeshLODIndex 				= MeshLODIndex;
	}

	// Process manual skin cache requests
	for (FHairGeometryCache::FData& Data : HairGeometryCache.Datas)
	{
		check(Data.LODData);
		AddSkinUpdatePass(
			GraphBuilder, 
			ShaderMap, 
			*Data.LODData, 
			Data.RequestedSections, 
			Data.PositionBuffer, 
			Data.PreviousPositionBuffer,
			Data.TangentBuffer);
	}

	AddHairSkinCacheDebugPass(GraphBuilder, ShaderMap, View, ShaderPrintData, Instances, HairGeometryCache);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Guide Instance Data

struct FGuideInstanceData
{
	int32 HairLODIndex = -1;
	int32 MeshLODIndex = -1;
	EGroomCacheType ActiveGroomCacheType = EGroomCacheType::None;
	EHairBindingType BindingType = EHairBindingType::NoneBinding;
	bool bSimulationEnable = false;
	bool bDeformationEnable = false;
	bool bGlobalDeformationEnable = false;

	FCachedGeometry MeshLODData;
	FHairGroupInstance* Instance = nullptr;

	bool NeedsMeshUpdate(bool bCheckMeshLODIndex=true) const
	{
		return 
			(!bCheckMeshLODIndex || (bCheckMeshLODIndex && MeshLODIndex >= 0)) &&
			(BindingType == EHairBindingType::Skinning) && 
			(bGlobalDeformationEnable || bSimulationEnable || bDeformationEnable || ActiveGroomCacheType==EGroomCacheType::Guides);
	}
};

/** Gather all the instance Data */
FORCEINLINE void GatherGuideInstanceData(const FHairStrandsInstances& Instances, FHairTransientResources& TransientResources, TArray<FGuideInstanceData>& GuideInstanceDatas, bool& bHasAnySimCacheInstances)
{
	// Gather all instances which require guides or RBF deformations
	bHasAnySimCacheInstances = false;
	GuideInstanceDatas.Reserve(Instances.Num());
	for (FHairStrandsInstance* AbstractInstance : Instances)
	{
		FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
		if (Instance->GeometryType == EHairGeometryType::NoneGeometry)
		{
			continue;
		}	
		check(Instance->HairGroupPublicData);

		FGuideInstanceData& InstanceData = GuideInstanceDatas.AddDefaulted_GetRef();
		InstanceData.Instance 					= Instance;
		InstanceData.ActiveGroomCacheType 		= GetHairInstanceCacheType(Instance);
		InstanceData.BindingType 				= Instance->BindingType;
		InstanceData.HairLODIndex 				= Instance->HairGroupPublicData->LODIndex;
		InstanceData.bSimulationEnable 			= Instance->HairGroupPublicData->IsSimulationEnable(InstanceData.HairLODIndex);
		InstanceData.bDeformationEnable 		= Instance->HairGroupPublicData->bIsDeformationEnable;
		InstanceData.bGlobalDeformationEnable 	= Instance->HairGroupPublicData->IsGlobalInterpolationEnable(InstanceData.HairLODIndex);
		InstanceData.MeshLODIndex				= Instance->HairGroupPublicData->MeshLODIndex;
		InstanceData.MeshLODData				= TransientResources.GetMeshLODData(Instance->RegisteredIndex, true /*bSim*/);

		if (InstanceData.ActiveGroomCacheType == EGroomCacheType::Guides)  { bHasAnySimCacheInstances = true; }
	}	
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Guide interpolation

static void RunHairStrandsInterpolation_Guide(
	FRDGBuilder& GraphBuilder,
	FSceneInterface* Scene,
	const FSceneView* View,
	const uint32 ViewUniqueID,
	const FHairStrandsInstances& Instances,
	const FShaderPrintData* ShaderPrintData,
	FHairTransientResources& TransientResources,
	FGlobalShaderMap* ShaderMap)
{
	check(IsInRenderingThread());

	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairGuideInterpolation, "HairGuideInterpolation");

	// Gather all instances which require guides or RBF deformations
	bool bHasAnySimCacheInstances = false;
	TArray<FGuideInstanceData> InstanceDatas;
	GatherGuideInstanceData(Instances, TransientResources, InstanceDatas, bHasAnySimCacheInstances);

	// Update dynamic mesh triangles
	// Guide update need to run only if simulation is enabled, or if RBF is enabled (since RFB are transfer through guides)
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.NeedsMeshUpdate())
		{		
			check(InstanceData.Instance->Guides.IsValid());
			check(InstanceData.Instance->Guides.HasValidRootData());
			check(InstanceData.Instance->Guides.DeformedRootResource->IsValid(InstanceData.MeshLODIndex));

			AddHairStrandUpdateMeshTrianglesPass(
				GraphBuilder,
				ShaderMap,
				InstanceData.MeshLODIndex,
				InstanceData.MeshLODData,
				InstanceData.Instance->Guides.RestRootResource,
				InstanceData.Instance->Guides.DeformedRootResource);
		}
	}

	// RBF sample position update
	// Guide update need to run only if simulation is enabled, or if RBF is enabled (since RFB are transfer through guides)
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.NeedsMeshUpdate())
		{		
			if (InstanceData.bGlobalDeformationEnable)
			{
				AddHairStrandInitMeshSamplesPass(
					GraphBuilder,
					ShaderMap,
					InstanceData.MeshLODIndex,
					InstanceData.MeshLODData,
					InstanceData.Instance->Guides.RestRootResource,
					InstanceData.Instance->Guides.DeformedRootResource);
			}
		}
	}

	// RBF weights update
	// Guide update need to run only if simulation is enabled, or if RBF is enabled (since RFB are transfer through guides)
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.NeedsMeshUpdate())
		{
			if (InstanceData.bGlobalDeformationEnable)
			{
				AddHairStrandUpdateMeshSamplesPass(
					GraphBuilder,
					ShaderMap,
					InstanceData.MeshLODIndex,
					InstanceData.MeshLODData,
					InstanceData.Instance->Guides.RestRootResource,
					InstanceData.Instance->Guides.DeformedRootResource);
			}
		}
	}

	// Update position offset for each instance (GPU)
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		const bool bRequiresCaching = bHasAnySimCacheInstances && (InstanceData.ActiveGroomCacheType == EGroomCacheType::Guides);
		if (InstanceData.NeedsMeshUpdate() && !bRequiresCaching)
		{
			check(InstanceData.MeshLODIndex >= 0);

			AddHairStrandUpdatePositionOffsetPass(
				GraphBuilder,
				ShaderMap,
				EHairPositionUpdateType::Guides,
				InstanceData.Instance->RegisteredIndex,
				InstanceData.HairLODIndex,
				InstanceData.MeshLODIndex,
				InstanceData.Instance->Guides.DeformedRootResource,
				InstanceData.Instance->Guides.DeformedResource);

			// Add manual transition for the GPU solver as Niagara does not track properly the RDG buffer, and so doesn't issue the correct transitions
			{
				GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, InstanceData.Instance->Guides.DeformedRootResource->GetLOD(InstanceData.MeshLODIndex)->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);

				if (InstanceData.bGlobalDeformationEnable)
				{
					GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, InstanceData.Instance->Guides.DeformedRootResource->GetLOD(InstanceData.MeshLODIndex)->GetDeformedSamplePositionsBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
					GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, InstanceData.Instance->Guides.DeformedRootResource->GetLOD(InstanceData.MeshLODIndex)->GetMeshSampleWeightsBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
				}
			}
			GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
		}
	}

	// Update position offset for each instance (CPU)
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		const bool bRequiresCaching = bHasAnySimCacheInstances && (InstanceData.ActiveGroomCacheType == EGroomCacheType::Guides);
		if (InstanceData.MeshLODIndex >= 0 && (InstanceData.BindingType != EHairBindingType::Skinning) && !bRequiresCaching && (InstanceData.bSimulationEnable || InstanceData.bDeformationEnable))
		{		
			AddHairStrandUpdatePositionOffsetPass(
				GraphBuilder,
				ShaderMap,
				EHairPositionUpdateType::Guides,
				InstanceData.Instance->RegisteredIndex,
				InstanceData.HairLODIndex,
				InstanceData.MeshLODIndex,
				nullptr,
				InstanceData.Instance->Guides.DeformedResource);

			// Add manual transition for the GPU solver as Niagara does not track properly the RDG buffer, and so doesn't issue the correct transitions
			GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Guide Deformation

static void RunHairStrandsDeformation_Guide(
	FRDGBuilder& GraphBuilder,
	FSceneInterface* Scene,
	const FSceneView* View,
	const uint32 ViewUniqueID,
	const FHairStrandsInstances& Instances,
	const FShaderPrintData* ShaderPrintData,
	FHairTransientResources& TransientResources,
	FGlobalShaderMap* ShaderMap)
{
	check(IsInRenderingThread());

	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairGuideInterpolation, "HairGuideInterpolation");

	// Gather all instances which require guides or RBF deformations
	bool bHasAnySimCacheInstances = false;
	TArray<FGuideInstanceData> InstanceDatas;
	GatherGuideInstanceData(Instances, TransientResources, InstanceDatas, bHasAnySimCacheInstances);

	// Apply deformation from graph deformer
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		if(InstanceData.Instance->Guides.bIsDeformationEnable && InstanceData.Instance->Guides.DeformedResource && InstanceData.Instance->Guides.DeformedResource->DeformerBuffer.Buffer)
		{
			FRDGBufferRef DstBuffer = Register(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current), ERDGImportedBufferFlags::None).Buffer;
			FRDGBufferRef SrcBuffer = Register(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->DeformerBuffer, ERDGImportedBufferFlags::None).Buffer;
			
			AddCopyBufferPass(GraphBuilder, DstBuffer, SrcBuffer);
		}
	}

	// Apply deformation from RBF
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		if (
			((InstanceData.Instance->Guides.bIsSimulationEnable || InstanceData.Instance->Guides.bIsDeformationEnable || InstanceData.Instance->Guides.bIsSimulationCacheEnable)) ||
			(!InstanceData.Instance->Guides.bHasGlobalInterpolation && !InstanceData.Instance->Guides.bIsSimulationEnable && !InstanceData.Instance->Guides.bIsDeformationEnable && !InstanceData.Instance->Guides.bIsSimulationCacheEnable) ||
			!IsHairStrandsBindingEnable()) continue;

		FRDGExternalBuffer RawDeformedPositionBuffer = InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current);
		FRDGImportedBuffer DeformedPositionBuffer = Register(GraphBuilder, RawDeformedPositionBuffer, ERDGImportedBufferFlags::CreateUAV);

		AddDeformSimHairStrandsPass(
			GraphBuilder,
			ShaderMap,
			InstanceData.Instance->RegisteredIndex,
			InstanceData.MeshLODIndex,
			InstanceData.Instance->Guides.RestResource->GetPointCount(),
			InstanceData.Instance->Guides.RestRootResource,
			InstanceData.Instance->Guides.DeformedRootResource,
			RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PositionBuffer),
			RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PointToCurveBuffer),
			DeformedPositionBuffer,
			InstanceData.Instance->Guides.RestResource->GetPositionOffset(),
			RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current)),
			InstanceData.Instance->Guides.bHasGlobalInterpolation,
			nullptr);
	}

	// Apply deformation from skeletal mesh bones
	for (FGuideInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.ActiveGroomCacheType == EGroomCacheType::None && InstanceData.Instance->Guides.bIsDeformationEnable && InstanceData.Instance->Guides.DeformedResource && InstanceData.Instance->DeformedComponent && (InstanceData.Instance->DeformedSection != INDEX_NONE))
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InstanceData.Instance->DeformedComponent))
			{
				if (FSkeletalMeshObject* SkeletalMeshObject = SkeletalMeshComponent->MeshObject)
				{
					const int32 LodIndex = SkeletalMeshObject->GetLOD();
					FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[LodIndex];
				
					if (LodRenderData->RenderSections.IsValidIndex(InstanceData.Instance->DeformedSection)) 
					{
						FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InstanceData.Instance->DeformedSection, false);

						// Guides deformation based on the skeletal mesh bones
						FRDGImportedBuffer GuideDeformResource = Register(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current), ERDGImportedBufferFlags::CreateUAV);
						AddDeformSimHairStrandsPass(
							GraphBuilder,
							ShaderMap,
							InstanceData.Instance->RegisteredIndex,
							InstanceData.MeshLODIndex,
							InstanceData.Instance->Guides.RestResource->GetPointCount(),
							InstanceData.Instance->Guides.RestRootResource,
							InstanceData.Instance->Guides.DeformedRootResource,
							RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PositionBuffer),
							RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PointToCurveBuffer),
							GuideDeformResource,
							InstanceData.Instance->Guides.RestResource->GetPositionOffset(),
							RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current)),
							InstanceData.Instance->Guides.bHasGlobalInterpolation,
							BoneBufferSRV);
					}
				}
			}
		}
	}

	// Apply guide cache update
	if (bHasAnySimCacheInstances)
	{
		for (FGuideInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.ActiveGroomCacheType == EGroomCacheType::Guides)
			{
				FScopeLock Lock(InstanceData.Instance->Debug.GroomCacheBuffers->GetCriticalSection());
				FGroomCacheGroupData* GroomCacheData0 = const_cast<FGroomCacheGroupData*>(&InstanceData.Instance->Debug.GroomCacheBuffers->GetCurrentFrameBuffer().GroupsData[InstanceData.Instance->Debug.GroupIndex]);
				FGroomCacheGroupData* GroomCacheData1 = const_cast<FGroomCacheGroupData*>(&InstanceData.Instance->Debug.GroomCacheBuffers->GetNextFrameBuffer().GroupsData[InstanceData.Instance->Debug.GroupIndex]);
	
				const float InterpolationFactor = InstanceData.Instance->Debug.GroomCacheBuffers->GetInterpolationFactor();
				
				// Update CPU position offset to GPU
				InstanceData.Instance->Guides.DeformedResource->SetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current, FMath::Lerp(GroomCacheData0->BoundingBox.GetCenter(), GroomCacheData1->BoundingBox.GetCenter(), InterpolationFactor));
				AddHairStrandUpdatePositionOffsetPass(
					GraphBuilder,
					ShaderMap,
					EHairPositionUpdateType::Guides,
					InstanceData.Instance->RegisteredIndex,
					InstanceData.HairLODIndex,
					InstanceData.MeshLODIndex,
					InstanceData.Instance->Guides.DeformedRootResource,
					InstanceData.Instance->Guides.DeformedResource);
	
				// Pass to upload GroomCache guide positions
				FGroomCacheResources CacheResources0 = CreateGroomCacheBuffer(GraphBuilder, GroomCacheData0->VertexData);
				FGroomCacheResources CacheResources1 = CreateGroomCacheBuffer(GraphBuilder, GroomCacheData1->VertexData);
				AddGroomCacheUpdatePass(
					GraphBuilder,
					ShaderMap,
					InstanceData.Instance->RegisteredIndex,
					InstanceData.Instance->Guides.RestResource->GetPointCount(),
					InstanceData.Instance->Guides.RestResource->GetCurveCount(),
					InterpolationFactor,
					1.f /*MaxHairRadius*/,
					InstanceData.Instance->Guides.RestResource->BulkData.HasFlags(FHairStrandsBulkData::EDataFlags::DataFlags_HasExtraEndingPoint),
					CacheResources0,
					CacheResources1,
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PositionBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->CurveBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PointToCurveBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current)),
					RegisterAsUAV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current)));
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Strands interpolation

void AddDrawDebugClusterPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	FHairTransientResources& TransientResources,
	const FShaderPrintData* ShaderPrintData,
	EGroomViewMode ViewMode,
	FHairStrandClusterData& HairClusterData);

static void RunHairStrandsInterpolation_Strands(
	FRDGBuilder& GraphBuilder,
	FSceneInterface* Scene,
	const TArray<const FSceneView*>& Views, 
	const FSceneView* View,
	const uint32 ViewUniqueID,
	const uint32 TotalRegisteredInstanceCount,
	const FHairStrandsInstances& Instances,
	const FShaderPrintData* ShaderPrintData,
	FHairTransientResources& TransientResources,
	FGlobalShaderMap* ShaderMap)
{
	if (Instances.IsEmpty()) { return; }
	if (!IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Scene->GetShaderPlatform())) { return;  }
	check(IsInRenderingThread());
	check(View);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsInterpolation, "HairInterpolation(Strands)");

	struct FInstanceRDGResources
	{
		FRDGBufferSRVRef PositionSRV = nullptr;
		FRDGBufferSRVRef PositionOffsetSRV = nullptr;
		FRDGBufferSRVRef TangentSRV = nullptr;

		FRDGBufferSRVRef DeformerPositionSRV = nullptr;
		FRDGBufferUAVRef PositionUAV = nullptr;
		FRDGBufferUAVRef TangentUAV = nullptr;

		FRDGImportedBuffer DeformedPosition;
		FRDGImportedBuffer DeformedPrevPosition;

		FRDGImportedBuffer Raytracing_PositionBuffer;
		FRDGImportedBuffer Raytracing_IndexBuffer;
	};

	struct FInstanceData
	{
		uint32 RegisteredIndex = ~0;
		int32 HairLODIndex = -1;
		int32 MeshLODIndex = -1;
		uint32 ActivePointCount = 0;
		uint32 ActiveCurveCount = 0;
		uint32 PreviousActivePointCount = 0;
		uint32 PreviousActiveCurveCount = 0;
		EGroomCacheType ActiveGroomCacheType = EGroomCacheType::None;
		EHairBindingType BindingType = EHairBindingType::NoneBinding;
		bool bSimulationEnable = false;
		bool bDeformationEnable = false;
		bool bGlobalDeformationEnable = false;
		bool bNeedDeformation = false;
		bool bNeedRaytracing = false;
		bool bNeedRaytracingUpdate = false;
		bool bNeedRaytracingBuild = false;
		bool bSupportVoxelization = false;

		FInstanceRDGResources RDGResources;

		FCachedGeometry MeshLODData;
		FHairGroupInstance* Instance = nullptr;
		FRDGHairStrandsCullingData CullingData;

		bool NeedsMeshUpdate(bool bCheckMeshLODIndex = true) const
		{
			return
				(!bCheckMeshLODIndex || (bCheckMeshLODIndex && MeshLODIndex >= 0)) &&
				(BindingType == EHairBindingType::Skinning) &&
				(bGlobalDeformationEnable || bSimulationEnable || bDeformationEnable);
		}
	};

	// Gather all strands instances
	const EHairViewRayTracingMask ViewRayTracingMask = RHI_RAYTRACING ? (View->Family->EngineShowFlags.PathTracing ? EHairViewRayTracingMask::PathTracing : EHairViewRayTracingMask::RayTracing) : EHairViewRayTracingMask::None;
	bool bHasAnySimCacheInstances = false;
	bool bHasAnyRenCacheInstances = false;
	bool bHasAnyRaytracingInstances = false;
	TArray<FInstanceData> InstanceDatas;
	InstanceDatas.Reserve(Instances.Num());
	for (FHairStrandsInstance* AbstractInstance : Instances)
	{
		FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
		if (Instance->GeometryType != EHairGeometryType::Strands)
		{
			continue;
		}	
		check(Instance->HairGroupPublicData);

		const bool bCastShadow = (Instance->Debug.Proxy->IsDrawnInGame() && Instance->Debug.Proxy->CastsDynamicShadow()) || Instance->Debug.Proxy->CastsHiddenShadow();

		FInstanceData& InstanceData = InstanceDatas.AddDefaulted_GetRef();
		InstanceData.RegisteredIndex			= Instance->RegisteredIndex;
		InstanceData.Instance 					= Instance;
		InstanceData.ActivePointCount 			= Instance->HairGroupPublicData->GetActiveStrandsPointCount();
		InstanceData.ActiveCurveCount 			= Instance->HairGroupPublicData->GetActiveStrandsCurveCount();
		InstanceData.PreviousActivePointCount 	= Instance->HairGroupPublicData->GetActiveStrandsPointCount(true /*bPrevious*/);
		InstanceData.PreviousActiveCurveCount 	= Instance->HairGroupPublicData->GetActiveStrandsCurveCount(true /*bPrevious*/);
		InstanceData.ActiveGroomCacheType 		= GetHairInstanceCacheType(Instance);
		InstanceData.BindingType 				= Instance->BindingType;
		InstanceData.HairLODIndex 				= Instance->HairGroupPublicData->LODIndex;
		InstanceData.bSimulationEnable 			= Instance->HairGroupPublicData->IsSimulationEnable(InstanceData.HairLODIndex);
		InstanceData.bDeformationEnable 		= Instance->HairGroupPublicData->bIsDeformationEnable;
		InstanceData.bGlobalDeformationEnable 	= Instance->HairGroupPublicData->IsGlobalInterpolationEnable(InstanceData.HairLODIndex);
		InstanceData.MeshLODIndex				= Instance->HairGroupPublicData->MeshLODIndex;
		InstanceData.bNeedDeformation 			= Instance->Strands.DeformedResource != nullptr;
		InstanceData.bNeedRaytracing			= false;
		InstanceData.bNeedRaytracingUpdate		= false;
		InstanceData.bNeedRaytracingBuild		= false;
		InstanceData.bSupportVoxelization		= Instance->Strands.Modifier.bSupportVoxelization && (bCastShadow || Instance->Debug.Proxy->AffectsIndirectLightingWhileHidden()); // Lumen also uses hair voxelization

		// Register resources
		FInstanceRDGResources RDGResources;
		if (InstanceData.bNeedDeformation)
		{		
			// Trach on which view the position has been update, so that we can ensure motion vector are coherent
			Instance->Strands.DeformedResource->GetUniqueViewID(FHairStrandsDeformedResource::Current) = ViewUniqueID;
			
			InstanceData.RDGResources.DeformedPosition		= Register(GraphBuilder, Instance->Strands.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current), ERDGImportedBufferFlags::CreateViews);
			InstanceData.RDGResources.DeformedPrevPosition	= Register(GraphBuilder, Instance->Strands.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Previous), ERDGImportedBufferFlags::CreateViews);

			InstanceData.RDGResources.PositionSRV 			= InstanceData.RDGResources.DeformedPosition.SRV;
			InstanceData.RDGResources.PositionUAV			= InstanceData.RDGResources.DeformedPosition.UAV;
			InstanceData.RDGResources.PositionOffsetSRV 	= RegisterAsSRV(GraphBuilder, Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current));		
			InstanceData.RDGResources.TangentSRV 			= RegisterAsSRV(GraphBuilder, Instance->Strands.DeformedResource->TangentBuffer);
			InstanceData.RDGResources.TangentUAV 			= RegisterAsUAV(GraphBuilder, Instance->Strands.DeformedResource->TangentBuffer);
			InstanceData.RDGResources.DeformerPositionSRV 	= RegisterAsSRV(GraphBuilder, Instance->Strands.DeformedResource->DeformerBuffer);
		}
		else
		{
			InstanceData.RDGResources.PositionSRV 			= RegisterAsSRV(GraphBuilder, Instance->Strands.RestResource->PositionBuffer);
			InstanceData.RDGResources.PositionUAV			= nullptr;
			InstanceData.RDGResources.PositionOffsetSRV 	= RegisterAsSRV(GraphBuilder, Instance->Strands.RestResource->PositionOffsetBuffer);
			InstanceData.RDGResources.TangentSRV 			= nullptr;  
			InstanceData.RDGResources.DeformerPositionSRV 	= nullptr;
		}

		#if RHI_RAYTRACING
		InstanceData.bNeedRaytracing = InstanceData.Instance->Strands.RenRaytracingResource && EnumHasAnyFlags(InstanceData.Instance->Strands.ViewRayTracingMask, ViewRayTracingMask);
		if (InstanceData.bNeedRaytracing)
		{
			InstanceData.RDGResources.Raytracing_PositionBuffer = Register(GraphBuilder, InstanceData.Instance->Strands.RenRaytracingResource->PositionBuffer, ERDGImportedBufferFlags::CreateViews);
			InstanceData.RDGResources.Raytracing_IndexBuffer 	= Register(GraphBuilder, InstanceData.Instance->Strands.RenRaytracingResource->IndexBuffer, ERDGImportedBufferFlags::CreateViews);
			bHasAnyRaytracingInstances = true;
		}
		#endif

		if (InstanceData.ActiveGroomCacheType == EGroomCacheType::Guides)  { bHasAnySimCacheInstances = true; }
		if (InstanceData.ActiveGroomCacheType == EGroomCacheType::Strands) { bHasAnyRenCacheInstances = true; }

		InstanceData.MeshLODData = TransientResources.GetMeshLODData(Instance->RegisteredIndex, false /*bSim*/);
		Instance->HairGroupPublicData->VFInput.Strands	= FHairGroupPublicData::FVertexFactoryInput::FStrands();

		if (InstanceData.MeshLODIndex >= 0 && (InstanceData.BindingType == EHairBindingType::Skinning || InstanceData.bGlobalDeformationEnable))
		{
			check(Instance->Strands.HasValidRootData());
			check(Instance->Strands.DeformedRootResource->IsValid(InstanceData.MeshLODIndex));
		}

		if (InstanceData.MeshLODIndex >= 0 && (InstanceData.bSimulationEnable || InstanceData.bDeformationEnable))
		{
			check(Instance->Strands.IsValid());
		}
	}

	FRDGExternalAccessQueue ExternalAccessQueue;
	const EGroomViewMode ViewMode = GetGroomViewMode(*View);

	// Early culling
	FHairStrandClusterData ClusterDatas;
	{
		TArray<FRDGBufferSRVRef> Transitions;
		Transitions.Reserve(InstanceDatas.Num());

		// Gather culling jobs
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.Instance->Strands.bCullingEnable)
			{
				AddInstanceToClusterData(InstanceData.Instance, ClusterDatas);
				Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Instance->HairGroupPublicData->GetCulledVertexIdBuffer()));
				Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Instance->HairGroupPublicData->GetDrawIndirectRasterComputeBuffer()));
			}
			else
			{
				InstanceData.Instance->HairGroupPublicData->SetCullingResultAvailable(false);
			}
		}

		// Culling pass
		if (Views.Num() > 0 && ClusterDatas.HairGroups.Num() > 0)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsClusterCulling, "HairStrandsClusterCulling");

			TRACE_CPUPROFILER_EVENT_SCOPE(ComputeHairStrandsClustersCulling);

			FRDGBufferUAVRef IndirectDispatchArgsUAVWithSkipBarrier = GraphBuilder.CreateUAV(TransientResources.IndirectDispatchArgsBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);
			AddClusterCullingPass(
				GraphBuilder,
				ShaderMap,
				Views[0],
				ShaderPrintData,
				ClusterDatas,
				IndirectDispatchArgsUAVWithSkipBarrier);

			AddTransitionPass(GraphBuilder, ShaderMap, View->GetShaderPlatform(), Transitions);
		}
	}
	
	// Update dynamic mesh triangles
	{
		TArray<FRDGBufferSRVRef> Transitions;
		Transitions.Reserve(InstanceDatas.Num());
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.MeshLODIndex >= 0 && (InstanceData.BindingType == EHairBindingType::Skinning || InstanceData.bGlobalDeformationEnable))
			{
				AddHairStrandUpdateMeshTrianglesPass(
					GraphBuilder,
					ShaderMap,
					InstanceData.MeshLODIndex,
					InstanceData.MeshLODData,
					InstanceData.Instance->Strands.RestRootResource,
					InstanceData.Instance->Strands.DeformedRootResource);
				Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.DeformedRootResource->GetLOD(InstanceData.MeshLODIndex)->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current)));
			}
		}
		AddTransitionPass(GraphBuilder, ShaderMap, View->GetShaderPlatform(), Transitions);
	}

	// Update position offset 
	{
		TArray<FRDGBufferSRVRef> Transitions;
		Transitions.Reserve(InstanceDatas.Num());

		// From GPU root triangle
		{
			for (FInstanceData& InstanceData : InstanceDatas)
			{
				if (InstanceData.MeshLODIndex >= 0 && (InstanceData.BindingType == EHairBindingType::Skinning || InstanceData.bGlobalDeformationEnable))
				{
					AddHairStrandUpdatePositionOffsetPass(
						GraphBuilder,
						ShaderMap,
						EHairPositionUpdateType::Strands,
						InstanceData.Instance->RegisteredIndex,
						InstanceData.HairLODIndex,
						InstanceData.MeshLODIndex,
						InstanceData.Instance->Strands.DeformedRootResource,
						InstanceData.Instance->Strands.DeformedResource);
					Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current)));
				}
			}
		}
	
		// From CPU AABB
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.MeshLODIndex >= 0 && InstanceData.BindingType != EHairBindingType::Skinning && (InstanceData.bSimulationEnable || InstanceData.bDeformationEnable))
			{
				AddHairStrandUpdatePositionOffsetPass(
					GraphBuilder,
					ShaderMap,
					EHairPositionUpdateType::Strands,
					InstanceData.Instance->RegisteredIndex,
					InstanceData.HairLODIndex,
					InstanceData.MeshLODIndex,
					nullptr,
					InstanceData.Instance->Strands.DeformedResource);
				Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current)));
			}
		}
		
		// From Sim Cache
		if (bHasAnySimCacheInstances)
		{
			for (FInstanceData& InstanceData : InstanceDatas)
			{
				if (InstanceData.ActiveGroomCacheType == EGroomCacheType::Guides && InstanceData.bNeedDeformation)
				{
					// Apply the same offset to the render strands for proper rendering
					const FVector OffsetPosition = InstanceData.Instance->Guides.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current);
					InstanceData.Instance->Strands.DeformedResource->SetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current, OffsetPosition);
		
					AddHairStrandUpdatePositionOffsetPass(
						GraphBuilder,
						ShaderMap,
						EHairPositionUpdateType::Strands,
						InstanceData.Instance->RegisteredIndex,
						0,
						InstanceData.MeshLODIndex,
						InstanceData.Instance->Strands.DeformedRootResource,
						InstanceData.Instance->Strands.DeformedResource);
					Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current)));
				}
			}
		}

		AddTransitionPass(GraphBuilder, ShaderMap, View->GetShaderPlatform(), Transitions);
	}


	// Clear cluster AABBs (used optionally  for voxel allocation & for culling)
	{
		uint32 TotalClusterAABBCount = 0;
		TransientResources.ClusterAABBOffetAndCounts.Init(FUintVector2::ZeroValue, TotalRegisteredInstanceCount); // Use TotalRegisteredInstanceCount because this buffer is indexed by RegisteredIndex
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.bNeedDeformation)
			{
				TransientResources.ClusterAABBOffetAndCounts[InstanceData.RegisteredIndex].X = TotalClusterAABBCount;
				TransientResources.ClusterAABBOffetAndCounts[InstanceData.RegisteredIndex].Y = InstanceData.Instance->HairGroupPublicData->ClusterCount;
				TotalClusterAABBCount += InstanceData.Instance->HairGroupPublicData->ClusterCount;
			}
		}
		TotalClusterAABBCount = FMath::Max(TotalClusterAABBCount, 1u);
		TransientResources.ClusterAABBBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, 6u * TotalClusterAABBCount), TEXT("Hair.Transient.ClusterAABBs"));
		TransientResources.ClusterAABBSRV = GraphBuilder.CreateSRV(TransientResources.ClusterAABBBuffer, PF_R32_SINT);

		FRDGBufferUAVRef ClusterAABBUAVSkipBarrier = GraphBuilder.CreateUAV(TransientResources.ClusterAABBBuffer, PF_R32_SINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferUAVRef GroupAABBUAVSkipBarrier = GraphBuilder.CreateUAV(TransientResources.GroupAABBBuffer, PF_R32_SINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		AddClearAABBPass(GraphBuilder, ShaderMap, TotalClusterAABBCount, ClusterAABBUAVSkipBarrier);
		AddClearAABBPass(GraphBuilder, ShaderMap, TotalRegisteredInstanceCount, GroupAABBUAVSkipBarrier);

		for (FInstanceData& InstanceData : InstanceDatas)
		{
			InstanceData.CullingData = ImportCullingData(GraphBuilder, InstanceData.Instance->HairGroupPublicData);
		}
	}

	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsInterpolationCurve, "HairStrandsInterpolationCurve");

		TRACE_CPUPROFILER_EVENT_SCOPE(HairStrandsInterpolationCurve);

		TArray<FRDGBufferSRVRef> Transitions;
		Transitions.Reserve(InstanceDatas.Num());
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.ActiveGroomCacheType != EGroomCacheType::Strands && InstanceData.bNeedDeformation)
			{
				const bool bValidGuide		= InstanceData.bNeedDeformation && (InstanceData.bSimulationEnable || InstanceData.bGlobalDeformationEnable || InstanceData.bDeformationEnable || InstanceData.Instance->Guides.bIsSimulationCacheEnable);// || (WITH_EDITOR && PatchMode == EHairPatchAttribute::GuideInflucence);
				const bool bUseSingleGuide	= InstanceData.bNeedDeformation && bValidGuide && InstanceData.Instance->Strands.InterpolationResource->UseSingleGuide();
				const bool bHasSkinning		= InstanceData.bNeedDeformation && InstanceData.BindingType == EHairBindingType::Skinning;
	
				AddHairStrandsInterpolationPass(
					GraphBuilder,
					ShaderMap,
					View->GetShaderPlatform(),
					ShaderPrintData,
					InstanceData.Instance,
					InstanceData.ActivePointCount,
					InstanceData.ActiveCurveCount,
					InstanceData.Instance->Strands.RestResource,
					InstanceData.Instance->Guides.RestResource,
					bValidGuide,
					InstanceData.Instance->Strands.RestResource->BulkData.Header.MaxPointPerCurve,
					InstanceData.MeshLODIndex,
					InstanceData.Instance->Strands.Modifier.HairLengthScale,
					InstanceData.Instance->Strands.HairInterpolationType,
					EHairGeometryType::Strands,
					InstanceData.CullingData,
					InstanceData.Instance->Strands.RestResource->GetPositionOffset(),
					bValidGuide ? InstanceData.Instance->Guides.RestResource->GetPositionOffset() : FVector::ZeroVector,
					InstanceData.RDGResources.PositionOffsetSRV,
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current)) : nullptr,
					bHasSkinning ? InstanceData.Instance->Strands.RestRootResource : nullptr,
					bHasSkinning && bValidGuide ? InstanceData.Instance->Guides.RestRootResource : nullptr,
					bHasSkinning ? InstanceData.Instance->Strands.DeformedRootResource : nullptr,
					bHasSkinning && bValidGuide ? InstanceData.Instance->Guides.DeformedRootResource : nullptr,
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.RestResource->PositionBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.RestResource->CurveBuffer),
					bUseSingleGuide,
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.InterpolationResource->CurveInterpolationBuffer) : nullptr,
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.InterpolationResource->PointInterpolationBuffer) : nullptr,
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PositionBuffer) : nullptr,
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current)) : nullptr,
					InstanceData.RDGResources.DeformerPositionSRV,
					InstanceData.RDGResources.PositionUAV,
					FHairStrandsLODDeformedRootResource::Current);
	
				Transitions.Add(InstanceData.RDGResources.PositionSRV);

				GraphBuilder.SetBufferAccessFinal(InstanceData.RDGResources.DeformedPosition.Buffer, ERHIAccess::SRVMask);
			}
		}

		AddTransitionPass(GraphBuilder, ShaderMap, View->GetShaderPlatform(), Transitions);
	}

	// Previous Position
	// * Transfer prev. position
	// * Or recompute interpolation pass for previous positions if needed
	for (FInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.ActiveGroomCacheType != EGroomCacheType::Strands && InstanceData.bNeedDeformation)
		{
			const bool bLODChanged = (InstanceData.Instance->HairGroupPublicData->VFInput.bHasLODSwitch && IsHairStrandsTransferPositionOnLODChange());
			const bool bNewCurves  = InstanceData.ActivePointCount > InstanceData.PreviousActivePointCount;
			if (bLODChanged || bNewCurves)
			{
				const uint32 PointOffset = bLODChanged ? 0 : InstanceData.PreviousActivePointCount;
				const uint32 PointCount  = bLODChanged ? InstanceData.ActivePointCount : InstanceData.ActivePointCount - InstanceData.PreviousActivePointCount;
				AddTransferPositionPass(
					GraphBuilder, 
					ShaderMap, 
					PointOffset,
					PointCount,
					InstanceData.ActivePointCount, 
					InstanceData.RDGResources.DeformedPosition.SRV, 
					InstanceData.RDGResources.DeformedPrevPosition.UAV);
				GraphBuilder.SetBufferAccessFinal(InstanceData.RDGResources.DeformedPrevPosition.Buffer, ERHIAccess::SRVMask);
			}
		}
	}

	// Render Strands cache update
	if (bHasAnyRenCacheInstances)
	{
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.ActiveGroomCacheType == EGroomCacheType::Strands)
			{
				FScopeLock Lock(InstanceData.Instance->Debug.GroomCacheBuffers->GetCriticalSection());
				
				FGroomCacheGroupData* GroomCacheData0 = const_cast<FGroomCacheGroupData*>(&InstanceData.Instance->Debug.GroomCacheBuffers->GetCurrentFrameBuffer().GroupsData[InstanceData.Instance->Debug.GroupIndex]);
				FGroomCacheGroupData* GroomCacheData1 = const_cast<FGroomCacheGroupData*>(&InstanceData.Instance->Debug.GroomCacheBuffers->GetNextFrameBuffer().GroupsData[InstanceData.Instance->Debug.GroupIndex]);
				const float InterpolationFactor = InstanceData.Instance->Debug.GroomCacheBuffers->GetInterpolationFactor();
				
				FGroomCacheResources CacheResources0 = CreateGroomCacheBuffer(GraphBuilder, GroomCacheData0->VertexData);
				FGroomCacheResources CacheResources1 = CreateGroomCacheBuffer(GraphBuilder, GroomCacheData1->VertexData);
				
				// Update position offset
				{
					const FVector OffsetPosition = FMath::Lerp(GroomCacheData0->BoundingBox.GetCenter(), GroomCacheData1->BoundingBox.GetCenter(), InterpolationFactor);
					InstanceData.Instance->Strands.DeformedResource->SetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current, OffsetPosition);
				}
				
				// Update max radius
				if (GroomCacheData0->VertexData.PointsRadius.Num() > 0)
				{
					InstanceData.Instance->Strands.Modifier.HairWidth = FMath::Lerp(GroomCacheData0->StrandData.MaxRadius, GroomCacheData1->StrandData.MaxRadius, InterpolationFactor) * 2.0f;
				}
				
				AddHairStrandUpdatePositionOffsetPass(
					GraphBuilder,
					ShaderMap,
					EHairPositionUpdateType::Strands,
					InstanceData.Instance->RegisteredIndex,
					0,
					InstanceData.MeshLODIndex,
					InstanceData.Instance->Strands.DeformedRootResource,
					InstanceData.Instance->Strands.DeformedResource);
				
				// Pass to upload GroomCache strands positions
				AddGroomCacheUpdatePass(
					GraphBuilder,
					ShaderMap,
					InstanceData.Instance->RegisteredIndex,
					InstanceData.ActivePointCount,
					InstanceData.ActiveCurveCount,
					InterpolationFactor,
					InstanceData.Instance->Strands.Modifier.HairWidth * 0.5f /*InMaxHairRadius*/,
					InstanceData.Instance->Strands.RestResource->BulkData.HasFlags(FHairStrandsBulkData::EDataFlags::DataFlags_HasExtraEndingPoint),
					CacheResources0,
					CacheResources1,
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.RestResource->PositionBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.RestResource->CurveBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.RestResource->PointToCurveBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current)),
					InstanceData.RDGResources.PositionUAV);
			}
		}
	}

	// Update tangent data based on the deformed positions
	for (FInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.bNeedDeformation)
		{
			AddHairTangentPass(
				GraphBuilder,
				ShaderMap,
				InstanceData.ActivePointCount,
				InstanceData.Instance->HairGroupPublicData,
				InstanceData.RDGResources.PositionSRV,
				InstanceData.RDGResources.TangentUAV);
		}
		else
		{
			FRDGExternalBuffer TangentBuffer = InstanceData.Instance->Strands.RestResource->GetTangentBuffer(GraphBuilder, ShaderMap, InstanceData.ActivePointCount, InstanceData.ActiveCurveCount);
			InstanceData.RDGResources.TangentSRV = RegisterAsSRV(GraphBuilder, TangentBuffer);
		}
	}

	// Patch attribute for debug visualization (guide influence or clusters visualization)
	const bool bNeedPatchPass = ViewMode == EGroomViewMode::RenderHairStrands || ViewMode == EGroomViewMode::Cluster || ViewMode == EGroomViewMode::ClusterAABB;
	if (bNeedPatchPass)
	{
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			EHairPatchAttribute PatchMode = EHairPatchAttribute::None;
			if (ViewMode == EGroomViewMode::RenderHairStrands && InstanceData.bNeedDeformation)	{ PatchMode = EHairPatchAttribute::GuideInflucence; }
			if (ViewMode == EGroomViewMode::Cluster || ViewMode == EGroomViewMode::ClusterAABB)	{ PatchMode = EHairPatchAttribute::ClusterInfluence; }

			if (PatchMode != EHairPatchAttribute::None)
			{
				// Create an debug buffer for storing cluster visualization data. This is only used for debug purpose, hence only enable in editor build.
				// Special case for debug mode were the attribute buffer is patch with some custom data to show hair properties (strands belonging to the same cluster, ...)
				if (InstanceData.Instance->Strands.DebugCurveAttributeBuffer.Buffer == nullptr)
				{
					CreateHairStrandsDebugAttributeBuffer(GraphBuilder, &InstanceData.Instance->Strands.DebugCurveAttributeBuffer, InstanceData.Instance->Strands.GetData().GetCurveAttributeSizeInBytes(), FName(InstanceData.Instance->Debug.MeshComponentName));
				}
				FRDGImportedBuffer OutRenCurveAttributeBuffer = Register(GraphBuilder, InstanceData.Instance->Strands.DebugCurveAttributeBuffer, ERDGImportedBufferFlags::CreateUAV);
			
				const bool bValidGuide = InstanceData.bNeedDeformation && (InstanceData.bSimulationEnable || InstanceData.bGlobalDeformationEnable || InstanceData.bDeformationEnable || InstanceData.Instance->Guides.bIsSimulationCacheEnable);// || (WITH_EDITOR && PatchMode == EHairPatchAttribute::GuideInflucence);
				const bool bUseSingleGuide	= InstanceData.bNeedDeformation && bValidGuide && InstanceData.Instance->Strands.InterpolationResource->UseSingleGuide();

				check(InstanceData.Instance->Strands.IsValid());
				AddPatchAttributePass(
					GraphBuilder,
					ShaderMap,
					InstanceData.ActiveCurveCount,
					PatchMode,
					bValidGuide,
					bUseSingleGuide,
					InstanceData.Instance->Strands.GetData(),
					Register(GraphBuilder, InstanceData.Instance->Strands.RestResource->CurveAttributeBuffer, ERDGImportedBufferFlags::CreateSRV).Buffer,
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.RestResource->CurveBuffer),
					RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.ClusterResource->CurveToClusterIdBuffer),
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.InterpolationResource->CurveInterpolationBuffer) : nullptr,
					bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Strands.InterpolationResource->PointInterpolationBuffer) : nullptr,
					OutRenCurveAttributeBuffer);
			}
		}
	}

	// Update the VF input with the update resources
	for (FInstanceData& InstanceData : InstanceDatas)
	{
		InstanceData.Instance->HairGroupPublicData->VFInput = InternalComputeHairStrandsVertexInputData(&GraphBuilder, InstanceData.Instance, ViewMode);
	}

	// 3. Compute cluster AABBs (used for LODing and voxelization)
	{
		FRDGBufferUAVRef GroupAABUAVSkipBarrier = GraphBuilder.CreateUAV(TransientResources.GroupAABBBuffer, PF_R32_SINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferUAVRef ClusterAABBUAVSkipBarrier = GraphBuilder.CreateUAV(TransientResources.ClusterAABBBuffer, PF_R32_SINT, ERDGUnorderedAccessViewFlags::SkipBarrier);

		const FVector& TranslatedWorldOffset = View->ViewMatrices.GetPreViewTranslation();
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			// Optim: If an instance does not voxelize it's data, then there is no need for having valid AABB
			bool bNeedGPUAABB = InstanceData.bSupportVoxelization;
			const EHairAABBUpdateType UpdateType = InstanceData.bNeedDeformation ? EHairAABBUpdateType::UpdateClusterAABB : EHairAABBUpdateType::UpdateGroupAABB;
				
			FHairStrandClusterData::FHairGroup* HairGroupCluster = nullptr;
			if (InstanceData.CullingData.bCullingResultAvailable)
			{
				// Sanity check
				check(InstanceData.Instance->Strands.bCullingEnable);
				HairGroupCluster = &ClusterDatas.HairGroups[InstanceData.Instance->HairGroupPublicData->ClusterDataIndex];
				if (!HairGroupCluster->bVisible)
				{
					bNeedGPUAABB = false;
				}
			}
				
			if (bNeedGPUAABB)
			{
				FRDGImportedBuffer Strands_CulledVertexCount = Register(GraphBuilder, InstanceData.Instance->HairGroupPublicData->GetDrawIndirectRasterComputeBuffer(), ERDGImportedBufferFlags::CreateSRV);
				AddHairClusterAABBPass(
					GraphBuilder,
					ShaderMap,
					InstanceData.ActiveCurveCount,
					TranslatedWorldOffset,
					ShaderPrintData,
					UpdateType,
					InstanceData.Instance,
					InstanceData.RDGResources.PositionOffsetSRV,
					HairGroupCluster,
					TransientResources.GetClusterOffset(InstanceData.Instance->RegisteredIndex),
					TransientResources.GetClusterCount(InstanceData.Instance->RegisteredIndex),
					InstanceData.RDGResources.PositionSRV,
					Strands_CulledVertexCount.SRV,
					ClusterAABBUAVSkipBarrier,
					GroupAABUAVSkipBarrier);
	
				TransientResources.bIsGroupAABBValid[InstanceData.RegisteredIndex] = true;
			}
		}

		// Run cluster debug view here (instead of GroomDebug.h/.cpp, as we need to have the (transient) cluster data 
		if (ViewMode == EGroomViewMode::Cluster || ViewMode == EGroomViewMode::ClusterAABB)
		{
			AddDrawDebugClusterPass(GraphBuilder, *View, ShaderMap, TransientResources, ShaderPrintData, ViewMode, ClusterDatas);
		}
	}

	#if RHI_RAYTRACING
	if (bHasAnyRaytracingInstances)
	{
		const uint32 ProceduralSplits = GetHairRaytracingProceduralSplits();

		// 1. Update raytracing geometry (update only if the view mask and the RT geometry mask match)
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.bNeedRaytracing)
			{
				// Note: VFInput.Strands.Common.Radius already contains RadiusScale from discrete LOD setup, for backward compatibility.
				// Only used CLOD scale when the geometry is dynamic
				const bool bIsDynamicGeometry = InstanceData.Instance->Strands.DeformedResource != nullptr;
				const float CLODScale = bIsDynamicGeometry ? InstanceData.Instance->HairGroupPublicData->ContinuousLODCoverageScale : 1.f;
				const float HairRadiusRT = InstanceData.Instance->HairGroupPublicData->VFInput.Strands.Common.RaytracingRadiusScale * InstanceData.Instance->HairGroupPublicData->VFInput.Strands.Common.Radius * CLODScale;
				const float HairRootScaleRT = InstanceData.Instance->HairGroupPublicData->VFInput.Strands.Common.RootScale;
				const float HairTipScaleRT = InstanceData.Instance->HairGroupPublicData->VFInput.Strands.Common.TipScale;

				InstanceData.bNeedRaytracingUpdate = bIsDynamicGeometry ||
					InstanceData.Instance->Strands.CachedHairScaledRadius != HairRadiusRT ||
					InstanceData.Instance->Strands.CachedHairRootScale != HairRootScaleRT ||
					InstanceData.Instance->Strands.CachedHairTipScale != HairTipScaleRT ||
					InstanceData.Instance->Strands.CachedProceduralSplits != ProceduralSplits;
				InstanceData.bNeedRaytracingBuild = !InstanceData.Instance->Strands.RenRaytracingResource->bIsRTGeometryInitialized;
				if (InstanceData.bNeedRaytracingBuild || InstanceData.bNeedRaytracingUpdate)
				{
					// The entire RT buffer needs to be updated for ensuring invalid segments marked. 
					// When building the RT geometry, the entire buffer is processed, and having partially updated 
					// buffer would cause the groom to have stale curve/points
					const uint32 ActivePointCount = InstanceData.ActivePointCount;
					const uint32 TotalPointCount = InstanceData.Instance->Strands.RestResource->GetPointCount();
					AddGenerateRaytracingGeometryPass(
						GraphBuilder,
						ShaderMap,
						ShaderPrintData,
						InstanceData.Instance->RegisteredIndex,
						ActivePointCount,
						TotalPointCount,
						InstanceData.Instance->Strands.RenRaytracingResource->RayTracingPrimitiveType,
						ProceduralSplits,
						HairRadiusRT,
						HairRootScaleRT,
						HairTipScaleRT,
						InstanceData.RDGResources.PositionOffsetSRV,
						InstanceData.CullingData,
						InstanceData.RDGResources.PositionSRV,
						InstanceData.RDGResources.TangentSRV,
						InstanceData.RDGResources.Raytracing_PositionBuffer.UAV,
						InstanceData.RDGResources.Raytracing_IndexBuffer.UAV);
			
					InstanceData.Instance->Strands.CachedHairScaledRadius = HairRadiusRT;
					InstanceData.Instance->Strands.CachedHairRootScale = HairRootScaleRT;
					InstanceData.Instance->Strands.CachedHairTipScale = HairTipScaleRT;
				}
			}
		}

		// 2. Build/update acceleration structure
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.bNeedRaytracing && (InstanceData.bNeedRaytracingBuild || InstanceData.bNeedRaytracingUpdate))
			{
				AddBuildStrandsAccelerationStructurePass(
					GraphBuilder,
					InstanceData.Instance,
					ProceduralSplits,
					InstanceData.bNeedRaytracingUpdate,
					InstanceData.RDGResources.Raytracing_PositionBuffer.Buffer,
					InstanceData.RDGResources.Raytracing_IndexBuffer.Buffer);
			}
		}
	}
	#endif

	if (ViewMode == EGroomViewMode::SimHairStrands)
	{
		for (FInstanceData& InstanceData : InstanceDatas)
		{
			if (InstanceData.Instance->Guides.NeedsGuides())
			{
				if (InstanceData.Instance->Guides.DeformedResource)
				{
					// Disable culling when drawing only guides, as the culling output has been computed for the strands, not for the guides.
					InstanceData.Instance->HairGroupPublicData->SetCullingResultAvailable(false);
					
					AddHairTangentPass(
						GraphBuilder,
						ShaderMap,
						InstanceData.Instance->Guides.RestResource->GetPointCount(),
						InstanceData.Instance->HairGroupPublicData,
						RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current)),
						RegisterAsUAV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->TangentBuffer));
				}
				InstanceData.Instance->HairGroupPublicData->VFInput = InternalComputeHairStrandsVertexInputData(&GraphBuilder, InstanceData.Instance, ViewMode);
			}
		}
	}

	// Sanity check
	for (FInstanceData& InstanceData : InstanceDatas)
	{
		check(InstanceData.Instance->HairGroupPublicData->VFInput.Strands.PositionBuffer.Buffer);
		InstanceData.Instance->Strands.UniformBuffer.UpdateUniformBufferImmediate(GraphBuilder.RHICmdList, InstanceData.Instance->GetHairStandsUniformShaderParameters(ViewMode));
		InstanceData.Instance->HairGroupPublicData->VFInput.GeometryType = EHairGeometryType::Strands;
		InstanceData.Instance->HairGroupPublicData->VFInput.LocalToWorldTransform = InstanceData.Instance->GetCurrentLocalToWorld();
		InstanceData.Instance->HairGroupPublicData->bSupportVoxelization = InstanceData.bSupportVoxelization;
	}

	ExternalAccessQueue.Submit(GraphBuilder);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Cards/Meshes interpolation

enum class EHairCardsSimulationType
{
	None,
	Guide,
	RBF
};

uint32 GetHairCardsInterpolationType();

static void RunHairStrandsInterpolation_Cards(
	FRDGBuilder& GraphBuilder,
	FSceneInterface* Scene,
	const TArray<const FSceneView*>& Views, 
	const FSceneView* View,
	const uint32 ViewUniqueID,
	const FHairStrandsInstances& Instances,
	const FShaderPrintData* ShaderPrintData,
	FHairTransientResources& TransientResources,
	FGlobalShaderMap* ShaderMap)
{
	const bool bCardSupported = IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Scene->GetShaderPlatform());
	const bool bMeshSupported = IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, Scene->GetShaderPlatform());

	if (Instances.IsEmpty() || (!bCardSupported && !bMeshSupported)) { return; }
	check(IsInRenderingThread());
	check(View);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairCardsInterpolation, "HairCardsInterpolation");

	struct FInstanceData
	{
		uint32 RegisteredIndex = ~0;
		int32 HairLODIndex = -1;
		int32 MeshLODIndex = -1;
		EHairBindingType BindingType = EHairBindingType::NoneBinding;
		bool bSimulationEnable = false;
		bool bDeformationEnable = false;
		bool bGlobalDeformationEnable = false;
		bool bValidGuide = false;
		bool bHasSkinning = false;
		bool bNeedDeformation = false;
		EHairCardsSimulationType CardsSimulationType = EHairCardsSimulationType::None;
		FHairGroupInstance::FCards::FLOD*  CardInstance = nullptr;
		FHairGroupInstance::FMeshes::FLOD* MeshInstance = nullptr;
		FCachedGeometry MeshLODData;
		FHairGroupInstance* Instance = nullptr;
	};

	// Gather all strands instances
	TArray<FInstanceData> InstanceDatas;
	TArray<uint32> MeshInstances;
	TArray<uint32> CardInstances;
	MeshInstances.Reserve(Instances.Num());
	CardInstances.Reserve(Instances.Num());
	for (FHairStrandsInstance* AbstractInstance : Instances)
	{
		FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
		check(Instance->HairGroupPublicData);
		
		if (Instance->GeometryType == EHairGeometryType::Cards || Instance->GeometryType == EHairGeometryType::Meshes)
		{
			const uint32 HairLODIndex = Instance->HairGroupPublicData->LODIndex;
			if (Instance->GeometryType == EHairGeometryType::Cards  && (!bCardSupported || !Instance->Cards.IsValid(HairLODIndex)))  { continue; }
			if (Instance->GeometryType == EHairGeometryType::Meshes && (!bMeshSupported || !Instance->Meshes.IsValid(HairLODIndex))) { continue; }

			if (Instance->GeometryType == EHairGeometryType::Cards)
			{	
				CardInstances.Add(InstanceDatas.Num());
			}
			else if (Instance->GeometryType == EHairGeometryType::Meshes)
			{	
				MeshInstances.Add(InstanceDatas.Num());
			}

			// Common instance parameters
			FInstanceData& InstanceData = InstanceDatas.AddDefaulted_GetRef();
			InstanceData.RegisteredIndex			= Instance->RegisteredIndex;
			InstanceData.Instance 					= Instance;
			InstanceData.BindingType 				= Instance->BindingType;
			InstanceData.HairLODIndex 				= HairLODIndex;
			InstanceData.bSimulationEnable 			= Instance->HairGroupPublicData->IsSimulationEnable(HairLODIndex);
			InstanceData.bDeformationEnable 		= Instance->HairGroupPublicData->bIsDeformationEnable;
			InstanceData.bGlobalDeformationEnable 	= Instance->HairGroupPublicData->IsGlobalInterpolationEnable(HairLODIndex);			
			InstanceData.MeshLODIndex				= Instance->HairGroupPublicData->MeshLODIndex;
			InstanceData.MeshLODData  				= TransientResources.GetMeshLODData(Instance->RegisteredIndex, false /*bSim*/);

			// Card or Mesh specific parameters
			if (Instance->GeometryType == EHairGeometryType::Cards)
			{
				check(Instance->Cards.LODs.IsValidIndex(InstanceData.HairLODIndex));
				InstanceData.CardInstance 			= &Instance->Cards.LODs[InstanceData.HairLODIndex];
				InstanceData.bValidGuide			= InstanceData.Instance->Guides.bIsSimulationEnable || InstanceData.Instance->Guides.bHasGlobalInterpolation || InstanceData.Instance->Guides.bIsDeformationEnable || InstanceData.Instance->Guides.bIsSimulationCacheEnable;
				InstanceData.bHasSkinning			= InstanceData.BindingType == EHairBindingType::Skinning && InstanceData.MeshLODIndex >= 0;
				InstanceData.bNeedDeformation		= InstanceData.bValidGuide || InstanceData.bHasSkinning;
				InstanceData.CardsSimulationType 	= EHairCardsSimulationType::None;
				if (InstanceData.bHasSkinning || InstanceData.bValidGuide)
				{
					switch (GetHairCardsInterpolationType())
					{
						case 0 : InstanceData.CardsSimulationType = EHairCardsSimulationType::None; break;
						case 1 : InstanceData.CardsSimulationType = EHairCardsSimulationType::Guide; break;
						case 2 : InstanceData.CardsSimulationType = EHairCardsSimulationType::RBF;  break;
					}
				}

				if (InstanceData.bNeedDeformation)
				{
					check(InstanceData.CardInstance->Guides.RestResource);
				}
				if (InstanceData.bSimulationEnable || InstanceData.bDeformationEnable)
				{
					check(InstanceData.CardInstance->Guides.IsValid());
				}
				if (InstanceData.BindingType == EHairBindingType::Skinning || InstanceData.bGlobalDeformationEnable)
				{
					check(InstanceData.CardInstance->Guides.IsValid());
					check(InstanceData.CardInstance->Guides.HasValidRootData());
					check(InstanceData.MeshLODIndex == -1 || InstanceData.CardInstance->Guides.DeformedRootResource->IsValid(InstanceData.MeshLODIndex));
				}
			}
			else if (Instance->GeometryType == EHairGeometryType::Meshes)
			{
				check(Instance->Meshes.IsValid(InstanceData.HairLODIndex));
				InstanceData.bNeedDeformation 	= Instance->Meshes.LODs[InstanceData.HairLODIndex].DeformedResource != nullptr;
				InstanceData.MeshInstance 		= &Instance->Meshes.LODs[InstanceData.HairLODIndex];
				if (InstanceData.bNeedDeformation)
				{
					check(Instance->BindingType == EHairBindingType::Skinning);
					check(Instance->Guides.IsValid());
					check(Instance->Guides.HasValidRootData());
					check(Instance->Guides.DeformedRootResource);
					// MeshLODIndex -1 indicates that skin cache is disabled and this is a workaround to prevent the editor from crashing 
					// An editor setting guildeline popup exists to inform the user that the skin cache should be enabled. 
					check(InstanceData.MeshLODIndex == -1 || Instance->Guides.DeformedRootResource->IsValid(InstanceData.MeshLODIndex));
				}
			}

			// Prepare VF input
			InstanceData.Instance->HairGroupPublicData->VFInput.Cards  					= FHairGroupPublicData::FVertexFactoryInput::FCards();
			InstanceData.Instance->HairGroupPublicData->VFInput.Meshes 					= FHairGroupPublicData::FVertexFactoryInput::FMeshes();
			InstanceData.Instance->HairGroupPublicData->VFInput.GeometryType 			= InstanceData.Instance->GeometryType;
			InstanceData.Instance->HairGroupPublicData->VFInput.LocalToWorldTransform 	= Instance->GetCurrentLocalToWorld();
			InstanceData.Instance->HairGroupPublicData->bSupportVoxelization 			= false;
		}
	}

	// Cards only - Update dynamic mesh triangles
	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		if (InstanceData.MeshLODIndex >= 0 && (InstanceData.BindingType == EHairBindingType::Skinning || InstanceData.bGlobalDeformationEnable))
		{
			AddHairStrandUpdateMeshTrianglesPass(
				GraphBuilder,
				ShaderMap,
				InstanceData.MeshLODIndex,
				InstanceData.MeshLODData,
				InstanceData.CardInstance->Guides.RestRootResource,
				InstanceData.CardInstance->Guides.DeformedRootResource);
		}
	}

	// Cards only - Update position offset (GPU)
	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		if (InstanceData.MeshLODIndex >= 0 && (InstanceData.BindingType == EHairBindingType::Skinning || InstanceData.bGlobalDeformationEnable))
		{
			AddHairStrandUpdatePositionOffsetPass(
				GraphBuilder,
				ShaderMap,
				EHairPositionUpdateType::Cards,
				InstanceData.RegisteredIndex,
				InstanceData.HairLODIndex,
				InstanceData.MeshLODIndex,
				InstanceData.CardInstance->Guides.DeformedRootResource,
				InstanceData.CardInstance->Guides.DeformedResource);
		}
	}

	// Cards only - Update position offset (CPU)
	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		if (InstanceData.MeshLODIndex >= 0 && InstanceData.BindingType != EHairBindingType::Skinning && !InstanceData.bGlobalDeformationEnable && (InstanceData.bSimulationEnable || InstanceData.bDeformationEnable))
		{
			AddHairStrandUpdatePositionOffsetPass(
				GraphBuilder,
				ShaderMap,
				EHairPositionUpdateType::Cards,
				InstanceData.RegisteredIndex,
				InstanceData.HairLODIndex,
				InstanceData.MeshLODIndex,
				nullptr,
				InstanceData.CardInstance->Guides.DeformedResource);
		}
	}

	ERHIPipeline CardPositionExternalAccessPipeline = ERHIPipeline::Graphics;
#if RHI_RAYTRACING
	// Also add async compute as queue which can access these external hair resources
	if (GSupportsEfficientAsyncCompute && GRHIGlobals.RayTracing.SupportsAsyncRayTraceDispatch && IsHairRayTracingEnabled())
	{
		EnumAddFlags(CardPositionExternalAccessPipeline, ERHIPipeline::AsyncCompute);
	}
#endif
	FRDGExternalAccessQueue ExternalAccessQueue;

	// Cards only - Deform guide with skinning and/or simulation
	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		FHairGroupInstance::FCards::FLOD& LOD = *InstanceData.CardInstance;

		if (InstanceData.bNeedDeformation)
		{
			// 1. Cards are deformed based on guides motion (simulation or RBF applied on guides)
			if (InstanceData.CardsSimulationType == EHairCardsSimulationType::Guide)
			{
				FRDGBufferUAVRef Guides_DeformedPositionUAV = RegisterAsUAV(GraphBuilder, LOD.Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current));

				const bool bUseSingleGuide = LOD.Guides.InterpolationResource->UseSingleGuide();

				FRDGHairStrandsCullingData CullingData;
				AddHairStrandsInterpolationPass(
					GraphBuilder,
					ShaderMap,
					View->GetShaderPlatform(),
					ShaderPrintData,
					InstanceData.Instance,
					LOD.Guides.RestResource->GetPointCount(),
					LOD.Guides.RestResource->GetCurveCount(),
					LOD.Guides.RestResource,
					InstanceData.Instance->Guides.RestResource,
					InstanceData.bValidGuide,
					LOD.Guides.RestResource->BulkData.Header.MaxPointPerCurve,
					InstanceData.MeshLODIndex,
					1.0f,
					LOD.Guides.HairInterpolationType,
					EHairGeometryType::Cards,
					CullingData,
					LOD.Guides.RestResource->GetPositionOffset(),
					InstanceData.bValidGuide ? InstanceData.Instance->Guides.RestResource->GetPositionOffset() : FVector::ZeroVector,
					RegisterAsSRV(GraphBuilder, LOD.Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current)),
					InstanceData.bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current)) : nullptr,
					InstanceData.bHasSkinning ? LOD.Guides.RestRootResource : nullptr ,
					InstanceData.bHasSkinning && InstanceData.bValidGuide ? InstanceData.Instance->Guides.RestRootResource : nullptr,
					InstanceData.bHasSkinning ? LOD.Guides.DeformedRootResource : nullptr,
					InstanceData.bHasSkinning && InstanceData.bValidGuide ? InstanceData.Instance->Guides.DeformedRootResource : nullptr,
					RegisterAsSRV(GraphBuilder, LOD.Guides.RestResource->PositionBuffer),
					RegisterAsSRV(GraphBuilder, LOD.Guides.RestResource->CurveBuffer),
					bUseSingleGuide,
					InstanceData.bValidGuide ? RegisterAsSRV(GraphBuilder, LOD.Guides.InterpolationResource->CurveInterpolationBuffer) : nullptr,
					InstanceData.bValidGuide ? RegisterAsSRV(GraphBuilder, LOD.Guides.InterpolationResource->PointInterpolationBuffer) : nullptr,
					InstanceData.bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.RestResource->PositionBuffer) : nullptr,
					InstanceData.bValidGuide ? RegisterAsSRV(GraphBuilder, InstanceData.Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current)) : nullptr,
					nullptr,
					Guides_DeformedPositionUAV,
					FHairStrandsLODDeformedRootResource::Current);
			}
		}
	}

	// Cards only - Deform final cards geometry (using guides)
	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		FHairGroupInstance::FCards::FLOD& LOD = *InstanceData.CardInstance;

		if (InstanceData.bNeedDeformation)
		{
			// 1. Cards are deformed based on guides motion (simulation or RBF applied on guides)
			if (InstanceData.CardsSimulationType == EHairCardsSimulationType::Guide)
			{
				AddHairCardsDeformationPass(
					GraphBuilder,
					ShaderMap,
					View->GetFeatureLevel(),
					ShaderPrintData,
					InstanceData.Instance,
					InstanceData.HairLODIndex,
					InstanceData.MeshLODIndex,
					CardPositionExternalAccessPipeline);
			}
		}
	}

	// Cards only - Deform final cards geometry (using RBF)
	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		FHairGroupInstance::FCards::FLOD& LOD = *InstanceData.CardInstance;

		if (InstanceData.bNeedDeformation)
		{
			// 2. Cards are deformed only based on skel. mesh RBF data)
			if (InstanceData.CardsSimulationType == EHairCardsSimulationType::RBF)
			{
				AddHairCardsRBFInterpolationPass(
					GraphBuilder,
					ShaderMap,
					InstanceData.MeshLODIndex,
					LOD.RestResource,
					LOD.DeformedResource,
					InstanceData.Instance->Guides.RestRootResource,
					InstanceData.Instance->Guides.DeformedRootResource);
			}
		}
	}

	for (uint32 InstanceIndex : CardInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		FHairGroupInstance::FCards::FLOD& LOD = *InstanceData.CardInstance;

		if (InstanceData.bNeedDeformation)
		{		
			// 2. Cards are deformed only based on skel. mesh RBF data)
			if (InstanceData.CardsSimulationType == EHairCardsSimulationType::RBF)
			{
				AddHairCardsRBFInterpolationPass(
					GraphBuilder,
					ShaderMap,
					InstanceData.MeshLODIndex,
					LOD.RestResource,
					LOD.DeformedResource,
					InstanceData.Instance->Guides.RestRootResource,
					InstanceData.Instance->Guides.DeformedRootResource);
			}

			if (LOD.DeformedResource)
			{
				ExternalAccessQueue.Add(Register(GraphBuilder, LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::Current), ERDGImportedBufferFlags::None).Buffer, ERHIAccess::SRVMask, CardPositionExternalAccessPipeline);
				ExternalAccessQueue.Add(Register(GraphBuilder, LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::Previous), ERDGImportedBufferFlags::None).Buffer, ERHIAccess::SRVMask);
			}
		}
	}

	const bool bIsReflectionCapture = View && View->bIsReflectionCapture;

	// Cards only - Build/update RT geometry
	#if RHI_RAYTRACING
	if (IsHairRayTracingEnabled() && IsRayTracingEnabled() && !bIsReflectionCapture)
	{
		for (uint32 InstanceIndex : CardInstances)
		{
			FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
			if (InstanceData.CardInstance->RaytracingResource)
			{
				AddBuildHairCardAccelerationStructurePass(
					GraphBuilder,
					InstanceData.Instance,
					InstanceData.HairLODIndex,
					InstanceData.bNeedDeformation);
			}
		}
	}
	#endif

	// Meshes only - Deform final mesh geometry (using RBF)
	for (uint32 InstanceIndex : MeshInstances)
	{
		FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
		if (InstanceData.bNeedDeformation)
		{
			AddHairMeshesRBFInterpolationPass(
				GraphBuilder,
				ShaderMap,
				InstanceData.MeshLODIndex,
				InstanceData.MeshInstance->RestResource,
				InstanceData.MeshInstance->DeformedResource,
				InstanceData.Instance->Guides.RestRootResource,
				InstanceData.Instance->Guides.DeformedRootResource);
	
			ExternalAccessQueue.Add(Register(GraphBuilder, InstanceData.MeshInstance->DeformedResource->GetBuffer(FHairMeshesDeformedResource::Current), ERDGImportedBufferFlags::None).Buffer, ERHIAccess::SRVMask);
		}
	}

	// Meshes only - Build/update RT geometry
	#if RHI_RAYTRACING
	if (IsHairRayTracingEnabled() && IsRayTracingEnabled() && !bIsReflectionCapture)
	{
		for (uint32 InstanceIndex : MeshInstances)
		{
			FInstanceData& InstanceData = InstanceDatas[InstanceIndex];
			if (InstanceData.MeshInstance->RaytracingResource)
			{
				AddBuildHairMeshAccelerationStructurePass(
					GraphBuilder,
					InstanceData.Instance,
					InstanceData.HairLODIndex,
					InstanceData.bNeedDeformation);
			}
		}
	}
	#endif

	ExternalAccessQueue.Submit(GraphBuilder);
}

// Return the LOD which should be used for a given screen size and LOD bias value
// This function is mirrored in HairStrandsClusterCommon.ush
static float GetHairInstanceLODIndex(const TArray<float>& InLODScreenSizes, float InScreenSize, float InLODBias)
{
	const uint32 LODCount = InLODScreenSizes.Num();
	check(LODCount > 0);

	float OutLOD = 0;
	if (LODCount > 1 && InScreenSize < InLODScreenSizes[0])
	{
		for (uint32 LODIt = 1; LODIt < LODCount; ++LODIt)
		{
			if (InScreenSize >= InLODScreenSizes[LODIt])
			{
				uint32 PrevLODIt = LODIt - 1;

				const float S_Delta = abs(InLODScreenSizes[PrevLODIt] - InLODScreenSizes[LODIt]);
				const float S = S_Delta > 0 ? FMath::Clamp(FMath::Abs(InScreenSize - InLODScreenSizes[LODIt]) / S_Delta, 0.f, 1.f) : 0;
				OutLOD = PrevLODIt + (1 - S);
				break;
			}
			else if (LODIt == LODCount - 1)
			{
				OutLOD = LODIt;
			}
		}
	}

	if (InLODBias != 0)
	{
		OutLOD = FMath::Clamp(OutLOD + InLODBias, 0.f, float(LODCount - 1));
	}
	return OutLOD;
}


static EHairGeometryType ConvertLODGeometryType(EHairGeometryType Type, bool InbUseCards, EShaderPlatform Platform)
{
	// Force cards only if it is enabled or fallback on cards if strands are disabled
	InbUseCards = (InbUseCards || !IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform)) && IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Platform);

	switch (Type)
	{
	case EHairGeometryType::Strands: return IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform) ? (InbUseCards ? EHairGeometryType::Cards : EHairGeometryType::Strands) : (InbUseCards ? EHairGeometryType::Cards : EHairGeometryType::NoneGeometry);
	case EHairGeometryType::Cards:   return IsHairStrandsEnabled(EHairStrandsShaderType::Cards, Platform) ? EHairGeometryType::Cards : EHairGeometryType::NoneGeometry;
	case EHairGeometryType::Meshes:  return IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, Platform) ? EHairGeometryType::Meshes : EHairGeometryType::NoneGeometry;
	}
	return EHairGeometryType::NoneGeometry;
}

static EHairGeometryType GetLODGeometryType(FHairGroupInstance* InInstance, int32 InLODIndex, EShaderPlatform InShaderPlatform=GMaxRHIShaderPlatform)
{
	const TArray<EHairGeometryType>& LODGeometryTypes = InInstance->HairGroupPublicData->GetLODGeometryTypes();
	check(LODGeometryTypes.IsValidIndex(InLODIndex));
	const bool bForceCards = GHairStrands_UseCards > 0 || InInstance->bForceCards;
	return ConvertLODGeometryType(LODGeometryTypes[InLODIndex], bForceCards, InShaderPlatform);
}

static void RunHairBufferSwap(const FHairStrandsInstances& Instances, const TArray<const FSceneView*> Views)
{
	EShaderPlatform ShaderPlatform = EShaderPlatform::SP_NumPlatforms;
	if (Views.Num() > 0)
	{
		ShaderPlatform = Views[0]->GetShaderPlatform();
	}

	for (FHairStrandsInstance* AbstractInstance : Instances)
	{
		FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);

		check(Instance);
		if (!Instance)
			continue;

		check(Instance->HairGroupPublicData);

		// Swap current/previous buffer for all LODs
		const bool bIsPaused = Views[0]->Family->bWorldIsPaused;
		if (!bIsPaused)
		{
			if (Instance->Guides.DeformedResource) { Instance->Guides.DeformedResource->SwapBuffer(); }
			if (Instance->Strands.DeformedResource) { Instance->Strands.DeformedResource->SwapBuffer(); }

			// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
			if (IsHairStrandContinuousDecimationReorderingEnabled())
			{
				if (Instance->Guides.DeformedRootResource) { Instance->Guides.DeformedRootResource->SwapBuffer(); }
				if (Instance->Strands.DeformedRootResource) { Instance->Strands.DeformedRootResource->SwapBuffer(); }
			}

			Instance->Debug.LastFrameIndex = Views[0]->Family->FrameNumber;
		}
	}
}

static void AddCopyHairStrandsPositionPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FHairStrandsRestResource& RestResources,
	FHairStrandsDeformedResource& DeformedResources)
{
	FRDGBufferRef RestBuffer = Register(GraphBuilder, RestResources.PositionBuffer, ERDGImportedBufferFlags::None).Buffer;
	FRDGBufferRef Deformed0Buffer = Register(GraphBuilder, DeformedResources.DeformedPositionBuffer[0], ERDGImportedBufferFlags::None).Buffer;
	FRDGBufferRef Deformed1Buffer = Register(GraphBuilder, DeformedResources.DeformedPositionBuffer[1], ERDGImportedBufferFlags::None).Buffer;
	AddCopyBufferPass(GraphBuilder, Deformed0Buffer, RestBuffer);
	AddCopyBufferPass(GraphBuilder, Deformed1Buffer, RestBuffer);
}

#if RHI_RAYTRACING
void AllocateRaytracingResources(FHairGroupInstance* Instance, bool bNeedDynamicResources, FName OwnerName)
{
	check(Instance->Strands.RenRaytracingResource == nullptr);

	if (bNeedDynamicResources)
	{
		// Allocate dynamic raytracing resources (owned by the groom component/instance)
		FHairResourceName ResourceName(FName(Instance->Debug.GroomAssetName), Instance->Debug.GroupIndex);
		Instance->Strands.RenRaytracingResource = new FHairStrandsRaytracingResource(Instance->Strands.GetData(), ResourceName, OwnerName);
		Instance->Strands.RenRaytracingResourceOwned = true;
	}
	else
	{
		if (Instance->Strands.RenRaytracingResourceShared)
		{
			// (Lazy) Allocate static raytracing resources (owned by the grooom asset)
			if (*Instance->Strands.RenRaytracingResourceShared == nullptr)
			{
				FName SharedOwnerName(Instance->Debug.GroomAssetName);
				FHairResourceName ResourceName(SharedOwnerName, Instance->Debug.GroupIndex);
				*Instance->Strands.RenRaytracingResourceShared = new FHairStrandsRaytracingResource(Instance->Strands.GetData(), ResourceName, SharedOwnerName);
			}
			Instance->Strands.RenRaytracingResourceOwned = false;
			Instance->Strands.RenRaytracingResource = *Instance->Strands.RenRaytracingResourceShared;
			check(Instance->Strands.RenRaytracingResource);
		}
	}
}

static void AllocateRaytracingResources(FHairGroupInstance* Instance, int32 InMeshLODIndex)
{
	const bool bNeedDynamicResources = Instance->Strands.DeformedResource != nullptr;
	if (IsHairRayTracingEnabled() && !Instance->Strands.RenRaytracingResource )
	{
		check(Instance->Strands.IsValid());

		#if RHI_ENABLE_RESOURCE_INFO
		FName OwnerName(FString::Printf(TEXT("%s [LOD%d]"), *Instance->Debug.MeshComponentName, InMeshLODIndex));
		#else
		FName OwnerName = NAME_None;
		#endif
		
		AllocateRaytracingResources(Instance, bNeedDynamicResources, OwnerName);
	}
	Instance->Strands.ViewRayTracingMask |= EHairViewRayTracingMask::PathTracing;
}

#endif


void AddHairStreamingRequest(FHairGroupInstance* Instance, int32 InLODIndex)
{
	if (Instance && InLODIndex >= 0)
	{
		check(Instance->HairGroupPublicData);

		// Hypothesis: the mesh LOD will be the same as the hair LOD
		const int32 MeshLODIndex = InLODIndex;
		
		// Insure that MinLOD is necessary taken into account if a force LOD is request (i.e., LODIndex>=0). If a Force LOD 
		// is not resquested (i.e., LODIndex<0), the MinLOD is applied after ViewLODIndex has been determined in the codeblock below
		const float MinLOD = FMath::Max(0, GHairStrandsMinLOD);
		float LODIndex = FMath::Max(InLODIndex, MinLOD);

		const TArray<bool>& LODVisibilities = Instance->HairGroupPublicData->GetLODVisibilities();
		const int32 LODCount = LODVisibilities.Num();

		LODIndex = FMath::Clamp(LODIndex, 0.f, float(LODCount - 1));

		const int32 IntLODIndex = FMath::Clamp(FMath::FloorToInt(LODIndex), 0, LODCount - 1);
		const bool bIsVisible = LODVisibilities[IntLODIndex];

		EHairGeometryType GeometryType = GetLODGeometryType(Instance, IntLODIndex);
		if (GeometryType == EHairGeometryType::Meshes)
		{
			if (!Instance->Meshes.IsValid(IntLODIndex))
			{
				GeometryType = EHairGeometryType::NoneGeometry;
			}
		}
		else if (GeometryType == EHairGeometryType::Cards)
		{
			if (!Instance->Cards.IsValid(IntLODIndex))
			{
				GeometryType = EHairGeometryType::NoneGeometry;
			}
		}
		else if (GeometryType == EHairGeometryType::Strands)
		{
			if (!Instance->Strands.IsValid())
			{
				GeometryType = EHairGeometryType::NoneGeometry;
			}
		}

		if (!bIsVisible)
		{
			GeometryType = EHairGeometryType::NoneGeometry;
		}

		const bool bSimulationEnable			= Instance->HairGroupPublicData->IsSimulationEnable(LODIndex);
		const bool bDeformationEnable			= Instance->HairGroupPublicData->bIsDeformationEnable;
		const bool bGlobalInterpolationEnable	= Instance->HairGroupPublicData->IsGlobalInterpolationEnable(LODIndex);
		const bool bSimulationCacheEnable		= Instance->HairGroupPublicData->bIsSimulationCacheEnable;
		const bool bLODNeedsGuides				= bSimulationEnable || bDeformationEnable || bGlobalInterpolationEnable || bSimulationCacheEnable;

		const EHairResourceLoadingType LoadingType = GetHairResourceLoadingType(GeometryType, int32(LODIndex));
		if (LoadingType != EHairResourceLoadingType::Async || GeometryType == EHairGeometryType::NoneGeometry)
		{
			return;
		}

		// Lazy allocation of resources
		// Note: Allocation will only be done if the resources is not initialized yet. Guides deformed position are also initialized from the Rest position at creation time.
		if (Instance->Guides.RestResource && bLODNeedsGuides)
		{
			if (Instance->Guides.RestRootResource)			{ Instance->Guides.RestRootResource->StreamInData(MeshLODIndex);}
			if (Instance->Guides.RestResource)				{ Instance->Guides.RestResource->StreamInData(); }
		}

		if (GeometryType == EHairGeometryType::Meshes)
		{
			FHairGroupInstance::FMeshes::FLOD& InstanceLOD = Instance->Meshes.LODs[IntLODIndex];
		}
		else if (GeometryType == EHairGeometryType::Cards)
		{
			FHairGroupInstance::FCards::FLOD& InstanceLOD = Instance->Cards.LODs[IntLODIndex];

			if (InstanceLOD.InterpolationResource)			{ InstanceLOD.InterpolationResource->StreamInData(); }
			if (InstanceLOD.DeformedResource)				{ InstanceLOD.DeformedResource->StreamInData(); }
			if (InstanceLOD.Guides.RestRootResource)		{ InstanceLOD.Guides.RestRootResource->StreamInData(MeshLODIndex); }
			if (InstanceLOD.Guides.RestResource)			{ InstanceLOD.Guides.RestResource->StreamInData(); }
			if (InstanceLOD.Guides.InterpolationResource)	{ InstanceLOD.Guides.InterpolationResource->StreamInData(); }
		}
		else if (GeometryType == EHairGeometryType::Strands)
		{
			if (Instance->Strands.RestRootResource)			{ Instance->Strands.RestRootResource->StreamInData(MeshLODIndex); }
			if (Instance->Strands.RestResource)				{ Instance->Strands.RestResource->StreamInData(); }
			if (Instance->Strands.ClusterResource)			{ Instance->Strands.ClusterResource->StreamInData(); }
			if (Instance->Strands.InterpolationResource)	{ Instance->Strands.InterpolationResource->StreamInData(); }
		}
	}
}

static FVector2f ComputeProjectedScreenPos(const FVector& InWorldPos, const FSceneView& View)
{
	// Compute the MinP/MaxP in pixel coord, relative to View.ViewRect.Min
	const FMatrix& WorldToView = View.ViewMatrices.GetWorldToView();
	const FMatrix& ViewToProj = View.ViewMatrices.GetViewToClip();
	const float NearClippingDistance = View.NearClippingDistance + SMALL_NUMBER;
	const FIntRect ViewRect = View.UnconstrainedViewRect;

	// Clamp position on the near plane to get valid rect even if bounds' points are behind the camera
	FPlane P_View = WorldToView.TransformFVector4(FVector4(InWorldPos, 1.f));
	if (P_View.Z <= NearClippingDistance)
	{
		P_View.Z = NearClippingDistance;
	}

	// Project from view to projective space
	FVector2D MinP(FLT_MAX, FLT_MAX);
	FVector2D MaxP(-FLT_MAX, -FLT_MAX);
	FVector2D ScreenPos;
	const bool bIsValid = FSceneView::ProjectWorldToScreen(P_View, ViewRect, ViewToProj, ScreenPos);

	// Clamp to pixel border
	ScreenPos = FIntPoint(FMath::FloorToInt(ScreenPos.X), FMath::FloorToInt(ScreenPos.Y));

	// Clamp to screen rect
	ScreenPos.X = FMath::Clamp(ScreenPos.X, ViewRect.Min.X, ViewRect.Max.X);
	ScreenPos.Y = FMath::Clamp(ScreenPos.Y, ViewRect.Min.Y, ViewRect.Max.Y);

	return FVector2f(ScreenPos.X, ScreenPos.Y);
}

static float ComputeActiveCurveCoverageScale(const FHairStrandsBulkData& InData, uint32 InAvailableCurveCount)
{
	const uint32 RestCurveCount = InData.Header.CurveCount;
	const float CurveRatio = InAvailableCurveCount / float(RestCurveCount);
	return InData.GetCoverageScale(CurveRatio);
}

static float ComputeActiveCurveRadiusScale(const FHairStrandsClusterResource* InResource, float InLODIndex)
{
	float Out = 1.f;
	if (InResource && InResource->BulkData.IsValid())
	{
		const FHairStrandsClusterBulkData& InData = InResource->BulkData;
		const uint32 MaxLODCount = InData.Header.LODInfos.Num();
		InLODIndex = FMath::Clamp(InLODIndex, 0.f,MaxLODCount-1);
		const int32 iLODIndex0 = FMath::Floor(InLODIndex);
		const int32 iLODIndex1 = FMath::Min(iLODIndex0+1,int32(MaxLODCount-1));
		const float S = InLODIndex - iLODIndex0;
		check(InData.Header.LODInfos.IsValidIndex(iLODIndex0));
		check(InData.Header.LODInfos.IsValidIndex(iLODIndex1));

		const float RadiusScale0 = InData.Header.LODInfos[iLODIndex0].RadiusScale;
		const float RadiusScale1 = InData.Header.LODInfos[iLODIndex1].RadiusScale;
		Out = iLODIndex0 != iLODIndex1 ? FMath::LerpStable(RadiusScale0, RadiusScale1, S) : RadiusScale0;
	}
	return Out;
}

static uint32 ComputeActiveCurveCount(float InScreenSize, float InAutoLODBias, uint32 InCurveCount, uint32 InClusterCount)
{
	const float Power = 1.f;  // This could be exposed per asset
	const float ScreenSizeBias = FMath::Clamp(InAutoLODBias + GetHairStrandsAutoLODBias(), -1.f, 1.f);
	uint32 OutCurveCount = InCurveCount * FMath::Pow(FMath::Clamp(InScreenSize + ScreenSizeBias, 0.f, 1.0f), Power);
	// Ensure there is at least 1 curve per cluster
	OutCurveCount = FMath::Max(InClusterCount, OutCurveCount);
	return FMath::Clamp(OutCurveCount, 1, InCurveCount);
}

struct FHairLOD
{
	float HairLODIndex = -1.f;
	float MinHairLOD = 0.f;
	uint32 HairLODCount = 0;

	float LODPredictedIndex = -1.f;
	float DebugScreenSize = 0.f;

	uint32 ContinuousLODPointCount = 0;
	uint32 ContinuousLODCurveCount = 0;
	float ContinuousLODScreenSize = 1.f;
	FVector2f ContinuousLODScreenPos = FVector2f(0,0);
	FBoxSphereBounds ContinuousLODBounds;
	float ContinuousLODCoverageScale = 1.f;
	float ContinuousLODRadiusScale = 1.f;
};

static FHairLOD ComputeHairLODIndex(const FHairGroupInstance* Instance, const TArray<const FSceneView*>& Views, bool bStrandsSupported)
{
	check(Instance);

	FHairLOD Out;

	// Insure that MinLOD is necessary taken into account if a force LOD is request (i.e., LODIndex>=0). If a Force LOD 
	// is not requested (i.e., LODIndex<0), the MinLOD is applied after ViewLODIndex has been determined in the codeblock below
	Out.HairLODCount = Instance->HairGroupPublicData->GetLODVisibilities().Num();
	Out.MinHairLOD = FMath::Max(0, GHairStrandsMinLOD);
	Out.HairLODIndex = IsHairVisibilityComputeRasterContinuousLODEnabled() ? 0.0 : (Instance->Debug.LODForcedIndex >= 0 ? FMath::Max(Instance->Debug.LODForcedIndex, Out.MinHairLOD) : -1.0f);

	FVector2f MaxContinuousLODScreenPos = FVector2f(0.f, 0.f);
	float LODViewIndex = -1;
	float MaxScreenSize_RestBound = 0.f;
	float MaxScreenSize_Bound = 0.f;
	const FSphere SphereBound = Instance->GetBounds().GetSphere();
	for (const FSceneView* View : Views)
	{
		const FVector3d BoundScale = Instance->LocalToWorld.GetScale3D();
		const FVector3f BoundExtent = Instance->Strands.IsValid() ? FVector3f(Instance->Strands.GetData().Header.BoundingBox.GetExtent()) : FVector3f(0,0,0);
		const float BoundRadius = FMath::Max3(BoundExtent.X, BoundExtent.Y, BoundExtent.Z) * FMath::Max3(BoundScale.X, BoundScale.Y, BoundScale.Z);

		const float ScreenSize_RestBound = FMath::Clamp(ComputeBoundsScreenSize(FVector4(SphereBound.Center, 1), BoundRadius, *View), 0.f, 1.0f);
		const float ScreenSize_Bound = FMath::Clamp(ComputeBoundsScreenSize(FVector4(SphereBound.Center, 1), SphereBound.W, *View), 0.f, 1.0f);

		const float CurrLODViewIndex = FMath::Max(Out.MinHairLOD, GetHairInstanceLODIndex(Instance->HairGroupPublicData->GetLODScreenSizes(), ScreenSize_Bound, Instance->Strands.Modifier.LODBias));

		MaxScreenSize_Bound = FMath::Max(MaxScreenSize_Bound, ScreenSize_Bound);
		MaxScreenSize_RestBound = FMath::Max(MaxScreenSize_RestBound, ScreenSize_RestBound);
		MaxContinuousLODScreenPos = ComputeProjectedScreenPos(SphereBound.Center, *View);

		// Select highest LOD across all views
		LODViewIndex = LODViewIndex < 0 ? CurrLODViewIndex : FMath::Min(LODViewIndex, CurrLODViewIndex);
	}

	if (Out.HairLODIndex < 0)
	{
		Out.HairLODIndex = LODViewIndex;
	}
	Out.HairLODIndex = FMath::Clamp(Out.HairLODIndex, 0.f, float(Out.HairLODCount - 1));

	// Feedback game thread with LOD selection 
	Out.LODPredictedIndex = LODViewIndex;
	Out.DebugScreenSize = MaxScreenSize_Bound;
	Out.ContinuousLODPointCount = Instance->HairGroupPublicData->RestPointCount;
	Out.ContinuousLODCurveCount = Instance->HairGroupPublicData->RestCurveCount;
	Out.ContinuousLODScreenSize = MaxScreenSize_RestBound;
	Out.ContinuousLODScreenPos = MaxContinuousLODScreenPos;
	Out.ContinuousLODBounds = SphereBound;
	Out.ContinuousLODCoverageScale = 1.f;
	Out.ContinuousLODRadiusScale = 1.f;

	// Extract the min/max LOD with strands. 
	// This assumes that these LODs are contiguous
	int32 MinLODIndexWithStrands = INDEX_NONE;
	int32 MaxLODIndexWithStrands = INDEX_NONE;

	// Extract only if active shader platform supports strands without conversion.
	if (bStrandsSupported)
	{
		int32 LODIndex = 0;
		const TArray<EHairGeometryType>& GeometryTypes = Instance->HairGroupPublicData->GetLODGeometryTypes();
		for (EHairGeometryType Type : GeometryTypes)
		{
			if (Type == EHairGeometryType::Strands)
			{
				if (MinLODIndexWithStrands == INDEX_NONE)
				{
					MinLODIndexWithStrands = LODIndex;
				}
				MaxLODIndexWithStrands = LODIndex;
			}
			++LODIndex;
		}
	}

	// Auto LOD
	const bool bNeedAutoLOD = MaxLODIndexWithStrands != INDEX_NONE && int32(Out.HairLODIndex) <= MaxLODIndexWithStrands;
	const float OriginalHairLODIndex = Out.HairLODIndex;
	if (Instance->Strands.ClusterResource && Instance->Strands.RestResource && Instance->bSupportStreaming)
	{
		uint32 EffectiveCurveCount = 0;
		if (bNeedAutoLOD && (Instance->HairGroupPublicData->bAutoLOD || UseHairStrandsForceAutoLOD()))
		{
			EffectiveCurveCount = ComputeActiveCurveCount(Out.ContinuousLODScreenSize, Instance->HairGroupPublicData->AutoLODBias, Instance->HairGroupPublicData->RestCurveCount, Instance->HairGroupPublicData->ClusterCount);
			Out.HairLODIndex = MinLODIndexWithStrands;
		}
		else
		{
			EffectiveCurveCount = Instance->Strands.ClusterResource->BulkData.GetCurveCount(Out.HairLODIndex);
		}
		check(EffectiveCurveCount <= uint32(Instance->Strands.GetData().Header.CurveToPointCount.Num()));

		Out.ContinuousLODCurveCount = EffectiveCurveCount;
		Out.ContinuousLODPointCount = EffectiveCurveCount > 0 ? Instance->Strands.GetData().Header.CurveToPointCount[EffectiveCurveCount - 1] : 0;
		Out.ContinuousLODCoverageScale = ComputeActiveCurveCoverageScale(Instance->Strands.GetData(), EffectiveCurveCount);
		Out.ContinuousLODRadiusScale = ComputeActiveCurveRadiusScale(Instance->Strands.ClusterResource, OriginalHairLODIndex); // Use OriginalHairLODIndex to query the correct CurveRadiusScale
	}

	return Out;
}

static void ApplyHairLOD(const FHairLOD& In, FHairGroupInstance* OutInstance)
{
	// Feedback game thread with LOD selection 
	OutInstance->Debug.LODPredictedIndex = In.LODPredictedIndex;
	OutInstance->HairGroupPublicData->DebugScreenSize = In.DebugScreenSize;
	OutInstance->HairGroupPublicData->ContinuousLODPreviousPointCount = OutInstance->HairGroupPublicData->ContinuousLODPointCount;
	OutInstance->HairGroupPublicData->ContinuousLODPreviousCurveCount = OutInstance->HairGroupPublicData->ContinuousLODCurveCount;
	OutInstance->HairGroupPublicData->ContinuousLODPointCount = In.ContinuousLODPointCount;
	OutInstance->HairGroupPublicData->ContinuousLODCurveCount = In.ContinuousLODCurveCount;
	OutInstance->HairGroupPublicData->ContinuousLODScreenSize = In.ContinuousLODScreenSize;
	OutInstance->HairGroupPublicData->ContinuousLODScreenPos = In.ContinuousLODScreenPos;
	OutInstance->HairGroupPublicData->ContinuousLODBounds = In.ContinuousLODBounds;
	OutInstance->HairGroupPublicData->ContinuousLODCoverageScale = In.ContinuousLODCoverageScale;
	OutInstance->HairGroupPublicData->ContinuousLODRadiusScale = In.ContinuousLODRadiusScale;
}

// Function for selecting, loading, & initializing LOD resources
static bool SelectValidLOD(
	FRDGBuilder& GraphBuilder, 
	FGlobalShaderMap* ShaderMap, 
	EShaderPlatform ShaderPlatform, 
	FHairGroupInstance* Instance, 
	float HairLODIndex,
	float PrevHairLODIndex, 
	float HairLODCount, 
	float MeshLODIndex, 
	float PrevMeshLODIndex,
	bool bHasPathTracingView)
{
	const int32 IntHairLODIndex = FMath::Clamp(FMath::FloorToInt(HairLODIndex), 0, HairLODCount - 1);
	const TArray<bool>& LODVisibilities = Instance->HairGroupPublicData->GetLODVisibilities();
	const bool bIsVisible = LODVisibilities[IntHairLODIndex];

	EHairGeometryType GeometryType = GetLODGeometryType(Instance, IntHairLODIndex, ShaderPlatform);

	// Skip cluster culling if strands have a single LOD (i.e.: LOD0), and the instance is static (i.e., no skinning, no simulation, ...)
	bool bCullingEnable = false;
	if (GeometryType == EHairGeometryType::Meshes)
	{
		if (!Instance->Meshes.IsValid(IntHairLODIndex))
		{
			GeometryType = EHairGeometryType::NoneGeometry;
		}
	}
	else if (GeometryType == EHairGeometryType::Cards)
	{
		if (!Instance->Cards.IsValid(IntHairLODIndex))
		{
			GeometryType = EHairGeometryType::NoneGeometry;
		}
	}
	else if (GeometryType == EHairGeometryType::Strands)
	{
		if (!Instance->Strands.IsValid())
		{
			GeometryType = EHairGeometryType::NoneGeometry;
		}
		else
		{
			const bool bNeedLODing		= HairLODIndex > 0;
			const bool bNeedDeformation = Instance->Strands.DeformedResource != nullptr;
			bCullingEnable = (bNeedLODing || bNeedDeformation);
		}
	}

	const bool bDoesBindingSupportMeshLOD = MeshLODIndex >= Instance->HairGroupPublicData->BindingMinMeshLOD;
	if (!bIsVisible || !bDoesBindingSupportMeshLOD)
	{
		GeometryType = EHairGeometryType::NoneGeometry;
	}

	// Transform ContinuousLOD CurveCount/PointCount into streaming request
	uint32 RequestedCurveCount = Instance->HairGroupPublicData->RestCurveCount;
	uint32 RequestedPointCount = Instance->HairGroupPublicData->RestPointCount;
	const bool bStreamingEnabled = GHairStrands_Streaming > 0;
	if (GeometryType == EHairGeometryType::Strands && bStreamingEnabled && Instance->bSupportStreaming)
	{
		// Round Curve/Point request to curve 'page'
		RequestedCurveCount = GetRoundedCurveCount(Instance->HairGroupPublicData->ContinuousLODCurveCount, Instance->HairGroupPublicData->RestCurveCount);
		RequestedPointCount = RequestedCurveCount > 0 ? Instance->Strands.GetData().Header.CurveToPointCount[RequestedCurveCount - 1] : 0;
	}

	const bool bSimulationEnable			= Instance->HairGroupPublicData->IsSimulationEnable(IntHairLODIndex);
	const bool bDeformationEnable			= Instance->HairGroupPublicData->bIsDeformationEnable;
	const bool bGlobalInterpolationEnable	= Instance->HairGroupPublicData->IsGlobalInterpolationEnable(IntHairLODIndex);
	const bool bSimulationCacheEnable		= Instance->HairGroupPublicData->bIsSimulationCacheEnable;
	const bool bLODNeedsGuides				= bSimulationEnable || bDeformationEnable || bGlobalInterpolationEnable || bSimulationCacheEnable;

	// If we're not rendering hair, and we don't need guides work, bail before any allocations.
	// This prevents hitching for EHairGeometryType::NoneGeometry which isn't accounted for in GetHairResourceLoadingType() and just returns EHairResourceLoadingType::Sync
	if (GeometryType == EHairGeometryType::NoneGeometry && !bLODNeedsGuides)
	{
		const int32 IntPrevHairLODIndex = FMath::Clamp(FMath::FloorToInt(PrevHairLODIndex), 0, HairLODCount - 1);
		const EHairBindingType CurrBindingType = Instance->HairGroupPublicData->GetBindingType(IntHairLODIndex);
		const EHairBindingType PrevBindingType = Instance->HairGroupPublicData->GetBindingType(IntPrevHairLODIndex);
 
		Instance->HairGroupPublicData->SetLODVisibility(bIsVisible);
		Instance->HairGroupPublicData->SetLODIndex(HairLODIndex);
		Instance->HairGroupPublicData->SetLODBias(0);
		Instance->HairGroupPublicData->SetMeshLODIndex(MeshLODIndex);
 
		Instance->HairGroupPublicData->VFInput.GeometryType = GeometryType;
		Instance->HairGroupPublicData->VFInput.BindingType = CurrBindingType;
		Instance->HairGroupPublicData->VFInput.bHasLODSwitch = IntPrevHairLODIndex != IntHairLODIndex;
		Instance->HairGroupPublicData->VFInput.bHasLODSwitchBindingType = CurrBindingType != PrevBindingType;
 
		Instance->GeometryType = GeometryType;
		Instance->BindingType = CurrBindingType;

		// Return true to avoid further retry. While the selected LOD is not renderable, it is 'valid' as 'hidden'
		return true;
	}

	const EHairResourceLoadingType LoadingType = GetHairResourceLoadingType(GeometryType, IntHairLODIndex);
	EHairResourceStatus ResourceStatus;
	ResourceStatus.Status 				= EHairResourceStatus::EStatus::None;
	ResourceStatus.AvailableCurveCount 	= Instance->HairGroupPublicData->RestCurveCount;

	// Lazy allocation of resources
	// Note: Allocation will only be done if the resources is not initialized yet. Guides deformed position are also initialized from the Rest position at creation time.
	if (Instance->Guides.IsValid() && bLODNeedsGuides)
	{
		if (Instance->Guides.RestRootResource)			{ Instance->Guides.RestRootResource->Allocate(GraphBuilder, LoadingType, ResourceStatus, MeshLODIndex); }
		if (Instance->Guides.RestResource)				{ Instance->Guides.RestResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
		if (Instance->Guides.DeformedRootResource)		{ Instance->Guides.DeformedRootResource->Allocate(GraphBuilder, LoadingType, ResourceStatus, MeshLODIndex); }
		if (Instance->Guides.DeformedResource)			
		{ 
			// Ensure the rest resources are correctly loaded prior to initialized and copy the rest position into the deformed positions
			if (Instance->Guides.RestResource && Instance->Guides.RestResource->bIsInitialized)
			{
				const bool bNeedCopy = !Instance->Guides.DeformedResource->bIsInitialized; 
				Instance->Guides.DeformedResource->Allocate(GraphBuilder, EHairResourceLoadingType::Sync); 
				if (bNeedCopy) 
				{ 
					AddCopyHairStrandsPositionPass(GraphBuilder, ShaderMap, *Instance->Guides.RestResource, *Instance->Guides.DeformedResource); 
				}
			}
		}
	}

	if (GeometryType == EHairGeometryType::Meshes)
	{
		FHairGroupInstance::FMeshes::FLOD& InstanceLOD = Instance->Meshes.LODs[IntHairLODIndex];

		if (InstanceLOD.DeformedResource)				{ InstanceLOD.DeformedResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
#if RHI_RAYTRACING
		if (InstanceLOD.RaytracingResource)				{ InstanceLOD.RaytracingResource->Allocate(GraphBuilder, LoadingType, ResourceStatus);}
#endif

		InstanceLOD.InitVertexFactory();
	}
	else if (GeometryType == EHairGeometryType::Cards)
	{
		FHairGroupInstance::FCards::FLOD& InstanceLOD = Instance->Cards.LODs[IntHairLODIndex];

		if (InstanceLOD.InterpolationResource)			{ InstanceLOD.InterpolationResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
		if (InstanceLOD.DeformedResource)				{ InstanceLOD.DeformedResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
		if (InstanceLOD.Guides.RestRootResource)		{ InstanceLOD.Guides.RestRootResource->Allocate(GraphBuilder, LoadingType, ResourceStatus, MeshLODIndex); }
		if (InstanceLOD.Guides.RestResource)			{ InstanceLOD.Guides.RestResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
		if (InstanceLOD.Guides.DeformedRootResource)	{ InstanceLOD.Guides.DeformedRootResource->Allocate(GraphBuilder, LoadingType, ResourceStatus, MeshLODIndex); }
		if (InstanceLOD.Guides.DeformedResource)		{ InstanceLOD.Guides.DeformedResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
		if (InstanceLOD.Guides.InterpolationResource)	{ InstanceLOD.Guides.InterpolationResource->Allocate(GraphBuilder, LoadingType, ResourceStatus); }
#if RHI_RAYTRACING
		if (InstanceLOD.RaytracingResource)				{ InstanceLOD.RaytracingResource->Allocate(GraphBuilder, LoadingType, ResourceStatus);}
#endif
		InstanceLOD.InitVertexFactory();
	}
	else if (GeometryType == EHairGeometryType::Strands)
	{
		check(Instance->HairGroupPublicData);

		if (Instance->Strands.RestRootResource)			{ Instance->Strands.RestRootResource->Allocate		(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, false/*bAllowDeallocation*/, MeshLODIndex); }
		if (Instance->Strands.RestResource)				{ Instance->Strands.RestResource->Allocate			(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, false/*bAllowDeallocation*/); }
		if (Instance->Strands.ClusterResource)			{ Instance->Strands.ClusterResource->Allocate		(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, false/*bAllowDeallocation*/); }
		if (Instance->Strands.InterpolationResource)	{ Instance->Strands.InterpolationResource->Allocate	(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, false/*bAllowDeallocation*/); }
		if (Instance->Strands.CullingResource)			{ Instance->Strands.CullingResource->Allocate		(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount); }

		if (Instance->Strands.DeformedRootResource)		{ Instance->Strands.DeformedRootResource->Allocate	(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, false/*bAllowDeallocation*/, MeshLODIndex); }
		if (Instance->Strands.DeformedResource)			{ Instance->Strands.DeformedResource->Allocate		(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount); }
#if RHI_RAYTRACING
		if (bHasPathTracingView)						{ AllocateRaytracingResources(Instance, MeshLODIndex); }
		if (Instance->Strands.RenRaytracingResource)	{ Instance->Strands.RenRaytracingResource->Allocate	(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, Instance->Strands.RenRaytracingResourceOwned/*bAllowDeallocation*/); }
#endif
		
		if (Instance->Strands.LinearSweptSphereVertexFactory != nullptr)
		{
			Instance->Strands.LinearSweptSphereVertexFactory->InitResources(GraphBuilder.RHICmdList);
		}
		else
		{
			check(Instance->Strands.VertexFactory != nullptr);
			Instance->Strands.VertexFactory->InitResources(GraphBuilder.RHICmdList);
		}

		// Early initialization, so that when filtering MeshBatch block in SceneVisiblity, we can use this value to know 
		// if the hair instance is visible or not (i.e., HairLengthScale > 0)
		Instance->HairGroupPublicData->VFInput.Strands.Common.LengthScale = Instance->Strands.Modifier.HairLengthScale;
	}

	// Only switch LOD if the data are ready to be used
	const bool bIsLODDataReady = !ResourceStatus.HasStatus(EHairResourceStatus::EStatus::Loading) || (bStreamingEnabled && ResourceStatus.AvailableCurveCount > 0);
	if (bIsLODDataReady)
	{
		const uint32 IntPrevHairLODIndex = FMath::FloorToInt(PrevHairLODIndex); 
		const EHairBindingType CurrBindingType = Instance->HairGroupPublicData->GetBindingType(IntHairLODIndex);
		const EHairBindingType PrevBindingType = Instance->HairGroupPublicData->GetBindingType(IntPrevHairLODIndex);

		Instance->HairGroupPublicData->SetLODVisibility(bIsVisible);
		Instance->HairGroupPublicData->SetLODIndex(HairLODIndex);
		Instance->HairGroupPublicData->SetLODBias(0);
		Instance->HairGroupPublicData->SetMeshLODIndex(MeshLODIndex);
		Instance->HairGroupPublicData->VFInput.GeometryType = GeometryType;
		Instance->HairGroupPublicData->VFInput.BindingType = CurrBindingType;
		Instance->HairGroupPublicData->VFInput.bHasLODSwitch = IntPrevHairLODIndex != IntHairLODIndex;
		Instance->HairGroupPublicData->VFInput.bHasLODSwitchBindingType = CurrBindingType != PrevBindingType;
		Instance->GeometryType = GeometryType;
		Instance->BindingType = CurrBindingType;
		Instance->Guides.bIsSimulationEnable = Instance->HairGroupPublicData->IsSimulationEnable(IntHairLODIndex);
		Instance->Guides.bHasGlobalInterpolation = Instance->HairGroupPublicData->IsGlobalInterpolationEnable(IntHairLODIndex);
		Instance->Guides.bIsDeformationEnable = Instance->HairGroupPublicData->bIsDeformationEnable;
		Instance->Guides.bIsSimulationCacheEnable = Instance->HairGroupPublicData->bIsSimulationCacheEnable;
		Instance->Strands.bCullingEnable = bCullingEnable;
	}

	// Ensure that if the binding type is set to Skinning (or has RBF enabled), the skel. mesh is valid and support skin cache.
	if ((Instance->Guides.bHasGlobalInterpolation || Instance->BindingType == EHairBindingType::Skinning) && MeshLODIndex < 0)
	{
		return false;
	}

	// If requested LOD's resources are not ready yet, and the previous LOD was invalid or if the MeshLODIndex has changed 
	// (which would required extra data loading) change the geometry to be invalid, so that it don't get processed this frame.
	const bool bIsLODValid = PrevHairLODIndex >= 0;
	const bool bHasMeshLODChanged = PrevMeshLODIndex != MeshLODIndex && (Instance->Guides.bHasGlobalInterpolation || Instance->BindingType == EHairBindingType::Skinning);
	if (!bIsLODDataReady && (!bIsLODValid || bHasMeshLODChanged || !bDoesBindingSupportMeshLOD))
	{
		return false;
	}

	// Adapt the number curve/point based on available curves/points
	if (GeometryType == EHairGeometryType::Strands && bStreamingEnabled)
	{
		if (!bIsLODDataReady)
		{
			return false;
		}
		check(bIsLODDataReady);
		check(Instance->Strands.IsValid());

		// Adapt CurveCount/PointCount/CoverageScale based on what is actually available
		const uint32 EffectiveCurveCount = FMath::Min(ResourceStatus.AvailableCurveCount, Instance->HairGroupPublicData->ContinuousLODCurveCount);
		Instance->HairGroupPublicData->ContinuousLODCurveCount = EffectiveCurveCount;
		Instance->HairGroupPublicData->ContinuousLODPointCount = EffectiveCurveCount > 0 ? Instance->Strands.GetData().Header.CurveToPointCount[EffectiveCurveCount - 1] : 0;
		Instance->HairGroupPublicData->ContinuousLODCoverageScale = ComputeActiveCurveCoverageScale(Instance->Strands.GetData(), EffectiveCurveCount); 
		Instance->HairGroupPublicData->ContinuousLODRadiusScale = ComputeActiveCurveRadiusScale(Instance->Strands.ClusterResource, HairLODIndex); // default is 1.f - Only used for backward compatibility
		check(Instance->HairGroupPublicData->ContinuousLODPointCount <= Instance->HairGroupPublicData->RestPointCount);
	}
	return true;
}

static void RunHairLODSelection(
	FRDGBuilder& GraphBuilder, 
	FSceneInterface* Scene,
	const FHairStrandsInstances& Instances, 
	const TArray<const FSceneView*>& Views, 
	FGlobalShaderMap* ShaderMap)
{
	EShaderPlatform ShaderPlatform = EShaderPlatform::SP_NumPlatforms;
	if (Views.Num() > 0)
	{
		ShaderPlatform = Views[0]->GetShaderPlatform();
	}
	
	// Detect if strands are supported by active shader platform
	bool bStrandsSupported = IsHairStrandsEnabled(EHairStrandsShaderType::Strands, ShaderPlatform);

	// Detect if one of the current view has path tracing enabled for allocating raytracing resources
	bool bHasPathTracingView = false;
	#if RHI_RAYTRACING
	for (const FSceneView* View : Views)
	{
		if (View->Family->EngineShowFlags.PathTracing)
		{
			bHasPathTracingView = true;
			break;
		}
	}
	#endif

	struct FInstanceData
	{
		FHairGroupInstance* Instance = nullptr;
		FHairGroupPublicData* HairGroupPublicData = nullptr;
		FHairLOD HairLOD;
		FTransform MeshLODLocalToWorld = FTransform::Identity;

		// Perform LOD selection based on all the views	
		// CPU LOD selection. 
		// * When enable the CPU LOD selection allow to change the geometry representation. 
		// * GPU LOD selecion allow fine grain LODing, but does not support representation changes (strands, cards, meshes)
		// 
		// Use the forced LOD index if set, otherwise compute the LOD based on the maximal screensize accross all views
		// Compute the view where the screen size is maximale
		int32 MeshLODIndex = -1;
		int32 PrevMeshLODIndex = -1.f;
		float PrevHairLODIndex = -1.f;

		// 0. Initial LOD index picking	
		// If continuous LOD is enabled, we bypass all other type of geometric representation, and only use LOD0

	};

	struct FUniqueGroom
	{
		uint32 InstanceCount = 0;
		FHairLOD HairLOD;
		uint32 AssetHash = 0;
		uint32 GroupIndex = 0;
		int32 MeshLODIndex = -1;
		uint32 RestCurveCount = 0;

		// Resources
		FHairStrandsRestResource*			RestResource = nullptr;
		FHairStrandsInterpolationResource*	InterpolationResource = nullptr;
		FHairStrandsClusterResource* 		ClusterResource = nullptr;
	#if RHI_RAYTRACING
		FHairStrandsRaytracingResource*		RaytracingResource = nullptr;
	#endif
	};
	TArray<FUniqueGroom> UniqueGroomAssets;
	UniqueGroomAssets.Reserve(Instances.Num());

	struct FUniqueGroomBinding
	{
		uint32 InstanceCount = 0;
		FHairLOD HairLOD;
		uint32 AssetHash = 0;
		uint32 GroupIndex = 0;
		int32 MeshLODIndex = -1;
		uint32 RestCurveCount = 0;

		// Resources
		FHairStrandsRestRootResource* RestRootResource = nullptr;
	};
	TArray<FUniqueGroomBinding> UniqueGroomBindingAssets;
	UniqueGroomAssets.Reserve(Instances.Num());

	if (!Instances.IsEmpty())
	{
		// Sync the skeletal mesh update tasks involved with LOD selection so that GetLOD returns a valid result.
		FSkeletalMeshUpdater::WaitForStage(GraphBuilder, ESkeletalMeshUpdateStage::Filter);
	}

	// Find unique groom assets
	TArray<FInstanceData> InstanceDatas;
	InstanceDatas.Reserve(Instances.Num());
	for (FHairStrandsInstance* AbstractInstance : Instances)
	{
		FHairGroupInstance* Instance = static_cast<FHairGroupInstance*>(AbstractInstance);
		check(Instance);
		check(Instance->HairGroupPublicData);

		FInstanceData& InstanceData = InstanceDatas.AddDefaulted_GetRef();
		InstanceData.Instance 				= Instance;
		InstanceData.HairGroupPublicData 	= Instance->HairGroupPublicData;
		InstanceData.PrevHairLODIndex 		= Instance->HairGroupPublicData->GetLODIndex();
		InstanceData.PrevMeshLODIndex 		= Instance->HairGroupPublicData->GetMeshLODIndex();
		InstanceData.HairLOD 				= ComputeHairLODIndex(Instance, Views, bStrandsSupported);
		InstanceData.MeshLODIndex 			= GetMeshLODIndex(GraphBuilder, ShaderMap, Scene, Instance, InstanceData.MeshLODLocalToWorld);

		const EHairGeometryType GeometryType = GetLODGeometryType(Instance, InstanceData.HairLOD.HairLODIndex, ShaderPlatform);
		const EHairBindingType BindingType = Instance->HairGroupPublicData->GetBindingType(InstanceData.HairLOD.HairLODIndex);

		// Extract unique Groom/GroomBinding assets
		// Only for strands geometry as, we only support fine-grain stream on strands resources
		if (GeometryType == EHairGeometryType::Strands)
		{
			// Groom
			{
				FUniqueGroom* Unique = UniqueGroomAssets.FindByPredicate([Instance](const FUniqueGroom& A) { return A.AssetHash == Instance->Debug.GroomAssetHash && A.GroupIndex == Instance->Debug.GroupIndex; });
				if (Unique)
				{
					if (InstanceData.HairLOD.ContinuousLODCurveCount > Unique->HairLOD.ContinuousLODCurveCount)
					{
						Unique->HairLOD = InstanceData.HairLOD;
					}
				}
				else
				{
					Unique 					= &UniqueGroomAssets.AddDefaulted_GetRef();
					Unique->AssetHash 		= Instance->Debug.GroomAssetHash;
					Unique->GroupIndex 		= Instance->Debug.GroupIndex;
					Unique->HairLOD 		= InstanceData.HairLOD;
					Unique->RestCurveCount 	= Instance->HairGroupPublicData->RestCurveCount;
				}
				++Unique->InstanceCount;

				if (!Unique->RestResource) 			{ Unique->RestResource 			= Instance->Strands.RestResource; }
				if (!Unique->InterpolationResource) { Unique->InterpolationResource = Instance->Strands.InterpolationResource; }
				if (!Unique->ClusterResource) 		{ Unique->ClusterResource 		= Instance->Strands.ClusterResource; }
			#if RHI_RAYTRACING
				if (!Unique->RaytracingResource && !Instance->Strands.RenRaytracingResourceOwned)
				{
					Unique->RaytracingResource 	= Instance->Strands.RenRaytracingResource;
				}
			#endif
			}
	
			// Groom binding
			if (BindingType == EHairBindingType::Skinning)
			{
				FUniqueGroomBinding* Unique = UniqueGroomBindingAssets.FindByPredicate([&InstanceData](const FUniqueGroomBinding& A) 
				{ 
					return 
						A.AssetHash == InstanceData.Instance->Debug.GroomBindingAssetHash && 
						A.GroupIndex == InstanceData.Instance->Debug.GroupIndex && 
						A.MeshLODIndex == InstanceData.MeshLODIndex; 
				});
				if (Unique)
				{
					// Sanity check
					check(Unique->MeshLODIndex == InstanceData.MeshLODIndex);
					check(Unique->GroupIndex == InstanceData.Instance->Debug.GroupIndex);

					if (InstanceData.HairLOD.ContinuousLODCurveCount > Unique->HairLOD.ContinuousLODCurveCount)
					{
						Unique->HairLOD = InstanceData.HairLOD;
					}
				}
				else
				{
					Unique 					= &UniqueGroomBindingAssets.AddDefaulted_GetRef();
					Unique->AssetHash 		= Instance->Debug.GroomBindingAssetHash;
					Unique->GroupIndex 		= Instance->Debug.GroupIndex;
					Unique->HairLOD 		= InstanceData.HairLOD;
					Unique->RestCurveCount 	= Instance->HairGroupPublicData->RestCurveCount;
					Unique->MeshLODIndex 	= InstanceData.MeshLODIndex;
				}
				++Unique->InstanceCount;

				if (!Unique->RestRootResource) { Unique->RestRootResource = Instance->Strands.RestRootResource; }
			}
		}
	}

	// Emit streaming request for each unique Groom/GroomBinding group
	bool bAllowToDeallocate = true;
	for (FUniqueGroom& Unique : UniqueGroomAssets)
	{
		const uint32 RequestedCurveCount = Unique.HairLOD.ContinuousLODCurveCount;
		const uint32 RequestedPointCount = Unique.HairLOD.ContinuousLODPointCount;

		EHairResourceStatus ResourceStatus;
		ResourceStatus.Status 				= EHairResourceStatus::EStatus::None;
		ResourceStatus.AvailableCurveCount 	= Unique.RestCurveCount;

		if (Unique.RestResource)			{ Unique.RestResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount); }
		if (Unique.ClusterResource)			{ Unique.ClusterResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount); }
		if (Unique.InterpolationResource)	{ Unique.InterpolationResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount); }
		#if RHI_RAYTRACING
		if (Unique.RaytracingResource)		{ Unique.RaytracingResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount); }
		#endif
	}
	for (FUniqueGroomBinding& Unique : UniqueGroomBindingAssets)
	{
		const uint32 RequestedCurveCount = Unique.HairLOD.ContinuousLODCurveCount;
		const uint32 RequestedPointCount = Unique.HairLOD.ContinuousLODPointCount;

		EHairResourceStatus ResourceStatus;
		ResourceStatus.Status 				= EHairResourceStatus::EStatus::None;
		ResourceStatus.AvailableCurveCount 	= Unique.RestCurveCount;

		if (Unique.RestRootResource)		{ Unique.RestRootResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async, ResourceStatus, RequestedCurveCount, RequestedPointCount, bAllowToDeallocate, Unique.MeshLODIndex); }
	}

	// LOD selection & load resources
	for (FInstanceData& InstanceData : InstanceDatas)
	{
		// 0. Initial LOD index picking	
		// If continuous LOD is enabled, we bypass all other type of geometric representation, and only use LOD0
		//const FHairLOD HairLOD = ComputeHairLODIndex(InstanceData.Instance, Views);
		ApplyHairLOD(InstanceData.HairLOD, InstanceData.Instance);

		// 1. Try to find an available LOD
		bool bFoundValidLOD = SelectValidLOD(
			GraphBuilder,
			ShaderMap,
			ShaderPlatform,
			InstanceData.Instance,
			InstanceData.HairLOD.HairLODIndex,
			InstanceData.PrevHairLODIndex,
			InstanceData.HairLOD.HairLODCount,
			InstanceData.MeshLODIndex,
			InstanceData.PrevMeshLODIndex,
			bHasPathTracingView);

		// 2. If no valid LOD are found, try to find a coarser LOD which is:
		// * loaded 
		// * does not require skinning binding 
		// * does not require global interpolation (as global interpolation requires skel. mesh data)
		// * does not require simulation (as simulation requires LOD to be selected on the game thread i.e., Predicted/Forced, and so we can override this LOD here)
		if (!bFoundValidLOD)
		{
			for (uint32 FallbackHairLODIndex = FMath::Clamp(FMath::FloorToInt(InstanceData.HairLOD.HairLODIndex + 1), 0, InstanceData.HairLOD.HairLODCount - 1); FallbackHairLODIndex < InstanceData.HairLOD.HairLODCount; ++FallbackHairLODIndex)
			{
				if ( InstanceData.HairGroupPublicData->GetBindingType(FallbackHairLODIndex) == EHairBindingType::Rigid &&
					!InstanceData.HairGroupPublicData->IsSimulationEnable(FallbackHairLODIndex) &&
					!InstanceData.HairGroupPublicData->IsGlobalInterpolationEnable(FallbackHairLODIndex))
				{
					bFoundValidLOD = SelectValidLOD(
						GraphBuilder,
						ShaderMap,
						ShaderPlatform,
						InstanceData.Instance,
						FallbackHairLODIndex,
						InstanceData.PrevHairLODIndex,
						InstanceData.HairLOD.HairLODCount,
						InstanceData.MeshLODIndex,
						InstanceData.PrevMeshLODIndex,
						bHasPathTracingView);
					break;
				}
			}
		}

		// 3. If no LOD are valid, then mark the instance as invalid for this frame
		if (!bFoundValidLOD)
		{
			InstanceData.Instance->GeometryType = EHairGeometryType::NoneGeometry;
			InstanceData.Instance->BindingType = EHairBindingType::NoneBinding;
		}

		// Cull instance if it is not visible in any View
		{
			const FPrimitiveSceneProxy* Proxy = InstanceData.Instance->Debug.Proxy;
			check(Proxy != nullptr);
			bool bVisibleInView = false;
			for (const FSceneView* View : Views)
			{
				if (Proxy->IsShown(View) || Proxy->IsShadowCast(View))
				{
					bVisibleInView = true;
					break;
				}
			}
			if (!bVisibleInView && !Proxy->AffectsIndirectLightingWhileHidden())
			{
				InstanceData.Instance->GeometryType = EHairGeometryType::NoneGeometry;
				InstanceData.Instance->BindingType = EHairBindingType::NoneBinding;
			}
		}
		// Update the local-to-world transform based on the binding type 
		if (GetHairSwapBufferType() == EHairBufferSwapType::BeginOfFrame || GetHairSwapBufferType() == EHairBufferSwapType::EndOfFrame)
		{
			InstanceData.Instance->Debug.SkinningPreviousLocalToWorld = InstanceData.Instance->Debug.SkinningCurrentLocalToWorld;
			InstanceData.Instance->Debug.SkinningCurrentLocalToWorld  = InstanceData.MeshLODLocalToWorld;
		}
		InstanceData.Instance->LocalToWorld = InstanceData.Instance->GetCurrentLocalToWorld();
	}
}


///////////////////////////////////////////////////////////////////////////////////////////////////
bool HasHairStrandsFolliculeMaskQueries();
void RunHairStrandsFolliculeMaskQueries(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap);

#if WITH_EDITOR
bool HasHairStrandsTexturesQueries();
void RunHairStrandsTexturesQueries(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const struct FShaderPrintData* ShaderPrintData);

bool HasHairStrandsPositionQueries();
void RunHairStrandsPositionQueries(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const struct FShaderPrintData* DebugShaderData, const FSceneView* SceneView);
#endif

static void RunHairStrandsProcess(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const struct FShaderPrintData* ShaderPrintData, const FSceneView* SceneView)
{
#if WITH_EDITOR
	if (HasHairStrandsTexturesQueries())
	{
		RunHairStrandsTexturesQueries(GraphBuilder, ShaderMap, ShaderPrintData);
	}

	if (HasHairStrandsPositionQueries())
	{
		RunHairStrandsPositionQueries(GraphBuilder, ShaderMap, ShaderPrintData, SceneView);
	}
#endif

	if (HasHairStrandsFolliculeMaskQueries())
	{
		RunHairStrandsFolliculeMaskQueries(GraphBuilder, ShaderMap);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void RunHairStrandsDebug(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FSceneInterface* Scene,
	const FSceneView& View,
	const FHairStrandsInstances& Instances,
	const FHairInstanceCullingResults& CullingResults,
	FHairTransientResources& TransientResources,
	const FShaderPrintData* ShaderPrintData,
	FRDGTextureRef SceneColor,
	FRDGTextureRef SceneDepth,
	FIntRect Viewport,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer);

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// HairStrands Bookmark API

void ProcessHairStrandsBookmark(
	FRDGBuilder* GraphBuilder,
	EHairStrandsBookmark Bookmark,
	FHairStrandsBookmarkParameters& Parameters)
{
	check(Parameters.Instances != nullptr);

	FHairStrandsInstances* Instances = Parameters.Instances;

	// Cards interpolation for ShadowView only needs to run when FrustumCulling is enabled
	if (Bookmark == EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_ShadowView && !IsInstanceFrustumCullingEnable())
	{
		return;
	}

	if (!Parameters.View || (Parameters.View->Family && !Parameters.View->Family->EngineShowFlags.Hair))
	{
		return;
	}

	if (IsInstanceFrustumCullingEnable())
	{
		Instances = nullptr;
		if (Bookmark == EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_PrimaryView)
		{
			Instances = &Parameters.CullingResults.VisibleInAnyViews_CardsOrMeshes_Primary;
		}
		else if (Bookmark == EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_ShadowView)
		{
			Instances = &Parameters.CullingResults.VisibleInAnyViews_CardsOrMeshes_Shadow;
		}
		else if (Bookmark == EHairStrandsBookmark::ProcessStrandsInterpolation)
		{
			Instances = &Parameters.CullingResults.VisibleInAnyViews_Strands;
		}
		else
		{
			Instances = Parameters.Instances;
		}
	}

	if (Bookmark == EHairStrandsBookmark::ProcessTasks)
	{
		const bool bHasHairStardsnProcess =
		#if WITH_EDITOR
			HasHairStrandsTexturesQueries() ||
			HasHairStrandsPositionQueries() ||
		#endif
			HasHairStrandsFolliculeMaskQueries();

		if (bHasHairStardsnProcess)
		{
			check(GraphBuilder);
			RunHairStrandsProcess(*GraphBuilder, Parameters.ShaderMap, Parameters.ShaderPrintData, Parameters.View);
		}
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessLODSelection)
	{
		if (GetHairSwapBufferType() == EHairBufferSwapType::BeginOfFrame)
		{
			RunHairBufferSwap(
				*Parameters.Instances,
				Parameters.AllViews);
		}
		check(GraphBuilder);

		RunHairLODSelection(
			*GraphBuilder,
			Parameters.Scene,
			*Parameters.Instances,
			Parameters.AllViews,
			Parameters.ShaderMap);

	}
	else if (Bookmark == EHairStrandsBookmark::ProcessEndOfFrame)
	{
		if (GetHairSwapBufferType() == EHairBufferSwapType::EndOfFrame)
		{
			RunHairBufferSwap(
				*Parameters.Instances,
				Parameters.AllViews);
		}
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessBindingSurfaceUpdate)
	{
		check(GraphBuilder);

		RunHairBindingSurfaceUpdate(
			*GraphBuilder,
			Parameters.Scene,
			Parameters.View,
			Parameters.ViewUniqueID,
			*Instances,
			Parameters.ShaderPrintData,
			*Parameters.TransientResources,
			Parameters.ShaderMap);
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessGuideInterpolation)
	{
		RunHairStrandsInterpolation_Guide(
			*GraphBuilder,
			Parameters.Scene,
			Parameters.View,
			Parameters.ViewUniqueID,
			*Instances,
			Parameters.ShaderPrintData,
			*Parameters.TransientResources,
			Parameters.ShaderMap);
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessGuideDeformation)
	{
		RunHairStrandsDeformation_Guide(
			*GraphBuilder,
			Parameters.Scene,
			Parameters.View,
			Parameters.ViewUniqueID,
			*Instances,
			Parameters.ShaderPrintData,
			*Parameters.TransientResources,
			Parameters.ShaderMap);
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_PrimaryView || Bookmark == EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_ShadowView)
	{
		check(GraphBuilder);
		RunHairStrandsInterpolation_Cards(
			*GraphBuilder,
			Parameters.Scene,
			Parameters.AllViews,
			Parameters.View,
			Parameters.ViewUniqueID,
			*Instances,
			Parameters.ShaderPrintData,
			*Parameters.TransientResources,
			Parameters.ShaderMap);
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessStrandsInterpolation)
	{
		check(GraphBuilder);
		check(Parameters.TransientResources);
		RunHairStrandsInterpolation_Strands(
			*GraphBuilder,
			Parameters.Scene,
			Parameters.AllViews,
			Parameters.View,
			Parameters.ViewUniqueID,
			Parameters.Instances->Num(),
			*Instances,
			Parameters.ShaderPrintData,
			*Parameters.TransientResources,
			Parameters.ShaderMap);
	}
	else if (Bookmark == EHairStrandsBookmark::ProcessDebug)
	{
		// Merge all visible instances
		FHairStrandsInstances DebugInstances;
		if (IsInstanceFrustumCullingEnable())
		{
			DebugInstances.Append(Parameters.CullingResults.VisibleInAnyViews_Strands);
			DebugInstances.Append(Parameters.CullingResults.VisibleInAnyViews_CardsOrMeshes_Primary);
			DebugInstances.Append(Parameters.CullingResults.VisibleInAnyViews_CardsOrMeshes_Shadow);
			DebugInstances.Sort([](const TRefCountPtr<FHairStrandsInstance>& A, const TRefCountPtr<FHairStrandsInstance>& B)
			{
				return A->RegisteredIndex < B->RegisteredIndex;
			});
			Instances = &DebugInstances;
		}

		check(GraphBuilder);
		check(Parameters.TransientResources);
		RunHairStrandsDebug(
			*GraphBuilder,
			Parameters.ShaderMap,
			Parameters.Scene,
			*Parameters.View,
			*Instances,
			Parameters.CullingResults,
			*Parameters.TransientResources,
			Parameters.ShaderPrintData,
			Parameters.SceneColorTexture,
			Parameters.SceneDepthTexture,
			Parameters.View->UnscaledViewRect,
			Parameters.View->ViewUniformBuffer);
	}
}
