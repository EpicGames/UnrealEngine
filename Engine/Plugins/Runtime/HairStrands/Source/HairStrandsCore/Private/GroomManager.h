// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "Shader.h"
#include "RenderResource.h"
#include "RenderGraphResources.h"
#include "HairStrandsInterface.h"
#include "HairStrandsMeshProjection.h"


////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc/Helpers

struct FHairStrandClusterData;
class FRDGPooledBuffer;
struct IPooledRenderTarget;
struct FRWBuffer;
struct FHairGroupInstance;
class  FHairGroupPublicData;
class  FRDGShaderResourceView;
class  FResourceArrayInterface;
class  FSceneView;
class  UTexture2D;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Mesh transfer and hair projection debug infos

enum class EHairStrandsProjectionMeshType
{
	RestMesh,
	DeformedMesh,
	SourceMesh,
	TargetMesh
};

enum EHairBufferSwapType
{
	BeginOfFrame,
	EndOfFrame,
	Tick,
	RenderFrame
};

EHairBufferSwapType GetHairSwapBufferType();

////////////////////////////////////////////////////////////////////////////////////////////////////
// Registrations & update

void RegisterHairStrands(struct FHairGroupInstance* Instance);
void UnregisterHairStrands(uint32 ComponentId);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

