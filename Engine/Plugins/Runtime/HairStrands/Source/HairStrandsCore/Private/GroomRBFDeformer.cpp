// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomRBFDeformer.h"
#include "Async/ParallelFor.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomBuilder.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "MeshAttributes.h"
#include "Misc/CoreMisc.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureDefines.h"
#include "Engine/Texture.h"
#include "GroomBindingBuilder.h"
#include "GroomBindingCommon.h"
#include "StaticMeshCompiler.h"

#define ALLOW_CURVE_SHUFFLING 0

namespace FHairStrandsDecimation
{
	void GetCurveShuffleIndices(TArray<uint32>& CurveRandomizedIndex, uint32 InCurveCount);
}

// HairStrandsMesh.usf
void InitMeshSamples(
	uint32 MaxVertexCount,
	const TArray<FVector3f>& VertexPositionsBuffer,
	uint32 MaxSampleCount,
	const TArray<uint32>& SampleIndicesBuffer,
	TArray<FVector3f>& OutSamplePositionsBuffer
)
{
	OutSamplePositionsBuffer.SetNum(MaxSampleCount);
	for (uint32 SampleIndex = 0; SampleIndex < MaxSampleCount; ++SampleIndex)
	{
		const uint32 VertexIndex = SampleIndicesBuffer[SampleIndex];
		check(VertexIndex < MaxVertexCount);
		if (VertexIndex < MaxVertexCount)
		{
			OutSamplePositionsBuffer[SampleIndex] = VertexPositionsBuffer[VertexIndex];
		}
	}
}

// HairStrandsMesh.usf
void UpdateMeshSamples(
	uint32 MaxSampleCount,
	const TArray<float>& InterpolationWeightsBuffer,
	const TArray<FVector4f>& RestSamplePositionsBuffer,
	const TArray<FVector3f>& DeformedSamplePositionsBuffer,
	TArray<FVector3f>& OutSampleDeformationsBuffer
)
{
	const uint32 EntryCount = FGroomRBFDeformer::GetEntryCount(MaxSampleCount);
	OutSampleDeformationsBuffer.SetNum(EntryCount);

	const bool bIsRBFLocalSpaceEnabled = !RestSamplePositionsBuffer.IsEmpty() && !DeformedSamplePositionsBuffer.IsEmpty() && UE::Groom::IsRBFLocalSpaceEnabled();
	const FVector4f RestSampleOffset = bIsRBFLocalSpaceEnabled ? RestSamplePositionsBuffer[0] : FVector4f(0, 0, 0, 0);
	const FVector3f DeformedSampleOffset = bIsRBFLocalSpaceEnabled ? DeformedSamplePositionsBuffer[0] : FVector3f(0, 0, 0);

	for (uint32 SampleIndex = 0; SampleIndex < EntryCount; ++SampleIndex)
	{
		uint32 WeightsOffset = SampleIndex * EntryCount;
		FVector3f SampleDeformation(FVector3f::ZeroVector);
		for (uint32 i = 0; i < MaxSampleCount; ++i, ++WeightsOffset)
		{
			SampleDeformation += InterpolationWeightsBuffer[WeightsOffset] * ((DeformedSamplePositionsBuffer[i]- DeformedSampleOffset) - (RestSamplePositionsBuffer[i]-RestSampleOffset));
		}

		OutSampleDeformationsBuffer[SampleIndex] = SampleDeformation;
	}
}

// HairStrandsGuideDeform.usf
FVector3f DisplacePosition(
	const FVector3f& RestControlPoint,
	uint32 SampleCount,
	const TArray<FVector4f>& RestSamplePositionsBuffer,
	const TArray<FVector3f>& DeformedSamplePositionsBuffer,
	const TArray<FVector3f>& MeshSampleWeightsBuffer
)
{
	const bool bIsRBFLocalSpaceEnabled = !RestSamplePositionsBuffer.IsEmpty() && !DeformedSamplePositionsBuffer.IsEmpty() && UE::Groom::IsRBFLocalSpaceEnabled();
	const FVector3f RestSampleOffset = bIsRBFLocalSpaceEnabled ? FVector3f(RestSamplePositionsBuffer[0]) : FVector3f(0, 0, 0);
	const FVector3f DeformedSampleOffset = bIsRBFLocalSpaceEnabled ? DeformedSamplePositionsBuffer[0] : FVector3f(0, 0, 0);

	FVector3f ControlPoint = RestControlPoint-RestSampleOffset;

	// Apply rbf interpolation from the samples set
	for (uint32 i = 0; i < SampleCount; ++i)
	{
		const FVector3f PositionDelta = RestControlPoint - RestSamplePositionsBuffer[i];
		const float FunctionValue = FMath::Sqrt(FVector3f::DotProduct(PositionDelta, PositionDelta) + 1);
		ControlPoint += FunctionValue * MeshSampleWeightsBuffer[i];
	}
	ControlPoint += MeshSampleWeightsBuffer[SampleCount];
	ControlPoint += MeshSampleWeightsBuffer[SampleCount + 1] * (RestControlPoint.X - RestSampleOffset.X);
	ControlPoint += MeshSampleWeightsBuffer[SampleCount + 2] * (RestControlPoint.Y - RestSampleOffset.Y);
	ControlPoint += MeshSampleWeightsBuffer[SampleCount + 3] * (RestControlPoint.Z - RestSampleOffset.Z);
	return ControlPoint + DeformedSampleOffset;
}

void DeformStrands(	
	const FHairStrandsDatas& HairStandsData,
	const TArray<FHairStrandsIndexFormat::Type>& PointToCurveBuffer,
	const TArray<FHairStrandsIndexFormat::Type>& RootToUniqueTriangleBuffer,
	const TArray<FHairStrandsRootBarycentricFormat::Type>& RootBarycentricBuffer,
	const TArray<FHairStrandsMeshTrianglePositionFormat::Type>& UniqueTrianglePositionBuffer_Rest,
	const TArray<FHairStrandsMeshTrianglePositionFormat::Type>& UniqueTrianglePositionBuffer_Deformed,
	uint32 VertexCount,
	uint32 SampleCount,
	const bool bHasRBF,
	const TArray<FVector3f>& RestPosePositionBuffer,
	const TArray<FVector4f>& RestSamplePositionsBuffer,
	const TArray<FVector3f>& DeformedSamplePositionsBuffer,
	const TArray<FVector3f>& MeshSampleWeightsBuffer,
	TArray<FVector3f>& OutDeformedPositionBuffer
)
{
	// Raw deformation with RBF
	OutDeformedPositionBuffer.SetNum(RestPosePositionBuffer.Num());
	ParallelFor(VertexCount, [&](uint32 VertexIndex)
	{
		const FVector3f& ControlPoint = RestPosePositionBuffer[VertexIndex];
		const FVector3f DisplacedPosition = DisplacePosition(ControlPoint, SampleCount, RestSamplePositionsBuffer, DeformedSamplePositionsBuffer, MeshSampleWeightsBuffer);
		OutDeformedPositionBuffer[VertexIndex] = bHasRBF ? DisplacedPosition : ControlPoint;
	});

	// Build curve shuffling indices
	TArray<uint32> CurveShuffleIndex;
	FHairStrandsDecimation::GetCurveShuffleIndices(CurveShuffleIndex, HairStandsData.GetNumCurves());

	// Compute correction for snapping the strands back to the surface, as the RBF introduces low frequency offset
	const uint32 CurveCount = HairStandsData.GetNumCurves();
	TArray<FVector4f> CorrectionOffsets;
	CorrectionOffsets.SetNum(CurveCount);
	ParallelFor(CurveCount, [&](uint32 CurveIndex)
	{
		const uint32 VertexOffset = HairStandsData.StrandsCurves.CurvesOffset[CurveIndex];

		// Sanity check
		const uint32 RootIndex = PointToCurveBuffer[VertexOffset];
		check(RootIndex == CurveIndex);

		// HairStandsData are *not* shuffled (for export purpose), while the root data have been built with shuffled data.
		// Due to this we have to revert the shuffling prior to looking up binding data.
	#if ALLOW_CURVE_SHUFFLING
		const uint32 ShuffleRootIndex = RootIndex;
	#else
		uint32 ShuffleRootIndex = RootIndex;
		for (uint32 It=0, Count=CurveShuffleIndex.Num();It<Count; ++It)
		{
			if (CurveShuffleIndex[It] == RootIndex)
			{
				ShuffleRootIndex = It;
				break;
			}
		}
	#endif
		const uint32 TriangleIndex = RootToUniqueTriangleBuffer[ShuffleRootIndex];


		const FVector3f& Rest_Position   = HairStandsData.StrandsPoints.PointsPosition[VertexOffset];
		const FVector3f& Deform_Position = OutDeformedPositionBuffer[VertexOffset];

		const uint32 PackedBarycentric = RootBarycentricBuffer[ShuffleRootIndex];
		const FVector2f B0 = FVector2f(FHairStrandsRootUtils::UnpackBarycentrics(PackedBarycentric));
		const FVector3f B  = FVector3f(B0.X, B0.Y, 1.f - B0.X - B0.Y);

		/* Strand hair roots translation and rotation in rest position relative to the bound triangle. Positions are relative to the rest root center */
		const FVector3f& Rest_V0 = UniqueTrianglePositionBuffer_Rest[TriangleIndex * 3 + 0];
		const FVector3f& Rest_V1 = UniqueTrianglePositionBuffer_Rest[TriangleIndex * 3 + 1];
		const FVector3f& Rest_V2 = UniqueTrianglePositionBuffer_Rest[TriangleIndex * 3 + 2];

		const FVector3f& Deform_V0 = UniqueTrianglePositionBuffer_Deformed[TriangleIndex * 3 + 0];
		const FVector3f& Deform_V1 = UniqueTrianglePositionBuffer_Deformed[TriangleIndex * 3 + 1];
		const FVector3f& Deform_V2 = UniqueTrianglePositionBuffer_Deformed[TriangleIndex * 3 + 2];

		const FVector3f Rest_RootPosition	=   Rest_V0 * B.X +   Rest_V1 * B.Y +   Rest_V2 * B.Z;
		const FVector3f Deform_RootPosition	= Deform_V0 * B.X + Deform_V1 * B.Y + Deform_V2 * B.Z;

		const FVector3f RestOffset = Rest_Position - Rest_RootPosition;
		const FVector3f SnappedDeformPosition = Deform_RootPosition + RestOffset;
		const FVector3f CorrectionOffset = SnappedDeformPosition - Deform_Position;

		CorrectionOffsets[CurveIndex] = FVector4f(CorrectionOffset, 0);
	});

	// Apply correction offset to each control points
	ParallelFor(VertexCount, [&](uint32 VertexIndex)
	{
		const uint32 CurveIndex = PointToCurveBuffer[VertexIndex];
		const FVector4f CorrectionOffset = CorrectionOffsets[CurveIndex];
		OutDeformedPositionBuffer[VertexIndex] += CorrectionOffset;
	});
}

TArray<FVector3f> GetDeformedHairStrandsPositions(
	const TArray<FVector3f>& MeshVertexPositionsBuffer_Target,
	const FHairStrandsDatas& HairStrandsData,
	const uint32 MeshLODIndex,
	const bool bHasRBF,
	const TArray<FHairStrandsIndexFormat::Type>& PointToCurveBuffer,
	const FHairStrandsRootData& RestLODData)
{
	// Init the mesh samples with the target mesh vertices
	const int32 MaxVertexCount = MeshVertexPositionsBuffer_Target.Num();
	const uint32 MaxSampleCount = RestLODData.SampleCount;
	const TArray<uint32>& SampleIndicesBuffer = RestLODData.MeshSampleIndicesBuffer;
	TArray<FVector3f> OutSamplePositionsBuffer;

	InitMeshSamples(MaxVertexCount, MeshVertexPositionsBuffer_Target, MaxSampleCount, SampleIndicesBuffer, OutSamplePositionsBuffer);

	const TArray<FHairStrandsIndexFormat::Type>& RootToUniqueTriangleBuffer = RestLODData.RootToUniqueTriangleIndexBuffer;

	// Update those vertices with the RBF interpolation weights
	const TArray<float>& InterpolationWeightsBuffer = RestLODData.MeshInterpolationWeightsBuffer;
	const TArray<FVector4f>& SampleRestPositionsBuffer = RestLODData.RestSamplePositionsBuffer;
	const TArray<FVector3f>& SampleDeformedPositionsBuffer = OutSamplePositionsBuffer;
	TArray<FVector3f> OutSampleDeformationsBuffer;

	UpdateMeshSamples(MaxSampleCount, InterpolationWeightsBuffer, SampleRestPositionsBuffer, SampleDeformedPositionsBuffer, OutSampleDeformationsBuffer);

	// Get the strands vertices positions centered at their bounding box
	const FHairStrandsPoints& Points = HairStrandsData.StrandsPoints;
	const uint32 VertexCount = Points.Num();

	TArray<FVector3f> OutPositions = Points.PointsPosition;

	// Use the vertex position of the binding, as the source asset might not have the same topology (in case the groom has been transfered from one mesh toanother using UV sharing)
	TArray<FHairStrandsMeshTrianglePositionFormat::Type> UniqueTrianglePositionBuffer_Rest = RestLODData.RestUniqueTrianglePositionBuffer;
	TArray<FHairStrandsMeshTrianglePositionFormat::Type> UniqueTrianglePositionBuffer_Deformed = RestLODData.RestUniqueTrianglePositionBuffer_TargetNonTransfered;
	check(UniqueTrianglePositionBuffer_Rest.Num() == UniqueTrianglePositionBuffer_Deformed.Num());

	// Deform the strands vertices with the deformed mesh samples
	const TArray<FVector3f>& RestPosePositionBuffer = OutPositions;
	const TArray<FVector3f>& MeshSampleWeightsBuffer = OutSampleDeformationsBuffer;
	TArray<FVector3f> OutDeformedPositionBuffer;

	DeformStrands(
		HairStrandsData,
		PointToCurveBuffer,
		RootToUniqueTriangleBuffer,
		RestLODData.RootBarycentricBuffer,
		UniqueTrianglePositionBuffer_Rest,
		UniqueTrianglePositionBuffer_Deformed,
		VertexCount, 
		MaxSampleCount, 
		bHasRBF,
		RestPosePositionBuffer, 
		SampleRestPositionsBuffer,
		SampleDeformedPositionsBuffer,
		MeshSampleWeightsBuffer, 
		OutDeformedPositionBuffer);

	return MoveTemp(OutDeformedPositionBuffer);
}

struct FRBFDeformedPositions 
{
	TArray<FVector3f> RenderStrands;
	TArray<FVector3f> GuideStrands;

	// Masked data
	TArray<bool> bIsRenderCurveValid;
	TArray<bool> bIsRenderVertexValid;
	bool bHasMaskedData = false;
};

#if WITH_EDITORONLY_DATA
static void ApplyDeformationToGroom(const TArray<FRBFDeformedPositions>& DeformedPositions, UGroomAsset* GroomAsset)
{
	// Notes:
	// * This function applies the deformed position onto the hair description.
	// * In addition, the groom can contain vertex/curves which needs to be masked (e.g., which are masked or cut.
	//   For handling this, the following function will first applied the deformed vertices and then remap all
	//   curves/vertices so that hair description contains only valid curves/vertices
	//
	// /!\ Only rendering strands can be masked. Guide always remains unchanged.

	// The deformation must be stored in the HairDescription to rebuild the hair data when the groom is loaded
	FHairDescription HairDescription = GroomAsset->GetHairDescription();

	// Check if any group has masked or trimmed data
	bool bHasMaskedData = false;
	bool bHasTrimmedData = false;
	{
		FHairDescriptionGroups HairDescriptionGroups;
		FGroomBuilder::BuildHairDescriptionGroups(HairDescription, HairDescriptionGroups);

		const uint32 GroupCount = DeformedPositions.Num();
		check(GroupCount == HairDescriptionGroups.HairGroups.Num());
		for (uint32 GroupIt = 0; GroupIt < GroupCount; ++GroupIt)
		{
			const FHairDescriptionGroup& DescGroup = HairDescriptionGroups.HairGroups[GroupIt];
			const FRBFDeformedPositions& RBFGroup  = DeformedPositions[GroupIt];
			if (RBFGroup.bHasMaskedData)
			{
				bHasMaskedData = true;
				break;
			}

			const bool bTrimPoint = (DescGroup.Info.Flags & uint32(EHairGroupInfoFlags::HasTrimmedPoint)) != 0;
			const bool bTrimCurve = (DescGroup.Info.Flags & uint32(EHairGroupInfoFlags::HasTrimmedCurve)) != 0;
			if (bTrimPoint || bTrimCurve)
			{
				bHasTrimmedData = true;
			}
		}
	}

	// If there are some masked/trimmed data, compute the curves & vertices remapping
	int32 CurrentStrandID = 0;
	int32 CurrentVertexID = 0;

	uint32 TotalNumStrands = 0;
	uint32 TotalNumVertices = 0;

	uint32 TotalValidNumStrands = 0;
	uint32 TotalValidNumVertices = 0;



	TotalNumVertices = HairDescription.GetNumVertices();
	TotalNumStrands = HairDescription.GetNumStrands();
	TotalValidNumVertices = TotalNumVertices;
	TotalValidNumStrands = TotalNumStrands;

	// Strands attributes as inputs
	TStrandAttributesConstRef<int> StrandNumVertices = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::VertexCount);
	TStrandAttributesConstRef<int> StrandGuides = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::Guide);
	TStrandAttributesConstRef<int> GroupIDs = HairDescription.StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::GroupID);

	// Guide and GroupID attributes are optional so must ensure they are available before using them
	bool bHasGuides = StrandGuides.IsValid();
	bool bHasGroupIDs = GroupIDs.IsValid();

	// We have to keep the same ordering of guides/strands and groups of vertices so that the deformed positions
	// are output in the right order in the HairDescription.
	// Thus, the deformed positions will be flattened into a single array
	struct FGroupInfo
	{
		// For debugging
		int32 NumRenderVertices = 0;
		int32 NumGuideVertices = 0;

		// For flattening the vertices into the array
		int32 CurrentRenderVertexIndex = 0;
		int32 CurrentGuideVertexIndex = 0;
	};

	TArray<FGroupInfo> GroupInfos;
	GroupInfos.SetNum(DeformedPositions.Num());

	const int32 GroomNumVertices = HairDescription.GetNumVertices();
	TArray<FVector3f> FlattenedDeformedPositions;
	FlattenedDeformedPositions.Reserve(GroomNumVertices);
	TArray<bool> FlattenedIsValid;
	FlattenedIsValid.Reserve(GroomNumVertices);

	// Mapping of GroupID to GroupIndex to preserver ordering
	TMap<int32, int32> GroupIDToGroupIndex;

	const int32 GroomNumStrands = HairDescription.GetNumStrands();
	check(GroomNumStrands < HAIR_MAX_NUM_CURVE_PER_GROUP);
	for (int32 StrandIndex = 0; StrandIndex < GroomNumStrands; ++StrandIndex)
	{
		FStrandID StrandID(StrandIndex);

		// Determine the group index to get the deformed positions from based on the strand group ID
		const int32 GroupID = bHasGroupIDs ? GroupIDs[StrandID] : 0;
		int32 GroupIndex = 0;
		int32* GroupIndexPtr = GroupIDToGroupIndex.Find(GroupID);
		if (GroupIndexPtr)
		{
			GroupIndex = *GroupIndexPtr;
		}
		else
		{
			GroupIndex = GroupIDToGroupIndex.Add(GroupID, GroupIDToGroupIndex.Num());
		}

		FGroupInfo& GroupInfo = GroupInfos[GroupIndex];

		// Determine the strand type: guide or render
		// Then, flattened the vertices positions from the selected group and strand type
		const int32 StrandGuide = bHasGuides ? StrandGuides[StrandID] : 0;
		const int32 NumVertices = StrandNumVertices[StrandID];
		if (StrandGuide > 0)
		{
			GroupInfo.NumGuideVertices += NumVertices;
			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				if (Index < HAIR_MAX_NUM_POINT_PER_CURVE)
				{
					if (GroupInfo.CurrentGuideVertexIndex < DeformedPositions[GroupIndex].GuideStrands.Num())
					{
						// Guide vertex are never masked
						FlattenedIsValid.Add(true);
						FlattenedDeformedPositions.Add(DeformedPositions[GroupIndex].GuideStrands[GroupInfo.CurrentGuideVertexIndex]);
						++GroupInfo.CurrentGuideVertexIndex;
					}
				}
				else
				{
					// Mark all invalid/trimmed control point as invalid and set their position to last control point to avoid bound computation issue
					FlattenedIsValid.Add(false);
					FlattenedDeformedPositions.Add(DeformedPositions[GroupIndex].GuideStrands[GroupInfo.CurrentGuideVertexIndex-1]);
				}
			}
		}
		else
		{
			GroupInfo.NumRenderVertices += NumVertices;
			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				if (Index < HAIR_MAX_NUM_POINT_PER_CURVE)
				{
					check(GroupInfo.CurrentRenderVertexIndex < DeformedPositions[GroupIndex].RenderStrands.Num());
					if (GroupInfo.CurrentRenderVertexIndex < DeformedPositions[GroupIndex].RenderStrands.Num())
					{
						FVertexID VertexID(FlattenedDeformedPositions.Num());
	
						const bool bIsValid = bHasMaskedData ? DeformedPositions[GroupIndex].bIsRenderVertexValid[GroupInfo.CurrentRenderVertexIndex] : true;
						FlattenedIsValid.Add(bIsValid);
						FlattenedDeformedPositions.Add(DeformedPositions[GroupIndex].RenderStrands[GroupInfo.CurrentRenderVertexIndex]);
						++GroupInfo.CurrentRenderVertexIndex;
					}
				}
				else
				{
					// Mark all invalid/trimmed control point as invalid and set their position to last control point to avoid bound computation issue
					FlattenedIsValid.Add(false);
					FlattenedDeformedPositions.Add(DeformedPositions[GroupIndex].RenderStrands[GroupInfo.CurrentRenderVertexIndex-1]);
				}
			}
		}
	}

	check(FlattenedDeformedPositions.Num() == GroomNumVertices);

	// Output the flattened deformed positions into the HairDescription
	TVertexAttributesRef<FVector3f> VertexPositions = HairDescription.VertexAttributes().GetAttributesRef<FVector3f>(HairAttribute::Vertex::Position);
	for (int32 VertexIndex = 0; VertexIndex < GroomNumVertices; ++VertexIndex)
	{
		FVertexID VertexID(VertexIndex);
		VertexPositions[VertexID] = FlattenedDeformedPositions[VertexIndex];
	}

	// Apply hair description masking/trimming if needed
	if (bHasMaskedData || bHasTrimmedData)
	{
		TVertexAttributesRef<float> RWVertexWidth = HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width);
		for (int32 VertexIndex = 0; VertexIndex < GroomNumVertices; ++VertexIndex)
		{
			if (!FlattenedIsValid[VertexIndex])
			{
				FVertexID VertexID(VertexIndex);
				RWVertexWidth[VertexID] = 0.f;
			}
		}
	}

	// Regenerate data based on the new/updated hair description
	{
		FHairDescriptionGroups HairDescriptionGroups;
		FGroomBuilder::BuildHairDescriptionGroups(HairDescription, HairDescriptionGroups);

		const int32 GroupCount = GroomAsset->GetHairGroupsInterpolation().Num();
		check(HairDescriptionGroups.HairGroups.Num() == GroupCount);
		for (int32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
		{
			const bool bIsValid = HairDescriptionGroups.IsValid();
			if (bIsValid)
			{
				const FHairDescriptionGroup& HairGroup = HairDescriptionGroups.HairGroups[GroupIndex];
				check(GroupIndex <= HairDescriptionGroups.HairGroups.Num());
				check(GroupIndex == HairGroup.Info.GroupIndex);
				const FHairInterpolationSettings& InterpolationSettings = GroomAsset->GetHairGroupsInterpolation()[GroupIndex].InterpolationSettings;
				const FHairGroupsLOD& HairGroupLOD = GroomAsset->GetHairGroupsLOD()[GroupIndex];

				FHairGroupInfo& HairGroupsInfo = GroomAsset->GetHairGroupsInfo()[GroupIndex];
				FHairGroupPlatformData& HairGroupsData = GroomAsset->GetHairGroupsPlatformData()[GroupIndex];

				FHairStrandsDatas StrandsData;
				FHairStrandsDatas GuidesData;
				FGroomBuilder::BuildData(HairGroup, GroomAsset->GetHairGroupsInterpolation()[GroupIndex], HairGroupsInfo, StrandsData, GuidesData, ALLOW_CURVE_SHUFFLING);

				FGroomBuilder::BuildBulkData(HairGroup.Info, GuidesData,  HairGroupsData.Guides.BulkData, false /*bAllowCompression*/);
				FGroomBuilder::BuildBulkData(HairGroup.Info, StrandsData, HairGroupsData.Strands.BulkData, true /*bAllowCompression*/);

				FHairStrandsInterpolationDatas StrandsInterpolationData;
				FGroomBuilder::BuildInterplationData(HairGroup.Info, StrandsData, GuidesData, InterpolationSettings, StrandsInterpolationData);
				FGroomBuilder::BuildInterplationBulkData(GuidesData, StrandsInterpolationData, HairGroupsData.Strands.InterpolationBulkData);

				FGroomBuilder::BuildClusterBulkData(StrandsData, HairDescriptionGroups.Bounds.SphereRadius, HairGroupLOD, HairGroupsData.Strands.ClusterBulkData);
			}
		}
	}

	GroomAsset->CommitHairDescription(MoveTemp(HairDescription), EHairDescriptionType::Source);
	GroomAsset->UpdateHairGroupsInfo();

	// Update/reimport the cards/meshes geometry which have been deformed prior to call this function
	GroomAsset->BuildCardsData();
	GroomAsset->BuildMeshesData();
}

static void ExtractSkeletalVertexPosition(
	const FSkeletalMeshRenderData* SkeletalMeshData,
	const uint32 MeshLODIndex,
	TArray<FVector3f>& OutMeshVertexPositionsBuffer)
{
	const uint32 VertexCount = SkeletalMeshData->LODRenderData[MeshLODIndex].StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	OutMeshVertexPositionsBuffer.SetNum(VertexCount);
	for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		OutMeshVertexPositionsBuffer[VertexIndex] = SkeletalMeshData->LODRenderData[MeshLODIndex].StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
	}
}

void DeformStaticMeshPositions(
	UStaticMesh* OutMesh,
	const TArray<FVector3f>& MeshVertexPositionsBuffer_Target,
	const FHairStrandsRootData& RestLODData)
{
	// Init the mesh samples with the target mesh vertices
	const int32 MaxVertexCount = MeshVertexPositionsBuffer_Target.Num();
	const uint32 MaxSampleCount = RestLODData.SampleCount;
	const TArray<uint32>& SampleIndicesBuffer = RestLODData.MeshSampleIndicesBuffer;
	TArray<FVector3f> OutSamplePositionsBuffer;

	InitMeshSamples(MaxVertexCount, MeshVertexPositionsBuffer_Target, MaxSampleCount, SampleIndicesBuffer, OutSamplePositionsBuffer);

	// Update those vertices with the RBF interpolation weights
	const TArray<float>& InterpolationWeightsBuffer = RestLODData.MeshInterpolationWeightsBuffer;
	const TArray<FVector4f>& SampleRestPositionsBuffer = RestLODData.RestSamplePositionsBuffer;
	const TArray<FVector3f>& SampleDeformedPositionsBuffer = OutSamplePositionsBuffer;
	TArray<FVector3f> OutSampleDeformationsBuffer;

	UpdateMeshSamples(MaxSampleCount, InterpolationWeightsBuffer, SampleRestPositionsBuffer, SampleDeformedPositionsBuffer, OutSampleDeformationsBuffer);

	// Deform the strands vertices with the deformed mesh samples
	const TArray<FVector3f>& MeshSampleWeightsBuffer = OutSampleDeformationsBuffer;

	// Raw deformation with RBF
	const uint32 MeshLODCount = OutMesh->GetNumSourceModels();
	for (uint32 MeshLODIt = 0; MeshLODIt < MeshLODCount; ++MeshLODIt)
	{
		FMeshDescription* MeshDescription = OutMesh->GetMeshDescription(MeshLODIt);
		TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);

		const uint32 VertexCount = VertexPositions.GetNumElements();
		ParallelFor(VertexCount, [&](uint32 VertexIndex)
		{
			FVertexID VertexID(VertexIndex);
			FVector3f& Point = VertexPositions[VertexID];
			Point = DisplacePosition(Point, MaxSampleCount, SampleRestPositionsBuffer, SampleDeformedPositionsBuffer, MeshSampleWeightsBuffer);
		});

		// Disable bMarkPackageDirty to make this thread safe.
		//
		// Package will be marked dirty by the caller.
		UStaticMesh::FCommitMeshDescriptionParams CommitParams;
		CommitParams.bMarkPackageDirty = false;
		OutMesh->CommitMeshDescription(MeshLODIt, CommitParams);
	}
}

namespace GroomDerivedDataCacheUtils
{
	FString BuildCardsDerivedDataKeySuffix(uint32 GroupIndex, const TArray<FHairLODSettings>& LODs, TArray<FHairGroupsCardsSourceDescription>& SourceDescriptions);
}
#endif // #if WITH_EDITORONLY_DATA

template<typename T>
float SampleTexture(const FUintPoint& InCoord, const uint8 ComponentIndex, const uint8 NumComponent, const FUintPoint& Resolution, const T* InData)
{
	const FUintPoint Coord(FMath::Min(Resolution.X - 1, InCoord.X), FMath::Min(Resolution.Y - 1, InCoord.Y));
	const uint32 Index = ComponentIndex + Coord.X * NumComponent + Coord.Y * Resolution.X * NumComponent;
	return InData[Index];
}

template<typename T>
float SampleTexture(const FVector2f& InUV, const uint8 InComponentIndex, const uint8 InNumComponent, const FUintPoint& InResolution, const uint8* InRawData, float InNormalizationScale)
{
	const T* TypedData = (const T*)InRawData;

	const FVector2f  P(float(InResolution.X) * InUV.X, float(InResolution.Y) * InUV.Y);
	const FUintPoint P00(FMath::Floor(P.X), FMath::Floor(P.Y));
	const FUintPoint P10 = P00 + FUintPoint(1, 0);
	const FUintPoint P11 = P00 + FUintPoint(1, 1);
	const FUintPoint P01 = P00 + FUintPoint(0, 1);

	const T T00 = SampleTexture<T>(P00, InComponentIndex, InNumComponent, InResolution, TypedData);
	const T T10 = SampleTexture<T>(P10, InComponentIndex, InNumComponent, InResolution, TypedData);
	const T T11 = SampleTexture<T>(P11, InComponentIndex, InNumComponent, InResolution, TypedData);
	const T T01 = SampleTexture<T>(P01, InComponentIndex, InNumComponent, InResolution, TypedData);

	const float V00 = float(T00) * InNormalizationScale;
	const float V10 = float(T10) * InNormalizationScale;
	const float V11 = float(T11) * InNormalizationScale;
	const float V01 = float(T01) * InNormalizationScale;

	const FVector2f S(FMath::Frac(P.X), FMath::Frac(P.Y));
	return FMath::Lerp(
		FMath::Lerp(V00, V10, S.X),
		FMath::Lerp(V01, V11, S.X),
		S.Y);
}

static float SampleMaskTexture(const FVector2f& InUV, const FUintPoint& InResolution, const uint8* InData, ETextureSourceFormat Format)
{
	switch(Format)
	{
		case TSF_G8:		return SampleTexture<uint8>		(InUV, 0, 1, InResolution, InData, 1.f/255.f);
		case TSF_BGRA8:		return SampleTexture<uint8>		(InUV, 3, 4, InResolution, InData, 1.f/255.f);
		case TSF_RGBA16:	return SampleTexture<uint16>	(InUV, 0, 4, InResolution, InData, 1.f/65536.f);
		case TSF_RGBA16F:	return SampleTexture<FFloat16>	(InUV, 0, 4, InResolution, InData, 1.f);
		case TSF_G16:		return SampleTexture<uint16>	(InUV, 0, 1, InResolution, InData, 1.f/65536.f);
		case TSF_RGBA32F:	return SampleTexture<float>		(InUV, 0, 4, InResolution, InData, 1.f);
		case TSF_R16F:		return SampleTexture<FFloat16>	(InUV, 0, 1, InResolution, InData, 1.f);
		case TSF_R32F:		return SampleTexture<float>		(InUV, 0, 1, InResolution, InData, 1.f);
		default: check(false);
	}
	return 1.0f;
}

void FGroomRBFDeformer::GetRBFDeformedGroomAsset(
	const UGroomAsset* InGroomAsset, 
	const UGroomBindingAsset* BindingAsset, 
	FTextureSource* MaskTextureSource, 
	const float MaskScale, 
	UGroomAsset* OutGroomAsset, 
	const ITargetPlatform* TargetPlatform,
	bool bShouldBuildStaticMeshes)
{
	// If bShouldBuildStaticMeshes is true, this function may only be called from the Game Thread
	check(!bShouldBuildStaticMeshes || IsInGameThread());

#if WITH_EDITORONLY_DATA
	if (InGroomAsset && BindingAsset && BindingAsset->GetTargetSkeletalMesh() && BindingAsset->GetSourceSkeletalMesh())
	{
		// Use the LOD0 skeletal mesh to extract the vertices used for the RBF weight computation
		const int32 MeshLODIndex = 0;

		// Apply RBF deformation to each group of guides and render strands
		const int32 NumGroups = InGroomAsset->GetHairGroupsPlatformData().Num();

		FGroomBindingBuilder::FInput Input;
		{
			Input.BindingType = EGroomBindingMeshType::SkeletalMesh;
			Input.NumInterpolationPoints = BindingAsset->GetNumInterpolationPoints();
			Input.MatchingSection = BindingAsset->GetMatchingSection();
			Input.SourceMeshLOD = 0;
			Input.TargetMeshMinLOD = 0;
			Input.bHasValidTarget = BindingAsset->HasValidTarget();
			Input.GroomAsset = BindingAsset->GetGroom();
			Input.SourceSkeletalMesh = BindingAsset->GetSourceSkeletalMesh();
			Input.TargetSkeletalMesh = BindingAsset->GetTargetSkeletalMesh();
			Input.SourceGeometryCache = nullptr;
			Input.TargetGeometryCache = nullptr;
			Input.bForceUseRunningPlatform = true;
		}

		// Compute root datas
		TArray<FHairRootGroupData> RootDatas;
		{
			if (TargetPlatform == nullptr)
			{
				TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
			}

			RootDatas.SetNum(NumGroups);
			for (int32 GroupIt = 0; GroupIt < NumGroups; ++GroupIt)
			{
				const bool bSucceed = BuildHairRootGroupData(Input, GroupIt, TargetPlatform, RootDatas[GroupIt]);
				check(bSucceed);
			}
		}

		// Use the vertices positions from the HairDescription instead of the GroomAsset since the latter
		// may contain decimated or auto-generated guides depending on the import settings
		// When building the hair description groups, do not add control points at the end of each curves, 
		// as this would make cause a mismatch ordering of the flatten CP's position vs. the hair description CP's positions
		FHairDescriptionGroups HairDescriptionGroups;
		FGroomBuilder::BuildHairDescriptionGroups(InGroomAsset->GetHairDescription(), HairDescriptionGroups, false /*bAllowAddEndControlPoint*/);

		TArray<FRBFDeformedPositions> DeformedPositions;
		DeformedPositions.SetNum(NumGroups);
		// Sanity check to insure that the groom has all the original vertices
		for (int32 GroupIt = 0; GroupIt < NumGroups; ++GroupIt)
		{
			check(InGroomAsset->GetHairGroupsInterpolation()[GroupIt].DecimationSettings.VertexDecimation == 1);
			check(InGroomAsset->GetHairGroupsInterpolation()[GroupIt].DecimationSettings.CurveDecimation == 1);
		}

		// Build Curve index for every vertices
		auto BuildPointToCurveMapping = [](const FHairStrandsDatas& In, TArray<uint32>& Out)
		{
			const uint32 CurveCount = In.GetNumCurves();
			Out.SetNum(In.GetNumPoints());

			for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
			{
				const uint32 RootIndex = In.StrandsCurves.CurvesOffset[CurveIndex];
				const uint32 PointCount = In.StrandsCurves.CurvesCount[CurveIndex];
				for (uint32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
				{
					Out[RootIndex + PointIndex] = CurveIndex; // RootIndex;
				}
			}
		};

		// Note that the GroupID from the HairGroups cannot be used as the GroupIndex since
		// the former may not be strictly increasing nor consecutive
		// but the ordering of the groups does represent the GroupIndex
		for (const FHairDescriptionGroup& Group : HairDescriptionGroups.HairGroups)
		{
			const uint32 GroupIndex = Group.Info.GroupIndex;
			const FHairStrandsRawDatas& OriginalGuides = Group.Guides;
			const FHairGroupPlatformData& HairGroupData = InGroomAsset->GetHairGroupsPlatformData()[GroupIndex];

			FHairStrandsDatas StrandsData;
			FHairStrandsDatas GuidesData;
			FHairGroupInfo DummyInfo;

			// Disable explicitely curve reordering as we want to preserve vertex mapping to write deformed postions back to the hair description, after RBF deformation
			FGroomBuilder::BuildData(Group, InGroomAsset->GetHairGroupsInterpolation()[GroupIndex], DummyInfo, StrandsData, GuidesData, ALLOW_CURVE_SHUFFLING);

			TArray<uint32> SimRootDataPointToCurveBuffer;
			TArray<uint32> RenRootDataPointToCurveBuffer;
			BuildPointToCurveMapping(GuidesData, SimRootDataPointToCurveBuffer);
			BuildPointToCurveMapping(StrandsData, RenRootDataPointToCurveBuffer);

			// Sanity check
			check(RootDatas[GroupIndex].MeshPositions.IsValidIndex(MeshLODIndex));
			check(RootDatas[GroupIndex].SimRootDatas.IsValidIndex(MeshLODIndex));
			check(RootDatas[GroupIndex].RenRootDatas.IsValidIndex(MeshLODIndex));

			// Get the deformed positions (== target mesh positions), i.e. the non-transfered position.
			const TArray<FVector3f>& MeshVertexPositionsBuffer_Target = RootDatas[GroupIndex].MeshPositions[MeshLODIndex];

			// Get deformed guides
			// If the groom override the value, we output dummy value for the guides, since they won't be used
			if (InGroomAsset->GetHairGroupsInterpolation()[GroupIndex].InterpolationSettings.GuideType != EGroomGuideType::Imported)
			{
				const uint32 OriginalVertexCount = OriginalGuides.StrandsPoints.Num();
				DeformedPositions[GroupIndex].GuideStrands.Init(FVector3f::ZeroVector, OriginalVertexCount);
			}
			else
			{
				const FHairStrandsRootData& SimRootData = RootDatas[GroupIndex].SimRootDatas[MeshLODIndex];

				const bool bHasRBF = InGroomAsset->IsGlobalInterpolationEnable(GroupIndex, 0/*HairLODIndex*/);

				DeformedPositions[GroupIndex].GuideStrands = GetDeformedHairStrandsPositions(
					MeshVertexPositionsBuffer_Target,
					GuidesData,
					MeshLODIndex,
					bHasRBF,
					SimRootDataPointToCurveBuffer,
					SimRootData);
			}

			// Get deformed render strands
			{
				FHairStrandsRootData RenRootData = RootDatas[GroupIndex].RenRootDatas[MeshLODIndex];

				// Transfer RBF weights from SimRootData to RenRootData, as RenRootData does not hold RBF sample data
				// This is transient data.
				{
					const FHairStrandsRootData& SimRootData = RootDatas[GroupIndex].SimRootDatas[MeshLODIndex];

					const FHairStrandsRootData& SimLODData = SimRootData;
					FHairStrandsRootData& RenLODData = RenRootData;
					RenLODData.SampleCount 						= SimLODData.SampleCount;
					RenLODData.MeshInterpolationWeightsBuffer 	= SimLODData.MeshInterpolationWeightsBuffer;
					RenLODData.MeshSampleIndicesBuffer 			= SimLODData.MeshSampleIndicesBuffer;
					RenLODData.RestSamplePositionsBuffer 		= SimLODData.RestSamplePositionsBuffer;
					RenLODData.MeshSampleSectionsBuffer			= SimLODData.MeshSampleSectionsBuffer;
					RenLODData.UniqueSectionIds					= SimLODData.UniqueSectionIds;
					RenLODData.MeshSectionCount					= SimLODData.MeshSectionCount;
				}

				const bool bHasRBF = InGroomAsset->IsGlobalInterpolationEnable(GroupIndex, 0/*HairLODIndex*/);

				DeformedPositions[GroupIndex].RenderStrands = GetDeformedHairStrandsPositions(
					MeshVertexPositionsBuffer_Target,
					StrandsData,
					MeshLODIndex,
					bHasRBF,
					RenRootDataPointToCurveBuffer,
					RenRootData);
			}

			// Trim strands based on mask texture
			DeformedPositions[GroupIndex].bIsRenderVertexValid.Init(true, StrandsData.GetNumPoints());
			DeformedPositions[GroupIndex].bIsRenderCurveValid.Init(true, StrandsData.GetNumCurves());
			DeformedPositions[GroupIndex].bHasMaskedData = false;
			if (MaskTextureSource)
			{
				check(MaskTextureSource->GetNumBlocks() == 1);

				const ETextureSourceFormat Format = MaskTextureSource->GetFormat();
				const FUintPoint Resolution(MaskTextureSource->GetSizeX(), MaskTextureSource->GetSizeY());

				// Access texture CPU data
				FTextureSourceBlock Block;
				MaskTextureSource->GetBlock(0, Block);
				const uint8* DataInBytes = MaskTextureSource->LockMipReadOnly(0 /*MipIndex*/);

				const uint32 CurveCount = StrandsData.GetNumCurves();
				for (uint32 CurveIndex =0; CurveIndex < CurveCount; ++CurveIndex)
				{
					const uint32 CurveOffset = StrandsData.StrandsCurves.CurvesOffset[CurveIndex];
					const uint16 CurveNumVertices = StrandsData.StrandsCurves.CurvesCount[CurveIndex];

					FVector2f RootUV = StrandsData.StrandsCurves.CurvesRootUV[CurveIndex];
					RootUV.Y = FMath::Clamp(1.f - RootUV.Y,0.f, 1.0f);
					const float Maskhreshold = SampleMaskTexture(RootUV, Resolution, DataInBytes, Format);
					const float CoordUThreshold = FMath::Lerp(1.f, Maskhreshold, MaskScale);

					DeformedPositions[GroupIndex].bIsRenderCurveValid[CurveIndex] = true;
					for (uint16 VertexIndex = 0; VertexIndex < CurveNumVertices; ++VertexIndex)
					{
						const uint32 PointGlobalIndex0 = VertexIndex + CurveOffset;
						const uint32 PointGlobalIndex1 = VertexIndex + CurveOffset + 1;

						DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex0] = true;

						// Trimmed valid
						if (CoordUThreshold < 1.f)
						{
							const float CoordU0 = StrandsData.StrandsPoints.PointsCoordU[PointGlobalIndex0];
							if (CoordUThreshold <= 0.f)
							{
								DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex0] = false;
								DeformedPositions[GroupIndex].bHasMaskedData = true;
							}
							else if (VertexIndex + 1 < CurveNumVertices)
							{
								DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex1] = true;

								// Interpolate position or masked vertex
								const float CoordU1 = StrandsData.StrandsPoints.PointsCoordU[PointGlobalIndex1];
								if (CoordU0 <= CoordUThreshold && CoordU1 > CoordUThreshold)
								{
									const float S = FMath::Clamp((CoordUThreshold - CoordU0) / FMath::Max(0.0001f, CoordU1 - CoordU0), 0.f, 1.f);
									StrandsData.StrandsPoints.PointsPosition[PointGlobalIndex1] =
										FMath::Lerp(
										StrandsData.StrandsPoints.PointsPosition[PointGlobalIndex0],
										StrandsData.StrandsPoints.PointsPosition[PointGlobalIndex1],
										S);

									DeformedPositions[GroupIndex].bHasMaskedData = true;
								}
								else if (CoordU0 > CoordUThreshold)
								{
									DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex0] = false;
									DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex1] = false;
								}
							}
							else if (CoordU0 > CoordUThreshold)
							{
								DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex0] = false;
								DeformedPositions[GroupIndex].bHasMaskedData = true;
							}

							// Mark the entire curve as masked if the first or second vertex are masked, because a curve needs to have at least two valid points
							if (!DeformedPositions[GroupIndex].bIsRenderVertexValid[PointGlobalIndex0] && (VertexIndex == 0 || VertexIndex == 1))
							{
								DeformedPositions[GroupIndex].bIsRenderCurveValid[CurveIndex] = false;
							}
						}
					}
				}

				MaskTextureSource->UnlockMip(0);
			}
		}

		// Apply changes onto cards and meshes (OutGroomASset already contain duplicated mesh asset
		for (FHairGroupsCardsSourceDescription& Desc : OutGroomAsset->GetHairGroupsCards())
		{
			if (UStaticMesh* Mesh = Desc.ImportedMesh)
			{
				if (Desc.GroupIndex >= 0)
				{
					if (bShouldBuildStaticMeshes)
					{
						Mesh->ConditionalPostLoad();
						Mesh->PreEditChange(nullptr);
					}
	
					check(RootDatas[Desc.GroupIndex].SimRootDatas.IsValidIndex(MeshLODIndex));

					// Load Sim root data, as they contains RBF weights
					const FHairStrandsRootData& SimRootData = RootDatas[Desc.GroupIndex].SimRootDatas[MeshLODIndex];
	
					// Get the deformed positions (== target mesh positions), i.e. the non-transfered position.
					const TArray<FVector3f>& MeshVertexPositionsBuffer_Target = RootDatas[Desc.GroupIndex].MeshPositions[MeshLODIndex];
					DeformStaticMeshPositions(Mesh, MeshVertexPositionsBuffer_Target, SimRootData);
				}
			}
		} 

		// Apply RBF deformation to mesh vertices
		for (FHairGroupsMeshesSourceDescription& Desc : OutGroomAsset->GetHairGroupsMeshes())
		{
			if (UStaticMesh* Mesh = Desc.ImportedMesh)
			{
				if (Mesh->GetNumSourceModels() == 0 || Desc.GroupIndex < 0 || Desc.GroupIndex >= InGroomAsset->GetNumHairGroups() || Desc.LODIndex == -1)
				{
					continue;
				}

				if (bShouldBuildStaticMeshes)
				{
					Mesh->ConditionalPostLoad();
					Mesh->PreEditChange(nullptr);
				}

				check(RootDatas[Desc.GroupIndex].SimRootDatas.IsValidIndex(MeshLODIndex));

				// Load Sim root data, as they contains RBF weights
				const FHairStrandsRootData& SimRootData = RootDatas[Desc.GroupIndex].SimRootDatas[MeshLODIndex];

				// Get the deformed positions (== target mesh positions), i.e. the non-transfered position.
				const TArray<FVector3f>& MeshVertexPositionsBuffer_Target = RootDatas[Desc.GroupIndex].MeshPositions[MeshLODIndex];
				DeformStaticMeshPositions(Mesh, MeshVertexPositionsBuffer_Target, SimRootData);
			}
		}

		// Build all static meshes in one batch
		if (bShouldBuildStaticMeshes)
		{
			TArray<UStaticMesh*> MeshesToBuild;
			MeshesToBuild.Reserve(OutGroomAsset->GetHairGroupsCards().Num() + OutGroomAsset->GetHairGroupsMeshes().Num());

			for (FHairGroupsCardsSourceDescription& Desc : OutGroomAsset->GetHairGroupsCards())
			{
				if (UStaticMesh* Mesh = Desc.ImportedMesh)
				{
					MeshesToBuild.Add(Mesh);
					Mesh->MarkPackageDirty();
				}
			}
			for (FHairGroupsMeshesSourceDescription& Desc : OutGroomAsset->GetHairGroupsMeshes())
			{
				if (UStaticMesh* Mesh = Desc.ImportedMesh)
				{
					MeshesToBuild.Add(Mesh);
					Mesh->MarkPackageDirty();
				}
			}

			UStaticMesh::BatchBuild(MeshesToBuild, /*bSilent*/ true);

			FStaticMeshCompilingManager::Get().FinishAllCompilation();
		}

		// Finally, the deformed guides and strands are applied to the GroomAsset
		ApplyDeformationToGroom(DeformedPositions, OutGroomAsset);
	}
#endif // #if WITH_EDITORONLY_DATA
}

uint32 FGroomRBFDeformer::GetEntryCount(uint32 InSampleCount)
{
	return HAIR_RBF_ENTRY_COUNT(InSampleCount);
}

uint32 FGroomRBFDeformer::GetWeightCount(uint32 InSampleCount)
{
	const uint32 EntryCount = GetEntryCount(InSampleCount);
	return EntryCount * EntryCount;
}

