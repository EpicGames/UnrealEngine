// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomResources.h"
#include "Engine/Texture2D.h"
#include "EngineUtils.h"
#include "GroomAssetImportData.h"
#include "GroomBuilder.h"
#include "GroomImportOptions.h"
#include "GroomSettings.h"
#include "RenderingThread.h"
#include "Engine/AssetUserData.h"
#include "HairStrandsVertexFactory.h"
#include "HairStrandsInterface.h"
#include "Misc/Paths.h"
#include "RHIStaticStates.h"
#include "RHIUtilities.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/BulkData.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "NiagaraSystem.h"
#include "Async/ParallelFor.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "RenderGraph.h"
#include "RenderGraphUtils.h"
#include "RenderTargetPool.h"
#include "RHIUtilities.h"
#include "GroomBindingBuilder.h"
#include "GroomRBFDeformer.h"
#include "UnifiedBuffer.h"
#if WITH_EDITORONLY_DATA
#include "Containers/StringView.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#endif

DEFINE_RENDER_COMMAND_PIPE(Groom, ERenderCommandPipeFlags::None);

static int32 GHairStrandsBulkData_ReleaseAfterUse = 0;
static FAutoConsoleVariableRef CVarHairStrandsBulkData_ReleaseAfterUse(TEXT("r.HairStrands.Strands.BulkData.ReleaseAfterUse"), GHairStrandsBulkData_ReleaseAfterUse, TEXT("Release CPU bulk data once hair groom/groom binding asset GPU resources are created. This saves memory"));

static int32 GHairStrandsBulkData_AsyncLoading = -1;
static int32 GHairCardsBulkData_AsyncLoading = -1;

static FAutoConsoleVariableRef CVarHairStrandsBulkData_AsyncLoading(TEXT("r.HairStrands.Strands.BulkData.AsyncLoading"), GHairStrandsBulkData_AsyncLoading, TEXT("Load hair strands data with async loading so that it is not blocking the rendering thread. This value define the MinLOD at which this happen. Default disabled (-1)"));
static FAutoConsoleVariableRef CVarHairCardsBulkData_AsyncLoading(TEXT("r.HairStrands.Cards.BulkData.AsyncLoading"), GHairCardsBulkData_AsyncLoading, TEXT("Load hair cards/meshes data with async loading so that it is not blocking the rendering thread. This value define the MinLOD at which this happen. Default disabled (-1)"));

static int32 GHairStrandsBulkData_Validation = 1;
static FAutoConsoleVariableRef CVarHairStrandsBulkData_Validation(TEXT("r.HairStrands.Strands.BulkData.Validation"), GHairStrandsBulkData_Validation, TEXT("Validate some hair strands data at serialization/loading time."));

static float GHairStrandsDebugVoxel_WorldSize = 0.3f;
static int32 GHairStrandsDebugVoxel_MaxSegmentPerVoxel = 2048;
static FAutoConsoleVariableRef CVarHairStrandsDebugVoxel_WorldSize(TEXT("r.HairStrands.DebugData.VoxelSize"), GHairStrandsDebugVoxel_WorldSize, TEXT("Voxel size use for creating debug data."));
static FAutoConsoleVariableRef CVarHairStrandsDebugVoxel_MaxSegmentPerVoxel(TEXT("r.HairStrands.DebugData.MaxSegmentPerVoxel"), GHairStrandsDebugVoxel_MaxSegmentPerVoxel, TEXT("Max number of segments per Voxel size when creating debug data."));

bool ValidateHairBulkData()
{
	return GHairStrandsBulkData_Validation > 0;
}

static void CreateExternalBufferViews(FRDGBuilder& GraphBuilder, FRDGBufferRef& InBuffer, FRDGExternalBuffer& OutBuffer, EPixelFormat Format = PF_Unknown)
{
	check(OutBuffer.Buffer);
	if (EnumHasAnyFlags(InBuffer->Desc.Usage, BUF_ShaderResource))
	{
		OutBuffer.SRV = OutBuffer.Buffer->GetOrCreateSRV(GraphBuilder.RHICmdList, FRDGBufferSRVDesc(InBuffer, Format));
	}
	if (EnumHasAnyFlags(InBuffer->Desc.Usage, BUF_UnorderedAccess))
	{
		OutBuffer.UAV = OutBuffer.Buffer->GetOrCreateUAV(GraphBuilder.RHICmdList, FRDGBufferUAVDesc(InBuffer, Format));
	}
	OutBuffer.Format = Format;
}

static void ConvertToExternalBufferWithViews(FRDGBuilder& GraphBuilder, FRDGBufferRef& InBuffer, FRDGExternalBuffer& OutBuffer, EPixelFormat Format = PF_Unknown)
{
	OutBuffer.Buffer = GraphBuilder.ConvertToExternalBuffer(InBuffer);
	CreateExternalBufferViews(GraphBuilder, InBuffer, OutBuffer, Format);
}

////////////////////////////////////////////////////////////////////////////////////

EHairResourceLoadingType GetHairResourceLoadingType(EHairGeometryType InGeometryType, int32 InLODIndex)
{
#if !WITH_EDITORONLY_DATA
	switch (InGeometryType)
	{
	case EHairGeometryType::Strands: return InLODIndex <= GHairStrandsBulkData_AsyncLoading ? EHairResourceLoadingType::Async : EHairResourceLoadingType::Sync;
	case EHairGeometryType::Cards:
	case EHairGeometryType::Meshes: return InLODIndex <= GHairCardsBulkData_AsyncLoading ? EHairResourceLoadingType::Async : EHairResourceLoadingType::Sync;
	}
	return EHairResourceLoadingType::Sync;
#else
	return EHairResourceLoadingType::Sync;
#endif
}

enum class EHairResourceUsageType : uint8
{
	Static,
	Dynamic
};

#define HAIRSTRANDS_RESOUCE_NAME(Type, Name) (Type == EHairStrandsResourcesType::Guides  ? TEXT(#Name "(Guides)") : (Type == EHairStrandsResourcesType::Strands ? TEXT(#Name "(Strands)") : TEXT(#Name "(Cards)")))

const TCHAR* ToHairResourceDebugName(const TCHAR* In, FHairResourceName& InDebugNames)
{
#if HAIR_RESOURCE_DEBUG_NAME
	FString& TempDebugName = InDebugNames.Names.AddDefaulted_GetRef();
	TempDebugName = In;
	if (InDebugNames.GroupIndex >= 0)
	{
		TempDebugName += TEXT("_GROUP") + FString::FromInt(InDebugNames.GroupIndex);
	}
	if (InDebugNames.LODIndex >= 0)
	{
		TempDebugName += TEXT("_LOD") + FString::FromInt(InDebugNames.LODIndex);
	}
	TempDebugName += TEXT("_") + InDebugNames.AssetName.ToString();
	return *TempDebugName;
#else
	return In;
#endif
}

/////////////////////////////////////////////////////////////////////////////////////////
// FRWBuffer utils 

template<typename FormatType>
void CreateBuffer(FRHICommandListBase& RHICmdList, const TArray<typename FormatType::Type>& InData, FRWBuffer& OutBuffer, const TCHAR* DebugName, ERHIAccess InitialAccess = ERHIAccess::SRVMask)
{
	const uint32 DataCount = InData.Num();
	const uint32 DataSizeInBytes = FormatType::SizeInByte*DataCount;

	if (DataSizeInBytes == 0) return;

	OutBuffer.Initialize(RHICmdList, FormatType::SizeInByte, DataCount, FormatType::Format, InitialAccess, BUF_Static, DebugName);
	void* BufferData = RHICmdList.LockBuffer(OutBuffer.Buffer, 0, DataSizeInBytes, RLM_WriteOnly);

	FMemory::Memcpy(BufferData, InData.GetData(), DataSizeInBytes);
	RHICmdList.UnlockBuffer(OutBuffer.Buffer);
}

template<typename FormatType>
void CreateBuffer(FRHICommandListBase& RHICmdList, uint32 InVertexCount, FRWBuffer& OutBuffer, const TCHAR* DebugName)
{
	const uint32 DataCount = InVertexCount;
	const uint32 DataSizeInBytes = FormatType::SizeInByte*DataCount;

	if (DataSizeInBytes == 0) return;

	OutBuffer.Initialize(RHICmdList, FormatType::SizeInByte, DataCount, FormatType::Format, ERHIAccess::UAVCompute, BUF_Static, DebugName);
	void* BufferData = RHICmdList.LockBuffer(OutBuffer.Buffer, 0, DataSizeInBytes, RLM_WriteOnly);
	FMemory::Memset(BufferData, 0, DataSizeInBytes);
	RHICmdList.UnlockBuffer(OutBuffer.Buffer);
}

template<typename FormatType>
void CreateBuffer(FRHICommandListBase& RHICmdList, const TArray<typename FormatType::Type>& InData, FHairCardsVertexBuffer& OutBuffer, const TCHAR* DebugName, const FName& OwnerName, ERHIAccess InitialAccess = ERHIAccess::SRVMask)
{
	const uint32 DataCount = InData.Num();
	const uint32 DataSizeInBytes = FormatType::SizeInByte * DataCount;

	if (DataSizeInBytes == 0)
	{
		return;
	}

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(DebugName, DataSizeInBytes)
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetInitialState(InitialAccess)
		.SetOwnerName(OwnerName)
		.SetInitActionInitializer();

	FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
	Initializer.WriteData(InData.GetData(), DataSizeInBytes);

	OutBuffer.VertexBufferRHI = Initializer.Finalize();
	OutBuffer.ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
		OutBuffer.VertexBufferRHI, 
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(FormatType::Format));
}

/////////////////////////////////////////////////////////////////////////////////////////
// RDG buffers utils 

static FRDGBufferDesc ApplyUsage(FRDGBufferDesc In, EHairResourceUsageType InUsage)
{
	if (InUsage != EHairResourceUsageType::Dynamic)
	{
		In.Usage = In.Usage & (~BUF_UnorderedAccess);
	}
	return In;
}

inline bool ReleaseAfterUse() { return GHairStrandsBulkData_ReleaseAfterUse > 0; }

inline void InternalSetBulkDataFlags(FByteBulkData& In)
{
	// Unloading of the bulk data is only supported on cooked build, as we can reload the data from the file/archieve
#if !WITH_EDITORONLY_DATA
	const bool bReleaseCPUData = ReleaseAfterUse();
	if (bReleaseCPUData)
	{
		In.SetBulkDataFlags(BULKDATA_SingleUse);
	}
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Regular loading from BulkData

FRDGBufferRef InternalCreateBufferRDG_FromBulkData(FRDGBuilder& GraphBuilder, FByteBulkData& InBulkData, FRDGExternalBuffer& Out, const EPixelFormat OutFormat, const FRDGBufferDesc Desc, const TCHAR* DebugName, const FName& OwnerName)
{
	InternalSetBulkDataFlags(InBulkData);

	const uint32 DataSizeInBytes = Desc.GetSize();
	check(InBulkData.GetBulkDataSize() >= DataSizeInBytes);
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	const uint8* Data = (const uint8*)InBulkData.Lock(LOCK_READ_ONLY);
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::MultiFrame);
	Buffer->SetOwnerName(OwnerName);
	if (Data && DataSizeInBytes)
	{
		#if !WITH_EDITORONLY_DATA
		if (ReleaseAfterUse())
		{
			GraphBuilder.QueueBufferUpload(Buffer, Data, DataSizeInBytes, [&InBulkData](const void* Ptr) { InBulkData.Unlock(); });
		}
		else
		#endif
		{
			GraphBuilder.QueueBufferUpload(Buffer, Data, DataSizeInBytes, ERDGInitialDataFlags::None);  // Copy data internally
			InBulkData.Unlock();
		}
	}
	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, OutFormat);
	return Buffer;
}

template<typename FormatType>
FRDGBufferRef InternalCreateVertexBufferRDG_FromBulkData(FRDGBuilder& GraphBuilder, FByteBulkData& InBulkData, uint32 InDataCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType)
{
	const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateBufferDesc(FormatType::SizeInByte, InDataCount), UsageType);
	return InternalCreateBufferRDG_FromBulkData(GraphBuilder, InBulkData, Out, FormatType::Format, Desc, DebugName, OwnerName);
}

template<typename FormatType>
FRDGBufferRef InternalCreateStructuredBufferRDG_FromBulkData(FRDGBuilder& GraphBuilder, FByteBulkData& InBulkData, uint32 InDataCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType)
{
	const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateStructuredDesc(FormatType::SizeInByte, InDataCount), UsageType);
	return InternalCreateBufferRDG_FromBulkData(GraphBuilder, InBulkData, Out, PF_Unknown, Desc, DebugName, OwnerName);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// HairBulkData loading


///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairTranscodeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairTranscodeCS);
	SHADER_USE_PARAMETER_STRUCT(FHairTranscodeCS, FGlobalShader);

	class FTranscoding : SHADER_PERMUTATION_BOOL("PERMUTATION_TRANSCODING");
	using FPermutationDomain = TShaderPermutationDomain<FTranscoding>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, OffsetInBytes)
		SHADER_PARAMETER(uint32, DataSizeInBytes)
		SHADER_PARAMETER(uint32, TotalSizeInBytes)

		SHADER_PARAMETER(uint32, UncompressedOffsetInBytes)
		SHADER_PARAMETER(uint32, UncompressedTotalSizeInBytes)

		SHADER_PARAMETER(FVector3f, PositionScale)
		SHADER_PARAMETER(FVector3f, PositionOffset)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, InBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 1024u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TRANSCODE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairTranscodeCS, "/Engine/Private/HairStrands/HairStrandsTranscode.usf", "MainCS", SF_Compute);

struct FHairTranscoding
{
	bool bEnabled = false;
	FVector3f PositionOffset = FVector3f::ZeroVector;
	FVector3f PositionScale = FVector3f::ZeroVector;

	uint32 UncompressedTotalSizeInBytes = 0;
	uint32 UncompressedOffsetInBytes = 0;
};

static void AddTranscodePass(
	FRDGBuilder& GraphBuilder,
	uint32 InOffsetInBytes,
	uint32 InNewDataSizeInSize,
	uint32 InTotalSizeInBytes,
	FRDGBufferRef In,
	FRDGBufferRef Out,
	const FHairTranscoding& InTranscoding)
{
	const FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	FHairTranscodeCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairTranscodeCS::FParameters>();
	Parameters->TotalSizeInBytes = InTotalSizeInBytes;
	Parameters->DataSizeInBytes = InNewDataSizeInSize;
	Parameters->OffsetInBytes = InOffsetInBytes;
	Parameters->InBuffer = GraphBuilder.CreateSRV(In);
	Parameters->OutBuffer = GraphBuilder.CreateUAV(Out);

	// Transcoding parameters
	Parameters->UncompressedTotalSizeInBytes = InTranscoding.UncompressedTotalSizeInBytes;
	Parameters->UncompressedOffsetInBytes = InTranscoding.UncompressedOffsetInBytes;
	Parameters->PositionScale = InTranscoding.PositionScale;
	Parameters->PositionOffset = InTranscoding.PositionOffset;

	// Data needs to be 4-bytes aligned
	check((Parameters->DataSizeInBytes & 0x3) == 0);
	check((Parameters->OffsetInBytes & 0x3) == 0);

	const uint32 Byte4Count  = Parameters->DataSizeInBytes >> 2u;
	const uint32 Byte16Count = Parameters->DataSizeInBytes >> 4u;
	const FIntVector DispatchGroupCount = InTranscoding.bEnabled ? 
		FComputeShaderUtils::GetGroupCount(Byte16Count, FHairTranscodeCS::GetGroupSize()): 
		FComputeShaderUtils::GetGroupCount(Byte4Count,  FHairTranscodeCS::GetGroupSize());

	const FShaderParametersMetadata* ParametersMetadata = FHairTranscodeCS::FParameters::FTypeInfo::GetStructMetadata();

	FHairTranscodeCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairTranscodeCS::FTranscoding>(InTranscoding.bEnabled ? 1u : 0u);

	TShaderMapRef<FHairTranscodeCS> ComputeShader(ShaderMap, PermutationVector);
	ClearUnusedGraphResources(ComputeShader, ParametersMetadata, Parameters);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::Transcode(Transcoding:%d)", InTranscoding.bEnabled ? 1u: 0u),
		ComputeShader,
		Parameters,
		DispatchGroupCount);
}

// Used for Buffer and StructuredBuffer
static FRDGBufferRef InternalCreateBufferRDG_FromHairBulkData(FRDGBuilder& GraphBuilder, FHairBulkContainer& InData, const FRDGBufferRef& In, const FRDGBufferDesc& BufferDesc, const FRDGBufferDesc& UploadDesc, const TCHAR* DebugName, const FName& OwnerName)
{
	check(InData.ChunkRequest);
	FHairStreamingRequest::FChunk& InChunk = *InData.ChunkRequest;

	const bool bDeallocate  = In != nullptr && InData.ChunkRequest->Status == FHairStreamingRequest::FChunk::EStatus::Unloading;
	const bool bCreate 		= In == nullptr || BufferDesc.GetSize() == UploadDesc.GetSize();
	const bool bCopy 		= In != nullptr && InChunk.Size > 0;
	const bool bReallocate 	= In != nullptr && In->Desc.GetSize() < InChunk.TotalSize;

	check(InChunk.Size > 0);
	// Either create a new buffer or append new data to existing buffer
	if (bDeallocate)
	{
		// 1. Create new (smaller) buffer 
		FRDGBufferDesc NewBufferDesc = BufferDesc;
		NewBufferDesc.Usage |= EBufferUsageFlags::UnorderedAccess;
		FRDGBufferRef Out = GraphBuilder.CreateBuffer(NewBufferDesc, DebugName, ERDGBufferFlags::MultiFrame);
		Out->SetOwnerName(OwnerName);

		// 2. Copy existing data from the old buffer to the new buffer
		check (Out->Desc.GetSize() <= In->Desc.GetSize());
		AddCopyBufferPass(GraphBuilder, Out, 0, In, 0, Out->Desc.GetSize());
		return Out;
	}
	else if (bCreate)
	{	
		check(BufferDesc.GetSize() >= UploadDesc.GetSize());

		FRDGBufferRef Out = GraphBuilder.CreateBuffer(BufferDesc, DebugName, ERDGBufferFlags::MultiFrame);
		Out->SetOwnerName(OwnerName);
		GraphBuilder.QueueBufferUpload(Out, InChunk.GetData(), InChunk.Size, ERDGInitialDataFlags::None);
		InChunk.Release();
		return Out;
	}
	else if (bCopy)
	{
		// 1. If the current buffer is too small for storing the new data, reallocate it
		FRDGBufferRef Out = In;
		if (bReallocate)
		{
			// 1.1 Create new buffer
			FRDGBufferDesc NewBufferDesc = BufferDesc;
			NewBufferDesc.Usage |= EBufferUsageFlags::UnorderedAccess;
			Out = GraphBuilder.CreateBuffer(NewBufferDesc, DebugName, ERDGBufferFlags::MultiFrame);
			Out->SetOwnerName(OwnerName);

			// 1.2 Copy existing data from the old buffer to the new buffer
			AddCopyBufferPass(GraphBuilder, Out, 0, In, 0, In->Desc.GetSize());
		}

		// 2. Upload missing data
		FRDGBufferRef UploadBuffer = GraphBuilder.CreateBuffer(UploadDesc, DebugName, ERDGBufferFlags::MultiFrame);
		GraphBuilder.QueueBufferUpload(UploadBuffer, InChunk.GetData(), InChunk.Size, ERDGInitialDataFlags::None);
		InChunk.Release();

		// 4. Append new data to the new/existing buffer
		AddCopyBufferPass(GraphBuilder, Out, InChunk.Offset, UploadBuffer, 0, InChunk.Size);

		// Return the new buffer if it needs to be extracted
		return Out;
		//return bReallocate ? Out : nullptr;
	}
	else
	{
		return nullptr;
	}
}

template<typename FormatType>
FRDGBufferRef InternalCreateVertexBufferRDG_FromHairBulkData(FRDGBuilder& GraphBuilder, FHairBulkContainer& InChunk, uint32 InDataCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType)
{
	// Fallback for non-streamble resources (e.g. guides)
	if (InChunk.ChunkRequest == nullptr)
	{
		return InternalCreateVertexBufferRDG_FromBulkData<FormatType>(GraphBuilder, InChunk.Data, InDataCount, Out, DebugName, OwnerName, UsageType);
	}

	const FRDGBufferRef In = Out.Buffer ? Register(GraphBuilder, Out, ERDGImportedBufferFlags::None).Buffer : nullptr;
	const FRDGBufferDesc BufferDesc = ApplyUsage(FRDGBufferDesc::CreateBufferDesc(FormatType::SizeInByte, FMath::DivideAndRoundUp(InChunk.ChunkRequest->TotalSize, FormatType::SizeInByte)), UsageType);
	const FRDGBufferDesc UploadDesc = ApplyUsage(FRDGBufferDesc::CreateBufferDesc(FormatType::SizeInByte, FMath::DivideAndRoundUp(InChunk.ChunkRequest->Size, FormatType::SizeInByte)), UsageType);
	if (BufferDesc.GetSize() == 0) 	{ Out.Buffer = nullptr; return nullptr; }
	if (UploadDesc.GetSize()==0) 	{ return nullptr; }
	if (FRDGBufferRef Buffer = InternalCreateBufferRDG_FromHairBulkData(GraphBuilder, InChunk, In, BufferDesc, UploadDesc, DebugName, OwnerName))
	{
		ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, FormatType::Format);
		return Buffer;
	}
	return nullptr;
}

template<typename FormatType>
FRDGBufferRef InternalCreateStructuredBufferRDG_FromHairBulkData(FRDGBuilder& GraphBuilder, FHairBulkContainer& InChunk, uint32 InDataCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType)
{
	// Fallback for non-streamble resources (e.g. guides)
	if (InChunk.ChunkRequest == nullptr)
	{
		return InternalCreateStructuredBufferRDG_FromBulkData<FormatType>(GraphBuilder, InChunk.Data, InDataCount, Out, DebugName, OwnerName, UsageType);
	}

	const FRDGBufferRef In = Out.Buffer ? Register(GraphBuilder, Out, ERDGImportedBufferFlags::None).Buffer : nullptr;
	const FRDGBufferDesc BufferDesc = ApplyUsage(FRDGBufferDesc::CreateStructuredDesc(FormatType::SizeInByte, FMath::DivideAndRoundUp(InChunk.ChunkRequest->TotalSize, FormatType::SizeInByte)), UsageType);
	const FRDGBufferDesc UploadDesc = ApplyUsage(FRDGBufferDesc::CreateStructuredDesc(FormatType::SizeInByte, FMath::DivideAndRoundUp(InChunk.ChunkRequest->Size, FormatType::SizeInByte)), UsageType);
	if (BufferDesc.GetSize() == 0) 	{ Out.Buffer = nullptr; return nullptr; }
	if (UploadDesc.GetSize()==0) 	{ return nullptr; }
	if (FRDGBufferRef Buffer = InternalCreateBufferRDG_FromHairBulkData(GraphBuilder, InChunk, In, BufferDesc, UploadDesc, DebugName, OwnerName))
	{
		ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out);
		return Buffer;
	}
	return nullptr;
}

FRDGBufferRef InternalCreateByteAddressBufferRDG_FromHairBulkData(FRDGBuilder& GraphBuilder, FHairBulkContainer& InHairBulkContainer, FRDGExternalBuffer& OutExternal, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType, FHairTranscoding InTranscoding = FHairTranscoding())
{
	FHairStreamingRequest::FChunk* Chunk = InHairBulkContainer.ChunkRequest;
	FByteBulkData* BulkData = &InHairBulkContainer.Data;

	// Fallback for non-streamble resources (e.g. guides, cards guides, ...)
	if (Chunk == nullptr)
	{
		const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateByteAddressDesc(BulkData->GetBulkDataSize()), UsageType);
		//return InternalCreateBufferRDG_FromBulkData(GraphBuilder, LocalBulkData, Out, PF_Unknown, Desc, DebugName, OwnerName);
		InternalSetBulkDataFlags(*BulkData);

		const uint32 DataSizeInBytes = Desc.GetSize();
		check(BulkData->GetBulkDataSize() >= DataSizeInBytes);
		if (DataSizeInBytes == 0)
		{
			OutExternal.Buffer = nullptr;
			return nullptr;
		}
	
		const uint8* Data = (const uint8*)BulkData->Lock(LOCK_READ_ONLY);
		FRDGBufferRef OutBuffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::MultiFrame);
		OutBuffer->SetOwnerName(OwnerName);
		if (Data && DataSizeInBytes)
		{
			#if !WITH_EDITORONLY_DATA
			if (ReleaseAfterUse())
			{
				GraphBuilder.QueueBufferUpload(OutBuffer, Data, DataSizeInBytes, [BulkData](const void* Ptr) { BulkData->Unlock(); });
			}
			else
			#endif
			{
				GraphBuilder.QueueBufferUpload(OutBuffer, Data, DataSizeInBytes, ERDGInitialDataFlags::None);  // Copy data internally
				BulkData->Unlock();
			}
		}
		ConvertToExternalBufferWithViews(GraphBuilder, OutBuffer, OutExternal, PF_Unknown);
		return OutBuffer;
	}
	else // Streamable resources
	{
		// Size of the final buffer, after decompression if any
		const uint32 BufferTotalSizeInBytes = InTranscoding.bEnabled ? InTranscoding.UncompressedTotalSizeInBytes : Chunk->TotalSize;
		const FRDGBufferRef In = OutExternal.Buffer ? Register(GraphBuilder, OutExternal, ERDGImportedBufferFlags::None).Buffer : nullptr;
		const FRDGBufferDesc BufferDesc = ApplyUsage(FRDGBufferDesc::CreateByteAddressDesc(BufferTotalSizeInBytes), UsageType);
		const FRDGBufferDesc UploadDesc = ApplyUsage(FRDGBufferDesc::CreateByteAddressDesc(Chunk->Size), UsageType);

		if (BufferDesc.GetSize()==0) { OutExternal.Buffer = nullptr; return nullptr; }
		if (UploadDesc.GetSize()==0) { return nullptr; }
	
		const bool bDeallocate  = In != nullptr && Chunk->Status == FHairStreamingRequest::FChunk::EStatus::Unloading;
		const bool bUploadData	= Chunk->Size > 0;
	
		// If no shrink, no create, no append, no action needed
		if (!bDeallocate && !bUploadData)
		{
			return nullptr;
		}
	
		// 1. Resize buffer to fit the new required size (shrink/increase)
		// Ensure the buffer is 16 bytes aligned for resources copy during resizing
		const uint32 TotalSizeInBytes = FMath::DivideAndRoundUp(BufferDesc.NumElements * BufferDesc.BytesPerElement, 16u) * 16u;
		FRDGBufferRef OutBuffer = ResizeByteAddressBufferIfNeeded(GraphBuilder, OutExternal.Buffer, TotalSizeInBytes, DebugName);
		OutBuffer->SetOwnerName(OwnerName);
		CreateExternalBufferViews(GraphBuilder, OutBuffer, OutExternal, PF_Unknown);
	
		// 2. Append data
		if (!bDeallocate && bUploadData)
		{
			// 2.1 Upload new data
			FRDGBufferRef UploadBuffer = GraphBuilder.CreateBuffer(UploadDesc, DebugName, ERDGBufferFlags::MultiFrame);
			UploadBuffer->SetOwnerName(OwnerName);
			GraphBuilder.QueueBufferUpload(UploadBuffer, Chunk->GetData(), Chunk->Size, ERDGInitialDataFlags::None); // Copy data internally
			Chunk->Release();
	
			// 2.2 Append new data to the new/existing buffer
			AddTranscodePass(GraphBuilder, Chunk->Offset, Chunk->Size, Chunk->TotalSize, UploadBuffer, OutBuffer, InTranscoding);
		}
		return OutBuffer;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Regular data loading

static void InternalCreateIndirectBufferRDG(FRDGBuilder& GraphBuilder, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName)
{
	FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(4, 4);
	Desc.Usage |= BUF_DrawIndirect;
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::None);
	Buffer->SetOwnerName(OwnerName);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Buffer, PF_R32_UINT), 0u);
	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, PF_R32_UINT);
}

static FRDGBufferRef InternalCreateVertexBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FRDGBufferDesc& Desc,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags,
	const FName& OwnerName)
{
	checkf(EnumHasAnyFlags(Desc.Usage, EBufferUsageFlags::VertexBuffer), TEXT("CreateVertexBuffer called with an FRDGBufferDesc underlying type that is not 'VertexBuffer'. Buffer: %s"), Name);
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, Name, ERDGBufferFlags::MultiFrame);
	Buffer->SetOwnerName(OwnerName);
	if (InitialData && InitialDataSize)
	{
		GraphBuilder.QueueBufferUpload(Buffer, InitialData, InitialDataSize, InitialDataFlags);
	}
	return Buffer;
}

template<typename FormatType>
FRDGBufferRef InternalCreateVertexBufferRDG(FRDGBuilder& GraphBuilder, const typename FormatType::Type* InData, uint32 InDataCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType, ERDGInitialDataFlags InitialDataFlags)
{
	const uint32 DataSizeInBytes = FormatType::SizeInByte * InDataCount;
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateBufferDesc(FormatType::SizeInByte, InDataCount), UsageType);
	FRDGBufferRef Buffer = InternalCreateVertexBuffer(
		GraphBuilder,
		DebugName,
		Desc,
		InData,
		DataSizeInBytes,
		InitialDataFlags,
		OwnerName);

	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, FormatType::Format);
	return Buffer;
}

template<typename FormatType>
FRDGBufferRef InternalCreateVertexBufferRDG(FRDGBuilder& GraphBuilder, const TArray<typename FormatType::Type>& InData, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType, ERDGInitialDataFlags InitialDataFlags=ERDGInitialDataFlags::NoCopy)
{
	const uint32 DataCount = InData.Num();
	const uint32 DataSizeInBytes = FormatType::SizeInByte * DataCount;
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateBufferDesc(FormatType::SizeInByte, InData.Num()), UsageType);
	FRDGBufferRef Buffer = InternalCreateVertexBuffer(
		GraphBuilder,
		DebugName,
		Desc,
		InData.GetData(),
		DataSizeInBytes,
		InitialDataFlags,
		OwnerName);

	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, FormatType::Format);
	return Buffer;
}

template<typename DataType>
FRDGBufferRef InternalCreateVertexBufferRDG(FRDGBuilder& GraphBuilder, const TArray<DataType>& InData, EPixelFormat Format, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType, ERDGInitialDataFlags InitialDataFlags=ERDGInitialDataFlags::NoCopy)
{
	const uint32 DataCount = InData.Num();
	const uint32 DataSizeInBytes = sizeof(DataType) * DataCount;
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateBufferDesc(sizeof(DataType), InData.Num()), UsageType);

	FRDGBufferRef Buffer = InternalCreateVertexBuffer(
		GraphBuilder,
		DebugName,
		Desc,
		InData.GetData(),
		DataSizeInBytes,
		InitialDataFlags,
		OwnerName);

	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, Format);
	return Buffer;
}

template<typename FormatType>
FRDGBufferRef InternalCreateVertexBufferRDG(FRDGBuilder& GraphBuilder, uint32 InVertexCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType Usage)
{
	// Sanity check
	check(Usage == EHairResourceUsageType::Dynamic);

	const uint32 DataCount = InVertexCount;
	const uint32 DataSizeInBytes = FormatType::SizeInByte * DataCount;
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(FormatType::SizeInByte, InVertexCount);
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::MultiFrame);
	Buffer->SetOwnerName(OwnerName);

	if (IsFloatFormat(FormatType::Format) || IsUnormFormat(FormatType::Format) || IsSnormFormat(FormatType::Format))
	{
		AddClearUAVFloatPass(GraphBuilder, GraphBuilder.CreateUAV(Buffer, FormatType::Format), 0.0f);
	}
	else
	{
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Buffer, FormatType::Format), 0u);
	}
	
	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, FormatType::Format);
	return Buffer;
}

template<typename FormatType>
FRDGBufferRef InternalCreateStructuredBufferRDG(FRDGBuilder& GraphBuilder, uint32 DataCount, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType Usage)
{
	// Sanity check
	check(Usage == EHairResourceUsageType::Dynamic);

	const uint32 DataSizeInBytes = FormatType::SizeInByte * DataCount;
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateStructuredDesc(FormatType::SizeInByte, DataCount), Usage);
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::MultiFrame);
	Buffer->SetOwnerName(OwnerName);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Buffer, FormatType::Format), 0u);
	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, FormatType::Format);
	return Buffer;
}

template<typename DataType>
FRDGBufferRef InternalCreateStructuredBufferRDG(FRDGBuilder& GraphBuilder, const TArray<DataType>& InData, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType, ERDGInitialDataFlags InitialDataFlags)
{
	const uint32 DataCount = InData.Num();
	const uint32 DataSizeInBytes = sizeof(DataType) * DataCount;
	if (DataSizeInBytes == 0)
	{
		Out.Buffer = nullptr;
		return nullptr;
	}

	FRDGBufferRef Buffer = CreateStructuredBuffer(GraphBuilder, DebugName, sizeof(DataType), InData.Num(), InData.GetData(), sizeof(DataType) * InData.Num());
	Buffer->SetOwnerName(OwnerName);
	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, EPixelFormat::PF_Unknown);
	return Buffer;
}

template<typename DataType>
FRDGBufferRef InternalCreateByteAddressBufferRDG(FRDGBuilder& GraphBuilder, const TArray<DataType>& InData, EPixelFormat Format, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType)
{
	const uint32 DataCount = InData.Num();
	const uint32 DataSizeInBytes = sizeof(DataType) * DataCount;
	Out.Buffer = nullptr;
	if (DataSizeInBytes != 0)
	{
		const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateByteAddressDesc(DataSizeInBytes), UsageType);
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::MultiFrame);
		Buffer->SetOwnerName(OwnerName);
		if (InData.GetData() && DataSizeInBytes)
		{
			GraphBuilder.QueueBufferUpload(Buffer, InData.GetData(), DataSizeInBytes, ERDGInitialDataFlags::None);
		}
		
		ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, Format);
		return Buffer;
	}
	return nullptr;
}

FRDGBufferRef InternalCreateByteAddressBufferRDG(FRDGBuilder& GraphBuilder, uint64 DataSizeInBytes, FRDGExternalBuffer& Out, const TCHAR* DebugName, const FName& OwnerName, EHairResourceUsageType UsageType)
{
	Out.Buffer = nullptr;
	if (DataSizeInBytes != 0)
	{
		const FRDGBufferDesc Desc = ApplyUsage(FRDGBufferDesc::CreateByteAddressDesc(DataSizeInBytes), UsageType);
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, DebugName, ERDGBufferFlags::MultiFrame);
		Buffer->SetOwnerName(OwnerName);
		ConvertToExternalBufferWithViews(GraphBuilder, Buffer, Out, PF_Unknown);
		return Buffer;
	}
	return nullptr;
}
/////////////////////////////////////////////////////////////////////////////////////////

static UTexture2D* CreateCardTexture(FIntPoint Resolution)
{
	UTexture2D* Out = nullptr;

	// Pass NAME_None as name to ensure an unique name is picked, so GC dont delete the new texture when it wants to delete the old one 
	Out = NewObject<UTexture2D>(GetTransientPackage(), NAME_None /*TEXT("ProceduralFollicleMaskTexture")*/, RF_Transient);
	Out->AddToRoot();
	Out->SetPlatformData(new FTexturePlatformData());
	Out->GetPlatformData()->SizeX = Resolution.X;
	Out->GetPlatformData()->SizeY = Resolution.Y;
	Out->GetPlatformData()->PixelFormat = PF_R32_FLOAT;
	Out->SRGB = false;

	const uint32 MipCount = 1; // FMath::Min(FMath::FloorLog2(Resolution), 5u);// Don't need the full chain
	for (uint32 MipIt = 0; MipIt < MipCount; ++MipIt)
	{
		const uint32 MipResolutionX = Resolution.X >> MipIt;
		const uint32 MipResolutionY = Resolution.Y >> MipIt;
		const uint32 SizeInBytes = sizeof(float) * MipResolutionX * MipResolutionY;

		FTexture2DMipMap* MipMap = new FTexture2DMipMap(MipResolutionX, MipResolutionY);
		Out->GetPlatformData()->Mips.Add(MipMap);
		MipMap->BulkData.Lock(LOCK_READ_WRITE);
		float* MipMemory = (float*)MipMap->BulkData.Realloc(SizeInBytes);
		for (uint32 Y = 0; Y < MipResolutionY; Y++)
			for (uint32 X = 0; X < MipResolutionX; X++)
			{
				MipMemory[X + Y * MipResolutionY] = X / float(MipResolutionX);
			}
		//FMemory::Memzero(MipMemory, SizeInBytes);
		MipMap->BulkData.Unlock();
	}
	Out->UpdateResource();

	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////
void CreateHairStrandsDebugAttributeBuffer(FRDGBuilder& GraphBuilder, FRDGExternalBuffer* DebugAttributeBuffer, uint32 SizeInBytes, const FName& OwnerName)
{
	if (SizeInBytes == 0 || !DebugAttributeBuffer)
	{
		return;
	}

	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(SizeInBytes), TEXT("Hair.Strands_DebugAttributeBuffer"), ERDGBufferFlags::MultiFrame);
	Buffer->SetOwnerName(OwnerName);
	ConvertToExternalBufferWithViews(GraphBuilder, Buffer, *DebugAttributeBuffer, PF_R32_UINT);
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairCommonResource::FHairCommonResource(EHairStrandsAllocationType InAllocationType, const FHairResourceName& InResourceName, const FName& InOwnerName, bool bInUseRenderGraph):
bUseRenderGraph(bInUseRenderGraph),
bIsInitialized(false),
AllocationType(InAllocationType),
ResourceName(InResourceName),
OwnerName(InOwnerName)
{
}

void FHairCommonResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	// 1. If the resource is Updated (ReleaseRHI, IniRHI), reset its loaded data counter
	if (!bIsInitialized)
	{
		InternalResetLoadedSize();
	}

	// 2. If the resouce creation is deferred, skip the initialization
	if (bIsInitialized || AllocationType == EHairStrandsAllocationType::Deferred || GUsingNullRHI) 
	{ 
		return; 
	}

	// 3. Loaded the data
	const bool bReady = InternalGetOrRequestData(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP);
	check(bReady);

	// 4. Allocate the resource, and update the data
	if (bUseRenderGraph)
	{
		FRDGBuilder GraphBuilder(RHICmdList.GetAsImmediate());
		InternalAllocate(GraphBuilder, HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP);
		GraphBuilder.Execute();
	}
	else
	{
		InternalAllocate();
	}
	bIsInitialized = true;
}

void FHairCommonResource::ReleaseRHI()
{
	InternalRelease();
	bIsInitialized = false;
}

bool FHairCommonResource::InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount) 
{ 
	// Initialize the streaming request so that resource which actually don't emit streaming request get correct #curve/#point/LODIndx
	// request when allocating its resources
	StreamingRequest.CurveCount = InRequestedCurveCount; 
	StreamingRequest.PointCount = InRequestedPointCount;  
	return true; 
}

void FHairCommonResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType)
{
	EHairResourceStatus Status;
	Status.Status = EHairResourceStatus::EStatus::None;
	Status.AvailableCurveCount = HAIR_MAX_NUM_CURVE_PER_GROUP;
	Allocate(GraphBuilder, LoadingType, Status, HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP);
}

void FHairCommonResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status)
{
	Allocate(GraphBuilder, LoadingType, Status, HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP);
}

bool NeedDeallocation(uint32 InRequest, uint32 InAvailable);
void FHairCommonResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, uint32 InRequestedCurveCount, uint32 InRequestedPointCount, bool bAllowDeallocation)
{
	check(AllocationType == EHairStrandsAllocationType::Deferred);

	// Check if there is already a request in flight	

	if (LoadingType == EHairResourceLoadingType::Sync)
	{
		if (!bIsInitialized)
		{
			FRenderResource::InitResource(GraphBuilder.RHICmdList); // Call RenderResource InitResource() so that the resources is marked as initialized
		}
		if (!bIsInitialized)
		{
			InternalAllocate(GraphBuilder, InRequestedCurveCount, InRequestedPointCount);
		}
		bIsInitialized = true;

		// Update the max curve count available
		MaxAvailableCurveCount = InRequestedCurveCount;

		Status |= EHairResourceStatus::EStatus::Valid;
	}
	else if (LoadingType == EHairResourceLoadingType::Async)
	{
		// 1. If all requested curve are already loaded, nothing to do
		if (bIsInitialized && MaxAvailableCurveCount >= InRequestedCurveCount) 
		{ 
			// Trim/Un-stream data if needed (ensure no streaming request is in-flight)
			if (bAllowDeallocation && NeedDeallocation(InRequestedCurveCount, MaxAvailableCurveCount) && StreamingRequest.IsNone())
			{
				StreamingRequest.CurveCount = InRequestedCurveCount;
				StreamingRequest.PointCount = InRequestedPointCount;
				if (FHairStrandsBulkCommon* Bulk = InternalGetBulkData())
				{
					Bulk->Unload(&StreamingRequest);
				}
				InternalAllocate(GraphBuilder, InRequestedCurveCount, InRequestedPointCount); // Do we need to pass something else for the deallocation

				// Update the max curve count available
				MaxAvailableCurveCount = StreamingRequest.CurveCount;

				// Reset streaming request. When the request is delete, the DDC request becomes cancelled. 
				StreamingRequest = FHairStreamingRequest();
			}

			Status |= EHairResourceStatus::EStatus::Valid; 
		}
		// 2. If more curves are requested, issue a streaming request
		else if (InternalGetOrRequestData(InRequestedCurveCount, InRequestedPointCount))
		{ 
			// 2.1 Curve data are availble, and update GPU resources
			if (!bIsInitialized)
			{
				FRenderResource::InitResource(GraphBuilder.RHICmdList); // Call RenderResource InitResource() so that the resources is marked as initialized
			}
			InternalAllocate(GraphBuilder, StreamingRequest.CurveCount, StreamingRequest.PointCount);
			bIsInitialized = true;

			// Update the max curve count available
			MaxAvailableCurveCount = StreamingRequest.CurveCount;

			// Reset streaming request. When the request is delete, the DDC request becomes cancelled. 
			StreamingRequest = FHairStreamingRequest();

			Status |= EHairResourceStatus::EStatus::Valid;
		}
		else
		{
			// 2.2 Curve data are not availble yet, postpone (new) resources creation
			Status |= EHairResourceStatus::EStatus::Loading; 
		}
	}

	Status.AddAvailableCurve(MaxAvailableCurveCount);
}

void FHairCommonResource::StreamInData()
{
	if (!bIsInitialized)
	{
		// TODO
		// InternalGetOrRequestData(HAIR_MAX_NUM_CURVE_PER_GROUP, HAIR_MAX_NUM_POINT_PER_GROUP);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairCardIndexBuffer::FHairCardIndexBuffer(const TArray<FHairCardsIndexFormat::Type>& InIndices, const FName& InOwnerName)
	:Indices(InIndices) 
{ 
	SetOwnerName(InOwnerName); 
}

void FHairCardIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FResourceArrayUploadArrayView UploadView(MakeConstArrayView(Indices));

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateIndex<FHairCardsIndexFormat::Type>(TEXT("FHairCardIndexBuffer"), Indices.Num())
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetOwnerName(GetOwnerName())
		.SetInitActionResourceArray(&UploadView)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVCompute);

	IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
}

FHairCardsRestResource::FHairCardsRestResource(const FHairCardsBulkData& InBulkData, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Immediate, InResourceName, InOwnerName, false),
	RestPositionBuffer(),
	RestIndexBuffer(InBulkData.Indices, InOwnerName),
	NormalsBuffer(),
	UVsBuffer(),
	MaterialsBuffer(),
	VertexColorsBuffer(),
	BulkData(InBulkData)
{

}

void FHairCardsRestResource::InternalAllocate()
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	const ERHIAccess InitialAccess = ERHIAccess::SRVMask;

	// These resources are kept as regular (i.e., non-RDG resources) as they need to be bound at the input assembly stage by the Vertex declaraction which requires FVertexBuffer type
	CreateBuffer<FHairCardsPositionFormat>(RHICmdList, BulkData.Positions, RestPositionBuffer, ToHairResourceDebugName(TEXT("Hair.CardsRest_PositionBuffer"), ResourceName), OwnerName, InitialAccess);
	CreateBuffer<FHairCardsNormalFormat>(RHICmdList, BulkData.Normals, NormalsBuffer, ToHairResourceDebugName(TEXT("Hair.CardsRest_NormalBuffer"), ResourceName), OwnerName, InitialAccess);
	CreateBuffer<FHairCardsUVFormat>(RHICmdList, BulkData.UVs, UVsBuffer, ToHairResourceDebugName(TEXT("Hair.CardsRest_UVBuffer"), ResourceName), OwnerName, InitialAccess);
	CreateBuffer<FHairCardsMaterialFormat>(RHICmdList, BulkData.Materials, MaterialsBuffer, ToHairResourceDebugName(TEXT("Hair.CardsRest_MaterialBuffer"), ResourceName), OwnerName, InitialAccess);
	if (BulkData.Header.bVertexColor)
	{
		CreateBuffer<FHairCardsMaterialFormat>(RHICmdList, BulkData.VertexColors, VertexColorsBuffer, ToHairResourceDebugName(TEXT("Hair.CardsRest_VertexColorBuffer"), ResourceName), OwnerName, InitialAccess);
	}
}

void FHairCardsRestResource::InternalRelease()
{
}

void FHairCardsRestResource::InitResource(FRHICommandListBase& RHICmdList)
{
	FRenderResource::InitResource(RHICmdList);
	RestIndexBuffer.InitResource(RHICmdList);
	RestPositionBuffer.InitResource(RHICmdList);
	NormalsBuffer.InitResource(RHICmdList);
	UVsBuffer.InitResource(RHICmdList);
	MaterialsBuffer.InitResource(RHICmdList);
	if (BulkData.Header.bVertexColor)
	{
		VertexColorsBuffer.InitResource(RHICmdList);
	}
}

void FHairCardsRestResource::ReleaseResource()
{
	FRenderResource::ReleaseResource();
	RestIndexBuffer.ReleaseResource();
	RestPositionBuffer.ReleaseResource();
	NormalsBuffer.ReleaseResource();
	UVsBuffer.ReleaseResource();
	MaterialsBuffer.ReleaseResource();
	VertexColorsBuffer.ReleaseResource();
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairCardsDeformedResource::FHairCardsDeformedResource(const FHairCardsBulkData& InBulkData, bool bInInitializedData, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	BulkData(InBulkData), bInitializedData(bInInitializedData)
{}

void FHairCardsDeformedResource::InternalAllocate(FRDGBuilder& GraphBuilder)
{
	if (bInitializedData)
	{
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.Positions, DeformedPositionBuffer[0], ToHairResourceDebugName(TEXT("Hair.CardsDeformedPosition(Current)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.Positions, DeformedPositionBuffer[1], ToHairResourceDebugName(TEXT("Hair.CardsDeformedPosition(Previous)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);

		InternalCreateVertexBufferRDG<FHairCardsNormalFormat>(GraphBuilder, BulkData.Normals, DeformedNormalBuffer, ToHairResourceDebugName(TEXT("Hair.CardsDeformedNormal"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}
	else
	{
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.GetNumVertices(), DeformedPositionBuffer[0], ToHairResourceDebugName(TEXT("Hair.CardsDeformedPosition(Current)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.GetNumVertices(), DeformedPositionBuffer[1], ToHairResourceDebugName(TEXT("Hair.CardsDeformedPosition(Previous)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);

		InternalCreateVertexBufferRDG<FHairCardsNormalFormat>(GraphBuilder, BulkData.GetNumVertices() * FHairCardsNormalFormat::ComponentCount, DeformedNormalBuffer, ToHairResourceDebugName(TEXT("Hair.CardsDeformedNormal"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}

	// Transition to SRV in case the deformation pass does not run in this graph (e.g. rendered only in shadows or captures).
	GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, DeformedPositionBuffer[0], ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask, ERHIPipeline::All);
	GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, DeformedPositionBuffer[1], ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);

	FRDGImportedBuffer CardsDeformedNormalRDGBuffer = Register(GraphBuilder, DeformedNormalBuffer, ERDGImportedBufferFlags::CreateSRV);
	GraphBuilder.UseExternalAccessMode(CardsDeformedNormalRDGBuffer.Buffer, ERHIAccess::SRVMask);
}

void FHairCardsDeformedResource::InternalRelease()
{
	DeformedPositionBuffer[0].Release();
	DeformedPositionBuffer[1].Release();
	DeformedNormalBuffer.Release();
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairMeshesRestResource::FHairMeshesRestResource(const FHairMeshesBulkData& InBulkData, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Immediate, InResourceName, InOwnerName, false),
	RestPositionBuffer(),
	IndexBuffer(InBulkData.Indices, InOwnerName),
	NormalsBuffer(),
	UVsBuffer(),
	VertexColorsBuffer(),
	BulkData(InBulkData)
{
	check(BulkData.GetNumVertices() > 0);
	check(IndexBuffer.Indices.Num() > 0);
}

void FHairMeshesRestResource::InternalAllocate()
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	const ERHIAccess InitialAccess = ERHIAccess::SRVMask;

	// These resources are kept as regular (i.e., non-RDG resources) as they need to be bound at the input assembly stage by the Vertex declaraction which requires FVertexBuffer type
	CreateBuffer<FHairCardsPositionFormat>(RHICmdList, BulkData.Positions, RestPositionBuffer, ToHairResourceDebugName(TEXT("Hair.MeshesRest_Positions"), ResourceName), OwnerName, InitialAccess);
	CreateBuffer<FHairCardsNormalFormat>(RHICmdList, BulkData.Normals, NormalsBuffer, ToHairResourceDebugName(TEXT("Hair.MeshesRest_Normals"), ResourceName), OwnerName, InitialAccess);
	CreateBuffer<FHairCardsUVFormat>(RHICmdList, BulkData.UVs, UVsBuffer, ToHairResourceDebugName(TEXT("Hair.MeshesRest_UVs"), ResourceName), OwnerName, InitialAccess);
	if (BulkData.Header.bVertexColor)
	{
		CreateBuffer<FHairCardsMaterialFormat>(RHICmdList, BulkData.VertexColors, VertexColorsBuffer, ToHairResourceDebugName(TEXT("Hair.MeshesRest_VertexColors"), ResourceName), OwnerName, InitialAccess);
	}
}

void FHairMeshesRestResource::InternalRelease()
{

}

void FHairMeshesRestResource::InitResource(FRHICommandListBase& RHICmdList)
{
	FRenderResource::InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
	RestPositionBuffer.InitResource(RHICmdList);
	NormalsBuffer.InitResource(RHICmdList);
	UVsBuffer.InitResource(RHICmdList);
	if (BulkData.Header.bVertexColor)
	{
		VertexColorsBuffer.InitResource(RHICmdList);
	}
}

void FHairMeshesRestResource::ReleaseResource()
{
	FRenderResource::ReleaseResource();
	IndexBuffer.ReleaseResource();
	RestPositionBuffer.ReleaseResource();
	NormalsBuffer.ReleaseResource();
	UVsBuffer.ReleaseResource();
	VertexColorsBuffer.ReleaseResource();
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairMeshesDeformedResource::FHairMeshesDeformedResource(const FHairMeshesBulkData& InBulkData, bool bInInitializedData, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	BulkData(InBulkData), bInitializedData(bInInitializedData)
{}

void FHairMeshesDeformedResource::InternalAllocate(FRDGBuilder& GraphBuilder)
{
	if (bInitializedData)
	{
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.Positions, DeformedPositionBuffer[0], ToHairResourceDebugName(TEXT("Hair.MeshesDeformed(Current)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.Positions, DeformedPositionBuffer[1], ToHairResourceDebugName(TEXT("Hair.MeshesDeformed(Previous)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}
	else
	{
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.GetNumVertices(), DeformedPositionBuffer[0], ToHairResourceDebugName(TEXT("Hair.MeshesDeformed(Current)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		InternalCreateVertexBufferRDG<FHairCardsPositionFormat>(GraphBuilder, BulkData.GetNumVertices(), DeformedPositionBuffer[1], ToHairResourceDebugName(TEXT("Hair.MeshesDeformed(Previous)"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}

	// Transition to SRV in case the mesh RBF deformation pass does not run in this graph (e.g. lazy allocation in a capture render).
	GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, DeformedPositionBuffer[0], ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask, ERHIPipeline::All);
	GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, DeformedPositionBuffer[1], ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
}

void FHairMeshesDeformedResource::InternalRelease()
{
	DeformedPositionBuffer[0].Release();
	DeformedPositionBuffer[1].Release();
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsRestResource::FHairStrandsRestResource(FHairStrandsBulkData& InBulkData, EHairStrandsResourcesType InCurveType, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	PositionBuffer(), PointAttributeBuffer(), CurveAttributeBuffer(), CurveMappingBuffer(), PointMappingBuffer(), PointToCurveBuffer(), BulkData(InBulkData), CurveType(InCurveType)
{
	MaxAvailableCurveCount = 0;

	// Sanity check
	check(!!(BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasData));
}

bool FHairStrandsRestResource::InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount)
{
	if (StreamingRequest.IsNone())
	{
#if !WITH_EDITORONLY_DATA
		if (BulkData.IsCookedBulkDataLoadedForCPUAccess())
		{
			// Data is already in memory
			StreamingRequest.CurveCount = InRequestedCurveCount; 
			StreamingRequest.PointCount = InRequestedPointCount;
			return true;
		}
		else
#endif
		{
			StreamingRequest.Request(InRequestedCurveCount, InRequestedPointCount, BulkData, false, false, false, OwnerName);
		}
	}
	return StreamingRequest.IsCompleted();
}

void FHairStrandsRestResource::InternalAllocate(FRDGBuilder& GraphBuilder)
{
	// If we enter this function, the request need to be completed
	check(StreamingRequest.IsCompleted());

	const uint32 PointCount = BulkData.GetNumPoints();
	const uint32 CurveCount = BulkData.GetNumCurves();

	// 1. Lock data, which force the loading data from files (on non-editor build/cooked data). These data are then uploaded to the GPU
	// 2. A local copy is done by the buffer uploader. This copy is discarded once the uploading is done.
	const bool bTranscodedPosition = !!(BulkData.Header.Flags & FHairStrandsBulkData::EDataFlags::DataFlags_HasTranscodedPosition);
	if (bTranscodedPosition)
	{
		FHairTranscoding PositionTranscoding;
		PositionTranscoding.bEnabled						= true;
		PositionTranscoding.PositionOffset 					= BulkData.Header.Transcoding.PositionOffset;
		PositionTranscoding.PositionScale 					= BulkData.Header.Transcoding.PositionScale;
		PositionTranscoding.UncompressedOffsetInBytes 		= 0;
		PositionTranscoding.UncompressedTotalSizeInBytes 	= 0;

		if (BulkData.Data.TranscodedPositions.ChunkRequest)
		{
			// Compressed data stored into chunk data
			const uint32 CompressedDataOffsetBytes = BulkData.Data.TranscodedPositions.ChunkRequest->Offset;
			const uint32 CompressedDataSizeInBytes = BulkData.Data.TranscodedPositions.ChunkRequest->TotalSize;

			// Number of compressed chunks
			const uint32 CompressedChunkCount_Offset = (CompressedDataOffsetBytes / FCompressedHairPositionsStrideInBytes);
			const uint32 CompressedChunkCount_Total  = (CompressedDataSizeInBytes / FCompressedHairPositionsStrideInBytes);

			// Uncompressed size/offset, i.e., size and offset of the final data (faster transcoding)
			PositionTranscoding.UncompressedOffsetInBytes 		= CompressedChunkCount_Offset * HAIR_POINT_COUNT_PER_COMPRESSED_POSITION_CHUNK * FPackedHairPositionStrideInBytes;
			PositionTranscoding.UncompressedTotalSizeInBytes 	= CompressedChunkCount_Total  * HAIR_POINT_COUNT_PER_COMPRESSED_POSITION_CHUNK * FPackedHairPositionStrideInBytes;
		}

		InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.TranscodedPositions, PositionBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PositionBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static, PositionTranscoding);
	}
	else
	{
		InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.Positions, PositionBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PositionBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
	}
	InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.CurveAttributes, CurveAttributeBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_CurveAttributeBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
	if (!!(BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasPointAttribute))
	{
		InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.PointAttributes, PointAttributeBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PointAttributeBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
	}
	else if (PointAttributeBuffer.Buffer == nullptr)
	{
		TArray<uint32> DummyAttribute;
		DummyAttribute.Add(0u);
		InternalCreateByteAddressBufferRDG(GraphBuilder, DummyAttribute, EPixelFormat::PF_R32_UINT, PointAttributeBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PointAttributeBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
		GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, PointAttributeBuffer, ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
	}
	if (!!(BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasPointMapping))
	{
		InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.PointMapping, PointMappingBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PointMappingBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
	}
	else
	{
		TArray<uint32> DummyAttribute;
		DummyAttribute.Add(0u);
		InternalCreateByteAddressBufferRDG(GraphBuilder, DummyAttribute, EPixelFormat::PF_R32_UINT, PointMappingBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PointMappingBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
		GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, PointMappingBuffer, ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
	}

	if (!!(BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasCurveMapping))
	{
		InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.CurveMapping, CurveMappingBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_CurveMappingBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
	}
	else if(CurveMappingBuffer.Buffer == nullptr)
	{
		TArray<uint32> DummyAttribute;
		DummyAttribute.Add(0u);
		InternalCreateByteAddressBufferRDG(GraphBuilder, DummyAttribute, EPixelFormat::PF_R32_UINT, CurveMappingBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_CurveMappingBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
		GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, CurveMappingBuffer, ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
	}
	
	InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.Curves, CurveBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_CurveBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);

	InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.PointToCurve, PointToCurveBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PointToCurveBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);

	// If the buffer is not null, it means it was been lazily allocated once. In such a case, The tangent buffer is recreated the new required size
	const bool bStaticTangentNeeded = TangentBuffer.Buffer != nullptr;
	if (bStaticTangentNeeded)
	{
		InternalCreateVertexBufferRDG<FHairStrandsTangentFormat>(GraphBuilder, PointCount * FHairStrandsTangentFormat::ComponentCount, TangentBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsRest_TangentBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		CachedTangentPointCount = 0; // Force recaching
	}

	if (PositionOffsetBuffer.Buffer == nullptr)
	{	
		TArray<FVector4f> RestOffset;
		RestOffset.Add((FVector3f)BulkData.GetPositionOffset());// LWC_TODO: precision loss
		InternalCreateStructuredBufferRDG<FHairStrandsPositionOffsetFormat::Type>(GraphBuilder, RestOffset, PositionOffsetBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRest_PositionOffsetBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static, ERDGInitialDataFlags::None);
		GraphBuilder.UseExternalAccessMode(Register(GraphBuilder, PositionOffsetBuffer, ERDGImportedBufferFlags::CreateSRV).Buffer, ERHIAccess::SRVMask);
	}
}

void AddHairTangentPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 PointCount,
	FHairGroupPublicData* HairGroupPublicData,
	FRDGBufferSRVRef PositionBuffer,
	FRDGBufferUAVRef OutTangentBuffer);

FRDGExternalBuffer FHairStrandsRestResource::GetTangentBuffer(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, uint32 ActivePointCount, uint32 ActiveCurveCount)
{
	const uint32 AllocatedPoint = TangentBuffer.Buffer ? TangentBuffer.Buffer->Desc.NumElements / FHairStrandsTangentFormat::ComponentCount : 0u;
	const uint32 RequestedPoint = ActivePointCount;

	// Lazy allocation
	const bool bReallocateTangentBuffer = TangentBuffer.Buffer == nullptr || ActivePointCount > AllocatedPoint;
	if (bReallocateTangentBuffer)
	{
		InternalCreateVertexBufferRDG<FHairStrandsTangentFormat>(GraphBuilder, ActivePointCount * FHairStrandsTangentFormat::ComponentCount, TangentBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsRest_TangentBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}

	// Lazy update tangent
	const bool bRecomputeTangent = bReallocateTangentBuffer || ((AllocatedPoint >= RequestedPoint) && (ActivePointCount > CachedTangentPointCount));
	if (bRecomputeTangent)
	{
		AddHairTangentPass(
			GraphBuilder,
			ShaderMap,
			ActivePointCount,
			nullptr,
			RegisterAsSRV(GraphBuilder, PositionBuffer),
			RegisterAsUAV(GraphBuilder, TangentBuffer));

		CachedTangentPointCount = ActivePointCount;
	}

	return TangentBuffer;
}

void FHairStrandsRestResource::InternalRelease()
{
	PositionBuffer.Release();
	PositionOffsetBuffer.Release();
	CurveAttributeBuffer.Release();
	PointAttributeBuffer.Release();
	PointToCurveBuffer.Release();
	TangentBuffer.Release();
	CurveBuffer.Release();
	CurveMappingBuffer.Release();
	PointMappingBuffer.Release();
	MaxAvailableCurveCount = 0;
	CachedTangentPointCount = 0;
	
	for (TPair<FString, FRDGExternalBuffer>& ExternalBuffer : ExternalBuffers)
	{
		ExternalBuffer.Value.Release();
	}
}

void FHairStrandsRestResource::InternalResetLoadedSize()
{
	BulkData.ResetLoadedSize();
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsDeformedResource::FHairStrandsDeformedResource(FHairStrandsBulkData& InBulkData, EHairStrandsResourcesType InCurveType, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	BulkData(InBulkData), CurveType(InCurveType)
{
	GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current)  = BulkData.GetPositionOffset();
	GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Previous) = BulkData.GetPositionOffset();
}

void FHairStrandsDeformedResource::InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount)
{
	// Need to update these to by dynamic based on the active number of point
	const uint32 PointCount = FMath::Min(InPointCount, BulkData.GetNumPoints());

	FRDGBufferRef Prev0 = Register(GraphBuilder, DeformedPositionBuffer[0], ERDGImportedBufferFlags::None).Buffer;
	FRDGBufferRef Prev1 = Register(GraphBuilder, DeformedPositionBuffer[1], ERDGImportedBufferFlags::None).Buffer;

	FRDGBufferRef Deformed0 = InternalCreateByteAddressBufferRDG(GraphBuilder, PointCount * BulkData.Header.Strides.PositionStride, DeformedPositionBuffer[0], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_DeformedPositionBuffer0), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	FRDGBufferRef Deformed1 = InternalCreateByteAddressBufferRDG(GraphBuilder, PointCount * BulkData.Header.Strides.PositionStride, DeformedPositionBuffer[1], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_DeformedPositionBuffer1), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	InternalCreateVertexBufferRDG<FHairStrandsTangentFormat>(GraphBuilder,  PointCount * FHairStrandsTangentFormat::ComponentCount, TangentBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_TangentBuffer), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);

	if (Prev0 != nullptr)
	{
		const uint32 SizeInBytes = FMath::Min(Prev0->Desc.GetSize(), Deformed0->Desc.GetSize());
		AddCopyBufferPass(GraphBuilder, Deformed0, 0, Prev0, 0, SizeInBytes);
		AddCopyBufferPass(GraphBuilder, Deformed1, 0, Prev1, 0, SizeInBytes);
	}
	 
	if (DeformedOffsetBuffer[0].Buffer == nullptr)
	{
		TArray<FVector4f> DefaultOffsets;
		DefaultOffsets.Add((FVector3f)BulkData.GetPositionOffset()); // LWC_TODO: precision loss
		InternalCreateStructuredBufferRDG<FHairStrandsPositionOffsetFormat::Type>(GraphBuilder, DefaultOffsets, DeformedOffsetBuffer[0], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_DeformedOffsetBuffer0), ResourceName), OwnerName, EHairResourceUsageType::Dynamic, ERDGInitialDataFlags::None);
		InternalCreateStructuredBufferRDG<FHairStrandsPositionOffsetFormat::Type>(GraphBuilder, DefaultOffsets, DeformedOffsetBuffer[1], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_DeformedOffsetBuffer1), ResourceName), OwnerName, EHairResourceUsageType::Dynamic, ERDGInitialDataFlags::None);
	}
	// Note: DeformerBuffer is optionally/lazily allocated by a mesh-deformer graph
}

FRDGExternalBuffer& FHairStrandsDeformedResource::GetDeformerBuffer(FRDGBuilder& GraphBuilder)
{
	// Lazy allocation and update
	if (DeformerBuffer.Buffer == nullptr)
	{
		InternalCreateByteAddressBufferRDG(GraphBuilder, BulkData.Header.PointCount * BulkData.Header.Strides.PositionStride, DeformerBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_DeformerBuffer), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}
	return DeformerBuffer;
}

FRDGExternalBuffer& FHairStrandsDeformedResource::GetDeformerCurveAttributeBuffer(FRDGBuilder& GraphBuilder)
{
	// Deformer curve attributes
	if (DeformerCurveAttributeBuffer.Buffer == nullptr)
	{
		const uint32 AllocSize = BulkData.GetCurveAttributeSizeInBytes();
		check(AllocSize > 0);
		InternalCreateByteAddressBufferRDG(GraphBuilder, AllocSize, DeformerCurveAttributeBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformed_DeformerCurveAttributeBuffer), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}
	return DeformerCurveAttributeBuffer;
}

FRDGExternalBuffer& FHairStrandsDeformedResource::GetDeformerPointAttributeBuffer(FRDGBuilder& GraphBuilder)
{
	// Deformer point attributes
	if (DeformerPointAttributeBuffer.Buffer == nullptr && (BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasPointAttribute) && BulkData.Data.PointAttributes.Data.GetBulkDataSize() > 0)
	{
		const uint32 AllocSize = BulkData.GetPointAttributeSizeInBytes();
		check(AllocSize > 0);
		InternalCreateByteAddressBufferRDG(GraphBuilder, AllocSize, DeformerPointAttributeBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsDeformedt_DeformerPointAttributeBuffer), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
	}
	return DeformerPointAttributeBuffer;
}

void FHairStrandsDeformedResource::InternalRelease()
{
	DeformedPositionBuffer[0].Release();
	DeformedPositionBuffer[1].Release();
	TangentBuffer.Release();
	DeformerBuffer.Release();
	DeformerPointAttributeBuffer.Release();
	DeformerCurveAttributeBuffer.Release();

	DeformedOffsetBuffer[0].Release();
	DeformedOffsetBuffer[1].Release();

	for (TPair<FString, FRDGExternalBuffer>& ExternalBuffer : ExternalBuffers)
	{
		ExternalBuffer.Value.Release();
	}
}

/////////////////////////////////////////////////////////////////////////////////////////
// Cluster culling resources
FHairStrandsClusterResource::FHairStrandsClusterResource(FHairStrandsClusterBulkData& InBulkData, const FHairResourceName& InResourceName, const FName& InOwnerName):
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	BulkData(InBulkData)
{
	MaxAvailableCurveCount = 0;
}

bool FHairStrandsClusterResource::InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount)
{
	if (StreamingRequest.IsNone())
	{
#if !WITH_EDITORONLY_DATA
		if (BulkData.IsCookedBulkDataLoadedForCPUAccess())
		{
			// Data is already in memory
			StreamingRequest.CurveCount = InRequestedCurveCount;
			StreamingRequest.PointCount = InRequestedPointCount;
			return true;
		}
		else
#endif
		{
			StreamingRequest.Request(InRequestedCurveCount, InRequestedPointCount, BulkData, false, false, false, OwnerName);
		}
	}
	return StreamingRequest.IsCompleted();
}

void FHairStrandsClusterResource::InternalAllocate(FRDGBuilder& GraphBuilder)
{
	check(StreamingRequest.IsCompleted());

	const uint32 EntryCount = FMath::DivideAndRoundUp(BulkData.Header.PointCount, HAIR_POINT_LOD_COUNT_PER_UINT);

	// Allocated once with all the cluster data (i.e., no streaming)
	if (ClusterInfoBuffer.Buffer == nullptr)
	{
		InternalCreateStructuredBufferRDG_FromHairBulkData<FHairClusterInfoFormat>(GraphBuilder, BulkData.Data.PackedClusterInfos, BulkData.Header.ClusterCount, ClusterInfoBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsCluster_ClusterInfoBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Static);
	}
	InternalCreateVertexBufferRDG_FromHairBulkData<FHairClusterIndexFormat>(GraphBuilder, BulkData.Data.CurveToClusterIds, BulkData.Header.CurveCount, CurveToClusterIdBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsCluster_CurveToClusterIds"), ResourceName), OwnerName, EHairResourceUsageType::Static);	
	InternalCreateVertexBufferRDG_FromHairBulkData<FHairClusterIndexFormat>(GraphBuilder, BulkData.Data.PointLODs, EntryCount, PointLODBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsCluster_PointLOD"), ResourceName), OwnerName, EHairResourceUsageType::Static);
}

void FHairStrandsClusterResource::InternalRelease()
{
	ClusterInfoBuffer.Release();
	CurveToClusterIdBuffer.Release();
	PointLODBuffer.Release();
	MaxAvailableCurveCount = 0;
}

void FHairStrandsClusterResource::InternalResetLoadedSize()
{
	BulkData.ResetLoadedSize();
}


/////////////////////////////////////////////////////////////////////////////////////////
// Cluster culling resources

struct FHairStrandsSintFormat
{
	typedef int32 Type;
	typedef int32 BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_SINT;
};

struct FHairStrandsUintFormat
{
	typedef uint32 Type;
	typedef uint32 BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsFloatFormat
{
	typedef float Type;
	typedef float BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_FLOAT;
};

FHairStrandsCullingResource::FHairStrandsCullingResource(uint32 InPointCount, uint32 InCurveCount, uint32 InClusterCount, const FHairResourceName& InResourceName, const FName& InOwnerName):
FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName)
{
	MaxPointCount = InPointCount;
	MaxCurveCount = InCurveCount;
	ClusterCount = InClusterCount;
	MaxAvailableCurveCount = 0;
}

void FHairStrandsCullingResource::InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount)
{
	if (ClusterCount > 0)
	{	
		const uint32 PointCount = FMath::Min(InPointCount, MaxPointCount);
		const uint32 CurveCount = FMath::Min(InCurveCount, MaxCurveCount);

		InternalCreateByteAddressBufferRDG(GraphBuilder, CurveCount * FHairStrandsUintFormat::SizeInByte, Resources.CulledCurveBuffer, TEXT("Hair.Cluster_CulledCurveBuffer"), GetOwnerName(), EHairResourceUsageType::Dynamic);
		InternalCreateVertexBufferRDG<FHairStrandsUintFormat>(GraphBuilder, PointCount, Resources.CulledVertexIdBuffer, TEXT("Hair.Cluster_CulledVertexIdBuffer"), GetOwnerName(), EHairResourceUsageType::Dynamic);
	
		// Only allocated once, as cluster count does not change
		if (Resources.DrawIndirectBuffer.Buffer == nullptr)
		{
			InternalCreateIndirectBufferRDG(GraphBuilder, Resources.DrawIndirectBuffer, TEXT("Hair.Cluster_DrawIndirectBuffer"), GetOwnerName());
			InternalCreateIndirectBufferRDG(GraphBuilder, Resources.DrawIndirectRasterComputeBuffer, TEXT("Hair.Cluster_DrawIndirectRasterComputeBuffer"), GetOwnerName());
			GraphBuilder.SetBufferAccessFinal(Register(GraphBuilder, Resources.DrawIndirectBuffer, ERDGImportedBufferFlags::None).Buffer, ERHIAccess::IndirectArgs);
		}
	}
}

void FHairStrandsCullingResource::InternalRelease()
{
	Resources.DrawIndirectBuffer.Release();
	Resources.DrawIndirectRasterComputeBuffer.Release();
	Resources.CulledCurveBuffer.Release();
	Resources.CulledVertexIdBuffer.Release();
	MaxAvailableCurveCount = 0;
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsLODRestRootResource::FHairStrandsLODRestRootResource(TArray<FHairStrandsRootBulkData>& InBulkDataArray, int32 InBulkDataIndex, EHairStrandsResourcesType InCurveType, uint32 InMeshLODIndex, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	BulkDataArray(InBulkDataArray), BulkDataIndex(InBulkDataIndex), CurveType(InCurveType), MeshLODIndex(InMeshLODIndex)
{
	MaxAvailableCurveCount = 0;

	Reset();
}

void FHairStrandsLODRestRootResource::Reset()
{
	check(MeshLODIndex == GetBulkData().Header.LODIndex);
	Status 		= EStatus::Invalid;
	SampleCount = GetBulkData().Header.SampleCount;
}

bool FHairStrandsLODRestRootResource::InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount)
{
	if (StreamingRequest.IsNone())
	{
		FHairStrandsRootBulkData& BulkData = GetBulkData();
#if !WITH_EDITORONLY_DATA
		if (BulkData.IsCookedBulkDataLoadedForCPUAccess())
		{
			// Data is already in memory
			StreamingRequest.CurveCount = InRequestedCurveCount;
			StreamingRequest.PointCount = InRequestedPointCount;
			return true;
		}
		else
#endif
		{
			StreamingRequest.Request(InRequestedCurveCount, InRequestedPointCount, BulkData, false, false, false, OwnerName);
		}
	}
	return StreamingRequest.IsCompleted();
}

void FHairStrandsLODRestRootResource::InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount)
{
	// Once empty, the MeshProjectionLODsneeds to be repopulate as it might be re-initialized. 
	// E.g., when a resource is updated, it is first released, then re-init. 
	if (!bIsInitialized)
	{
		Reset();
	}
	FHairStrandsRootBulkData& BulkData = GetBulkData();
	check(BulkData.Header.PointCount > 0);
	{
		// Clamp to the max supported
		InCurveCount = FMath::Min(InCurveCount, BulkData.Header.RootCount);

		const bool bIsLODInitialized = (Status == EStatus::Completed || Status == EStatus::Initialized) && InCurveCount <= AvailableCurveCount;
		if (bIsLODInitialized)
		{
			return;
		}

		{
			InternalCreateVertexBufferRDG_FromHairBulkData<FHairStrandsRootBarycentricFormat>(GraphBuilder, BulkData.Data.RootBarycentricBuffer, BulkData.Header.RootCount, RootBarycentricBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_RootTriangleBarycentricBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
			InternalCreateVertexBufferRDG_FromHairBulkData<FHairStrandsRootToUniqueTriangleIndexFormat>(GraphBuilder, BulkData.Data.RootToUniqueTriangleIndexBuffer, BulkData.Header.RootCount, RootToUniqueTriangleIndexBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_RootToUniqueTriangleIndexBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
			InternalCreateVertexBufferRDG_FromHairBulkData<FHairStrandsUniqueTriangleIndexFormat>(GraphBuilder, BulkData.Data.UniqueTriangleIndexBuffer, BulkData.Header.UniqueTriangleCount, UniqueTriangleIndexBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_UniqueTriangleIndexBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
			InternalCreateVertexBufferRDG_FromHairBulkData<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, BulkData.Data.RestUniqueTrianglePositionBuffer, BulkData.Header.UniqueTriangleCount * 3, RestUniqueTrianglePositionBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_RestUniqueTrianglePosition0Buffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
		}

		// Allocate only once
		if (BulkData.Header.SampleCount > 0 && MeshInterpolationWeightsBuffer.Buffer == nullptr)
		{
			const uint32 InteroplationWeightCount = BulkData.Data.MeshInterpolationWeightsBuffer.GetBulkDataSize() / sizeof(FHairStrandsWeightFormat::Type);
			const uint32 WeightCount = FGroomRBFDeformer::GetWeightCount(BulkData.Header.SampleCount);
			check(WeightCount == InteroplationWeightCount); // Sanity check

			InternalCreateStructuredBufferRDG_FromHairBulkData<FHairStrandsWeightFormat>(GraphBuilder, BulkData.Data.MeshInterpolationWeightsBuffer, InteroplationWeightCount, MeshInterpolationWeightsBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_MeshInterpolationWeightsBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
			InternalCreateVertexBufferRDG_FromHairBulkData<FHairStrandsRBFSampleIndexFormat>(GraphBuilder, BulkData.Data.MeshSampleIndicesAndSectionsBuffer, BulkData.Header.SampleCount, MeshSampleIndicesAndSectionsBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_MeshSampleIndicesAndSectionsBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
			InternalCreateStructuredBufferRDG_FromHairBulkData<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, BulkData.Data.RestSamplePositionsBuffer, BulkData.Header.SampleCount, RestSamplePositionsBuffer, ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRestRoot_RestSamplePositionsBuffer), ResourceName), OwnerName, EHairResourceUsageType::Static);
		}

		Status = EStatus::Completed;
		AvailableCurveCount = InCurveCount;
		SampleCount = BulkData.Header.SampleCount;
	}
}

void FHairStrandsLODRestRootResource::InternalRelease()
{
	Status = EStatus::Invalid;
	RootBarycentricBuffer.Release();
	RootToUniqueTriangleIndexBuffer.Release();
	UniqueTriangleIndexBuffer.Release();
	RestUniqueTrianglePositionBuffer.Release();
	SampleCount = 0;
	MeshInterpolationWeightsBuffer.Release();
	MeshSampleIndicesAndSectionsBuffer.Release();
	RestSamplePositionsBuffer.Release();
	AvailableCurveCount = 0;
}

void FHairStrandsLODRestRootResource::InternalResetLoadedSize()
{
	// Conditionnaly reset loaded size at under certain circumpstances, the bulk data array might having reset 
	// (due to pending build, while some resource are still in flight/being destroyed). This shouldn't be needed, 
	// but can't figure out why we end in such a case at the moment.
	if (IsBulkDataValid())
	{
		GetBulkData().ResetLoadedSize();
	}
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsRestRootResource::FHairStrandsRestRootResource(TArray<FHairStrandsRootBulkData>& InBulkDatas, EHairStrandsResourcesType InCurveType, const FHairResourceName& InResourceName, const FName& InOwnerName)
: CurveType(InCurveType)
{
	int32 Index = 0;
	for (FHairStrandsRootBulkData& LODBulkData : InBulkDatas)
	{
		LODs.Add(new FHairStrandsLODRestRootResource(InBulkDatas, Index, InCurveType, LODBulkData.Header.LODIndex, InResourceName, InOwnerName));
		Index++;
	}
}

void FHairStrandsRestRootResource::StreamInData(int32 InMeshLODIndex)
{
	if (IsDataValid(InMeshLODIndex))
	{
		GetLOD(InMeshLODIndex)->StreamInData();
	}
}

void FHairStrandsRestRootResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, int32 InMeshLODIndex)
{
	if (IsDataValid(InMeshLODIndex))
	{
		GetLOD(InMeshLODIndex)->Allocate(GraphBuilder, LoadingType, Status);
	}
}

void FHairStrandsRestRootResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, uint32 InRequestedCurveCount, uint32 InRequestedPointCount, bool bAllowDeallocation, int32 InMeshLODIndex)
{
	if (IsDataValid(InMeshLODIndex))
	{
		GetLOD(InMeshLODIndex)->Allocate(GraphBuilder, LoadingType, Status, InRequestedCurveCount, InRequestedPointCount, bAllowDeallocation);
	}
}

void FHairStrandsRestRootResource::ReleaseResource()
{
	for (FHairStrandsLODRestRootResource* LOD : LODs)
	{
		LOD->ReleaseResource();
		delete LOD;
	}
	LODs.Empty();
}

void FHairStrandsRestRootResource::InternalResetLoadedSize()
{
	for (FHairStrandsLODRestRootResource* LOD : LODs)
	{
		LOD->InternalResetLoadedSize();
	}
}

void FHairStrandsRestRootResource::BeginInitResource()
{
	for (FHairStrandsLODRestRootResource* LOD : LODs)
	{
		::BeginInitResource(LOD);
	}
}

void FHairStrandsRestRootResource::BeginUpdateResourceRHI()
{
	for (FHairStrandsLODRestRootResource* LOD : LODs)
	{
		::BeginUpdateResourceRHI(LOD);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsLODDeformedRootResource::FHairStrandsLODDeformedRootResource(FHairStrandsRestRootResource* InRestResources, EHairStrandsResourcesType InCurveType, const uint32 InMeshLODIndex, const FHairResourceName& InResourceName, const FName& InOwnerName)
:	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName)
,	MeshLODIndex(InMeshLODIndex) 
,	CurveType(InCurveType)
{
	check(InRestResources);

	if (InRestResources->IsDataValid(InMeshLODIndex))
	{
		FHairStrandsLODRestRootResource* LODData = InRestResources->GetLOD(InMeshLODIndex);
		RootCount = LODData->GetBulkData().Header.RootCount;
		SampleCount = LODData->GetBulkData().Header.SampleCount;
	}

	Status = EStatus::Invalid;
}

void FHairStrandsLODDeformedRootResource::InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount)
{
	if (RootCount > 0 && Status == EStatus::Invalid)
	{
		Status = EStatus::Initialized;
		AvailableCurveCount = FMath::Min(InCurveCount, RootCount);
		if (SampleCount > 0)
		{
			const uint32 RBFEntryCount = FGroomRBFDeformer::GetEntryCount(SampleCount);

			InternalCreateStructuredBufferRDG<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, SampleCount, DeformedSamplePositionsBuffer[0], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRootDeformed_DeformedSamplePositionsBuffer0), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
			InternalCreateStructuredBufferRDG<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, RBFEntryCount, MeshSampleWeightsBuffer[0], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRootDeformed_MeshSampleWeightsBuffer0), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);

			// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
			if (IsHairStrandContinuousDecimationReorderingEnabled())
			{
				InternalCreateStructuredBufferRDG<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, SampleCount, DeformedSamplePositionsBuffer[1], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRootDeformed_DeformedSamplePositionsBuffer1), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
				InternalCreateStructuredBufferRDG<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, RBFEntryCount, MeshSampleWeightsBuffer[1], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRootDeformed_MeshSampleWeightsBuffer1), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
			}
		}

		InternalCreateVertexBufferRDG<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, RootCount * 3, DeformedUniqueTrianglePositionBuffer[0], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRootDeformed_DeformedUniqueTrianglePosition0Buffer0), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);

		// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
		if (IsHairStrandContinuousDecimationReorderingEnabled())
		{
			InternalCreateVertexBufferRDG<FHairStrandsMeshTrianglePositionFormat>(GraphBuilder, RootCount * 3, DeformedUniqueTrianglePositionBuffer[1], ToHairResourceDebugName(HAIRSTRANDS_RESOUCE_NAME(CurveType, Hair.StrandsRootDeformed_DeformedUniqueTrianglePosition0Buffer1), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		}
	}
}

void FHairStrandsLODDeformedRootResource::InternalRelease()
{
	Status = EStatus::Invalid;
	AvailableCurveCount = 0;
	DeformedUniqueTrianglePositionBuffer[0].Release();
	DeformedSamplePositionsBuffer[0].Release();
	MeshSampleWeightsBuffer[0].Release();

	// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
	if (IsHairStrandContinuousDecimationReorderingEnabled())
	{
		DeformedUniqueTrianglePositionBuffer[1].Release();
		DeformedSamplePositionsBuffer[1].Release();
		MeshSampleWeightsBuffer[1].Release();
	}
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsDeformedRootResource::FHairStrandsDeformedRootResource(FHairStrandsRestRootResource* InRestResources, EHairStrandsResourcesType InCurveType, const FHairResourceName& InResourceName, const FName& InOwnerName)
:	CurveType(InCurveType)
{
	check(InRestResources);
	LODs.Reserve(InRestResources->LODs.Num());
	for (FHairStrandsLODRestRootResource* InLOD : InRestResources->LODs)
	{
		check(InLOD);
		LODs.Add(new FHairStrandsLODDeformedRootResource(InRestResources, InCurveType, InLOD->MeshLODIndex, InResourceName, InOwnerName));
	}
}

void FHairStrandsDeformedRootResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, int32 InMeshLODIndex)
{
	if (IsDataValid(InMeshLODIndex))
	{
		GetLOD(InMeshLODIndex)->Allocate(GraphBuilder, LoadingType, Status);
	}
}

void FHairStrandsDeformedRootResource::Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, uint32 InRequestedCurveCount, uint32 InRequestedPointCount, bool bAllowDeallocation, int32 InMeshLODIndex)
{
	if (IsDataValid(InMeshLODIndex))
	{
		GetLOD(InMeshLODIndex)->Allocate(GraphBuilder, LoadingType, Status, InRequestedCurveCount, InRequestedPointCount, bAllowDeallocation);
	}
}

void FHairStrandsDeformedRootResource::ReleaseResource()
{
	for (FHairStrandsLODDeformedRootResource* LOD : LODs)
	{
		LOD->ReleaseResource();
		delete LOD;
	}
	LODs.Empty();
}

void FHairStrandsDeformedRootResource::SwapBuffer()
{
	for (FHairStrandsLODDeformedRootResource* LOD : LODs)
	{
		LOD->SwapBuffer();
	}
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsInterpolationResource::FHairStrandsInterpolationResource(FHairStrandsInterpolationBulkData& InBulkData, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
CurveInterpolationBuffer(), PointInterpolationBuffer(),BulkData(InBulkData)
{
	MaxAvailableCurveCount = 0;

	// Sanity check
	check(!!(BulkData.Header.Flags & FHairStrandsInterpolationBulkData::DataFlags_HasData));
}

bool FHairStrandsInterpolationResource::InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount)
{
	if (StreamingRequest.IsNone())
	{
#if !WITH_EDITORONLY_DATA
		if (BulkData.IsCookedBulkDataLoadedForCPUAccess())
		{
			// Data is already in memory
			StreamingRequest.CurveCount = InRequestedCurveCount;
			StreamingRequest.PointCount = InRequestedPointCount;
			return true;
		}
		else
#endif
		{
			StreamingRequest.Request(InRequestedCurveCount, InRequestedPointCount, BulkData, false, false, false, OwnerName);
		}
	}
	return StreamingRequest.IsCompleted();
}

void FHairStrandsInterpolationResource::InternalAllocate(FRDGBuilder& GraphBuilder)
{
	// If we enter this function, the request need to be completed
	check(StreamingRequest.IsCompleted());

	InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.CurveInterpolation, CurveInterpolationBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsInterpolation_CurveInterpolationBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Static);
	InternalCreateByteAddressBufferRDG_FromHairBulkData(GraphBuilder, BulkData.Data.PointInterpolation, PointInterpolationBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsInterpolation_PointInterpolationBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Static);
}

void FHairStrandsInterpolationResource::InternalRelease()
{
	CurveInterpolationBuffer.Release();
	PointInterpolationBuffer.Release();
	MaxAvailableCurveCount = 0;
}

void FHairStrandsInterpolationResource::InternalResetLoadedSize()
{
	BulkData.ResetLoadedSize();
}

/////////////////////////////////////////////////////////////////////////////////////////

FHairCardsInterpolationResource::FHairCardsInterpolationResource(FHairCardsInterpolationBulkData& InBulkData, const FHairResourceName& InResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, InResourceName, InOwnerName),
	InterpolationBuffer(), BulkData(InBulkData)
{
}

void FHairCardsInterpolationResource::InternalAllocate(FRDGBuilder& GraphBuilder)
{
	InternalCreateByteAddressBufferRDG(GraphBuilder, BulkData.Interpolation, FHairCardsInterpolationFormat::Format, InterpolationBuffer, ToHairResourceDebugName(TEXT("Hair.CardsInterpolation_InterpolationBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Static);
}

void FHairCardsInterpolationResource::InternalRelease()
{
	InterpolationBuffer.Release();
}

/////////////////////////////////////////////////////////////////////////////////////////

#if RHI_RAYTRACING
bool GetSupportHairStrandsProceduralPrimitive(EShaderPlatform InShaderPlatform);
bool GetSupportHairStrandsLinearSweptSpheres(EShaderPlatform InShaderPlatform);

static void ConvertPointToRaytracingVertexCount(EHairStrandsRayTracingPrimitiveType RayTracingPrimitiveType, uint32 InPointCount, uint32& OutVertexCount, uint32& OutIndexCount)
{
	if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::LinearSweptSphere)
	{
		// 1 vertex at the beginning to indicate invalid hair strand
		OutVertexCount = InPointCount + 1;
		OutIndexCount  = InPointCount * 2;
	}
	else if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::Procedural)
	{
		// only allocate space for primitive AABBs
		OutVertexCount = InPointCount * 2 * STRANDS_PROCEDURAL_INTERSECTOR_MAX_SPLITS;
		OutIndexCount  = 0;
	}
	else
	{
		// triangle fallback path
		OutVertexCount = InPointCount * 4;
		OutIndexCount  = InPointCount * 8 * 3;
	}
}

// RT geometry for strands is built as a 4 sided cylinder
//   each vertex of the curve becomes 4 points
//   each curve segment turns into 2*4=8 triangles (3 indices for each)
// there is some waste due to the degenerate triangles emitted from the end points of each curve
// total memory usage is: 4*float4 + 8*uint3 = 40 bytes per vertex
// The previous implementation used a "cross" layout without an index buffer
// which used 6*float4 = 48 bytes per vertex
// NOTE: the vertex buffer is a float4 because it is registered as a UAV for the compute shader to work
// TODO: use a plain float vertex buffer with 3x the entries instead to save memory? (float3 UAVs are not allowed)
FHairStrandsRaytracingResource::FHairStrandsRaytracingResource(const FHairStrandsBulkData& InData, const FHairResourceName& ResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, ResourceName, InOwnerName), bOwnBuffers(true)
{
	MaxVertexCount = 0;
	MaxIndexCount = 0;
	MaxAvailableCurveCount = 0;

	RayTracingPrimitiveType = EHairStrandsRayTracingPrimitiveType::Triangle;
	if (GetSupportHairStrandsLinearSweptSpheres(GMaxRHIShaderPlatform) && IsHairStrandsLinearSweptSpheresEnabled())
	{
		RayTracingPrimitiveType = EHairStrandsRayTracingPrimitiveType::LinearSweptSphere;
	}
	else if (GetSupportHairStrandsProceduralPrimitive(GMaxRHIShaderPlatform))
	{
		RayTracingPrimitiveType = EHairStrandsRayTracingPrimitiveType::Procedural;
	}
}

FHairStrandsRaytracingResource::FHairStrandsRaytracingResource(const FHairCardsBulkData& InData, const FHairResourceName& ResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, ResourceName, InOwnerName),
	MaxVertexCount(InData.GetNumVertices()), bOwnBuffers(false)
{}

FHairStrandsRaytracingResource::FHairStrandsRaytracingResource(const FHairMeshesBulkData& InData, const FHairResourceName& ResourceName, const FName& InOwnerName) :
	FHairCommonResource(EHairStrandsAllocationType::Deferred, ResourceName, InOwnerName),
	MaxVertexCount(InData.GetNumVertices()), bOwnBuffers(false)
{}

void FHairStrandsRaytracingResource::InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount)
{
	if (bOwnBuffers)
	{
		ConvertPointToRaytracingVertexCount(RayTracingPrimitiveType, InPointCount, MaxVertexCount, MaxIndexCount);

		InternalCreateVertexBufferRDG<FHairStrandsRaytracingFormat>(GraphBuilder, MaxVertexCount, PositionBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsRaytracing_PositionBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);
		InternalCreateStructuredBufferRDG<FHairStrandsIndexFormat>(GraphBuilder, MaxIndexCount, IndexBuffer, ToHairResourceDebugName(TEXT("Hair.StrandsRaytracing_IndexBuffer"), ResourceName), OwnerName, EHairResourceUsageType::Dynamic);

		// Force reallocating and rebuilding the RT geometry
		RayTracingGeometry.ReleaseRHI();
		RayTracingGeometry.SetInitializer(FRayTracingGeometryInitializer{});
		bIsRTGeometryInitialized = false;
	}
}

void FHairStrandsRaytracingResource::InternalRelease()
{
	PositionBuffer.Release();
	IndexBuffer.Release();
	RayTracingGeometry.ReleaseRHI();
	RayTracingGeometry.SetInitializer(FRayTracingGeometryInitializer{});
	bIsRTGeometryInitialized = false;
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Debug data

static uint32 ToLinearCoord(const FIntVector& T, const FIntVector& Resolution)
{
	// Morton instead for better locality?
	return T.X + T.Y * Resolution.X + T.Z * Resolution.X * Resolution.Y;
}

static FIntVector ToCoord(const FVector3f& T, const FIntVector& Resolution, const FVector3f& MinBound, const float VoxelSize)
{
	const FVector3f C = (T - MinBound) / VoxelSize;
	return FIntVector(
		FMath::Clamp(FMath::FloorToInt(C.X), 0, Resolution.X - 1),
		FMath::Clamp(FMath::FloorToInt(C.Y), 0, Resolution.Y - 1),
		FMath::Clamp(FMath::FloorToInt(C.Z), 0, Resolution.Z - 1));
}

void CreateHairStrandsDebugDatas(
	const FHairStrandsDatas& InData,
	FHairStrandsDebugDatas& Out)
{
	const FVector3f BoundSize = FVector3f(InData.BoundingBox.Max) - FVector3f(InData.BoundingBox.Min);
	Out.VoxelDescription.VoxelSize = FMath::Clamp(GHairStrandsDebugVoxel_WorldSize, 0.1f, 10.f);
	Out.VoxelDescription.VoxelResolution = FIntVector(FMath::CeilToInt(BoundSize.X / Out.VoxelDescription.VoxelSize), FMath::CeilToInt(BoundSize.Y / Out.VoxelDescription.VoxelSize), FMath::CeilToInt(BoundSize.Z / Out.VoxelDescription.VoxelSize));
	Out.VoxelDescription.VoxelMinBound = FVector3f(InData.BoundingBox.Min);
	Out.VoxelDescription.VoxelMaxBound = FVector3f(Out.VoxelDescription.VoxelResolution) * Out.VoxelDescription.VoxelSize + FVector3f(InData.BoundingBox.Min);
	Out.VoxelOffsetAndCount.Init(FHairStrandsDebugDatas::FOffsetAndCount(), Out.VoxelDescription.VoxelResolution.X * Out.VoxelDescription.VoxelResolution.Y * Out.VoxelDescription.VoxelResolution.Z);
	Out.VoxelDescription.MaxSegmentPerVoxel = 0;

	uint32 AllocationCount = 0;
	TArray<TArray<FHairStrandsDebugDatas::FVoxel>> TempVoxelData;

	const uint32 MaxNumberOfSegmentPerVoxel = uint32(FMath::Clamp(GHairStrandsDebugVoxel_MaxSegmentPerVoxel, 16, 64000));

	// Fill in voxel (TODO: make it parallel)
	const uint32 CurveCount = InData.StrandsCurves.Num();
	for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
	{
		const uint32 PointOffset = InData.StrandsCurves.CurvesOffset[CurveIndex];
		const uint32 PointCount = InData.StrandsCurves.CurvesCount[CurveIndex];

		for (uint32 PointIndex = 0; PointIndex < PointCount - 1; ++PointIndex)
		{
			const uint32 Index0 = PointOffset + PointIndex;
			const uint32 Index1 = PointOffset + PointIndex + 1;
			const FVector3f& P0 = InData.StrandsPoints.PointsPosition[Index0];
			const FVector3f& P1 = InData.StrandsPoints.PointsPosition[Index1];
			const FVector3f Segment = P1 - P0;

			const float Length = Segment.Size();
			const uint32 StepCount = FMath::CeilToInt(Length / (0.25f * Out.VoxelDescription.VoxelSize));
			uint32 PrevLinearCoord = ~0;
			for (uint32 StepIt = 0; StepIt < StepCount + 1; ++StepIt)
			{
				const FVector3f P = P0 + Segment * StepIt / float(StepCount);
				const FIntVector Coord = ToCoord(P, Out.VoxelDescription.VoxelResolution, Out.VoxelDescription.VoxelMinBound, Out.VoxelDescription.VoxelSize);
				const uint32 LinearCoord = ToLinearCoord(Coord, Out.VoxelDescription.VoxelResolution);
				if (LinearCoord != PrevLinearCoord)
				{
					if (Out.VoxelOffsetAndCount[LinearCoord].Count == 0)
					{
						Out.VoxelOffsetAndCount[LinearCoord].Offset = TempVoxelData.Num();
						TempVoxelData.Add(TArray<FHairStrandsDebugDatas::FVoxel>());
					}

					if (Out.VoxelOffsetAndCount[LinearCoord].Count+1 < MaxNumberOfSegmentPerVoxel)
					{
						const uint32 Offset = Out.VoxelOffsetAndCount[LinearCoord].Offset;
						Out.VoxelOffsetAndCount[LinearCoord].Count += 1;
						TempVoxelData[Offset].Add({Index0, Index1});
					}

					Out.VoxelDescription.MaxSegmentPerVoxel = FMath::Max(Out.VoxelDescription.MaxSegmentPerVoxel, Out.VoxelOffsetAndCount[LinearCoord].Count);

					PrevLinearCoord = LinearCoord;

					++AllocationCount;
				}
			}
		}
	}

	check(Out.VoxelDescription.MaxSegmentPerVoxel < MaxNumberOfSegmentPerVoxel);
	Out.VoxelData.Reserve(AllocationCount);

	for (int32 Index = 0, Count = Out.VoxelOffsetAndCount.Num(); Index < Count; ++Index)
	{
		if (Out.VoxelOffsetAndCount[Index].Count > 0)
		{
			const uint32 ArrayIndex = Out.VoxelOffsetAndCount[Index].Offset;
			const uint32 NewOffset = Out.VoxelData.Num();
			Out.VoxelOffsetAndCount[Index].Offset = NewOffset;

			for (FHairStrandsDebugDatas::FVoxel& Voxel : TempVoxelData[ArrayIndex])
			{
				Voxel.Index1 = NewOffset;
				Out.VoxelData.Add(Voxel);
			}
		}
		else
		{
			Out.VoxelOffsetAndCount[Index].Offset = 0;
		}

		// Sanity check
		//check(Out.VoxelOffsetAndCount[Index].Offset + Out.VoxelOffsetAndCount[Index].Count == Out.VoxelData.Num());
	}

	check(Out.VoxelData.Num()>0);
}

void CreateHairStrandsDebugResources(FRDGBuilder& GraphBuilder, const FHairStrandsDebugDatas* In, FHairStrandsDebugResources* Out)
{
	check(In);
	check(Out);

	FRDGBufferRef VoxelOffsetAndCount = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("HairStrandsDebug_VoxelOffsetAndCount"),
		sizeof(FHairStrandsDebugDatas::FOffsetAndCount),
		In->VoxelOffsetAndCount.Num(),
		In->VoxelOffsetAndCount.GetData(),
		sizeof(FHairStrandsDebugDatas::FOffsetAndCount) * In->VoxelOffsetAndCount.Num());

	FRDGBufferRef VoxelData = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("HairStrandsDebug_VoxelData"),
		sizeof(FHairStrandsDebugDatas::FVoxel),
		In->VoxelData.Num(),
		In->VoxelData.GetData(),
		sizeof(FHairStrandsDebugDatas::FVoxel) * In->VoxelData.Num());

	Out->VoxelDescription = In->VoxelDescription;
	Out->VoxelOffsetAndCount = ConvertToExternalAccessBuffer(GraphBuilder, VoxelOffsetAndCount);
	Out->VoxelData = ConvertToExternalAccessBuffer(GraphBuilder, VoxelData);
}
