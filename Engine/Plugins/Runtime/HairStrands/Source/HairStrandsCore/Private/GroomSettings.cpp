// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomSettings)


FArchive& operator<<(FArchive& Ar, FGroomBuildSettings& Settings)
{
	Ar << Settings.bOverrideGuides;
	Ar << Settings.bRandomizeGuide;
	Ar << Settings.HairToGuideDensity;
	Ar << Settings.InterpolationQuality;
	Ar << Settings.InterpolationDistance;
	Ar << Settings.bRandomizeGuide;
	Ar << Settings.bUseUniqueGuide;

	return Ar;
}

