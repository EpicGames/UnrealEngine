// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomTextureBuilder.h"
#include "Engine/Texture2D.h"
#include "HairStrandsCore.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "GroomAsset.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "ShaderPrintParameters.h"
#include "CommonRenderResources.h"
#include "HairStrandsMeshProjection.h"
#include "Engine/StaticMesh.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "HairStrandsDefinitions.h"
#include "RenderCaptureInterface.h"

#if WITH_EDITORONLY_DATA
#include "DerivedDataRequestOwner.h"
#endif

// Enable this to force looping texture generation pass and automatically 
// triggering RenderDoc capture when generating groom textures
#define DEBUG_TEXTURE_GENERATION 0
#define DEBUG_TEXTURE_RENDERDOC 0

///////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY_STATIC(LogGroomTextureBuilder, Log, All);

#define LOCTEXT_NAMESPACE "GroomTextureBuilder"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Follicle texture generation

void FGroomTextureBuilder::AllocateFollicleTextureResources(UTexture2D* Out)
{
	if (!Out)
	{
		return;
	}
	AllocateFollicleTextureResources(Out, Out->GetSizeX(), Out->GetNumMips());
}

void FGroomTextureBuilder::AllocateFollicleTextureResources(UTexture2D* Out, const FIntPoint& Resolution, uint32 MipCount)
{
	if (!Out)
	{
		return;
	}
	
#if WITH_EDITOR
	Out->PreEditChange(nullptr);
#endif

	FTextureFormatSettings FormatSettings;
	FormatSettings.CompressionNone = true;
	FormatSettings.CompressionSettings = TC_Masks;
	FormatSettings.SRGB = false;

#if WITH_EDITORONLY_DATA
	Out->Source.Init(Resolution.X, Resolution.Y, 1, MipCount, ETextureSourceFormat::TSF_BGRA8, nullptr);
#endif // #if WITH_EDITORONLY_DATA
	Out->LODGroup = TEXTUREGROUP_EffectsNotFiltered; // Mipmap filtering, no compression
#if WITH_EDITORONLY_DATA
	Out->SetLayerFormatSettings(0, FormatSettings);
#endif // #if WITH_EDITORONLY_DATA

	Out->SetPlatformData(new FTexturePlatformData());
	Out->GetPlatformData()->SizeX = Resolution.X;
	Out->GetPlatformData()->SizeY = Resolution.Y;
	Out->GetPlatformData()->PixelFormat = PF_B8G8R8A8;

	// No need to create the resources when using the CPU path.
	//Out->UpdateResource();
}

#if WITH_EDITOR
UTexture2D* FGroomTextureBuilder::CreateGroomFollicleMaskTexture(const UGroomAsset* GroomAsset, uint32 InResolution)
{
	if (!GroomAsset)
	{
		return nullptr;
	}

	const FString PackageName = GroomAsset->GetOutermost()->GetName();
	const FIntPoint Resolution(InResolution, InResolution);
	UObject* FollicleMaskAsset = FHairStrandsCore::CreateTexture(PackageName, Resolution, TEXT("_FollicleTexture"), FGroomTextureBuilder::AllocateFollicleTextureResources);
	return (UTexture2D*)FollicleMaskAsset;
}
#endif

struct FPixel
{
	uint8 V[4];
	FPixel() 
	{ 
		V[0] = 0;
		V[1] = 0;
		V[2] = 0;
		V[3] = 0;
	};

	FORCEINLINE uint8& Get(uint32 C) { return V[C]; }
	uint32 ToUint() const
	{
		return V[0] | (V[1] << 8) | (V[2] << 16) | (V[3] << 24);
	}
};

// CPU raster
static void RasterToTexture(int32 Resolution, int32 KernelExtent, uint32 Channel, const FHairStrandsDatas& InStrandsData, FPixel* OutPixels)
{
	const uint32 Attribute = InStrandsData.GetAttributes();
	if (!HasHairAttribute(Attribute, EHairAttribute::RootUV))
	{
		return;
	}

	const uint32 CurveCount = InStrandsData.GetNumCurves();
	for (uint32 CurveIndex = 0; CurveIndex < CurveCount; ++CurveIndex)
	{
		FVector2f RootUV = InStrandsData.StrandsCurves.CurvesRootUV[CurveIndex];
		RootUV.Y = FMath::Clamp(1.f - RootUV.Y, 0.f, 1.f);

		const FIntPoint RootCoord(
			FMath::Clamp(int32(RootUV.X * Resolution), 0, Resolution - 1),
			FMath::Clamp(int32(RootUV.Y * Resolution), 0, Resolution - 1));

		for (int32 Y = -KernelExtent; Y <= KernelExtent; ++Y)
		for (int32 X = -KernelExtent; X <= KernelExtent; ++X)
		{
			const FIntPoint Coord = RootCoord + FIntPoint(X, Y);
			if (Coord.X < 0 || Coord.X >= Resolution || Coord.Y < 0 || Coord.Y >= Resolution)
				continue;

			const FVector2D fCoord(Coord.X + 0.5f, Coord.Y+0.5f);
			const float Distance = FVector2D::Distance(fCoord, RootCoord);
			const float V = FMath::Clamp(1.f - (Distance / KernelExtent), 0.f, 1.f);

			const uint32 V8Bits = FMath::Clamp(uint32(V * 0xFF), 0u, 0xFFu);

			const uint32 LinearCoord = Coord.X + Coord.Y * Resolution;
			OutPixels[LinearCoord].Get(Channel) = FMath::Max(uint32(OutPixels[LinearCoord].Get(Channel)), V8Bits);
		}
	}
}

// GPU raster
static void InternalGenerateFollicleTexture_GPU(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	bool bCopyDataBackToCPU,
	const FIntPoint Resolution,
	const uint32 MipCount,
	const EPixelFormat Format,
	const uint32 InKernelSizeInPixels,
	const TArray<FRDGBufferRef>& InRootUVBuffers_R,
	const TArray<FRDGBufferRef>& InRootUVBuffers_G,
	const TArray<FRDGBufferRef>& InRootUVBuffers_B,
	const TArray<FRDGBufferRef>& InRootUVBuffers_A,
	UTexture2D*	OutTexture)
{
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;

	if (OutTexture == nullptr || !OutTexture->GetResource() || !OutTexture->GetResource()->GetTexture2DRHI() ||
		(InRootUVBuffers_R.Num() == 0 &&
		 InRootUVBuffers_G.Num() == 0 &&
		 InRootUVBuffers_B.Num() == 0 &&
		 InRootUVBuffers_A.Num() == 0))
	{
		return;
	}
	check(Resolution.X == Resolution.Y);

	FRDGTextureRef FollicleMaskTexture = nullptr;
	if (InRootUVBuffers_R.Num())
	{
		GenerateFolliculeMask(
			GraphBuilder,
			ShaderMap,
			Format,
			Resolution,
			MipCount,
			InKernelSizeInPixels,
			0,
			InRootUVBuffers_R,
			FollicleMaskTexture);
	}
	if (InRootUVBuffers_G.Num())
	{
		GenerateFolliculeMask(
			GraphBuilder,
			ShaderMap,
			Format,
			Resolution,
			MipCount,
			InKernelSizeInPixels,
			1,
			InRootUVBuffers_G,
			FollicleMaskTexture);
	}
	if (InRootUVBuffers_B.Num())
	{
		GenerateFolliculeMask(
			GraphBuilder,
			ShaderMap,
			Format,
			Resolution,
			MipCount,
			InKernelSizeInPixels,
			2,
			InRootUVBuffers_B,
			FollicleMaskTexture);
	}
	if (InRootUVBuffers_A.Num())
	{
		GenerateFolliculeMask(
			GraphBuilder,
			ShaderMap,
			Format,
			Resolution,
			MipCount,
			InKernelSizeInPixels,
			3,
			InRootUVBuffers_A,
			FollicleMaskTexture);
	}
	AddComputeMipsPass(GraphBuilder, ShaderMap, FollicleMaskTexture);

	// Select if the generated texture should be copy back to a CPU texture for being saved, or directly used
	AddReadbackTexturePass(
		GraphBuilder,
		RDG_EVENT_NAME("CopyRDGToTexture2D"),
		FollicleMaskTexture,
		[FollicleMaskTexture, MipCount, OutTexture](FRHICommandListImmediate& RHICmdList)
	{
		if (OutTexture && OutTexture->GetResource() && OutTexture->GetResource()->GetTexture2DRHI())
		{
			FRHITexture* DstTexture = OutTexture->GetResource()->GetTexture2DRHI();
			RHICmdList.Transition(FRHITransitionInfo(DstTexture, ERHIAccess::SRVMask, ERHIAccess::CopyDest));

			FRHICopyTextureInfo CopyInfo;
			CopyInfo.NumMips = MipCount;
			RHICmdList.CopyTexture(
				FollicleMaskTexture->GetRHI(),
				DstTexture,
				CopyInfo);

			RHICmdList.Transition(FRHITransitionInfo(DstTexture, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
		}
	});
}


// CPU path
static void InternalBuildFollicleTexture_CPU(const TArray<FFollicleInfo>& InInfos, UTexture2D* OutTexture)
{
#if WITH_EDITORONLY_DATA
	const FIntPoint Resolution(OutTexture->GetSizeX(), OutTexture->GetSizeY());
	check(OutTexture->GetSizeX() == OutTexture->GetSizeY());

	uint8* OutData = OutTexture->Source.LockMip(0);
	FMemory::Memzero(OutData, Resolution.X * Resolution.Y * sizeof(uint32));
	FPixel* Pixels = (FPixel*)OutData;
	for (const FFollicleInfo& Info : InInfos)
	{
		if (!Info.GroomAsset)
		{
			continue;
		}

		// The output pixel format is PF_B8G8R8A8. So remap channel to map onto the RGBA enum Info.Channel
		uint32 Channel = 0;
		switch (Info.Channel)
		{
			case FFollicleInfo::B: Channel = 0; break;
			case FFollicleInfo::G: Channel = 1; break;
			case FFollicleInfo::R: Channel = 2; break;
			case FFollicleInfo::A: Channel = 3; break;
		}

		uint32 GroupIndex = 0;
		for (const FHairGroupPlatformData& HairGroupData : Info.GroomAsset->GetHairGroupsPlatformData())
		{
			FHairStrandsDatas StrandsData;
			FHairStrandsDatas GuidesData;
			Info.GroomAsset->GetHairStrandsDatas(GroupIndex, StrandsData, GuidesData);

			RasterToTexture(Resolution.X, Info.KernelSizeInPixels, Channel, StrandsData, Pixels);
			++GroupIndex;
		}
	}
	OutTexture->Source.UnlockMip(0);
	OutTexture->DeferCompression = true; // This forces reloading data when the asset is saved
	OutTexture->MarkPackageDirty();
#endif // #if WITH_EDITORONLY_DATA
}

struct FFollicleInfoGPU
{
	FHairStrandsDatas*		StrandsData = nullptr;
	FFollicleInfo::EChannel	Channel = FFollicleInfo::R;
	uint32					KernelSizeInPixels = 0;
	bool					bGPUOnly = false;
};

// GPU path
static void InternalBuildFollicleTexture_GPU(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const TArray<FFollicleInfoGPU>& InInfos,
	const FIntPoint Resolution,
	const uint32 MipCount,
	const EPixelFormat InFormat,
	UTexture2D*	OutTexture)
{
	uint32 KernelSizeInPixels = ~0;
	TArray<FRDGBufferRef> RootUVBuffers[4];
	bool bCopyDataBackToCPU = false;

	for (const FFollicleInfoGPU& Info : InInfos)
	{
		if (KernelSizeInPixels == ~0)
		{
			KernelSizeInPixels = Info.KernelSizeInPixels;
			bCopyDataBackToCPU = !Info.bGPUOnly;
		}

		// Create root UVs buffer
		if (Info.StrandsData && HasHairAttribute(Info.StrandsData->GetAttributes(), EHairAttribute::RootUV))
		{
			const uint32 DataCount = Info.StrandsData->StrandsCurves.CurvesRootUV.Num();
			const uint32 DataSizeInBytes = sizeof(FVector2f) * DataCount;
			check(DataSizeInBytes != 0);

			const FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(sizeof(FVector2f), DataCount);
			FRDGBufferRef RootBuffer = CreateVertexBuffer(
				GraphBuilder,
				TEXT("RootUVBuffer"),
				Desc,
				Info.StrandsData->StrandsCurves.CurvesRootUV.GetData(),
				DataSizeInBytes,
				ERDGInitialDataFlags::None);
			RootUVBuffers[Info.Channel].Add(RootBuffer);
		}
	}

	const EPixelFormat Format = bCopyDataBackToCPU ? PF_B8G8R8A8 : PF_R8G8B8A8;
	InternalGenerateFollicleTexture_GPU(GraphBuilder, ShaderMap, bCopyDataBackToCPU, Resolution, MipCount, Format, KernelSizeInPixels, RootUVBuffers[0], RootUVBuffers[1], RootUVBuffers[2], RootUVBuffers[3], OutTexture);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Asynchronous queuing for follicle texture mask generation
struct FFollicleQuery
{
	TArray<FFollicleInfoGPU> Infos;
	FIntPoint Resolution = 0;
	uint32 MipCount = 0;
	EPixelFormat Format = PF_R8G8B8A8;
	UTexture2D*	OutTexture = nullptr;

};
TQueue<FFollicleQuery> GFollicleQueries;

bool HasHairStrandsFolliculeMaskQueries()
{
	return !GFollicleQueries.IsEmpty();
}

void RunHairStrandsFolliculeMaskQueries(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap)
{
	FFollicleQuery Q;
	while (GFollicleQueries.Dequeue(Q))
	{
		if (Q.Infos.Num() > 0 && Q.MipCount>0 && Q.Resolution.X > 0 && Q.Resolution.Y > 0)
		{
			InternalBuildFollicleTexture_GPU(GraphBuilder, ShaderMap, Q.Infos, Q.Resolution, Q.MipCount, Q.Format, Q.OutTexture);

			// Release all strands data
			for (FFollicleInfoGPU& Info : Q.Infos)
			{
				if (Info.StrandsData)
				{
					delete Info.StrandsData;
					Info.StrandsData = nullptr;
				}
			}
		}
	}
}

void FGroomTextureBuilder::BuildFollicleTexture(const TArray<FFollicleInfo>& InCPUInfos, UTexture2D* OutTexture, bool bUseGPU)
{
	if (!OutTexture || InCPUInfos.Num() == 0)
	{
		UE_LOGF(LogGroomTextureBuilder, Warning, "[Groom] Error - Follicle texture can't be created/rebuilt.");
		return;
	}

	if (bUseGPU)
	{
		// Asynchronous (GPU)
		const FIntPoint Resolution(OutTexture->GetSizeX(), OutTexture->GetSizeY());
		const uint32 MipCount = OutTexture->GetNumMips();
		if (MipCount > 0 && Resolution.X > 0 && Resolution.Y > 0)
		{
			TArray<FFollicleInfoGPU> GPUInfos;
			for (const FFollicleInfo& CPUInfo : InCPUInfos)
			{
				if (!CPUInfo.GroomAsset || CPUInfo.GroomAsset->GetNumHairGroups() == 0)
				{
					UE_LOGF(LogHairStrands, Warning, "[Groom] Error - Groom follicle texture can be entirely created/rebuilt as some groom assets seems invalid.");
					continue;
				}

				for (uint32 GroupIndex = 0, GroupCount = CPUInfo.GroomAsset->GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
				{
					FHairStrandsDatas* StrandsData = new FHairStrandsDatas();
					FHairStrandsDatas GuidesData;
				#if WITH_EDITORONLY_DATA
					CPUInfo.GroomAsset->GetHairStrandsDatas(GroupIndex, *StrandsData, GuidesData);
				#endif

					if (StrandsData->StrandsCurves.CurvesRootUV.Num())
					{
						FFollicleInfoGPU& GPUInfo = GPUInfos.AddDefaulted_GetRef();
						GPUInfo.Channel = CPUInfo.Channel;
						GPUInfo.KernelSizeInPixels = CPUInfo.KernelSizeInPixels;
						GPUInfo.StrandsData = StrandsData;
						GPUInfo.bGPUOnly = CPUInfo.bGPUOnly;
					}
					else
					{
						delete StrandsData;
					}
				}
			}

			const EPixelFormat Format = OutTexture->GetPixelFormat(0);
			ENQUEUE_RENDER_COMMAND(FFollicleTextureQuery)(/*UE::RenderCommandPipe::Groom,*/
			[Resolution, MipCount, Format, GPUInfos, OutTexture](FRHICommandListImmediate& RHICmdList)
			{
				if (OutTexture->GetResource())
				{
					GFollicleQueries.Enqueue({ GPUInfos, Resolution, MipCount, Format, OutTexture });
				}
			});
		}
	}
	else
	{
		// Synchronous (CPU)
		InternalBuildFollicleTexture_CPU(InCPUInfos, OutTexture);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Strands texture generation

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairStrandTextureClearCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandTextureClearCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandTextureClearCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, LayoutIndex)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TEXTURE_CLEAR"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandTextureClearCS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainCS", SF_Compute);

static void AddTextureClearPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FIntPoint& Resolution,
	const EHairTextureLayout Layout,
	FRDGTextureRef& OutTexture)
{
	FHairStrandTextureClearCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandTextureClearCS::FParameters>();
	Parameters->LayoutIndex = uint32(Layout);
	Parameters->Resolution = Resolution;
	Parameters->OutTexture = GraphBuilder.CreateUAV(OutTexture);

	TShaderMapRef<FHairStrandTextureClearCS> ComputeShader(ShaderMap);
	FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(Resolution, FIntPoint(8, 8));

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrandsTexture::Clear"),
		ComputeShader,
		Parameters,
		DispatchCount);
}


///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairStrandTextureCopyCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandTextureCopyCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandTextureCopyCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, TextureIndex)
		SHADER_PARAMETER(uint32, TextureCount)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, InTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TEXTURE_COPY"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandTextureCopyCS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainCS", SF_Compute);

static FRDGTextureRef AddTextureCopyPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 InTextureIndex,
	const FRDGTextureRef& InTexture)
{
	FRDGTextureRef OutTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(InTexture->Desc.Extent, InTexture->Desc.Format, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_External), TEXT("Hair.Readback.TransientCopy"));

	FHairStrandTextureCopyCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandTextureCopyCS::FParameters>();
	Parameters->TextureIndex = InTextureIndex;
	Parameters->TextureCount = InTexture->Desc.Depth;
	Parameters->Resolution = InTexture->Desc.Extent;
	Parameters->InTexture = InTexture;
	Parameters->OutTexture = GraphBuilder.CreateUAV(OutTexture);

	TShaderMapRef<FHairStrandTextureCopyCS> ComputeShader(ShaderMap);
	FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(Parameters->Resolution, FIntPoint(8, 8));

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrandsTexture::Copy"),
		ComputeShader,
		Parameters,
		DispatchCount);

	return OutTexture;
}



///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairStrandTextureDilationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandTextureDilationCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandTextureDilationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, LayoutIndex)
		SHADER_PARAMETER(uint32, TextureCount)
		SHADER_PARAMETER(uint32, bClearPass)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTestTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, TargetTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, OriginalTriangleMaskTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTriangleMaskTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, TargetTriangleMaskTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TEXTURE_DILATION"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandTextureDilationCS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainCS", SF_Compute);

static void AddTextureDilationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FIntPoint& Resolution,
	const EHairTextureLayout Layout,
	const bool bClearPass,
	FRDGTextureRef& DepthTestTexture,
	FRDGTextureRef& OriginalTriangleMaskTexture,
	FRDGTextureRef& SourceTriangleMaskTexture,
	FRDGTextureRef& TargetTriangleMaskTexture,
	FRDGTextureRef& SourceTexture,
	FRDGTextureRef& TargetTexture)
{
	FHairStrandTextureDilationCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandTextureDilationCS::FParameters>();
	Parameters->TextureCount 			= HAIR_CARDS_MAX_TEXTURE_COUNT;
	Parameters->LayoutIndex				= uint32(Layout);
	Parameters->bClearPass				= bClearPass ? 1u : 0u;
	Parameters->Resolution				= Resolution;
	Parameters->DepthTestTexture		= DepthTestTexture;
	Parameters->OriginalTriangleMaskTexture= OriginalTriangleMaskTexture;
	Parameters->SourceTriangleMaskTexture  = SourceTriangleMaskTexture;
	Parameters->TargetTriangleMaskTexture  = GraphBuilder.CreateUAV(TargetTriangleMaskTexture);
	Parameters->SourceTexture 			= SourceTexture;
	Parameters->TargetTexture 			= GraphBuilder.CreateUAV(TargetTexture);

	TShaderMapRef<FHairStrandTextureDilationCS> ComputeShader(ShaderMap);
	FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(Resolution, FIntPoint(8, 8));

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrandsTexture::Dilation"),
		ComputeShader,
		Parameters,
		DispatchCount);
}

static void InternalAllocateStrandsTexture(UTexture2D* Out, const FIntPoint& Resolution, uint32 MipCount, EPixelFormat Format, ETextureSourceFormat SourceFormat)
{
#if WITH_EDITOR
	Out->PreEditChange(nullptr);
#endif

	FTextureFormatSettings FormatSettings;
	FormatSettings.SRGB = false;
	FormatSettings.CompressionNone = true;
	FormatSettings.CompressionSettings = TC_Masks;
#if WITH_EDITORONLY_DATA
	Out->Source.Init(Resolution.X, Resolution.Y, 1, MipCount, SourceFormat, nullptr);
	Out->SetLayerFormatSettings(0, FormatSettings);
#endif // #if WITH_EDITORONLY_DATA

	Out->SetPlatformData(new FTexturePlatformData());
	Out->GetPlatformData()->SizeX = Resolution.X;
	Out->GetPlatformData()->SizeY = Resolution.Y;
	Out->GetPlatformData()->PixelFormat = Format;

	//Out->UpdateResource();
}

static void InternalAllocateStrandsTexture_Generic(UTexture2D* Out, const FIntPoint& Resolution, uint32 MipCount)
{
	InternalAllocateStrandsTexture(Out, Resolution, 1, PF_B8G8R8A8, ETextureSourceFormat::TSF_BGRA8);
}

#if WITH_EDITOR
FStrandsTexturesOutput FGroomTextureBuilder::CreateGroomStrandsTexturesTexture(const UGroomAsset* GroomAsset, uint32 InResolution, EHairTextureLayout InLayout)
{
	FStrandsTexturesOutput Output;

	if (!GroomAsset)
	{
		return Output;
	}

	const FString PackageName = GroomAsset->GetOutermost()->GetName();
	const FIntPoint Resolution(InResolution, InResolution);
	const uint32 TextureCount = GetHairTextureLayoutTextureCount(InLayout);
	Output.Textures.SetNum(TextureCount);
	for (uint32 TextureIt=0;TextureIt<TextureCount;++TextureIt)
	{
		Output.Textures[TextureIt] = FHairStrandsCore::CreateTexture(PackageName, Resolution, FString(TEXT("_")) + GetHairTextureLayoutTextureName(InLayout, TextureIt, false/*bDetail*/), InternalAllocateStrandsTexture_Generic);
	}
	return Output;
}
#endif

class FHairStrandsTextureVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTextureVS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTextureVS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(uint32, UVsChannelIndex)
		SHADER_PARAMETER(uint32, UVsChannelCount)
		SHADER_PARAMETER_SRV(Buffer, VertexBuffer)
		SHADER_PARAMETER_SRV(Buffer, UVsBuffer)
		SHADER_PARAMETER_SRV(Buffer, NormalsBuffer)
	END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_VERTEX"), 1);
	}
};

class FHairStrandsTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTexturePS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(uint32, UVsChannelIndex)
		SHADER_PARAMETER(uint32, UVsChannelCount)
		SHADER_PARAMETER_SRV(Buffer, VertexBuffer)
		SHADER_PARAMETER_SRV(Buffer, UVsBuffer)
		SHADER_PARAMETER_SRV(Buffer, NormalsBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_PIXEL"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsTextureVS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FHairStrandsTexturePS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceRawParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCommonParameters, Common)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceResourceRawParameters, Resources)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCullingRawParameters, Culling)
END_SHADER_PARAMETER_STRUCT()

class FHairStrandsTextureCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTextureCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(FIntPoint, TileSize)
		SHADER_PARAMETER(FIntPoint, TileOffsetInPixels)
		SHADER_PARAMETER(uint32, LayoutIndex)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(float, MaxDistance)
		SHADER_PARAMETER(int32, TracingDirection)
		SHADER_PARAMETER_STRUCT(FHairStrandsInstanceRawParameters, InVF)

		SHADER_PARAMETER(uint32, UVsChannelIndex)
		SHADER_PARAMETER(uint32, UVsChannelCount)

		SHADER_PARAMETER(FVector3f, Voxel_MinBound)
		SHADER_PARAMETER(FVector3f, Voxel_MaxBound)
		SHADER_PARAMETER(FIntVector, Voxel_Resolution)
		SHADER_PARAMETER(float, Voxel_Size)
		SHADER_PARAMETER(uint32, Voxel_MaxSegmentPerVoxel)

		SHADER_PARAMETER(uint32, Voxel_OffsetAndCount_MaxCount)
		SHADER_PARAMETER(uint32, Voxel_Data_MaxCount)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, Voxel_OffsetAndCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, Voxel_Data)

		SHADER_PARAMETER_SRV(Buffer, VertexBuffer)
		SHADER_PARAMETER_SRV(Buffer, UVsBuffer)
		SHADER_PARAMETER_SRV(Buffer, NormalsBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutTexture)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PositionTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TangentTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BitangentTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, NormalTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UVTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_COMPUTE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsTextureCS, "/Engine/Private/HairStrands/HairStrandsTexturesGeneration.usf", "MainCS", SF_Compute);


const int32 HairStrandsTextureTileSize = 1024;

static FIntPoint GetTileResolution(FIntPoint In)
{
	FIntPoint Out;
	Out.X = FMath::DivideAndRoundUp(In.X, HairStrandsTextureTileSize);
	Out.Y = FMath::DivideAndRoundUp(In.Y, HairStrandsTextureTileSize);
	return Out;
}

struct FHairStrandsRDGTextures
{
	EHairTextureLayout Layout = EHairTextureLayout::Layout0;
	FIntPoint TileCount = FIntPoint::ZeroValue; // Total tile count
	FIntPoint TileCoord = FIntPoint::ZeroValue; // Current tile coord

	FRDGTextureRef Texture;
	FRDGTextureRef TriangleMaskTexture = nullptr;
	FRDGTextureRef DepthTestTexture = nullptr;

	bool IsCompleted() const { return TileCoord.X >= TileCount.X || TileCoord.Y >= TileCount.Y;  }
};

struct FHairStrandsRTs
{
	EHairTextureLayout Layout = EHairTextureLayout::Layout0;
	FIntPoint TileCount; // Total tile count
	FIntPoint TileCoord; // Current tile coord

	TRefCountPtr<IPooledRenderTarget> Texture;
	TRefCountPtr<IPooledRenderTarget> TriangleMaskTexture;
	TRefCountPtr<IPooledRenderTarget> DepthTestTexture;
	
	bool IsCompleted() const { return TileCoord.X >= TileCount.X || TileCoord.Y >= TileCount.Y; }
};

struct FStrandsTexturesReadback
{
	TArray<TUniquePtr<FRHIGPUTextureReadback>> Textures;
	uint32 NotReadyFrameCount = 0;

	bool IsReady() const
	{
		for (const TUniquePtr<FRHIGPUTextureReadback>& Texture : Textures)
		{
			if (!Texture->IsReady())
			{
				return false;
			}
		}
		return true;
	}
};

// Decompose into tile
static void InternalGenerateHairStrandsTextures(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FShaderPrintData* ShaderPrintData,
	const bool bClear,
	const float InMaxDistance, 
	const int32 InTracingDirection,

	const uint32 VertexCount,
	const uint32 PrimitiveCount,

	const uint32 VertexBaseIndex,
	const uint32 IndexBaseIndex,

	const uint32 UVsChannelIndex,
	const uint32 UVsChannelCount,

	FRHIBuffer* InMeshIndexBuffer,
	FRHIShaderResourceView* InMeshVertexBuffer,
	FRHIShaderResourceView* InMeshUVsBuffer,
	FRHIShaderResourceView* InMeshNormalsBuffer,

	const FVector3f& VoxelMinBound,
	const FVector3f& VoxelMaxBound,
	const FIntVector& VoxelResolution,
	float VoxelSize,
	uint32 VoxelMaxSegmentPerVoxel,
	FRDGBufferRef VoxelOffsetAndCount,
	FRDGBufferRef VoxelData,
	
	FHairStrandsInstanceRawParameters& Instance,
	FHairStrandsRDGTextures& Out)
{
	const FIntPoint OutputResolution = Out.Texture->Desc.Extent;
	const FIntPoint OutTileCoord = Out.TileCoord;
	const int32 TileSize = HairStrandsTextureTileSize;
	const FIntPoint TileOffsetInPixels(OutTileCoord.X * TileSize, OutTileCoord.Y * TileSize);

	FRDGTextureRef PositionTexture 	= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputResolution, PF_A32B32G32R32F, 	FClearValueBinding(FLinearColor(0, 0, 0, 1)), TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("HairTextureGeneration.Position"));
	FRDGTextureRef TangentTexture 	= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputResolution, PF_FloatRGBA, 		FClearValueBinding(FLinearColor(0, 0, 0, 1)), TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("HairTextureGeneration.Tangent"));
	FRDGTextureRef BitangentTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputResolution, PF_FloatRGBA, 		FClearValueBinding(FLinearColor(0, 0, 0, 1)), TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("HairTextureGeneration.Bitangent"));
	FRDGTextureRef NormalTexture 	= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputResolution, PF_FloatRGBA, 		FClearValueBinding(FLinearColor(0, 0, 0, 1)), TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("HairTextureGeneration.Normal"));
	FRDGTextureRef UVTexture 		= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputResolution, PF_G32R32F, 		FClearValueBinding(FLinearColor(0, 0, 0, 1)), TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("HairTextureGeneration.UV"));

	// Raster geometry
	// This pass emits a GBuffer like output. It avoids having multiple PS invocations due to overdraw and lack of early-Z testing (due to SV_Depth output)
	{
		FHairStrandsTexturePS::FParameters* ParametersPS = GraphBuilder.AllocParameters<FHairStrandsTexturePS::FParameters>();
		ParametersPS->OutputResolution = OutputResolution;
		ParametersPS->VertexCount = VertexCount;
		ParametersPS->VertexBuffer = InMeshVertexBuffer;
		ParametersPS->UVsBuffer = InMeshUVsBuffer;
		ParametersPS->NormalsBuffer = InMeshNormalsBuffer;
		ParametersPS->UVsChannelIndex = UVsChannelIndex;
		ParametersPS->UVsChannelCount = UVsChannelCount;
		ParametersPS->RenderTargets[0] = FRenderTargetBinding(PositionTexture, ERenderTargetLoadAction::EClear);
		ParametersPS->RenderTargets[1] = FRenderTargetBinding(TangentTexture, ERenderTargetLoadAction::EClear);
		ParametersPS->RenderTargets[2] = FRenderTargetBinding(BitangentTexture, ERenderTargetLoadAction::EClear);
		ParametersPS->RenderTargets[3] = FRenderTargetBinding(NormalTexture, ERenderTargetLoadAction::EClear);
		ParametersPS->RenderTargets[4] = FRenderTargetBinding(UVTexture, ERenderTargetLoadAction::EClear);
		ParametersPS->RenderTargets[5] = FRenderTargetBinding(Out.TriangleMaskTexture, bClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad);
		ParametersPS->RenderTargets.DepthStencil = FDepthStencilBinding(
			Out.DepthTestTexture,
			bClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ENoAction,
			FExclusiveDepthStencil::DepthWrite_StencilNop);

		TShaderMapRef<FHairStrandsTextureVS> VertexShader(ShaderMap);
		TShaderMapRef<FHairStrandsTexturePS> PixelShader(ShaderMap);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::TexturePS"),
			ParametersPS,
			ERDGPassFlags::Raster,
			[ParametersPS, VertexShader, PixelShader, InMeshIndexBuffer, VertexCount, PrimitiveCount, IndexBaseIndex, VertexBaseIndex, OutputResolution, OutTileCoord, TileOffsetInPixels](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FHairStrandsTextureVS::FParameters ParametersVS;
				ParametersVS.OutputResolution	= ParametersPS->OutputResolution;
				ParametersVS.VertexCount 		= ParametersPS->VertexCount;
				ParametersVS.VertexBuffer 		= ParametersPS->VertexBuffer;
				ParametersVS.UVsBuffer 			= ParametersPS->UVsBuffer;
				ParametersVS.NormalsBuffer 		= ParametersPS->NormalsBuffer;
				ParametersVS.UVsChannelIndex	= ParametersPS->UVsChannelIndex;
				ParametersVS.UVsChannelCount	= ParametersPS->UVsChannelCount;

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<
					CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
					CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
					CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
					CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
					CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero> ::GetRHI();

				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_LessEqual>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *ParametersPS);

				RHICmdList.SetStreamSource(0, nullptr, 0);
				RHICmdList.SetViewport(0, 0, 0.0f, OutputResolution.X, OutputResolution.Y, 1.0f);
			
				// The visibility pass is no longer tile in order to avoid issue at tile borde, preventing to compute derivative during the tracing step
				RHICmdList.DrawIndexedPrimitive(InMeshIndexBuffer, VertexBaseIndex, 0, VertexCount, IndexBaseIndex, PrimitiveCount, 1);
			});
	}

	// Evaluate material
	{
		FHairStrandsTextureCS::FParameters* ParametersCS = GraphBuilder.AllocParameters<FHairStrandsTextureCS::FParameters>();
		ParametersCS->TileSize = TileSize;
		ParametersCS->TileOffsetInPixels = TileOffsetInPixels;
		ParametersCS->LayoutIndex = uint32(Out.Layout);
		ParametersCS->OutputResolution = OutputResolution;
		ParametersCS->VertexCount = VertexCount;
		ParametersCS->VertexBuffer = InMeshVertexBuffer;
		ParametersCS->UVsBuffer = InMeshUVsBuffer;
		ParametersCS->NormalsBuffer = InMeshNormalsBuffer;
		ParametersCS->MaxDistance = FMath::Abs(InMaxDistance);
		ParametersCS->TracingDirection = InTracingDirection;
		ParametersCS->UVsChannelIndex = UVsChannelIndex;
		ParametersCS->UVsChannelCount = UVsChannelCount;
		ParametersCS->InVF = Instance;
		ParametersCS->Voxel_MinBound = VoxelMinBound;
		ParametersCS->Voxel_MaxBound = VoxelMaxBound;
		ParametersCS->Voxel_Resolution = VoxelResolution;
		ParametersCS->Voxel_Size = VoxelSize;
		ParametersCS->Voxel_MaxSegmentPerVoxel = VoxelMaxSegmentPerVoxel;
		ParametersCS->Voxel_OffsetAndCount = GraphBuilder.CreateSRV(VoxelOffsetAndCount);
		ParametersCS->Voxel_Data = GraphBuilder.CreateSRV(VoxelData);
		ParametersCS->Voxel_OffsetAndCount_MaxCount = VoxelOffsetAndCount->Desc.NumElements;
		ParametersCS->Voxel_Data_MaxCount = VoxelData->Desc.NumElements;
		
		ParametersCS->PositionTexture 	= PositionTexture;
		ParametersCS->TangentTexture 	= TangentTexture;
		ParametersCS->BitangentTexture  = BitangentTexture;
		ParametersCS->NormalTexture 	= NormalTexture;
		ParametersCS->UVTexture 		= UVTexture;
		ParametersCS->OutTexture 		= GraphBuilder.CreateUAV(Out.Texture);

		if (ShaderPrintData)
		{
			ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, ParametersCS->ShaderPrintParameters);
		}

		TShaderMapRef<FHairStrandsTextureCS> ComputeShader(ShaderMap);
		const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(TileSize, 8), FMath::DivideAndRoundUp(TileSize, 8), 1);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::TextureCS"),
			ComputeShader,
			ParametersCS,
			DispatchCount);
	}
}


static FHairStrandsRDGTextures CreateTextures(FRDGBuilder& GraphBuilder, FIntPoint InResolution, EHairTextureLayout Layout)
{
	FHairStrandsRDGTextures Out;
	Out.Layout = Layout;

	FRDGTextureDesc Desc = FRDGTextureDesc::Create2DArray(InResolution, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ShaderResource | TexCreate_External, HAIR_CARDS_MAX_TEXTURE_COUNT);
	Out.Texture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.Texture"));

	// Textures for depth testing
	Desc.Format = PF_R32_UINT;
	Desc.ClearValue = FClearValueBinding::Black;
	Out.TriangleMaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.TriangleMaskTexture"));
	
	Desc.ClearValue = FClearValueBinding(1);
	Desc.Format = PF_DepthStencil;
	Desc.Flags = TexCreate_DepthStencilTargetable;
	Out.DepthTestTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.DepthTestTexture"));

	Out.TileCount = GetTileResolution(InResolution);
	Out.TileCoord = FIntPoint(0,0);

	return Out;
}

static FHairStrandsRDGTextures RegisterTextures(FRDGBuilder& GraphBuilder, const FHairStrandsRTs& In)
{
	FHairStrandsRDGTextures Out;
	Out.Layout = In.Layout;
	Out.Texture = GraphBuilder.RegisterExternalTexture(In.Texture);
	Out.DepthTestTexture = GraphBuilder.RegisterExternalTexture(In.DepthTestTexture);
	Out.TriangleMaskTexture = GraphBuilder.RegisterExternalTexture(In.TriangleMaskTexture);
	Out.TileCoord = In.TileCoord;
	Out.TileCount = In.TileCount;
	return Out;
}

static void ExportTextures(FRDGBuilder& GraphBuilder, const FHairStrandsRDGTextures& In, FHairStrandsRTs& Out)
{
	Out.Layout = In.Layout;
	GraphBuilder.QueueTextureExtraction(In.Texture, &Out.Texture);
	GraphBuilder.QueueTextureExtraction(In.TriangleMaskTexture, &Out.TriangleMaskTexture);
	GraphBuilder.QueueTextureExtraction(In.DepthTestTexture, &Out.DepthTestTexture);
	Out.TileCoord = In.TileCoord;
	Out.TileCount = In.TileCount;
}

void GetHairStrandsAttributeParameter(const FHairStrandsBulkData& In, FHairStrandsInstanceAttributeParameters& Out);

static bool TraceTextures(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FStrandsTexturesInfo& InInfo,
	const FShaderPrintData* ShaderPrintData,
	FHairStrandsRDGTextures& Output)
{
	USkeletalMesh* SkeletalMesh = (USkeletalMesh*)InInfo.SkeletalMesh;
	UStaticMesh* StaticMesh = (UStaticMesh*)InInfo.StaticMesh;

	if (!SkeletalMesh && !StaticMesh)
	{
		return true;
	}

	const bool bUseSkeletalMesh = SkeletalMesh != nullptr;

	bool bClear = Output.TileCoord == FIntPoint(0,0);
	const uint32 GroupCount = InInfo.GroomAsset->GetNumHairGroups();
	for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
	{
		if (!InInfo.GroupIndices.Contains(GroupIndex))
		{
			continue;
		}

		FHairGroupResources& GroupResource = InInfo.GroomAsset->GetHairGroupsResources()[GroupIndex];
		FHairGroupsRendering& RenderingData = InInfo.GroomAsset->GetHairGroupsRendering()[GroupIndex];

		FRDGBufferRef VoxelOffsetAndCount = GraphBuilder.RegisterExternalBuffer(GroupResource.Debug.Resource->VoxelOffsetAndCount);
		FRDGBufferRef VoxelData = GraphBuilder.RegisterExternalBuffer(GroupResource.Debug.Resource->VoxelData);

		{
			FRHIShaderResourceView* PositionBuffer = nullptr;
			FRHIShaderResourceView* UVsBuffer = nullptr;
			FRHIShaderResourceView* TangentBuffer = nullptr;
			FBufferRHIRef IndexBuffer = nullptr;
			uint32 TotalVertexCount = 0;
			uint32 TotalIndexCount = 0;
			uint32 UVsChannelIndex = InInfo.UVChannelIndex;
			uint32 UVsChannelCount = 0;
			uint32 NumPrimitives = 0;
			uint32 IndexBaseIndex = 0;
			uint32 VertexBaseIndex = 0;

			// Skeletal mesh
			if (bUseSkeletalMesh)
			{
				const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
				const uint32 MeshLODIndex = FMath::Clamp(InInfo.LODIndex, 0u, uint32(RenderData->LODRenderData.Num()));

				const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[MeshLODIndex];
				const uint32 SectionCount = LODData.RenderSections.Num();
				const uint32 SectionIdx = FMath::Clamp(InInfo.SectionIndex, 0u, SectionCount);
				const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIdx];

				PositionBuffer = LODData.StaticVertexBuffers.PositionVertexBuffer.GetSRV();
				UVsBuffer = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();
				TangentBuffer = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
				IndexBuffer = LODData.MultiSizeIndexContainer.GetIndexBuffer()->IndexBufferRHI;
				TotalIndexCount = LODData.MultiSizeIndexContainer.GetIndexBuffer()->Num();
				UVsChannelCount = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
				TotalVertexCount = LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();

				NumPrimitives = Section.NumTriangles;
				IndexBaseIndex = Section.BaseIndex;
				VertexBaseIndex = Section.BaseVertexIndex;
			}
			// Static mesh
			else
			{
				const uint32 MeshLODIndex = FMath::Clamp(InInfo.LODIndex, 0u, uint32(StaticMesh->GetNumLODs()));

				const FStaticMeshLODResources& LODData = StaticMesh->GetLODForExport(MeshLODIndex);
				const uint32 SectionCount = LODData.Sections.Num();
				const uint32 SectionIdx = FMath::Clamp(InInfo.SectionIndex, 0u, SectionCount);
				const FStaticMeshSection& Section = LODData.Sections[SectionIdx];

				PositionBuffer = LODData.VertexBuffers.PositionVertexBuffer.GetSRV();
				UVsBuffer = LODData.VertexBuffers.StaticMeshVertexBuffer.GetTexCoordsSRV();
				TangentBuffer = LODData.VertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
				IndexBuffer = LODData.IndexBuffer.IndexBufferRHI;
				TotalIndexCount = LODData.IndexBuffer.GetNumIndices();
				UVsChannelCount = LODData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
				TotalVertexCount = LODData.VertexBuffers.PositionVertexBuffer.GetNumVertices();

				NumPrimitives = Section.NumTriangles;
				IndexBaseIndex = Section.FirstIndex;
				VertexBaseIndex = 0;
			}

			// Ensure the rest resources are loaded when rendering the strands textures
			// Use EHairResourceLoadingType::Async, as the we can't use ::Sync because the resource is expected to be loading in async
			// If a streaming request is in-flight, ensure it is done and allocate the resource
			GroupResource.Strands.RestResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async);
		#if WITH_EDITORONLY_DATA
			if (GroupResource.Strands.RestResource->StreamingRequest.DDCRequestOwner)
			{
				GroupResource.Strands.RestResource->StreamingRequest.DDCRequestOwner->Wait();
				GroupResource.Strands.RestResource->Allocate(GraphBuilder, EHairResourceLoadingType::Async);
			}
		#endif	

			// Manually setting up parameters, since we don't have a real instance
			FHairStrandsInstanceRawParameters Instance;
			Instance.Common.GroupIndex = GroupIndex;
			Instance.Common.GroupCount = GroupCount;
			Instance.Common.PointCount = GroupResource.Strands.RestResource->BulkData.GetNumPoints();
			Instance.Common.CurveCount = GroupResource.Strands.RestResource->BulkData.GetNumCurves();
			Instance.Common.SourcePointCount = GroupResource.Strands.RestResource->BulkData.GetNumSourcePoints();
			Instance.Common.SourceCurveCount = GroupResource.Strands.RestResource->BulkData.GetNumSourceCurves();
			Instance.Common.Radius = RenderingData.GeometrySettings.HairWidth * 0.5f;
			Instance.Common.RootScale = 1.0f;
			Instance.Common.TipScale = 1.0f;
			Instance.Common.LengthScale = 1.0f;
			Instance.Common.Length = GroupResource.Strands.RestResource->BulkData.Header.MaxLength;
			Instance.Common.PositionOffset = FVector3f(GroupResource.Strands.RestResource->GetPositionOffset());
			GetHairStrandsAttributeParameter(GroupResource.Strands.RestResource->BulkData, Instance.Common.Attributes);

			Instance.Resources.PositionBuffer = GroupResource.Strands.RestResource->PositionBuffer.SRV;
			Instance.Resources.CurveAttributeBuffer = GroupResource.Strands.RestResource->CurveAttributeBuffer.SRV;
			Instance.Resources.PointAttributeBuffer = GroupResource.Strands.RestResource->PointAttributeBuffer.SRV;
			Instance.Resources.PointToCurveBuffer = GroupResource.Strands.RestResource->PointToCurveBuffer.SRV;

			InternalGenerateHairStrandsTextures(
				GraphBuilder,
				ShaderMap,
				ShaderPrintData,
				bClear,
				InInfo.MaxTracingDistance,
				InInfo.TracingDirection,

				TotalVertexCount,
				NumPrimitives,
				VertexBaseIndex,
				IndexBaseIndex,

				UVsChannelIndex,
				UVsChannelCount,

				IndexBuffer,
				PositionBuffer,
				UVsBuffer,
				TangentBuffer,

				GroupResource.Debug.Resource->VoxelDescription.VoxelMinBound,
				GroupResource.Debug.Resource->VoxelDescription.VoxelMaxBound,
				GroupResource.Debug.Resource->VoxelDescription.VoxelResolution,
				GroupResource.Debug.Resource->VoxelDescription.VoxelSize,
				GroupResource.Debug.Resource->VoxelDescription.MaxSegmentPerVoxel,
				VoxelOffsetAndCount,
				VoxelData,
				Instance,
				Output);


			bClear = false;
		}
	}

	// Increment processed tile
	// When debugging, never complete texture generation to force pass execution every frame
	#if !DEBUG_TEXTURE_GENERATION
	++Output.TileCoord.X;
	if (Output.TileCoord.X >= Output.TileCount.X)
	{
		Output.TileCoord.X = 0;
		++Output.TileCoord.Y;
	}
	#endif

	return true;
}

static FHairStrandsRDGTextures DilateTextures(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FHairStrandsRDGTextures& In, uint32 Dilation)
{
	const FIntPoint OutputResolution = In.Texture->Desc.Extent;

	FHairStrandsRDGTextures Out[2];
	// Overwrite the TriangleMaskTexture which will be reinitialized from the depth test texture over the iteration
	Out[0] = In;
	Out[0].TriangleMaskTexture = CreateTextures(GraphBuilder, OutputResolution, In.Layout).TriangleMaskTexture;
	Out[1] = CreateTextures(GraphBuilder, OutputResolution, In.Layout);

	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out[0].TriangleMaskTexture), 0u);

	uint32 SourceIndex = 0;
	uint32 TargetIndex = 0;
	for (uint32 DilationIt=0; DilationIt < Dilation; ++DilationIt)
	{
		TargetIndex = (SourceIndex + 1) % 2;

		AddTextureDilationPass(
			GraphBuilder,
			ShaderMap,
			OutputResolution,
			In.Layout,
			DilationIt == 0,
			In.DepthTestTexture,
			In.TriangleMaskTexture,
			Out[SourceIndex].TriangleMaskTexture,
			Out[TargetIndex].TriangleMaskTexture,
			Out[SourceIndex].Texture,
			Out[TargetIndex].Texture);

		SourceIndex = TargetIndex;
	}
	return Out[SourceIndex];
}

static void CopyTextures(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const EHairTextureLayout InLayout, const FHairStrandsRDGTextures& In, FStrandsTexturesReadback& Out)
{
	const uint32 TextureCount = GetHairTextureLayoutTextureCount(InLayout);
	for (uint32 TextureIt=0; TextureIt<TextureCount;++TextureIt)
	{
		// Split texture array into separate texture
		FRDGTextureRef Texture = AddTextureCopyPass(GraphBuilder, ShaderMap, TextureIt, In.Texture);

		// Enqueue copy to readback texture
		AddEnqueueCopyPass(GraphBuilder, Out.Textures[TextureIt].Get(), Texture);
	}
}

static void CopyReadbackToTexture(TUniquePtr<FRHIGPUTextureReadback>& In, UTexture2D* Out)
{
#if WITH_EDITORONLY_DATA
	uint32 BytePerPixels = 0;
	uint32 Width = Out->Source.GetSizeX();
	uint32 Height = Out->Source.GetSizeY();

	ETextureSourceFormat SrcFormat = Out->Source.GetFormat();
	switch (SrcFormat)
	{
	case TSF_BGRA8:
		BytePerPixels = 4; break;
	case TSF_G16:
		BytePerPixels = 2; break;
	};

	int32 RowPitchInPixels = 0;
	void* LockedData = In->Lock(RowPitchInPixels); // This forces a GPU stall
	uint8* InDataRGBA8 = (uint8*)LockedData;
	Out->Source.Init(Width, Height, 1, 1, SrcFormat, InDataRGBA8);
	In->Unlock();

	Out->DeferCompression = true; // This forces reloading data when the asset is saved
	Out->MarkPackageDirty();
#endif // WITH_EDITORONLY_DATA
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Asynchronous queuing for hair strands texture generation

struct FStrandsTexturesQuery
{
	FStrandsTexturesInfo		Info;
	FStrandsTexturesOutput		Output;
};

struct FStrandsTexturesProcess
{
	FStrandsTexturesInfo		Info;
	FHairStrandsRTs				InProgress;
	FStrandsTexturesReadback	Readback;
	FStrandsTexturesOutput		Output;
};

TQueue<FStrandsTexturesQuery>		GStrandsTexturesQueries;
TQueue<FStrandsTexturesProcess*>	GStrandsTexturesProcessQueue;
TQueue<FStrandsTexturesProcess*>	GStrandsTexturesReadbacks;

bool HasHairStrandsTexturesQueries()
{
	return !GStrandsTexturesQueries.IsEmpty() || !GStrandsTexturesProcessQueue.IsEmpty() || !GStrandsTexturesReadbacks.IsEmpty();
}

void RunHairStrandsTexturesQueries(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const struct FShaderPrintData* DebugShaderData)
{
	// Operations are ordered in reverse to ensure they are processed on independent frames to avoid TDR on heavy grooms.
	
	// 3. Readback data
	{
		TArray<FStrandsTexturesProcess*> NotReady;
		FStrandsTexturesProcess* R = nullptr;
		while (GStrandsTexturesReadbacks.Dequeue(R))
		{
			if (R)
			{
				if (R->Readback.IsReady() || ++R->Readback.NotReadyFrameCount > 32)
				{
					for (uint32 TextureIt = 0, TextureCount = R->Readback.Textures.Num(); TextureIt < TextureCount; ++TextureIt)
					{
						CopyReadbackToTexture(R->Readback.Textures[TextureIt], R->Output.Textures[TextureIt]);
					}
					R->Readback.Textures.Empty();
					delete R;
				}
				else
				{
					NotReady.Add(R);
				}
			}
		}
	
		for (FStrandsTexturesProcess* E : NotReady)
		{
			GStrandsTexturesReadbacks.Enqueue(E);
		}
	}

	// 2. Process passes
	{
		TArray<FStrandsTexturesProcess*> NotCompleted;
		FStrandsTexturesProcess* R = nullptr;
		while (GStrandsTexturesProcessQueue.Dequeue(R))
		{
			if (R)
			{
			#if DEBUG_TEXTURE_RENDERDOC
				RenderCaptureInterface::FScopedCapture RenderCapture(true, GraphBuilder, TEXT("PassX"));
			#endif
				FHairStrandsRDGTextures Textures = RegisterTextures(GraphBuilder, R->InProgress);

				TraceTextures(GraphBuilder, ShaderMap, R->Info, DebugShaderData, Textures);

			// When debugging, never complete texture generation to force pass execution every frame
			#if !DEBUG_TEXTURE_GENERATION
				if (Textures.IsCompleted())
				{
					Textures = DilateTextures(GraphBuilder, ShaderMap, Textures, R->Info.Dilation);
					CopyTextures(GraphBuilder, ShaderMap, R->Info.Layout, Textures, R->Readback);
					GStrandsTexturesReadbacks.Enqueue(R);
				}
				else
			#endif
				{
					ExportTextures(GraphBuilder, Textures, R->InProgress);
					NotCompleted.Add(R);
				}
			}
		}
		
		for (FStrandsTexturesProcess* E : NotCompleted)
		{
			GStrandsTexturesProcessQueue.Enqueue(E);
		}
	}

	// 1. Create textures
	{
		FStrandsTexturesQuery Q;
		while (GStrandsTexturesQueries.Dequeue(Q))
		{
			FStrandsTexturesProcess* R = new FStrandsTexturesProcess();

			auto GetTextureName = [](uint32 In)
			{
				switch (In)
				{	
					case 0: return TEXT("Hair.Readback.Texture0");
					case 1: return TEXT("Hair.Readback.Texture1");
					case 2: return TEXT("Hair.Readback.Texture2");
					case 3: return TEXT("Hair.Readback.Texture3");
					case 4: return TEXT("Hair.Readback.Texture4");
					case 5: return TEXT("Hair.Readback.Texture5");
				}
				return  TEXT("Hair.Readback.Unknown");
			};

			const uint32 TextureCount = GetHairTextureLayoutTextureCount(Q.Info.Layout);
			R->Readback.Textures.SetNum(TextureCount);
			for (uint32 TextureIt = 0; TextureIt < TextureCount; ++TextureIt)
			{
				R->Readback.Textures[TextureIt] = MakeUnique<FRHIGPUTextureReadback>(GetTextureName(TextureIt));
			}	
			R->Output = Q.Output;
			R->Info = Q.Info;

		#if DEBUG_TEXTURE_RENDERDOC
			RenderCaptureInterface::FScopedCapture RenderCapture(true, GraphBuilder, TEXT("Pass0"));
		#endif

			FHairStrandsRDGTextures Textures = CreateTextures(GraphBuilder, FIntPoint(FMath::Clamp(Q.Info.Resolution, 512u, 16384u), FMath::Clamp(Q.Info.Resolution, 512u, 16384u)), Q.Info.Layout);
			AddTextureClearPass(GraphBuilder, ShaderMap, Q.Info.Resolution, Q.Info.Layout, Textures.Texture);

			TraceTextures(GraphBuilder, ShaderMap, Q.Info, DebugShaderData, Textures);

			if (Textures.IsCompleted())
			{
				Textures = DilateTextures(GraphBuilder, ShaderMap, Textures, R->Info.Dilation);
				CopyTextures(GraphBuilder, ShaderMap, R->Info.Layout, Textures, R->Readback);
				GStrandsTexturesReadbacks.Enqueue(R);
			}
			else
			{
				ExportTextures(GraphBuilder, Textures, R->InProgress);
				GStrandsTexturesProcessQueue.Enqueue(R);
			}
		}
	}
}

#if WITH_EDITOR
void FGroomTextureBuilder::BuildStrandsTextures(const FStrandsTexturesInfo& InInfo, const FStrandsTexturesOutput& Output)
{
	GStrandsTexturesQueries.Enqueue({ InInfo, Output });
}
#endif

#undef LOCTEXT_NAMESPACE

