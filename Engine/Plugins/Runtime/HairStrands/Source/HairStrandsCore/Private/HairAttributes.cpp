// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairAttributes.h"
#include "HairStrandsDefinitions.h"
#include "HairStrandsDatas.h"
#include "Hash/CityHash.h"

namespace HairAttribute
{
	const FName Vertex::Color("groom_color");
	const FName Vertex::Roughness("groom_roughness");
	const FName Vertex::AO("groom_ao");
	const FName Vertex::Position("position");
	const FName Vertex::Width("groom_width");

	const FName Strand::Color("groom_color");
	const FName Strand::Roughness("groom_roughness");
	const FName Strand::GroupID("groom_group_id");
	const FName Strand::Guide("groom_guide");
	const FName Strand::ID("groom_id");
	const FName Strand::ClumpID("groom_clumpid");
	const FName Strand::RootUV("groom_root_uv");
	const FName Strand::VertexCount("vertexcount");
	const FName Strand::Width("groom_width");
	const FName Strand::ClosestGuides("groom_closest_guides");
	const FName Strand::GuideWeights("groom_guide_weights");
	const FName Strand::BasisType("groom_basis_type");
	const FName Strand::CurveType("groom_curve_type");
	const FName Strand::Knots("groom_knots");
	const FName Strand::GroupName("groom_group_name");
	const FName Strand::GroupCardsName("groom_group_cards_id");

	const FName Groom::Color("groom_color");
	const FName Groom::Roughness("groom_roughness");
	const FName Groom::Width("groom_width");
	const FName Groom::MajorVersion("groom_version_major");
	const FName Groom::MinorVersion("groom_version_minor");
	const FName Groom::Tool("groom_tool");
	const FName Groom::Properties("groom_properties");
}

static_assert(uint32(EHairAttribute::Count) < 32u);
bool HasHairAttribute(uint32 In, EHairAttribute InAttribute)
{ 
	return (In & (1u << uint32(InAttribute))) != 0; 
}

void SetHairAttribute(uint32& Out, EHairAttribute InAttribute) 
{ 
	Out |= (1u << uint32(InAttribute)); 
}

bool HasHairAttributeFlags(uint32 In, EHairAttributeFlags InFlag)
{ 
	return (In & (1u << uint32(InFlag))) != 0; 
}

void SetHairAttributeFlags(uint32& Out, EHairAttributeFlags InFlag) 
{ 
	Out |= (1u << uint32(InFlag));
}

const TCHAR* GetHairAttributeText(EHairAttribute In, uint32 InFlags)
{
	// If a new optional attribute is added, please add its UI/text description here
	static_assert(uint32(EHairAttribute::Count) == 8);

	switch (In)
	{
	case EHairAttribute::Width:						return TEXT("Width");
	case EHairAttribute::RootUV:					return HasHairAttributeFlags(InFlags, EHairAttributeFlags::HasRootUDIM) ? TEXT("RootUV(UDIM)") : TEXT("RootUV");
	case EHairAttribute::ClumpID:					return HasHairAttributeFlags(InFlags, EHairAttributeFlags::HasMultipleClumpIDs) ? TEXT("ClumpID(3)") : TEXT("ClumpID");
	case EHairAttribute::StrandID:					return TEXT("StrandID");
	case EHairAttribute::PrecomputedGuideWeights:	return TEXT("ImportedGuideWeights");
	case EHairAttribute::Color:						return TEXT("Color");
	case EHairAttribute::Roughness:					return TEXT("Roughness");
	case EHairAttribute::AO:						return TEXT("AO");
	}
	return TEXT("UNKNOWN");
}

bool FHairStrandsPoints::HasAttribute(EHairAttribute In) const
{
	if (In == EHairAttribute::Width)		{ return PointsRadius.Num() > 0; }
	if (In == EHairAttribute::Color)		{ return PointsBaseColor.Num() > 0; }
	if (In == EHairAttribute::Roughness)	{ return PointsRoughness.Num() > 0; }
	if (In == EHairAttribute::AO)			{ return PointsAO.Num() > 0; }
	return false;
}

bool FHairStrandsCurves::HasAttribute(EHairAttribute In) const
{
	if (In == EHairAttribute::RootUV)					{ return CurvesRootUV.Num() > 0; }
	if (In == EHairAttribute::PrecomputedGuideWeights)	{ return CurvesClosestGuideIDs.Num() > 0 && CurvesClosestGuideWeights.Num() > 0; }
	if (In == EHairAttribute::ClumpID)					{ return ClumpIDs.Num() > 0; }
	if (In == EHairAttribute::StrandID)					{ return StrandIDs.Num() > 0; }
	return false;
}

bool FHairStrandsCurves::HasPrecomputedWeights() const
{ 
	return HasAttribute(EHairAttribute::PrecomputedGuideWeights); 
}

uint32 FHairStrandsCommonDatas::GetAttributes() const
{
	uint32 Out = 0;
	for (uint32 It=0;It<uint32(EHairAttribute::Count); ++It)
	{	
		const EHairAttribute Attribute = (EHairAttribute)It;
		if (StrandsPoints.HasAttribute(Attribute) || StrandsCurves.HasAttribute(Attribute))
		{
			SetHairAttribute(Out, Attribute);
		}
	}
	return Out;
}

uint32 FHairStrandsCommonDatas::GetAttributeFlags() const
{
	return StrandsCurves.AttributeFlags;
}

void FHairStrandsDatas::CopyPoint(const FHairStrandsDatas& In, FHairStrandsDatas& Out, uint32 InAttributes, uint32 InIndex, uint32 OutIndex)
{
	// Multiply by MaxRadius as HairStrandsBuilder::BuildInternalData will recompute MaxRadius based on the actual Radius not the not the normalized radius.
	Out.StrandsPoints.PointsPosition[OutIndex]	= In.StrandsPoints.PointsPosition[InIndex];
	Out.StrandsPoints.PointsCoordU[OutIndex]	= In.StrandsPoints.PointsCoordU[InIndex];
	Out.StrandsPoints.PointsRadius[OutIndex]	= In.StrandsPoints.PointsRadius[InIndex];

	if (HasHairAttribute(InAttributes, EHairAttribute::Color))		{ Out.StrandsPoints.PointsBaseColor[OutIndex] = In.StrandsPoints.PointsBaseColor[InIndex]; }
	if (HasHairAttribute(InAttributes, EHairAttribute::Roughness))	{ Out.StrandsPoints.PointsRoughness[OutIndex] = In.StrandsPoints.PointsRoughness[InIndex]; }
	if (HasHairAttribute(InAttributes, EHairAttribute::AO))			{ Out.StrandsPoints.PointsAO[OutIndex] = In.StrandsPoints.PointsAO[InIndex]; }

	if(!Out.StrandsPoints.PointsMapping.IsEmpty())
	{ 
		Out.StrandsPoints.PointsMapping[OutIndex] = In.StrandsPoints.PointsMapping.IsValidIndex(InIndex) ? In.StrandsPoints.PointsMapping[InIndex] : InIndex;
	}
}


void FHairStrandsDatas::CopyPointLerp(const FHairStrandsDatas& In, FHairStrandsDatas& Out, uint32 InAttributes, uint32 InIndex0, uint32 InIndex1, float InAlpha, uint32 OutIndex)
{
	// Multiply by MaxRadius as HairStrandsBuilder::BuildInternalData will recompute MaxRadius based on the actual Radius not the not the normalized radius.
	Out.StrandsPoints.PointsPosition[OutIndex]	= FMath::Lerp(In.StrandsPoints.PointsPosition[InIndex0], In.StrandsPoints.PointsPosition[InIndex1], InAlpha);
	Out.StrandsPoints.PointsCoordU[OutIndex]	= FMath::Lerp(In.StrandsPoints.PointsCoordU[InIndex0], In.StrandsPoints.PointsCoordU[InIndex1], InAlpha);
	Out.StrandsPoints.PointsRadius[OutIndex]	= FMath::Lerp(In.StrandsPoints.PointsRadius[InIndex0], In.StrandsPoints.PointsRadius[InIndex1], InAlpha);

	if (HasHairAttribute(InAttributes, EHairAttribute::Color))		{ Out.StrandsPoints.PointsBaseColor[OutIndex]	= FMath::Lerp(In.StrandsPoints.PointsBaseColor[InIndex0], In.StrandsPoints.PointsBaseColor[InIndex1], InAlpha); }
	if (HasHairAttribute(InAttributes, EHairAttribute::Roughness))	{ Out.StrandsPoints.PointsRoughness[OutIndex]	= FMath::Lerp(In.StrandsPoints.PointsRoughness[InIndex0], In.StrandsPoints.PointsRoughness[InIndex1], InAlpha); }
	if (HasHairAttribute(InAttributes, EHairAttribute::AO))			{ Out.StrandsPoints.PointsAO[OutIndex]			= FMath::Lerp(In.StrandsPoints.PointsAO[InIndex0], In.StrandsPoints.PointsAO[InIndex1], InAlpha); }

	if (!Out.StrandsPoints.PointsMapping.IsEmpty())
	{
		const int32 PointIndex = (InAlpha < 0.5) ? InIndex0 : InIndex1;
		Out.StrandsPoints.PointsMapping[OutIndex] = In.StrandsPoints.PointsMapping.IsValidIndex(PointIndex) ? In.StrandsPoints.PointsMapping[PointIndex] : PointIndex;
	}
}

void FHairStrandsDatas::CopyCurve(const FHairStrandsDatas& In, FHairStrandsDatas& Out, uint32 InAttributes, uint32 InIndex, uint32 OutIndex)
{
	Out.StrandsCurves.CurvesLength[OutIndex] = In.StrandsCurves.CurvesLength[InIndex];
	if (HasHairAttribute(InAttributes, EHairAttribute::RootUV))
	{
		Out.StrandsCurves.CurvesRootUV[OutIndex] = In.StrandsCurves.CurvesRootUV[InIndex];
	}
	if (HasHairAttribute(InAttributes, EHairAttribute::PrecomputedGuideWeights))
	{
		Out.StrandsCurves.CurvesClosestGuideIDs[OutIndex] = In.StrandsCurves.CurvesClosestGuideIDs[InIndex];
		Out.StrandsCurves.CurvesClosestGuideWeights[OutIndex] = In.StrandsCurves.CurvesClosestGuideWeights[InIndex];
	}
	if (HasHairAttribute(InAttributes, EHairAttribute::StrandID))
	{
		Out.StrandsCurves.StrandIDs[OutIndex] = In.StrandsCurves.StrandIDs[InIndex];
	}
	if (HasHairAttribute(InAttributes, EHairAttribute::ClumpID))
	{
		Out.StrandsCurves.ClumpIDs[OutIndex] = In.StrandsCurves.ClumpIDs[InIndex];
	}

	if (!Out.StrandsCurves.CurvesMapping.IsEmpty())
	{
		Out.StrandsCurves.CurvesMapping[OutIndex] = In.StrandsCurves.CurvesMapping.IsValidIndex(InIndex) ? In.StrandsCurves.CurvesMapping[InIndex] : InIndex;
	}
}

FHairStrandsDatas::FHash FHairStrandsCommonDatas::GetHash() const
{
	FHairStrandsDatas::FHash Out;
	Out.CurveCountHash 	= CityHash64WithSeed((const char*)StrandsCurves.CurvesCount.GetData(), StrandsCurves.CurvesCount.Num() * sizeof(uint16), Out.CurveCountHash);
	Out.CurveOffsetHash = CityHash64WithSeed((const char*)StrandsCurves.CurvesOffset.GetData(), StrandsCurves.CurvesOffset.Num() * sizeof(uint32), Out.CurveOffsetHash);
	Out.PositionHash 	= CityHash64WithSeed((const char*)StrandsPoints.PointsPosition.GetData(), StrandsPoints.PointsPosition.Num() * sizeof(FVector3f), Out.PositionHash);
	return Out;
}