// Copyright Epic Games, Inc. All Rights Reserved. 

#include "HairCardsBuilder.h"
#include "Engine/Texture2D.h"
#include "HairStrandsCore.h"
#include "HairStrandsDatas.h"
#include "HairCardsDatas.h"
#include "GroomBuilder.h"

#include "Math/Box.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "GroomAsset.h"
#include "HairStrandsInterface.h"
#include "SceneView.h"
#include "Containers/ResourceArray.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "HAL/ConsoleManager.h"
#include "ShaderPrintParameters.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopedSlowTask.h"
#include "CommonRenderResources.h"
#include "Engine/StaticMesh.h"
#include "MeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshOperations.h"
#include "TextureResource.h"

#if WITH_EDITOR

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable:6294) // Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed.
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Core>
#include <Eigen/Dense>
#include <Eigen/Sparse>
#include <Eigen/SparseLU>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

static FBox ToFBox3d(const FBox3f& In)
{
	return FBox(FVector(In.Min), FVector(In.Max));
}

#define LOCTEXT_NAMESPACE "GroomCardBuilder"

////////////////////////////////////////////////////////////////////////////////////////////////
// Build public functions

namespace FHairCardsBuilder
{

FString GetVersion()
{
	// Important to update the version when cards building or importing changes
	return TEXT("13i");
}

#define HAIR_CARD_UV_ROOTUV 0
#define HAIR_CARD_UV_CARDUV 1
#define HAIR_CARD_UV_COUNT  2

// Example of layout with 3 texture coords
//  -----------------------------------------------  -----------------------------------------------
// |          Vertex0          ||          Vertex1          ||          Vertex2          ||
// | 0.xy   0.zw   1.xy   1.zw || 0.xy   0.zw   1.xy   1.zw || 0.xy   0.zw   1.xy   1.zw ||
// | UV0  | UV1  | UV2  | ---  || UV0  | UV1  | UV2  | ---  || UV0  | UV1  | UV2  | ---  ||
//  -----------------------------------------------  -----------------------------------------------
static void SetPackedUV(uint32 VertexIndex, uint32 UVIndex, uint32 UVCount, const FVector2f& InUV, TArray<FVector4f>& OutPackedUVs)
{
	const uint32 StrideInUint = FMath::DivideAndRoundUp(UVCount, 2u);
	uint32 UVVertexIndex = VertexIndex * StrideInUint + (UVIndex / 2);
	if (UVIndex % 2 == 0)
	{
		OutPackedUVs[UVVertexIndex].X = InUV.X; 
		OutPackedUVs[UVVertexIndex].Y = InUV.Y; 
	}
	else
	{
		OutPackedUVs[UVVertexIndex].Z = InUV.X; 
		OutPackedUVs[UVVertexIndex].W = InUV.Y; 
	}
}
static FVector2f GetPackedUV(uint32 VertexIndex, uint32 UVIndex, uint32 UVCount, const TArray<FVector4f>& InPackedUVs)
{
	const uint32 StrideInUint = FMath::DivideAndRoundUp(UVCount, 2u);
	const uint32 UVVertexIndex = VertexIndex * StrideInUint + (UVIndex / 2);
	const FVector4f PackedUV = InPackedUVs[UVVertexIndex];
	return (UVIndex % 2 == 0) ? FVector2f(PackedUV.X, PackedUV.Y) : FVector2f(PackedUV.Z, PackedUV.W);
}

static uint32 GetNumPackedUVs(uint32 PointCount, uint32 NumUVs)
{
	static_assert(FHairCardsGeometry::FUVChannel::GetCardUVIndex() == HAIR_CARD_UV_CARDUV);

	// Packed 2 UVs into a single FVector4f
	const uint32 StrideInUint = FMath::DivideAndRoundUp(NumUVs, 2u);
	return PointCount * StrideInUint;
}

static bool InternalCreateCardsGuides(
	FHairCardsGeometry& InCards,
	FHairStrandsRawDatas& OutGuides,
	TArray<float>& OutCardLengths)
{
	// Build the guides from the triangles that form the card
	// The guides are derived from the line that passes through the middle of each quad

	const uint32 NumCards = InCards.IndexOffsets.Num();
	const uint32 MaxGuidePoints = InCards.Indices.Num() / 3;
	OutCardLengths.SetNumZeroed(NumCards);

	OutGuides.StrandsPoints.PointsPosition.Reserve(MaxGuidePoints);
	OutGuides.StrandsPoints.PointsRadius.Reserve(MaxGuidePoints);
	OutGuides.StrandsPoints.PointsCoordU.Reserve(MaxGuidePoints);
	OutGuides.StrandsCurves.SetNum(NumCards, 0u);

	InCards.CoordU.SetNumZeroed(InCards.Positions.Num());
	InCards.LocalUVs.SetNumZeroed(InCards.Positions.Num());

	// Find the principal direction of the atlas by comparing the number of segment along U and along V
	bool bIsMainDirectionU = true;
	{
		uint32 MainDirectionUCount = 0;
		uint32 ValidCount = 0;
		for (uint32 CardIt = 0; CardIt < NumCards; ++CardIt)
		{
			const uint32 NumTriangles = InCards.IndexCounts[CardIt] / 3;
			const uint32 VertexOffset = InCards.IndexOffsets[CardIt];
			struct FIndexAndCoord { uint32 Index; float TexCoord; };
			struct FSimilarUVVertices
			{
				float TexCoord = 0;
				TArray<uint32> Indices;
				TArray<FIndexAndCoord> AllIndices; // Store vertex index and Tex.coord perpendicular to the principal axis (stored in TexCoord)
			};
			TArray<FSimilarUVVertices> SimilarVertexU;
			TArray<FSimilarUVVertices> SimilarVertexV;

			auto AddSimilarUV = [](TArray<FSimilarUVVertices>& In, uint32 Index, float TexCoord, float MinorTexCoord, float Threshold)
			{
				bool bFound = false;
				for (int32 It = 0, Count = In.Num(); It < Count; ++It)
				{
					if (FMath::Abs(In[It].TexCoord - TexCoord) < Threshold)
					{
						// We add only unique vertices per segment, so that the average position land in the center of the cards
						In[It].Indices.AddUnique(Index);
						In[It].AllIndices.Add({ Index, MinorTexCoord });
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					FSimilarUVVertices& SimilarUV = In.AddDefaulted_GetRef();
					SimilarUV.TexCoord = TexCoord;
					SimilarUV.Indices.Add(Index);
					SimilarUV.AllIndices.Add({ Index, MinorTexCoord });
				}
			};

			// Iterate over all triangles of a cards, and find vertices which share either same U or same V. We add them to separate lists. 
			// We then use the following heuristic: the main axis will have more segments. This is what determine the principal axis of the cards
			const float UVCoordTreshold = 1.f / 1024.f; // 1 pixel for a 1k texture
			for (uint32 TriangleIt = 0; TriangleIt < NumTriangles; ++TriangleIt)
			{
				const uint32 VertexIndexOffset = VertexOffset + TriangleIt * 3;
				for (uint32 VertexIt = 0; VertexIt < 3; ++VertexIt)
				{
					const uint32 VertexIndex = InCards.Indices[VertexIndexOffset + VertexIt];
					const FVector2f UV = InCards.UVs[HAIR_CARD_UV_CARDUV].UV[VertexIndex];
					AddSimilarUV(SimilarVertexU, VertexIndex, UV.X, UV.Y, UVCoordTreshold);
					AddSimilarUV(SimilarVertexV, VertexIndex, UV.Y, UV.X, UVCoordTreshold);
				}
			}

			// Use global UV orientation
			// Use global segment count orientation

			// Find the perpendicular direction by comparing the number of segment along U and along V
			const bool bIsValid = SimilarVertexU.Num() != SimilarVertexV.Num() ? 1u : 0u;
			if (bIsValid)
			{
				++ValidCount;
				MainDirectionUCount += SimilarVertexU.Num() > SimilarVertexV.Num() ? 1u : 0u;
			}
		}

		bIsMainDirectionU = (float(MainDirectionUCount) / float(FMath::Max(1u,ValidCount))) > 0.5f;
	}

	for (uint32 CardIt = 0; CardIt < NumCards; ++CardIt)
	{
		const uint32 NumTriangles = InCards.IndexCounts[CardIt] / 3;
		const uint32 VertexOffset = InCards.IndexOffsets[CardIt];
		struct FIndexAndCoord { uint32 Index; float TexCoord; };
		struct FSimilarUVVertices
		{
			float TexCoord = 0;
			TArray<uint32> Indices;
			TArray<FIndexAndCoord> AllIndices; // Store vertex index and Tex.coord perpendicular to the principal axis (stored in TexCoord)
		};
		TArray<FSimilarUVVertices> SimilarVertexU;
		TArray<FSimilarUVVertices> SimilarVertexV;

		auto AddSimilarUV = [](TArray<FSimilarUVVertices>& In, uint32 Index, float TexCoord, float MinorTexCoord, float Threshold)
		{
			bool bFound = false;
			for (int32 It = 0, Count = In.Num(); It < Count; ++It)
			{
				if (FMath::Abs(In[It].TexCoord - TexCoord) < Threshold)
				{
					// We add only unique vertices per segment, so that the average position land in the center of the cards
					In[It].Indices.AddUnique(Index);
					In[It].AllIndices.Add({Index, MinorTexCoord});
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				FSimilarUVVertices& SimilarUV = In.AddDefaulted_GetRef();
				SimilarUV.TexCoord = TexCoord;
				SimilarUV.Indices.Add(Index);
				SimilarUV.AllIndices.Add({Index, MinorTexCoord});
			}
		};

		// Iterate over all triangles of a cards, and find vertices which share either same U or same V. We add them to separate lists. 
		// We then use the following heuristic: the main axis will have more segments. This is what determine the principal axis of the cards
		const float UVCoordTreshold = 1.f / 1024.f; // 1 pixel for a 1k texture
		for (uint32 TriangleIt = 0; TriangleIt < NumTriangles; ++TriangleIt)
		{
			const uint32 VertexIndexOffset = VertexOffset + TriangleIt * 3;
			for (uint32 VertexIt = 0; VertexIt < 3; ++VertexIt)
			{
				const uint32 VertexIndex = InCards.Indices[VertexIndexOffset + VertexIt];
				const FVector2f UV = InCards.UVs[HAIR_CARD_UV_CARDUV].UV[VertexIndex];
				AddSimilarUV(SimilarVertexU, VertexIndex, UV.X, UV.Y, UVCoordTreshold);
				AddSimilarUV(SimilarVertexV, VertexIndex, UV.Y, UV.X, UVCoordTreshold);
			}
		}

		// Find the perpendicular direction by comparing the number of segment along U and along V
		TArray<FVector3f> CenterPoints;
		{
			// Sort vertices along the main axis so that, when we iterate through them, we get a correct linear ordering
			TArray<FSimilarUVVertices>& SimilarVertex = bIsMainDirectionU ? SimilarVertexU : SimilarVertexV;
			SimilarVertex.Sort([](const FSimilarUVVertices& A, const FSimilarUVVertices& B)
			{
				return A.TexCoord < B.TexCoord;
			});

			// For each group of vertex with similar 'principal' tex coord, sort them with growing 'perpendicular/secondary' tex coord
			for (FSimilarUVVertices& Similar : SimilarVertex)
			{
				Similar.AllIndices.Sort([](const FIndexAndCoord& A, const FIndexAndCoord& B)
				{
					return A.TexCoord < B.TexCoord;
				});

				// Compute normalize coordinate
				float MinTexCoord = Similar.AllIndices[0].TexCoord;
				float MaxTexCoord = Similar.AllIndices[0].TexCoord;
				for (const FIndexAndCoord& A : Similar.AllIndices)
				{
					MinTexCoord = FMath::Min(A.TexCoord, MinTexCoord);
					MaxTexCoord = FMath::Max(A.TexCoord, MaxTexCoord);
				}
				MaxTexCoord = FMath::Max(MaxTexCoord, MinTexCoord + KINDA_SMALL_NUMBER);
				for (FIndexAndCoord& A : Similar.AllIndices)
				{
					A.TexCoord = (A.TexCoord - MinTexCoord) / (MaxTexCoord - MinTexCoord);
				}
			}

			CenterPoints.Reserve(SimilarVertex.Num());
			FVector3f PrevCenterPoint = FVector3f::ZeroVector;
			float TotalLength = 0;
			for (const FSimilarUVVertices& Similar : SimilarVertex)
			{
				// Compute avg center point of the guide
				FVector3f CenterPoint = FVector3f::ZeroVector;
				for (uint32 VertexIndex : Similar.Indices)
				{
					CenterPoint += InCards.Positions[VertexIndex];
				}
				CenterPoint /= Similar.Indices.Num() > 0 ? Similar.Indices.Num() : 1;

				// Update length along the guide
				if (CenterPoints.Num() > 0)
				{
					const float SegmentLength = (CenterPoint - PrevCenterPoint).Size();
					TotalLength += SegmentLength;
				}

				// Update neighbor vertex with current length (to compute the parametric distance at the end)
				for (const FIndexAndCoord& VertexIndex : Similar.AllIndices)
				{
					InCards.CoordU[VertexIndex.Index] = TotalLength;
					InCards.LocalUVs[VertexIndex.Index].X = TotalLength;
					InCards.LocalUVs[VertexIndex.Index].Y = VertexIndex.TexCoord;
				}

				CenterPoints.Add(CenterPoint);
				PrevCenterPoint = CenterPoint;
			}

			if (SimilarVertex.Num() > 1)
			{
				// Normalize length to have a parametric distance
				for (const FSimilarUVVertices& Similar : SimilarVertex)
				{
					for (uint32 VertexIndex : Similar.Indices)
					{
						InCards.CoordU[VertexIndex] /= TotalLength;
						InCards.LocalUVs[VertexIndex].X = InCards.CoordU[VertexIndex];
					}
				}
			}
		}

		// Insure that cards as at least two points to build a segment as a lot of the runtime code assume at have
		// at least one valid segment
		check(CenterPoints.Num() > 0);
		if (CenterPoints.Num() == 1)
		{
			const FVector3f CenterPoint = CenterPoints[0];
			const float SegmentSize = 0.5f;
			const FVector3f P1 = CenterPoint + SegmentSize * (CenterPoint - InCards.BoundingBox.GetCenter()).GetSafeNormal();
			CenterPoints.Add(P1);
		}

		// Compute and store the guide's total length
		const uint32 PointCount = CenterPoints.Num();
		float TotalLength = 0.f;
		for (uint32 PointIt = 0; PointIt < PointCount - 1; ++PointIt)
		{
			TotalLength += (CenterPoints[PointIt + 1] - CenterPoints[PointIt]).Size();
		}
		OutCardLengths[CardIt] = TotalLength;

		const uint32 PointOffset = OutGuides.StrandsPoints.PointsPosition.Num();

		OutGuides.StrandsCurves.CurvesCount[CardIt] = PointCount;
		OutGuides.StrandsCurves.CurvesOffset[CardIt] = PointOffset;
		OutGuides.StrandsCurves.CurvesLength[CardIt] = TotalLength;
		OutGuides.Flags = 0u; // Card's guide don't have any particular flag (no shuffling, no extra point, no decimated points/curves, ...)
		if (OutGuides.StrandsCurves.CurvesRootUV.Num()) OutGuides.StrandsCurves.CurvesRootUV[CardIt] = FVector2f(0, 0);

		float CurrentLength = 0;
		for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
		{
			const FVector3f P0 = CenterPoints[PointIt];

			OutGuides.StrandsPoints.PointsPosition.Add(P0);
			OutGuides.StrandsPoints.PointsCoordU.Add(FMath::Clamp(CurrentLength / TotalLength, 0.f, 1.f));
			OutGuides.StrandsPoints.PointsRadius.Add(1);

			// Simple geometric normal based on adjacent vertices
			if (PointIt < PointCount - 1)
			{
				const FVector3f P1 = CenterPoints[PointIt + 1];
				const float SegmentLength = (P1 - P0).Size();
				CurrentLength += SegmentLength;
			}
		}
	}

	return true;
}

// Build interpolation data between the cards geometry and the guides
void InternalCreateCardsInterpolation(
	const FHairCardsGeometry& InCards,
	const FHairStrandsRawDatas& InGuides,
	FHairCardsInterpolationDatas& Out)
{		
	const uint32 TotalVertexCount = InCards.Positions.Num();
	Out.SetNum(TotalVertexCount);

	// For each cards, and for each cards vertex,
	// Compute the closest guide points (two guide points to interpolation in-between), 
	// and compute their indices and lerping value
	const uint32 CardsCount = InCards.IndexOffsets.Num();
	for (uint32 CardIt = 0; CardIt < CardsCount; ++CardIt)
	{
		const uint32 GuidePointOffset = InGuides.StrandsCurves.CurvesOffset[CardIt];
		const uint32 GuidePointCount = InGuides.StrandsCurves.CurvesCount[CardIt];
		check(GuidePointCount >= 2);

		const uint32 IndexOffset = InCards.IndexOffsets[CardIt];
		const uint32 IndexCount  = InCards.IndexCounts[CardIt];

		for (uint32 IndexIt = 0; IndexIt < IndexCount; ++IndexIt)
		{
			const uint32 VertexIndex = InCards.Indices[IndexOffset + IndexIt];
			const float CoordU = InCards.CoordU[VertexIndex];
			check(CoordU >= 0.f && CoordU <= 1.f);

			uint32 GuideIndex0 = GuidePointOffset + 0;
			uint32 GuideIndex1 = GuidePointOffset + 1;
			float  GuideLerp   = 0;
			bool bFoundMatch = false;
			for (uint32 GuidePointIt = 0; GuidePointIt < GuidePointCount-1; ++GuidePointIt)
			{
				const float GuideCoordU0 = InGuides.StrandsPoints.PointsCoordU[GuidePointOffset + GuidePointIt];
				const float GuideCoordU1 = InGuides.StrandsPoints.PointsCoordU[GuidePointOffset + GuidePointIt + 1];
				if (GuideCoordU0 <= CoordU && CoordU <= GuideCoordU1)
				{
					GuideIndex0 = GuidePointOffset + GuidePointIt;
					GuideIndex1 = GuidePointOffset + GuidePointIt + 1;
					const float LengthDiff = GuideCoordU1 - GuideCoordU0;
					GuideLerp = (CoordU - GuideCoordU0) / (LengthDiff>0 ? LengthDiff : 1);
					GuideLerp = FMath::Clamp(GuideLerp, 0.f, 1.f);
					bFoundMatch = true;
					break;
				}
			}

			if (!bFoundMatch)
			{
				GuideIndex0 = GuidePointOffset + GuidePointCount - 2;
				GuideIndex1 = GuidePointOffset + GuidePointCount - 1;
				GuideLerp = 1;
			}

			Out.PointsSimCurvesIndex[VertexIndex] = CardIt;
			Out.PointsSimCurvesVertexIndex[VertexIndex] = GuideIndex0;
			Out.PointsSimCurvesVertexLerp[VertexIndex] = GuideLerp;
		}
	}
}

void SanitizeMeshDescription(FMeshDescription* MeshDescription)
{
	if (!MeshDescription)
		return;

	bool bHasInvalidNormals = false;
	bool bHasInvalidTangents = false;
	FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(*MeshDescription, bHasInvalidNormals, bHasInvalidTangents);
	if (bHasInvalidNormals || bHasInvalidTangents)
	{
		FStaticMeshOperations::ComputeTriangleTangentsAndNormals(*MeshDescription, THRESH_POINTS_ARE_SAME);

		EComputeNTBsFlags Options = EComputeNTBsFlags::UseMikkTSpace | EComputeNTBsFlags::BlendOverlappingNormals;
		Options |= bHasInvalidNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
		Options |= bHasInvalidTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
		FStaticMeshOperations::ComputeTangentsAndNormals(*MeshDescription, Options);
	}
}

static float PackCardLengthAndGroupIndex(float InCardLength, uint32 InCardGroupIndex)
{
	// Encode cards length & group index into the .W component of position
	const uint32 EncodedW = FFloat16(InCardLength).Encoded | (InCardGroupIndex << 16u);
	return *(float*)&EncodedW;
}

static uint32 PackMaterialAttribute(const FVector3f& InBaseColor, float InRoughness)
{
	// Encode the base color in (cheap) sRGB in XYZ. The W component remains unused
	return
		(uint32(FMath::Sqrt(InBaseColor.X) * 255.f)    )|
		(uint32(FMath::Sqrt(InBaseColor.Y) * 255.f)<<8 )|
		(uint32(FMath::Sqrt(InBaseColor.Z) * 255.f)<<16)|
		(uint32(InRoughness                * 255.f)<<24);
}

static bool InternalImportGeometry_WithGeneratedGuides(
	const UStaticMesh* StaticMesh,
	const FHairStrandsDatas& InStrandsData,			// Used for extracting & assigning root UV to cards data
	const FHairStrandsVoxelData& InStrandsVoxelData,// Used for transfering & assigning group index to cards data
	FHairCardsDatas& Out,
	FHairCardsBulkData& OutBulk,
	FHairStrandsDatas& OutGuides,
	FHairCardsInterpolationBulkData& OutInterpolationBulkData)
{
	const uint32 MeshLODIndex = 0;

	// Note: * if there are multiple section we only import the first one. Support for multiple section could be added later on. 
	//       * use a local copy of the mesh description as multiple card assets referencing the same mesh description could be 
	//         built in parallel. We later call SanitizeMeshDescription which modify the mesh description and is not thread safe
	FMeshDescription LocalMeshDescription;
	StaticMesh->CloneMeshDescription(0, LocalMeshDescription);
	FMeshDescription* MeshDescription = &LocalMeshDescription;
	const uint32 VertexCount = MeshDescription->Vertices().Num();
	uint32 IndexCount  = MeshDescription->Triangles().Num() * 3;

	// Basic sanity check. Need at least one triangle
	if (IndexCount < 3)
	{
		return false;
	}

	// Build a set with all the triangleIDs
	// This will be used to find all the connected triangles forming cards
	TSet<FTriangleID> TriangleIDs;
	for (const FTriangleID& TriangleID : MeshDescription->Triangles().GetElementIDs())
	{
		TriangleIDs.Add(TriangleID);
	}

	// Find the cards triangle based on triangles adjancy
	uint32 CardsIndexCountReserve = 0;
	TArray<TSet<FTriangleID>> TrianglesCards;
	while (TriangleIDs.Num() != 0)
	{
		TSet<FTriangleID>& TriangleCard = TrianglesCards.AddDefaulted_GetRef();

		TQueue<FTriangleID> AdjacentTriangleIds;
		FTriangleID AdjacentTriangleId = *TriangleIDs.CreateIterator();
		AdjacentTriangleIds.Enqueue(AdjacentTriangleId);
		TriangleIDs.Remove(AdjacentTriangleId);
		TriangleCard.Add(AdjacentTriangleId);

		while (AdjacentTriangleIds.Dequeue(AdjacentTriangleId))
		{
			TArray<FTriangleID> AdjacentTriangles = MeshDescription->GetTriangleAdjacentTriangles(AdjacentTriangleId);
			for (const FTriangleID& A : AdjacentTriangles)
			{
				if (TriangleIDs.Contains(A))
				{
					AdjacentTriangleIds.Enqueue(A);
					TriangleIDs.Remove(A);
					TriangleCard.Add(A);
				}
			}
		}
		CardsIndexCountReserve += TriangleCard.Num() * 3;
	}
	IndexCount = CardsIndexCountReserve;

	// Fill in vertex indices and the cards indices offset/count
	uint32 GlobalIndex = 0;
	Out.Cards.Indices.Reserve(CardsIndexCountReserve);
	for (const TSet<FTriangleID>& TrianglesCard : TrianglesCards)
	{
		Out.Cards.IndexOffsets.Add(GlobalIndex);
		Out.Cards.IndexCounts.Add(TrianglesCard.Num()*3);
		for (const FTriangleID& TriangleId : TrianglesCard)
		{
			TArrayView<const FVertexInstanceID> VertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleId);
			check(VertexInstanceIDs.Num() == 3);
			FVertexInstanceID VI0 = VertexInstanceIDs[0];
			FVertexInstanceID VI1 = VertexInstanceIDs[1];
			FVertexInstanceID VI2 = VertexInstanceIDs[2];

			FVertexID V0 = MeshDescription->GetVertexInstanceVertex(VI0);
			FVertexID V1 = MeshDescription->GetVertexInstanceVertex(VI1);
			FVertexID V2 = MeshDescription->GetVertexInstanceVertex(VI2);

			Out.Cards.Indices.Add(V0.GetValue());
			Out.Cards.Indices.Add(V1.GetValue());
			Out.Cards.Indices.Add(V2.GetValue());

			GlobalIndex += 3;
		}
	}

	const uint32 MeshDescriptionNumChannels = MeshDescription->GetNumUVElementChannels();
	const uint32 NumUVSource = MeshDescriptionNumChannels;
	const uint32 NumUVs = NumUVSource + 1 /*RootUV*/;

	// Fill vertex data
	Out.Cards.Positions.SetNumZeroed(VertexCount);
	Out.Cards.Normals.SetNumZeroed(VertexCount);
	Out.Cards.Tangents.SetNumZeroed(VertexCount);
	Out.Cards.VertexColors.Empty();
	Out.Cards.UVs.SetNumZeroed(NumUVs);
	for (uint32 UVIt = 0;UVIt<NumUVs; ++UVIt)
	{
		Out.Cards.UVs[UVIt].UV.Init(FVector2f::ZeroVector, VertexCount);
	}
	TArray<float> TangentFrameSigns;
	TangentFrameSigns.SetNumZeroed(VertexCount);

	Out.Cards.BoundingBox.Init();

	SanitizeMeshDescription(MeshDescription);
	const TVertexAttributesRef<const FVector3f> VertexPositions					= MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	const TVertexInstanceAttributesRef<const FVector3f> VertexInstanceNormals	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	const TVertexInstanceAttributesRef<const FVector3f> VertexInstanceTangents	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
	const TVertexInstanceAttributesRef<const float> VertexInstanceBinormalSigns	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);
	const TVertexInstanceAttributesRef<const FVector2f> VertexInstanceUVs		= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	const TVertexInstanceAttributesRef<const FVector4f> VertexColors			= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);

	const bool bHasVertexColor = VertexColors.GetNumElements() > 0;
	if (bHasVertexColor)
	{
		Out.Cards.VertexColors.SetNumZeroed(VertexCount);
	}
	for (const FVertexID VertexId : MeshDescription->Vertices().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> VertexInstanceIds = MeshDescription->GetVertexVertexInstanceIDs(VertexId);
		if (VertexInstanceIds.Num() == 0)
		{
			continue;
		}

		FVertexInstanceID VertexInstanceId0 = VertexInstanceIds[0]; // Assume no actual duplicated data.

		const uint32 VertexIndex = VertexId.GetValue();
		check(VertexIndex < VertexCount);
		Out.Cards.Positions[VertexIndex]	= VertexPositions[VertexId];
		Out.Cards.Tangents[VertexIndex]		= VertexInstanceTangents[VertexInstanceId0];
		Out.Cards.Normals[VertexIndex]		= VertexInstanceNormals[VertexInstanceId0];
		TangentFrameSigns[VertexIndex] 		= VertexInstanceBinormalSigns[VertexInstanceId0];
		if (bHasVertexColor)
		{
			Out.Cards.VertexColors[VertexIndex] = FLinearColor(VertexColors[VertexInstanceId0]);
		}
		for (uint32 UVIt = 0; UVIt < NumUVSource; ++UVIt)
		{
			Out.Cards.UVs[HAIR_CARD_UV_CARDUV + UVIt].UV[VertexIndex] = VertexInstanceUVs.Get(VertexInstanceId0, UVIt);
		}
		Out.Cards.BoundingBox += Out.Cards.Positions[VertexIndex];
	}
	OutBulk.Header.BoundingBox = ToFBox3d(Out.Cards.BoundingBox);
	OutBulk.Header.bVertexColor = bHasVertexColor;
	OutBulk.Header.NumUVs = NumUVs;

	// Fill in render resources (do we need to keep it separated? e.g, format compression, packing)
	const uint32 PointCount = Out.Cards.Positions.Num();
	OutBulk.Positions.SetNumZeroed(PointCount);
	OutBulk.Normals.SetNumZeroed(PointCount * FHairCardsNormalFormat::ComponentCount);
	OutBulk.UVs.SetNumZeroed(GetNumPackedUVs(PointCount, NumUVs));
	OutBulk.Materials.SetNumZeroed(PointCount);
	OutBulk.VertexColors.Empty();
	if (bHasVertexColor)
	{
		OutBulk.VertexColors.SetNumZeroed(PointCount);
	}
	for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
	{
		OutBulk.Positions[PointIt] = FVector4f(Out.Cards.Positions[PointIt], 0);
		OutBulk.Normals[PointIt * 2] = FVector4f(Out.Cards.Tangents[PointIt], 0);
		OutBulk.Normals[PointIt * 2 + 1] = FVector4f(Out.Cards.Normals[PointIt], TangentFrameSigns[PointIt]);
		OutBulk.Materials[PointIt] = 0;
		if (bHasVertexColor)
		{
			OutBulk.VertexColors[PointIt] = Out.Cards.VertexColors[PointIt].ToFColor(true/*sRGB*/).Bits;
		}
	}

	OutBulk.Indices.SetNumZeroed(IndexCount);
	for (uint32 IndexIt = 0; IndexIt < IndexCount; ++IndexIt)
	{
		OutBulk.Indices[IndexIt] = Out.Cards.Indices[IndexIt];
	}

	OutBulk.DepthTexture = nullptr;
	OutBulk.TangentTexture = nullptr;
	OutBulk.CoverageTexture = nullptr;
	OutBulk.AttributeTexture = nullptr;

	const uint32 CardsCount = Out.Cards.IndexOffsets.Num();

	FHairStrandsRawDatas OutRawGuides;
	TArray<float> CardLengths;
	CardLengths.Reserve(CardsCount);
	bool bSuccess = InternalCreateCardsGuides(Out.Cards, OutRawGuides, CardLengths);
	if (bSuccess)
	{
		FHairCardsInterpolationDatas InterpolationData;
		InternalCreateCardsInterpolation(Out.Cards, OutRawGuides, InterpolationData);

		// Fill out the interpolation data
		static_assert(sizeof(FHairCardsInterpolationVertex) == HAIR_INTERPOLATION_CARDS_GUIDE_STRIDE);
		OutInterpolationBulkData.Interpolation.SetNumZeroed(PointCount);
		for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
		{
			const uint32 InterpVertexIndex = InterpolationData.PointsSimCurvesVertexIndex[PointIt];
			const float VertexLerp = InterpolationData.PointsSimCurvesVertexLerp[PointIt];
			FHairCardsInterpolationVertex PackedData;
			PackedData.VertexIndex = InterpVertexIndex;
			PackedData.VertexLerp = FMath::Clamp(uint32(VertexLerp * 0xFF), 0u, 0xFFu);
			OutInterpolationBulkData.Interpolation[PointIt] = PackedData;
		}
	}

	// Used voxelized hair group (from hair strands) to assign hair group index to card vertices
	TArray<FVector3f> CardBaseColor;
	TArray<float> CardsRoughness;
	TArray<uint8> CardGroupIndices;
	CardGroupIndices.Init(0u, CardsCount);					// Per-card
	CardBaseColor.Init(FVector3f::ZeroVector, PointCount);	// Per-vertex
	CardsRoughness.Init(0.f, PointCount);					// Per-vertex
	if (InStrandsVoxelData.IsValid())
	{
		for (uint32 CardIt = 0; CardIt < CardsCount; ++CardIt)
		{
			// 1. For each cards' vertex, query the closest group index, and build an histogram 
			//    of the group index covering the card
			TArray<uint8> GroupBins;
			GroupBins.Init(0, 64u);
			const uint32 CardsIndexCount = Out.Cards.IndexCounts[CardIt];
			const uint32 CardsIndexOffset = Out.Cards.IndexOffsets[CardIt];

			for (uint32 IndexIt = 0; IndexIt < CardsIndexCount; ++IndexIt)
			{
				const uint32 VertexIndex = Out.Cards.Indices[CardsIndexOffset + IndexIt];
				const FVector3f& P = Out.Cards.Positions[VertexIndex];

				FHairStrandsVoxelData::FData VoxelData = InStrandsVoxelData.GetData(P);
				if (VoxelData.GroupIndex != FHairStrandsVoxelData::InvalidGroupIndex)
				{
					check(VoxelData.GroupIndex < GroupBins.Num());
					GroupBins[VoxelData.GroupIndex]++;
					CardBaseColor[VertexIndex] = VoxelData.BaseColor;
					CardsRoughness[VertexIndex] = VoxelData.Roughness;
				}
			}

			// 2. Since a cards can covers several group, select the group index covering the larger 
			//    number of cards' vertices
			uint32 MaxBinCount = 0u;
			for (uint8 GroupIndex=0,GroupCount=GroupBins.Num(); GroupIndex<GroupCount; ++GroupIndex)
			{
				if (GroupBins[GroupIndex] > MaxBinCount)
				{
					CardGroupIndices[CardIt] = GroupIndex;
					MaxBinCount = GroupBins[GroupIndex];
				}
			}
		}
	}

	// Patch Cards Position to store cards length into the W component
	for (uint32 CardIt = 0; CardIt < CardsCount; ++CardIt)
	{
		const uint32 CardsIndexCount = Out.Cards.IndexCounts[CardIt];
		const uint32 CardsIndexOffset = Out.Cards.IndexOffsets[CardIt];

		const float CardLength = CardLengths[CardIt];
		for (uint32 IndexIt = 0; IndexIt < CardsIndexCount; ++IndexIt)
		{
			const uint32 VertexIndex = Out.Cards.Indices[CardsIndexOffset + IndexIt];
			const float CoordU = Out.Cards.CoordU[VertexIndex];

			// Instead of storing the interpolated card length, store the actual max length of the card, 
			// as reconstructing the strands length, based on interpolated CardLength will be too prone to numerical issue.
			// This means that the strand length retrieves in shader will be an over estimate of the actual length
			OutBulk.Positions[VertexIndex].W = PackCardLengthAndGroupIndex(CardLength, CardGroupIndices[CardIt]);

			OutBulk.Materials[VertexIndex] = PackMaterialAttribute(CardBaseColor[VertexIndex], CardsRoughness[VertexIndex]);
		}
	}

	// Patch Cards RootUV by transferring the guides root UV onto the cards
	if (InStrandsData.IsValid())
	{
		// 1. Extract all roots
		struct FStrandsRootData
		{
			FVector3f Position;
			FVector2f RootUV;
		};
		TArray<FStrandsRootData> StrandsRoots;
		{
			const uint32 InAttribute = InStrandsData.GetAttributes();
			const uint32 CurveCount = InStrandsData.StrandsCurves.Num();
			StrandsRoots.Reserve(CurveCount);
			for (uint32 CurveIt = 0; CurveIt < CurveCount; ++CurveIt)
			{
				const uint32 Offset = InStrandsData.StrandsCurves.CurvesOffset[CurveIt];
				FStrandsRootData& RootData = StrandsRoots.AddDefaulted_GetRef();
				RootData.Position = InStrandsData.StrandsPoints.PointsPosition[Offset];
				RootData.RootUV = HasHairAttribute(InAttribute, EHairAttribute::RootUV) ? InStrandsData.StrandsCurves.CurvesRootUV[CurveIt] : FVector2f(0,0);
			}
		}

		// 2. Extract cards root points
		struct FRootIndexAndPosition { uint32 Index; FVector3f Position; };
		struct FCardsRootData
		{
			FRootIndexAndPosition Root0;
			FRootIndexAndPosition Root1;
			uint32 CardsIndex;
		};
		TArray<FCardsRootData> CardsRoots;
		{
			CardsRoots.Reserve(CardsCount);
			for (uint32 CardIt = 0; CardIt < CardsCount; ++CardIt)
			{
				const uint32 CardsIndexCount = Out.Cards.IndexCounts[CardIt];
				const uint32 CardsIndexOffset = Out.Cards.IndexOffsets[CardIt];

				// Extract card vertices which are roots points
				TArray<FRootIndexAndPosition> RootPositions;
				for (uint32 IndexIt = 0; IndexIt < CardsIndexCount; ++IndexIt)
				{
					const uint32 VertexIndex = Out.Cards.Indices[CardsIndexOffset + IndexIt];
					if (Out.Cards.CoordU[VertexIndex] == 0)
					{
						RootPositions.Add({VertexIndex, Out.Cards.Positions[VertexIndex]});
					}
				}

				// Select the two most representative root points
				if (RootPositions.Num() > 0)
				{
					FCardsRootData& Cards = CardsRoots.AddDefaulted_GetRef();
					Cards.CardsIndex = CardIt;
					if (RootPositions.Num() == 1)
					{
						Cards.Root0 = RootPositions[0];
						Cards.Root1 = RootPositions[0];
					}
					else if (RootPositions.Num() == 2)
					{
						const bool bInvert = Out.Cards.LocalUVs[RootPositions[0].Index].Y > Out.Cards.LocalUVs[RootPositions[1].Index].Y;
						Cards.Root0 = bInvert ? RootPositions[1] : RootPositions[0];
						Cards.Root1 = bInvert ? RootPositions[0] : RootPositions[1];
					}
					else
					{
						// Sort according to local V coord
						RootPositions.Sort([&](const FRootIndexAndPosition& A, const FRootIndexAndPosition& B)
						{
							return Out.Cards.LocalUVs[A.Index].Y < Out.Cards.LocalUVs[B.Index].Y;
						});

						Cards.Root0 = RootPositions[0];
						Cards.Root1 = RootPositions[RootPositions.Num()-1];
					}
				}
			}
		}
		
		struct FRootUVData
		{
			FVector2f RootUV0;
			FVector2f RootUV1;
		};
		TArray<FRootUVData> OutRootUVDatas;
		OutRootUVDatas.SetNumZeroed(CardsRoots.Num());

		// 3. Find cards root / curve root

		// 3.1 Find closet root UV
		// /!\ N^2 loop: the number of cards should be relatively small
		ParallelFor(CardsRoots.Num(), 
		[
			&CardsRoots,
			&StrandsRoots,
			&Out,
			&OutBulk,
			&OutRootUVDatas
		] (uint32 CardRootIt) 
		//for (const FCardsRootData& CardsRoot : CardsRoots)
		{
			const FCardsRootData& CardsRoot = CardsRoots[CardRootIt];

			auto FindRootUV = [&](const FVector3f CardsRootPosition0, const FVector3f CardsRootPosition1, FVector2f& OutRootUV0, FVector2f& OutRootUV1)
			{
				float ClosestDistance1 = FLT_MAX;
				float ClosestDistance0 = FLT_MAX;
				OutRootUV0 = FVector2f::ZeroVector;
				OutRootUV1 = FVector2f::ZeroVector;
				for (const FStrandsRootData& StrandsRoot : StrandsRoots)
				{
					const float Distance0 = FVector3f::Distance(StrandsRoot.Position, CardsRootPosition0);
					const float Distance1 = FVector3f::Distance(StrandsRoot.Position, CardsRootPosition1);

					if (Distance0 < ClosestDistance0)
					{
						ClosestDistance0 = Distance0;
						OutRootUV0 = StrandsRoot.RootUV;
					}

					if (Distance1 < ClosestDistance1)
					{
						ClosestDistance1 = Distance1;
						OutRootUV1 = StrandsRoot.RootUV;
					}
				}

			};

			FRootUVData& OutRootUVData = OutRootUVDatas[CardRootIt];
			FindRootUV(CardsRoot.Root0.Position, CardsRoot.Root1.Position, OutRootUVData.RootUV0, OutRootUVData.RootUV1);
		});

		// 3.2 Apply root UV to all cards vertices
		// Apply this in a serial manner to avoid race in data writing. Some cards might share certain vertices, for which different RootUV could be written to.
		for (uint32 CardRootIt = 0, CardRootCount = CardsRoots.Num(); CardRootIt < CardRootCount; ++CardRootIt)
		{
			const FRootUVData& RootUVData = OutRootUVDatas[CardRootIt];
			const FCardsRootData& CardsRoot = CardsRoots[CardRootIt];

			const uint32 CardsIndexCount = Out.Cards.IndexCounts[CardsRoot.CardsIndex];
			const uint32 CardsIndexOffset = Out.Cards.IndexOffsets[CardsRoot.CardsIndex];
			for (uint32 IndexIt = 0; IndexIt < CardsIndexCount; ++IndexIt)
			{
				const uint32 VertexIndex = Out.Cards.Indices[CardsIndexOffset + IndexIt];

				// Linearly interpolate between the two roots based on the local V coordinate 
				// * LocalU is along the card 
				// * LocalV is across the card
				//  U
				//  ^  ____
				//  | |    |
				//  | |____| Card
				//  | |    |
				//  | |____|
				//     ----> V
				// Root0  Root1
				const float TexCoordV = Out.Cards.LocalUVs[VertexIndex].Y;
				const FVector2f RootUV = FMath::Lerp(RootUVData.RootUV0, RootUVData.RootUV1, TexCoordV);
				Out.Cards.UVs[HAIR_CARD_UV_ROOTUV].UV[VertexIndex] = RootUV;
			}
		}

	}

	// Pack texture coords
	for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
	{
		for (uint32 UVIt = 0; UVIt < NumUVs; ++UVIt)
		{
			SetPackedUV(PointIt, UVIt, NumUVs, Out.Cards.UVs[UVIt].UV[PointIt], OutBulk.UVs);
		}
	}

	// Build the FHairStrandsData from the FHairStrandsRawData
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGroomBuilder::BuildData(OutRawGuides, OutGuides);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return bSuccess;
}


static bool InternalImportGeometry_WithImportedGuides(
	const UStaticMesh* StaticMesh,
	const FHairStrandsDatas& InGuides,
	const FHairStrandsDatas& InStrandsData,			// Used for extracting & assigning root UV to cards data
	const FHairStrandsVoxelData& InStrandsVoxelData,// Used for transfering & assigning group index to cards data
	FHairCardsBulkData& OutBulk,
	FHairStrandsDatas& OutGuides,
	FHairCardsInterpolationBulkData& OutInterpolationBulkData)
{
	// Note: * if there are multiple section we only import the first one. Support for multiple section could be added later on. 
	//       * use a local copy of the mesh description as multiple card assets referencing the same mesh description could be 
	//         built in parallel. We later call SanitizeMeshDescription which modify the mesh description and is not thread safe
	FMeshDescription LocalMeshDescription;
	StaticMesh->CloneMeshDescription(0, LocalMeshDescription);
	FMeshDescription* MeshDescription = &LocalMeshDescription;
	const uint32 PointCount = MeshDescription->Vertices().Num();
	const uint32 IndexCount  = MeshDescription->Triangles().Num() * 3;

	const uint32 MeshLODIndex = 0;

	SanitizeMeshDescription(MeshDescription);
	const TVertexAttributesRef<const FVector3f> VertexPositions					= MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	const TVertexInstanceAttributesRef<const FVector3f> VertexInstanceNormals	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	const TVertexInstanceAttributesRef<const FVector3f> VertexInstanceTangents	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
	const TVertexInstanceAttributesRef<const float> VertexInstanceBinormalSigns	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);
	const TVertexInstanceAttributesRef<const FVector2f> VertexInstanceUVs		= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	const TVertexInstanceAttributesRef<const FVector4f> VertexColors			= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);

	const bool bHasVertexColor = VertexColors.GetNumElements() > 0;
	const uint32 NumSourceUVs = VertexInstanceUVs.GetNumChannels();
	const uint32 NumUVs = NumSourceUVs + 1u /*RootUV*/;

	// Write out BulkData
	OutBulk.Positions.SetNumZeroed(PointCount);
	OutBulk.Normals.SetNumZeroed(PointCount * FHairCardsNormalFormat::ComponentCount);
	OutBulk.UVs.SetNumZeroed(GetNumPackedUVs(PointCount, NumUVs));
	OutBulk.Materials.SetNumZeroed(PointCount);
	OutBulk.VertexColors.Empty();
	if (bHasVertexColor)
	{
		OutBulk.VertexColors.SetNumZeroed(PointCount);
	}
	FBox3f BoundingBox = FBox3f(EForceInit::ForceInit);
	for (const FVertexID VertexId : MeshDescription->Vertices().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> VertexInstanceIds = MeshDescription->GetVertexVertexInstanceIDs(VertexId);
		if (VertexInstanceIds.Num() == 0)
		{
			continue;
		}

		FVertexInstanceID VertexInstanceId0 = VertexInstanceIds[0]; // Assume no actual duplicated data.

		const uint32 VertexIndex = VertexId.GetValue();
		check(VertexIndex < PointCount);
		OutBulk.Positions[VertexIndex] 		= FVector4f(VertexPositions[VertexId], 0);		
		OutBulk.Normals[VertexIndex * 2]	= FVector4f(VertexInstanceTangents[VertexInstanceId0], 0);
		OutBulk.Normals[VertexIndex * 2 + 1]= FVector4f(VertexInstanceNormals[VertexInstanceId0], VertexInstanceBinormalSigns[VertexInstanceId0] /*TangentFrameSigns*/);
		OutBulk.Materials[VertexIndex] 		= 0;
		if (bHasVertexColor)
		{
			OutBulk.VertexColors[VertexIndex]= FLinearColor(VertexColors[VertexInstanceId0]).ToFColor(true/*sRGB*/).Bits;
		}
		// UVs are packed 2 channels into a single float4
		// * UVs[0]   -> Root UV - Not set here, but will be 'patched' later once guides & interpolation data are built
		// * UVs[1-N] -> Card UV & else
		for (uint32 UVIndex=0;UVIndex<NumSourceUVs;++UVIndex)
		{
			const FVector2f UV = VertexInstanceUVs.Get(VertexInstanceId0, UVIndex);

			SetPackedUV(VertexIndex, HAIR_CARD_UV_CARDUV + UVIndex, NumUVs, UV, OutBulk.UVs);
		}

		BoundingBox += OutBulk.Positions[VertexIndex];
	}
	OutBulk.Header.BoundingBox = ToFBox3d(BoundingBox);
	OutBulk.Header.bVertexColor = bHasVertexColor;
	OutBulk.Header.NumUVs = NumUVs;

	TArray<FVector3f> CardBaseColor;
	TArray<float> CardsRoughness;
	TArray<uint8> CardGroupIndices;
	TArray<float> CardLengths;
	CardGroupIndices.Init(0u, PointCount);					// Per-vertex
	CardBaseColor.Init(FVector3f::ZeroVector, PointCount);	// Per-vertex
	CardsRoughness.Init(0.f, PointCount);					// Per-vertex
	CardLengths.Init(0.f, PointCount);						// Per-vertex

	// Compute material properties per vertex
	if (InStrandsVoxelData.IsValid())
	{
		for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
		{
			const FVector4f& P = OutBulk.Positions[PointIt];

			FHairStrandsVoxelData::FData VoxelData = InStrandsVoxelData.GetData(P);
			if (VoxelData.GroupIndex != FHairStrandsVoxelData::InvalidGroupIndex)
			{
				CardBaseColor[PointIt] = VoxelData.BaseColor;
				CardsRoughness[PointIt] = VoxelData.Roughness;
				CardGroupIndices[PointIt] = VoxelData.GroupIndex;
			}
		}
	}

	// Build interpolation data
	{		
		struct FClosestGuide
		{
			uint32 GuideIndex = ~0u;
			uint32 PointIndex = ~0u;
			float U = 0;
			float Distance = FLT_MAX;
		};

		TArray<FClosestGuide> ClosestGuides;
		ClosestGuides.Init(FClosestGuide(), PointCount);

		FHairCardsInterpolationDatas InterpolationData;
		InterpolationData.SetNum(PointCount);
		OutInterpolationBulkData.Interpolation.SetNumZeroed(PointCount);

		TArray<float> CoordUs;
		CoordUs.Init(0, PointCount);

		// For each cards, and for each cards vertex,
		// Compute the closest guide points (two guide points to interpolation in-between), 
		// and compute their indices and lerping value
		const uint32 GuideCount = InGuides.GetNumCurves();
		for (uint32 GuideIt = 0; GuideIt < GuideCount; ++GuideIt)
		{
			const uint32 GuidePointOffset = InGuides.StrandsCurves.CurvesOffset[GuideIt];
			const uint32 GuidePointCount = InGuides.StrandsCurves.CurvesCount[GuideIt];
			check(GuidePointCount >= 2);

			uint32 GuideIndex0 = GuidePointOffset + 0;
			uint32 GuideIndex1 = GuidePointOffset + 1;
			float  GuideLerp   = 0;
			bool bFoundMatch = false;
			for (uint32 GuidePointIt = 0; GuidePointIt < GuidePointCount-1; ++GuidePointIt)
			{
				const uint32 I0 = GuidePointOffset + GuidePointIt;
				const uint32 I1 = I0+1u;

				const FVector3f& GPoint0 = InGuides.StrandsPoints.PointsPosition[I0];
				const FVector3f& GPoint1 = InGuides.StrandsPoints.PointsPosition[I1];
				const float GuideCoordU0 = InGuides.StrandsPoints.PointsCoordU[I0];
				const float GuideCoordU1 = InGuides.StrandsPoints.PointsCoordU[I1];

				const FVector3f G01Vector = GPoint1 - GPoint0;
				const float G01Norm2 = FMath::Max(FVector3f::DotProduct(G01Vector,G01Vector), KINDA_SMALL_NUMBER);

				// For each point of the mesh, compute its distance to the guide's segment
				for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
				{
					// Project P onto the segment.
					const FVector3f P = OutBulk.Positions[PointIt];
					float U = FVector3f::DotProduct(P-GPoint0, G01Vector) / G01Norm2;
					U = FMath::Clamp(U, 0.f, 1.f);
					const FVector3f PP = U * G01Vector + GPoint0;
					const float Dist = (P-PP).Length();

					// If new projection is closer than the older one, change guides
					FClosestGuide& ClosestGuide = ClosestGuides[PointIt];
					if (Dist < ClosestGuide.Distance)
					{
						ClosestGuide.GuideIndex = GuideIt;
						ClosestGuide.PointIndex = GuidePointIt;
						ClosestGuide.U = U;
						ClosestGuide.Distance = Dist;
						CoordUs[PointIt] = FMath::Lerp(GuideCoordU0, GuideCoordU1, ClosestGuide.U);
						CardLengths[PointIt] = InGuides.StrandsCurves.CurvesLength[GuideIt];
					}
				}
			}
		}

		// Compute the interpolation data
		for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
		{
			const FClosestGuide& ClosestGuide = ClosestGuides[PointIt];

			// Interpolation data
			{
				check(ClosestGuide.Distance < FLT_MAX);
				InterpolationData.PointsSimCurvesIndex[PointIt] = ClosestGuide.GuideIndex;
				InterpolationData.PointsSimCurvesVertexIndex[PointIt] = InGuides.StrandsCurves.CurvesOffset[ClosestGuide.GuideIndex] + ClosestGuide.PointIndex;
				InterpolationData.PointsSimCurvesVertexLerp[PointIt] = ClosestGuide.U;
			}
			// Interpolation bulk data
			{
				FHairCardsInterpolationVertex PackedData;
				PackedData.VertexIndex = InterpolationData.PointsSimCurvesVertexIndex[PointIt];
				PackedData.VertexLerp = FMath::Clamp(uint32(InterpolationData.PointsSimCurvesVertexLerp[PointIt] * 0xFF), 0u, 0xFFu);
				OutInterpolationBulkData.Interpolation[PointIt] = PackedData;
			}
			// Root UV bulk data
			{
				const FVector2f RootUV = InGuides.StrandsCurves.CurvesRootUV[ClosestGuide.GuideIndex];
				SetPackedUV(PointIt, HAIR_CARD_UV_ROOTUV, NumUVs, RootUV, OutBulk.UVs);
			}
		}
	}

	// Encode material bulk data
	for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
	{
		OutBulk.Positions[PointIt].W = PackCardLengthAndGroupIndex(CardLengths[PointIt], CardGroupIndices[PointIt]);
		OutBulk.Materials[PointIt] = PackMaterialAttribute(CardBaseColor[PointIt], CardsRoughness[PointIt]);
	}

	// Fill in vertex indices and the cards indices offset/count
	OutBulk.Indices.Reserve(IndexCount);
	for (const FTriangleID& TriangleId : MeshDescription->Triangles().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> VertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleId);
		check(VertexInstanceIDs.Num() == 3);
		FVertexInstanceID VI0 = VertexInstanceIDs[0];
		FVertexInstanceID VI1 = VertexInstanceIDs[1];
		FVertexInstanceID VI2 = VertexInstanceIDs[2];

		FVertexID V0 = MeshDescription->GetVertexInstanceVertex(VI0);
		FVertexID V1 = MeshDescription->GetVertexInstanceVertex(VI1);
		FVertexID V2 = MeshDescription->GetVertexInstanceVertex(VI2);

		OutBulk.Indices.Add(V0.GetValue());
		OutBulk.Indices.Add(V1.GetValue());
		OutBulk.Indices.Add(V2.GetValue());
	}

	OutGuides = InGuides;

	OutBulk.DepthTexture = nullptr;
	OutBulk.TangentTexture = nullptr;
	OutBulk.CoverageTexture = nullptr;
	OutBulk.AttributeTexture = nullptr;
	
	return true;
}

bool ImportGeometry(
	const UStaticMesh* StaticMesh,
	const FHairStrandsDatas& InGuidesData,
	const FHairStrandsDatas& InStrandsData,
	const FHairStrandsVoxelData& InStrandsVoxelData,
	const bool bGenerateGuidesFromCardGeometry,
	FHairCardsBulkData& OutBulk,
	FHairStrandsDatas& OutGuides,
	FHairCardsInterpolationBulkData& OutInterpolationBulkData)
{
	if (bGenerateGuidesFromCardGeometry)
	{
		FHairCardsDatas CardData;
		return InternalImportGeometry_WithGeneratedGuides(
			StaticMesh,
			InStrandsData,
			InStrandsVoxelData,
			CardData,
			OutBulk,
			OutGuides,
			OutInterpolationBulkData);
	}
	else
	{
		return InternalImportGeometry_WithImportedGuides(
			StaticMesh,
			InGuidesData,
			InStrandsData,
			InStrandsVoxelData,
			OutBulk,
			OutGuides,
			OutInterpolationBulkData);
	}
}

bool ExtractCardsData(const UStaticMesh* StaticMesh, const FHairStrandsDatas& InStrandsData, FHairCardsDatas& Out)
{
	FHairStrandsVoxelData DummyStrandsVoxelData;
	FHairCardsBulkData OutBulk;
	FHairStrandsDatas OutGuides;
	FHairCardsInterpolationBulkData OutInterpolationBulkData;
	return InternalImportGeometry_WithGeneratedGuides(
		StaticMesh,
		InStrandsData,
		DummyStrandsVoxelData,
		Out,
		OutBulk,
		OutGuides,
		OutInterpolationBulkData); 
}

} // namespace FHairCardsBuilder

namespace FHairMeshesBuilder
{
FString GetVersion()
{
	// Important to update the version when meshes building or importing changes
	return TEXT("4g");
}

void BuildGeometry(
	const FBox& InBox,
	FHairMeshesBulkData& OutBulk)
{
	const FVector3f Center = (FVector3f)InBox.GetCenter();
	const FVector3f Extent = (FVector3f)InBox.GetExtent();

	// Simple (incorrect normal/tangent) cube geomtry in place of the hair rendering
	const uint32 TotalPointCount = 8;
	const uint32 TotalIndexCount = 36;

	FHairMeshesDatas Out;

	Out.Meshes.Positions.SetNumZeroed(TotalPointCount);
	Out.Meshes.Normals.SetNumZeroed(TotalPointCount);
	Out.Meshes.Tangents.SetNumZeroed(TotalPointCount);
	Out.Meshes.Indices.SetNumZeroed(TotalIndexCount);
	Out.Meshes.UVs.SetNumZeroed(HAIR_CARD_UV_COUNT);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV.SetNumZeroed(TotalPointCount);
	Out.Meshes.UVs[HAIR_CARD_UV_ROOTUV].UV.Init(FVector2f(0,0), TotalPointCount);

	Out.Meshes.Positions[0] = Center + FVector3f(-Extent.X, -Extent.Y, -Extent.Z);
	Out.Meshes.Positions[1] = Center + FVector3f(+Extent.X, -Extent.Y, -Extent.Z);
	Out.Meshes.Positions[2] = Center + FVector3f(+Extent.X, +Extent.Y, -Extent.Z);
	Out.Meshes.Positions[3] = Center + FVector3f(-Extent.X, +Extent.Y, -Extent.Z);
	Out.Meshes.Positions[4] = Center + FVector3f(-Extent.X, -Extent.Y, +Extent.Z);
	Out.Meshes.Positions[5] = Center + FVector3f(+Extent.X, -Extent.Y, +Extent.Z);
	Out.Meshes.Positions[6] = Center + FVector3f(+Extent.X, +Extent.Y, +Extent.Z);
	Out.Meshes.Positions[7] = Center + FVector3f(-Extent.X, +Extent.Y, +Extent.Z);

	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[0] = FVector2f(0, 0);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[1] = FVector2f(1, 0);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[2] = FVector2f(1, 1);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[3] = FVector2f(0, 1);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[4] = FVector2f(0, 0);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[5] = FVector2f(1, 0);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[6] = FVector2f(1, 1);
	Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[7] = FVector2f(0, 1);

	Out.Meshes.Normals[0] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[1] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[2] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[3] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[4] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[5] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[6] = FVector3f(0, 0, 1);
	Out.Meshes.Normals[7] = FVector3f(0, 0, 1);

	Out.Meshes.Tangents[0] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[1] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[2] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[3] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[4] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[5] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[6] = FVector3f(1, 0, 0);
	Out.Meshes.Tangents[7] = FVector3f(1, 0, 0);

	Out.Meshes.Indices[0] = 0;
	Out.Meshes.Indices[1] = 1;
	Out.Meshes.Indices[2] = 2;
	Out.Meshes.Indices[3] = 0;
	Out.Meshes.Indices[4] = 2;
	Out.Meshes.Indices[5] = 3;

	Out.Meshes.Indices[6] = 4;
	Out.Meshes.Indices[7] = 5;
	Out.Meshes.Indices[8] = 6;
	Out.Meshes.Indices[9] = 4;
	Out.Meshes.Indices[10] = 6;
	Out.Meshes.Indices[11] = 7;

	Out.Meshes.Indices[12] = 0;
	Out.Meshes.Indices[13] = 1;
	Out.Meshes.Indices[14] = 5;
	Out.Meshes.Indices[15] = 0;
	Out.Meshes.Indices[16] = 5;
	Out.Meshes.Indices[17] = 4;

	Out.Meshes.Indices[18] = 2;
	Out.Meshes.Indices[19] = 3;
	Out.Meshes.Indices[20] = 7;
	Out.Meshes.Indices[21] = 2;
	Out.Meshes.Indices[22] = 7;
	Out.Meshes.Indices[23] = 6;

	Out.Meshes.Indices[24] = 1;
	Out.Meshes.Indices[25] = 2;
	Out.Meshes.Indices[26] = 6;
	Out.Meshes.Indices[27] = 1;
	Out.Meshes.Indices[28] = 6;
	Out.Meshes.Indices[29] = 5;

	Out.Meshes.Indices[30] = 3;
	Out.Meshes.Indices[31] = 0;
	Out.Meshes.Indices[32] = 4;
	Out.Meshes.Indices[33] = 3;
	Out.Meshes.Indices[34] = 4;
	Out.Meshes.Indices[35] = 7;

	Out.Meshes.BoundingBox.Init();

	// Fill in render resources (do we need to keep it separated? e.g, format compression, packing)
	const uint32 NumUVs = 2; // RootUV+CardUV
	const uint32 PointCount = Out.Meshes.Positions.Num();
	OutBulk.Positions.SetNumZeroed(PointCount);
	OutBulk.Normals.SetNumZeroed(PointCount * FHairCardsNormalFormat::ComponentCount);
	OutBulk.UVs.SetNumZeroed(FHairCardsBuilder::GetNumPackedUVs(PointCount, NumUVs));
	OutBulk.VertexColors.Empty();
	for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
	{
		OutBulk.Positions[PointIt] = FVector4f(Out.Meshes.Positions[PointIt], 0);
		OutBulk.Normals[PointIt * 2] = FVector4f(Out.Meshes.Tangents[PointIt], 0);
		OutBulk.Normals[PointIt * 2 + 1] = FVector4f(Out.Meshes.Normals[PointIt], 1);
		FHairCardsBuilder::SetPackedUV(PointIt, HAIR_CARD_UV_CARDUV, NumUVs, Out.Meshes.UVs[HAIR_CARD_UV_CARDUV].UV[PointIt], OutBulk.UVs);
		FHairCardsBuilder::SetPackedUV(PointIt, HAIR_CARD_UV_ROOTUV, NumUVs, FVector2f::ZeroVector, OutBulk.UVs);

		Out.Meshes.BoundingBox += FVector4f(OutBulk.Positions[PointIt]);
	}
	OutBulk.Header.BoundingBox = ToFBox3d(Out.Meshes.BoundingBox);
	OutBulk.Header.bVertexColor = false;
	OutBulk.Header.NumUVs = NumUVs;

	const uint32 IndexCount = Out.Meshes.Indices.Num();
	OutBulk.Indices.SetNumZeroed(IndexCount);
	for (uint32 IndexIt = 0; IndexIt < IndexCount; ++IndexIt)
	{
		OutBulk.Indices[IndexIt] = Out.Meshes.Indices[IndexIt];
	}
}

void ImportGeometry(
	const UStaticMesh* StaticMesh,
	FHairMeshesBulkData& OutBulk)
{
	FHairMeshesDatas Out;

	const uint32 MeshLODIndex = 0;

	// Note: if there are multiple section we only import the first one. Support for multiple section could be added later on. 
	const FStaticMeshRenderData* MeshRenderData = StaticMesh->GetRenderData();
	check(MeshRenderData != nullptr);
	check(MeshRenderData->CurrentFirstLODIdx == MeshLODIndex);
	const FStaticMeshLODResources& LODData = StaticMesh->GetLODForExport(MeshLODIndex);
	const uint32 VertexCount = LODData.VertexBuffers.PositionVertexBuffer.GetNumVertices();
	const uint32 IndexCount = LODData.IndexBuffer.GetNumIndices();
	const bool bHasVertexColor = LODData.VertexBuffers.ColorVertexBuffer.GetNumVertices() > 0;
	const uint32 NumUVsSource = LODData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	const uint32 NumUVs = NumUVsSource + 1; // +1 for RootUV
	
	Out.Meshes.Positions.SetNumZeroed(VertexCount);
	Out.Meshes.Normals.SetNumZeroed(VertexCount);
	Out.Meshes.Tangents.SetNumZeroed(VertexCount);
	Out.Meshes.VertexColors.Empty();
	Out.Meshes.Indices.SetNumZeroed(IndexCount);
	Out.Meshes.UVs.SetNumZeroed(NumUVs);
	for (uint32 UVIt = 0; UVIt < NumUVs; ++UVIt)
	{
		Out.Meshes.UVs[UVIt].UV.Init(FVector2f::ZeroVector, VertexCount);
	}

	if (bHasVertexColor)
	{
		Out.Meshes.VertexColors.SetNumZeroed(VertexCount);
		LODData.VertexBuffers.ColorVertexBuffer.GetVertexColors(Out.Meshes.VertexColors);
	}

	Out.Meshes.BoundingBox.Init();
	for (uint32 VertexIt = 0; VertexIt < VertexCount; ++VertexIt)
	{
		Out.Meshes.Positions[VertexIt]	= LODData.VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIt);
		Out.Meshes.Tangents[VertexIt]	= FVector4f(LODData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertexIt));
		Out.Meshes.Normals[VertexIt]	= FVector4f(LODData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIt));

		for (uint32 UVIt=0; UVIt<NumUVsSource; ++UVIt)
		{
			const FVector2f UV = LODData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIt, UVIt);
			Out.Meshes.UVs[HAIR_CARD_UV_CARDUV + UVIt].UV[VertexIt] = UV;
		}

		Out.Meshes.BoundingBox += Out.Meshes.Positions[VertexIt];
	}

	for (uint32 IndexIt = 0; IndexIt < IndexCount; ++IndexIt)
	{
		Out.Meshes.Indices[IndexIt] = LODData.IndexBuffer.GetIndex(IndexIt);
	}

	// Fill in render resources (do we need to keep it separated? e.g, format compression, packing)
	OutBulk.Header.BoundingBox = ToFBox3d(Out.Meshes.BoundingBox);
	OutBulk.Header.bVertexColor = bHasVertexColor;
	OutBulk.Header.NumUVs = NumUVs;

	const uint32 PointCount = Out.Meshes.Positions.Num();
	OutBulk.Positions.SetNumZeroed(PointCount);
	OutBulk.Normals.SetNumZeroed(PointCount * FHairCardsNormalFormat::ComponentCount);
	OutBulk.UVs.SetNumZeroed(FHairCardsBuilder::GetNumPackedUVs(PointCount, NumUVs));
	OutBulk.VertexColors.Empty();
	if (bHasVertexColor)
	{
		OutBulk.VertexColors.SetNumZeroed(PointCount);
	}
	for (uint32 PointIt = 0; PointIt < PointCount; ++PointIt)
	{
		OutBulk.Positions[PointIt] = FVector4f(Out.Meshes.Positions[PointIt], 0);
		OutBulk.Normals[PointIt * 2] = FVector4f(Out.Meshes.Tangents[PointIt], 0);
		OutBulk.Normals[PointIt * 2 + 1] = FVector4f(Out.Meshes.Normals[PointIt], 1);
		if (bHasVertexColor)
		{
			OutBulk.VertexColors[PointIt] = Out.Meshes.VertexColors[PointIt].Bits;
		}
		for (uint32 UVIt=0; UVIt<NumUVs; ++UVIt)
		{
			const FVector2f CardUVs = Out.Meshes.UVs[UVIt].UV[PointIt];
			FHairCardsBuilder::SetPackedUV(PointIt, UVIt, NumUVs, CardUVs, OutBulk.UVs);
		}
	}

	OutBulk.Indices.SetNumZeroed(IndexCount);
	for (uint32 IndexIt = 0; IndexIt < IndexCount; ++IndexIt)
	{
		OutBulk.Indices[IndexIt] = Out.Meshes.Indices[IndexIt];
	}
}

} // namespace FHairMeshesBuilder


namespace FHairCardsBuilder
{

// Utility class to construct MeshDescription instances
class FMeshDescriptionBuilder
{
public:
	void SetMeshDescription(FMeshDescription* Description);

	/** Append vertex and return new vertex ID */
	FVertexID AppendVertex(const FVector3f& Position);

	/** Append new vertex instance and return ID */
	FVertexInstanceID AppendInstance(const FVertexID& VertexID);

	/** Set the Normal of a vertex instance*/
	void SetInstanceNormal(const FVertexInstanceID& InstanceID, const FVector3f& Normal);

	/** Set the UV of a vertex instance */
	void SetInstanceUV(const FVertexInstanceID& InstanceID, const FVector2f& InstanceUV, int32 UVLayerIndex = 0);

	/** Set the number of UV layers */
	void SetNumUVLayers(int32 NumUVLayers);

	/** Enable per-triangle integer attribute named PolyTriGroups */
	void EnablePolyGroups();

	/** Create a new polygon group and return it's ID */
	FPolygonGroupID AppendPolygonGroup();

	/** Set the PolyTriGroups attribute value to a specific GroupID for a Polygon */
	void SetPolyGroupID(const FPolygonID& PolygonID, int GroupID);

	/** Append a triangle to the mesh with the given PolygonGroup ID */
	FPolygonID AppendTriangle(const FVertexID& Vertex0, const FVertexID& Vertex1, const FVertexID& Vertex2, const FPolygonGroupID& PolygonGroup);

	/** Append a triangle to the mesh with the given PolygonGroup ID, and optionally with triangle-vertex UVs and Normals */
	FPolygonID AppendTriangle(const FVertexID* Triangle, const FPolygonGroupID& PolygonGroup, const FVector2f* VertexUVs = nullptr, const FVector3f* VertexNormals = nullptr);

	/**
	 * Append an arbitrary polygon to the mesh with the given PolygonGroup ID, and optionally with polygon-vertex UVs and Normals
	 * Unique Vertex instances will be created for each polygon-vertex.
	 */
	FPolygonID AppendPolygon(const TArray<FVertexID>& Vertices, const FPolygonGroupID& PolygonGroup, const TArray<FVector2f>* VertexUVs = nullptr, const TArray<FVector3f>* VertexNormals = nullptr);

	/**
	 * Append a triangle to the mesh using the given vertex instances and PolygonGroup ID
	 */
	FPolygonID AppendTriangle(const FVertexInstanceID& Instance0, const FVertexInstanceID& Instance1, const FVertexInstanceID& Instance2, const FPolygonGroupID& PolygonGroup);

protected:
	FMeshDescription* MeshDescription;

	TVertexAttributesRef<FVector3f> VertexPositions;
	TVertexInstanceAttributesRef<FVector2f> InstanceUVs;
	TVertexInstanceAttributesRef<FVector3f> InstanceNormals;
	TVertexInstanceAttributesRef<FVector4f> InstanceColors;

	TPolygonAttributesRef<int> PolyGroups;
};

namespace ExtendedMeshAttribute
{
	const FName PolyTriGroups("PolyTriGroups");
}

void FMeshDescriptionBuilder::SetMeshDescription(FMeshDescription* Description)
{
	this->MeshDescription	= Description;
	this->VertexPositions	= MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	this->InstanceUVs		= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	this->InstanceNormals	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	this->InstanceColors	= MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);
}

void FMeshDescriptionBuilder::EnablePolyGroups()
{
	PolyGroups = MeshDescription->PolygonAttributes().GetAttributesRef<int>(ExtendedMeshAttribute::PolyTriGroups);
	if (PolyGroups.IsValid() == false)
	{
		MeshDescription->PolygonAttributes().RegisterAttribute<int>(ExtendedMeshAttribute::PolyTriGroups, 1, 0, EMeshAttributeFlags::AutoGenerated);
		PolyGroups = MeshDescription->PolygonAttributes().GetAttributesRef<int>(ExtendedMeshAttribute::PolyTriGroups);
		check(PolyGroups.IsValid());
	}
}

FVertexID FMeshDescriptionBuilder::AppendVertex(const FVector3f& Position)
{
	FVertexID VertexID = MeshDescription->CreateVertex();
	VertexPositions.Set(VertexID, Position);
	return VertexID;
}

FPolygonGroupID FMeshDescriptionBuilder::AppendPolygonGroup()
{
	return MeshDescription->CreatePolygonGroup();
}


FVertexInstanceID FMeshDescriptionBuilder::AppendInstance(const FVertexID& VertexID)
{
	return MeshDescription->CreateVertexInstance(VertexID);
}

void FMeshDescriptionBuilder::SetInstanceNormal(const FVertexInstanceID& InstanceID, const FVector3f& Normal)
{
	if (InstanceNormals.IsValid())
	{
		InstanceNormals.Set(InstanceID, Normal);
	}
}

void FMeshDescriptionBuilder::SetInstanceUV(const FVertexInstanceID& InstanceID, const FVector2f& InstanceUV, int32 UVLayerIndex)
{
	if (InstanceUVs.IsValid() && ensure(UVLayerIndex < InstanceUVs.GetNumChannels()))
	{
		InstanceUVs.Set(InstanceID, UVLayerIndex, InstanceUV);
	}
}

void FMeshDescriptionBuilder::SetNumUVLayers(int32 NumUVLayers)
{
	if (ensure(InstanceUVs.IsValid()))
	{
		InstanceUVs.SetNumChannels(NumUVLayers);
	}
}

FPolygonID FMeshDescriptionBuilder::AppendTriangle(const FVertexInstanceID& Instance0, const FVertexInstanceID& Instance1, const FVertexInstanceID& Instance2, const FPolygonGroupID& PolygonGroup)
{
	TArray<FVertexInstanceID> Polygon;
	Polygon.Add(Instance0);
	Polygon.Add(Instance1);
	Polygon.Add(Instance2);

	const FPolygonID NewPolygonID = MeshDescription->CreatePolygon(PolygonGroup, Polygon);

	return NewPolygonID;
}

void FMeshDescriptionBuilder::SetPolyGroupID(const FPolygonID& PolygonID, int GroupID)
{
	PolyGroups.Set(PolygonID, 0, GroupID);
}

void ConvertCardsGeometryToMeshDescription(const FHairCardsGeometry& In, FMeshDescription& Out)
{
	Out.Empty();

	// Do not write out RootUV
	const uint32 NumUVs = FMath::Max(1, In.UVs.Num()-1);

	FMeshDescriptionBuilder Builder;
	Builder.SetMeshDescription(&Out);
	Builder.EnablePolyGroups();
	Builder.SetNumUVLayers(NumUVs);

	// create vertices
	TArray<FVertexInstanceID> VertexInstanceIDs;
	const uint32 VertexCount = In.GetNumVertices();
	VertexInstanceIDs.Reserve(VertexCount);
	for (uint32 VIndex=0; VIndex<VertexCount; ++VIndex)
	{
		FVertexID VertexID = Builder.AppendVertex(In.Positions[VIndex]);
		FVertexInstanceID InstanceID = Builder.AppendInstance(VertexID);
		Builder.SetInstanceNormal(InstanceID, In.Normals[VIndex]);
		for (uint32 UVIt = 0; UVIt < NumUVs; ++UVIt)
		{
			Builder.SetInstanceUV(InstanceID, In.UVs[HAIR_CARD_UV_CARDUV + UVIt].UV[VIndex], UVIt);
		}

		VertexInstanceIDs.Add(InstanceID);
	}

	// Build the polygroup, i.e. the triangles belonging to the same cards
	const int32 TriangleCount = In.GetNumTriangles();
	const uint32 CardsCount = In.IndexCounts.Num();
	TArray<int32> TriangleToCardGroup;
	TriangleToCardGroup.Reserve(TriangleCount);
	for (uint32 GroupId = 0; GroupId < CardsCount; ++GroupId)
	{
		const uint32 IndexOffset = In.IndexOffsets[GroupId];
		const uint32 IndexCount  = In.IndexCounts[GroupId];
		const uint32 GroupTriangleCount = IndexCount / 3;
		for (uint32 TriangleId = 0; TriangleId < GroupTriangleCount; ++TriangleId)
		{
			TriangleToCardGroup.Add(GroupId);
		}
	}

	// build the polygons
	FPolygonGroupID ZeroPolygonGroupID = Builder.AppendPolygonGroup();
	for (int32 TriID=0; TriID < TriangleCount; ++TriID)
	{
		// transfer material index to MeshDescription polygon group (by convention)
		FPolygonGroupID UsePolygonGroupID = ZeroPolygonGroupID;

		int32 VertexIndex0 = In.Indices[TriID * 3 + 0];
		int32 VertexIndex1 = In.Indices[TriID * 3 + 1];
		int32 VertexIndex2 = In.Indices[TriID * 3 + 2];

		FVertexInstanceID VertexInstanceID0 = VertexInstanceIDs[VertexIndex0];
		FVertexInstanceID VertexInstanceID1 = VertexInstanceIDs[VertexIndex1];
		FVertexInstanceID VertexInstanceID2 = VertexInstanceIDs[VertexIndex2];

		FPolygonID NewPolygonID = Builder.AppendTriangle(VertexInstanceID0, VertexInstanceID1, VertexInstanceID2, UsePolygonGroupID);
		Builder.SetPolyGroupID(NewPolygonID, TriangleToCardGroup[TriID]);
	}
}

} // namespace FHairCardsExporter

#endif // WITH_EDITOR 


#undef LOCTEXT_NAMESPACE
