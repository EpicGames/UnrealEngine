// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairCardsDatas.h"

// Serialization code for cards structures
// Note that if there are changes in the serialized structures,
// including the types used in them such as the RenderData structures,
// CustomVersion will be required to handle the changes

void FHairCardsBulkData::Serialize(FArchive& Ar)
{
	Ar << Positions;
	Ar << Normals;
	Ar << UVs;
	Ar << Materials;
	Ar << VertexColors;
	Ar << Indices;
	Ar << Header.BoundingBox;
	Ar << Header.NumUVs;
	Ar << Header.bVertexColor;
}

FArchive& operator<<(FArchive& Ar, FHairCardsInterpolationVertex& CardInterpVertex)
{
	if (Ar.IsLoading())
	{
		uint32 Value;
		Ar << Value;
		CardInterpVertex.VertexIndex = Value & 0x00FFFFFF; // first 24 bits
		CardInterpVertex.VertexLerp = Value >> 24;
	}
	else
	{
		uint32 Value = CardInterpVertex.VertexIndex | (CardInterpVertex.VertexLerp << 24);
		Ar << Value;
	}

	return Ar;
}

void FHairMeshesBulkData::Serialize(FArchive& Ar)
{
	Ar << Positions;
	Ar << Normals;
	Ar << UVs;
	Ar << VertexColors;
	Ar << Indices;
	Ar << Header.BoundingBox;
	Ar << Header.NumUVs;
	Ar << Header.bVertexColor;
}


/////////////////////////////////////////////////////////////////////////////////////////
void FHairCardsInterpolationDatas::SetNum(const uint32 NumPoints)
{
	PointsSimCurvesIndex.SetNumZeroed(NumPoints);
	PointsSimCurvesVertexIndex.SetNumZeroed(NumPoints);
	PointsSimCurvesVertexLerp.SetNumZeroed(NumPoints);
}

void FHairCardsInterpolationDatas::Reset()
{
	PointsSimCurvesIndex.Empty();
	PointsSimCurvesVertexIndex.Empty();
	PointsSimCurvesVertexLerp.Empty();
}

void FHairCardsInterpolationBulkData::Serialize(FArchive& Ar)
{
	Ar << Interpolation;
}