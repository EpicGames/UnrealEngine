// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StrandHairCardsFactory.cpp: hair cards vertex factory implementation
=============================================================================*/

#include "HairCardsVertexFactory.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "MaterialDomain.h"
#include "MeshDrawShaderBindings.h"
#include "MeshBatch.h"
#include "ShaderParameterUtils.h"
#include "Rendering/ColorVertexBuffer.h"
#include "MeshMaterialShader.h"
#include "HairStrandsInterface.h"
#include "GroomInstance.h"
#include "SystemTextures.h" 
#include "GlobalRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"

#define HAIR_CARDS_VF_PRIMITIVEID_STREAM_INDEX 13
static int32 GHairCardCoverageBias = 0;
static FAutoConsoleVariableRef CVarHairCardCoverageBias(TEXT("r.HairStrands.Cards.CoverageBias"), GHairCardCoverageBias, TEXT("Apply a texture LOD bias to coverage texture"));

template<typename T> inline void VFC_BindParam(FMeshDrawSingleShaderBindings& ShaderBindings, const FShaderResourceParameter& Param, T* Value) { if (Param.IsBound() && Value) ShaderBindings.Add(Param, Value); }
template<typename T> inline void VFC_BindParam(FMeshDrawSingleShaderBindings& ShaderBindings, const FShaderParameter& Param, const T& Value) { if (Param.IsBound()) ShaderBindings.Add(Param, Value); }

enum class EHairCardsFactoryFlags : uint32
{
	Texture0    = 0x1,
	Texture1    = 0x2,
	Texture2    = 0x4,
	Texture3    = 0x8,
	Texture4    = 0x10,
	Texture5    = 0x20,
	InvertedUV  = 0x40,
	VertexColor = 0x80
};

static void SetTextures(
	const TArray<FTextureReferenceRHIRef>& InTextures,
	const TArray<FSamplerStateRHIRef>& InSamplers,
	const uint32 InLayoutIndex,
	FHairCardsVertexFactoryUniformShaderParameters& Out)
{
	Out.LayoutIndex = InLayoutIndex;
	Out.TextureCount = InTextures.Num();
	for (uint32 TextureIt = 0, TextureCount = InTextures.Num(); TextureIt < TextureCount; ++TextureIt)
	{
		if (InTextures[TextureIt] != nullptr)
		{
			Out.Flags |= (1u<<TextureIt);
			switch (TextureIt)
			{
				case 0:
				{
					Out.Texture0Texture = InTextures[TextureIt];
					Out.Texture0Sampler = InSamplers[TextureIt];
				} break;
				case 1:
				{
					Out.Texture1Texture = InTextures[TextureIt];
					Out.Texture1Sampler = InSamplers[TextureIt];
				} break;
				case 2:
				{
					Out.Texture2Texture = InTextures[TextureIt];
					Out.Texture2Sampler = InSamplers[TextureIt];
				} break;
				case 3:
				{
					Out.Texture3Texture = InTextures[TextureIt];
					Out.Texture3Sampler = InSamplers[TextureIt];
				} break;
				case 4:
				{
					Out.Texture4Texture = InTextures[TextureIt];
					Out.Texture4Sampler = InSamplers[TextureIt];
				} break;
				case 5:
				{
					Out.Texture5Texture = InTextures[TextureIt];
					Out.Texture5Sampler = InSamplers[TextureIt];
				} break;
				default:
				{
					// Only support up to 6 textures for now
					check(false);
				}

			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Cards based vertex factory
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FHairCardsVertexFactoryUniformShaderParameters, "HairCardsVF");

FHairCardsUniformBuffer CreateHairCardsVFUniformBuffer(
	const FHairGroupInstance* Instance,
	const uint32 LODIndex,
	EHairGeometryType GeometryType, 
	bool bSupportsManualVertexFetch)
{
	FHairCardsVertexFactoryUniformShaderParameters UniformParameters;

	if (GeometryType == EHairGeometryType::Cards)
	{
		const FHairGroupInstance::FCards::FLOD& LOD = Instance->Cards.LODs[LODIndex];
		const bool bHasVertexColor = LOD.RestResource->BulkData.Header.bVertexColor;

		// Cards atlas UV are inverted so fetching needs to be inverted on the y-axis
		UniformParameters.Flags = 
			(LOD.RestResource->bInvertUV ? uint32(EHairCardsFactoryFlags::InvertedUV) : 0u) |
			(bHasVertexColor ? uint32(EHairCardsFactoryFlags::VertexColor) : 0u);
		UniformParameters.AttributeTextureIndex = 0; // TODO pack attribute texture/channel description. Currently use hardcoded layout in shader attribute code.
		UniformParameters.AttributeChannelIndex = 0; // TODO pack attribute texture/channel description. Currently use hardcoded layout in shader attribute code.
		UniformParameters.MaxVertexCount = LOD.RestResource->GetVertexCount();
		UniformParameters.CoverageBias = FMath::Clamp(GHairCardCoverageBias, -16.f, 16.f);
		UniformParameters.NumUVs = LOD.RestResource->BulkData.Header.NumUVs;

		// When the geometry is not-dynamic (no binding to skeletal mesh, no simulation), only a single vertex buffer is allocated. 
		// In this case we force the buffer index to 0
		const bool bIsDynamic = LOD.DeformedResource != nullptr;
		if (bIsDynamic)
		{
			UniformParameters.PositionBuffer = LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::EFrameType::Current).SRV.GetReference();
			UniformParameters.PreviousPositionBuffer = LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::EFrameType::Previous).SRV.GetReference();
			UniformParameters.NormalsBuffer = LOD.DeformedResource->DeformedNormalBuffer.SRV.GetReference();
		}
		else
		{
			UniformParameters.PositionBuffer = LOD.RestResource->RestPositionBuffer.ShaderResourceViewRHI;
			UniformParameters.PreviousPositionBuffer = LOD.RestResource->RestPositionBuffer.ShaderResourceViewRHI;
			UniformParameters.NormalsBuffer = LOD.RestResource->NormalsBuffer.ShaderResourceViewRHI.GetReference();
		}

		UniformParameters.UVsBuffer = LOD.RestResource->UVsBuffer.ShaderResourceViewRHI.GetReference();
		UniformParameters.MaterialsBuffer = LOD.RestResource->MaterialsBuffer.ShaderResourceViewRHI.GetReference();
		if (bHasVertexColor)
		{
			UniformParameters.VertexColorsBuffer = LOD.RestResource->VertexColorsBuffer.ShaderResourceViewRHI.GetReference();
		}
		else
		{
			UniformParameters.VertexColorsBuffer = UniformParameters.NormalsBuffer; // Reuse normal buffer as a dummy input for material buffer, since it not used for meshes (material data is fetch through textures)
		}

		SetTextures(LOD.RestResource->Textures, LOD.RestResource->Samplers, LOD.RestResource->LayoutIndex, UniformParameters);
	}
	else if (GeometryType == EHairGeometryType::Meshes)
	{
		const FHairGroupInstance::FMeshes::FLOD& LOD = Instance->Meshes.LODs[LODIndex];
		const bool bHasVertexColor = LOD.RestResource->BulkData.Header.bVertexColor;

		// When the geometry is not-dynamic (no binding to skeletal mesh, no simulation), only a single vertex buffer is allocated. 
		// In this case we force the buffer index to 0
		const bool bIsDynamic = LOD.DeformedResource != nullptr;
		if (bIsDynamic)
		{
			UniformParameters.PositionBuffer = LOD.DeformedResource->GetBuffer(FHairMeshesDeformedResource::EFrameType::Current).SRV.GetReference();
			UniformParameters.PreviousPositionBuffer = LOD.DeformedResource->GetBuffer(FHairMeshesDeformedResource::EFrameType::Previous).SRV.GetReference();
		}
		else
		{
			UniformParameters.PositionBuffer = LOD.RestResource->RestPositionBuffer.ShaderResourceViewRHI;
			UniformParameters.PreviousPositionBuffer = LOD.RestResource->RestPositionBuffer.ShaderResourceViewRHI;
		}

		// Meshes UV are not inverted so no need to invert the y-axis
		UniformParameters.Flags = (bHasVertexColor ? uint32(EHairCardsFactoryFlags::VertexColor) : 0u);
		UniformParameters.AttributeTextureIndex = 0; // TODO pack attribute texture/channel description. Currently use hardcoded layout in shader attribute code.
		UniformParameters.AttributeChannelIndex = 0; // TODO pack attribute texture/channel description. Currently use hardcoded layout in shader attribute code.
		UniformParameters.MaxVertexCount = LOD.RestResource->GetVertexCount();
		UniformParameters.CoverageBias = 0;
		UniformParameters.NumUVs = LOD.RestResource->BulkData.Header.NumUVs;

		UniformParameters.NormalsBuffer = LOD.RestResource->NormalsBuffer.ShaderResourceViewRHI.GetReference();
		UniformParameters.UVsBuffer = LOD.RestResource->UVsBuffer.ShaderResourceViewRHI.GetReference();
		UniformParameters.MaterialsBuffer = UniformParameters.NormalsBuffer; // Reuse normal buffer as a dummy input for material buffer, since it not used for meshes (material data is fetch through textures)
		if (bHasVertexColor)
		{
			UniformParameters.VertexColorsBuffer = LOD.RestResource->VertexColorsBuffer.ShaderResourceViewRHI.GetReference();
		}
		else
		{
			UniformParameters.VertexColorsBuffer = UniformParameters.NormalsBuffer; // Reuse normal buffer as a dummy input for material buffer, since it not used for meshes (material data is fetch through textures)
		}

		SetTextures(LOD.RestResource->Textures, LOD.RestResource->Samplers, LOD.RestResource->LayoutIndex, UniformParameters);
	}

	if (!bSupportsManualVertexFetch)
	{
		UniformParameters.PositionBuffer         = GNullVertexBuffer.VertexBufferSRV;
		UniformParameters.PreviousPositionBuffer = GNullVertexBuffer.VertexBufferSRV;
		UniformParameters.NormalsBuffer          = GNullVertexBuffer.VertexBufferSRV;
		UniformParameters.UVsBuffer              = GNullVertexBuffer.VertexBufferSRV;
		UniformParameters.MaterialsBuffer        = GNullVertexBuffer.VertexBufferSRV;
		UniformParameters.VertexColorsBuffer     = GNullVertexBuffer.VertexBufferSRV;
	}

	FRHITexture* DefaultTexture = GBlackTexture->TextureRHI;
	FSamplerStateRHIRef DefaultSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	if (!UniformParameters.Texture0Texture)	{ UniformParameters.Texture0Texture = DefaultTexture; }
	if (!UniformParameters.Texture1Texture)	{ UniformParameters.Texture1Texture = DefaultTexture; }
	if (!UniformParameters.Texture2Texture)	{ UniformParameters.Texture2Texture = DefaultTexture; }
	if (!UniformParameters.Texture3Texture)	{ UniformParameters.Texture3Texture = DefaultTexture; }
	if (!UniformParameters.Texture4Texture)	{ UniformParameters.Texture4Texture = DefaultTexture; }
	if (!UniformParameters.Texture5Texture)	{ UniformParameters.Texture5Texture = DefaultTexture; }

	if (!UniformParameters.Texture0Sampler) { UniformParameters.Texture0Sampler = DefaultSampler; }
	if (!UniformParameters.Texture1Sampler) { UniformParameters.Texture1Sampler = DefaultSampler; }
	if (!UniformParameters.Texture2Sampler) { UniformParameters.Texture2Sampler = DefaultSampler; }
	if (!UniformParameters.Texture3Sampler) { UniformParameters.Texture3Sampler = DefaultSampler; }
	if (!UniformParameters.Texture4Sampler) { UniformParameters.Texture4Sampler = DefaultSampler; }
	if (!UniformParameters.Texture5Sampler) { UniformParameters.Texture5Sampler = DefaultSampler; }

	return TUniformBufferRef<FHairCardsVertexFactoryUniformShaderParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_MultiFrame);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Cards based vertex factory

class FHairCardsVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FHairCardsVertexFactoryShaderParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
	) const
	{
		const FHairCardsVertexFactory* VF = static_cast<const FHairCardsVertexFactory*>(VertexFactory);
		check(VF);

		const int32 LODIndex = VF->Data.LODIndex;

		const FHairGroupInstance* Instance = VF->Data.Instance;
		const EHairGeometryType InstanceGeometryType = VF->Data.GeometryType;
		check(Instance);
		check(LODIndex >= 0);

		// Decode VertexFactoryUserData as VertexFactoryUniformBuffer
		FRHIUniformBuffer* VertexFactoryUniformBuffer = nullptr;
		check(InstanceGeometryType == EHairGeometryType::Cards || InstanceGeometryType == EHairGeometryType::Meshes);
		if (InstanceGeometryType == EHairGeometryType::Cards)
		{
			const FHairGroupInstance::FCards::FLOD& LOD = Instance->Cards.LODs[LODIndex];
			check(LOD.UniformBuffer);
			VertexFactoryUniformBuffer = LOD.UniformBuffer;
		}
		else if (InstanceGeometryType == EHairGeometryType::Meshes)
		{
			const FHairGroupInstance::FMeshes::FLOD& LOD = Instance->Meshes.LODs[LODIndex];
			check(LOD.UniformBuffer);
			VertexFactoryUniformBuffer = LOD.UniformBuffer;
		}
		
		check(VertexFactoryUniformBuffer);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FHairCardsVertexFactoryUniformShaderParameters>(), VertexFactoryUniformBuffer);
	}
};

IMPLEMENT_TYPE_LAYOUT(FHairCardsVertexFactoryShaderParameters);

FHairCardsVertexFactory::FHairCardsVertexFactory(FHairGroupInstance* Instance, uint32 LODIndex, EHairGeometryType GeometryType, EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, const char* InDebugName)
	: FVertexFactory(InFeatureLevel)
	, DebugName(InDebugName)
{
	Data.Instance = Instance;
	Data.LODIndex = LODIndex;
	Data.GeometryType = GeometryType;
}
/**
 * Should we cache the material's shadertype on this platform with this vertex factory? 
 */
bool FHairCardsVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	return (Parameters.MaterialParameters.MaterialDomain == MD_Surface && Parameters.MaterialParameters.bIsUsedWithHairStrands && IsHairStrandsSupported(EHairStrandsShaderType::Cards, Parameters.Platform)) || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

void FHairCardsVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	const bool bUseGPUSceneAndPrimitiveIdStream = Parameters.VertexFactoryType->SupportsPrimitiveIdStream() && UseGPUScene(Parameters.Platform);
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bUseGPUSceneAndPrimitiveIdStream);
	OutEnvironment.SetDefine(TEXT("HAIR_CARD_MESH_FACTORY"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("MANUAL_VERTEX_FETCH"), RHISupportsManualVertexFetch(Parameters.Platform) ? TEXT("1") : TEXT("0"));	
}

void FHairCardsVertexFactory::ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors)
{
	// This is not relevant to hair cards at the moment, as instancing is not supported anyway (need to bind unique resource per card instance
	#if 0
	if (Type->SupportsPrimitiveIdStream() 
		&& UseGPUScene(Platform, GetMaxSupportedFeatureLevel(Platform)) 
		&& ParameterMap.ContainsParameterAllocation(FPrimitiveUniformShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
	{
		OutErrors.AddUnique(*FString::Printf(TEXT("Shader attempted to bind the Primitive uniform buffer even though Vertex Factory %s computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(PrimitiveId).Member instead of Primitive.Member."), Type->GetName()));
	}
	#endif
}

void FHairCardsVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	if (!PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
	{ 
		// Manual vertex fetch is available for this factory so only primitive ID stream is used
		Elements.Add(FVertexElement(0, 0, VET_UInt, HAIR_CARDS_VF_PRIMITIVEID_STREAM_INDEX, 0, true));
	}

	const bool bManualFetch = RHISupportsManualVertexFetch(GMaxRHIShaderPlatform);
	if (!bManualFetch)
	{
		uint8 NumStreams = Elements.Num();
		
		Elements.Add(FVertexElement(0+NumStreams, 0, FHairCardsPositionFormat::VertexElementType, 0, FHairCardsPositionFormat::SizeInByte, false)); // Current Position
		Elements.Add(FVertexElement(0+NumStreams, 0, FHairCardsPositionFormat::VertexElementType, 1, FHairCardsPositionFormat::SizeInByte, false)); // Previous Position

		uint8 NormalOffset0 = 0u;
		uint8 NormalOffset1 = FHairCardsNormalFormat::SizeInByte;
		Elements.Add(FVertexElement(1+NumStreams, NormalOffset0, FHairCardsNormalFormat::VertexElementType, 2, FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount, false)); // Normal
		Elements.Add(FVertexElement(1+NumStreams, NormalOffset1, FHairCardsNormalFormat::VertexElementType, 3, FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount, false)); // Normal

		Elements.Add(FVertexElement(2+NumStreams, 0, FHairCardsMaterialFormat::VertexElementType, 4, FHairCardsMaterialFormat::SizeInByte, false)); // Material
		Elements.Add(FVertexElement(2+NumStreams, 0, FHairCardsMaterialFormat::VertexElementType, 5, FHairCardsMaterialFormat::SizeInByte, false)); // VertexColor
		
		Elements.Add(FVertexElement(3+NumStreams, 0, FHairCardsUVFormat::VertexElementType, 6, FHairCardsUVFormat::SizeInByte, false));		
		Elements.Add(FVertexElement(3+NumStreams, 0, FHairCardsUVFormat::VertexElementType, 7, FHairCardsUVFormat::SizeInByte, false));
	}
}

EPrimitiveIdMode FHairCardsVertexFactory::GetPrimitiveIdMode(ERHIFeatureLevel::Type In) const
{
	return PrimID_DynamicPrimitiveShaderData;
}

void FHairCardsVertexFactory::SetData(const FDataType& InData)
{
	Data = InData;
	UpdateRHI(FRHICommandListImmediate::Get());
}

/**
* Copy the data from another vertex factory
* @param Other - factory to copy from
*/
void FHairCardsVertexFactory::Copy(const FHairCardsVertexFactory& Other)
{
	FHairCardsVertexFactory* VertexFactory = this;
	const FDataType* DataCopy = &Other.Data;
	ENQUEUE_RENDER_COMMAND(FHairStrandsVertexFactoryCopyData)(/*UE::RenderCommandPipe::Groom,*/
	[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
	{
		VertexFactory->Data = *DataCopy;
	});
	BeginUpdateResourceRHI(this);
}

void FHairCardsVertexFactory::InitResources(FRHICommandListBase& RHICmdList)
{
	if (bIsInitialized)
		return;

	FVertexFactory::InitResource(RHICmdList); //Call VertexFactory/RenderResources::InitResource() to mark the resource as initialized();

	bIsInitialized = true;
	bNeedsDeclaration = true;
	// We create different streams based on feature level
	check(HasValidFeatureLevel());

	const ERHIFeatureLevel::Type CurrentFeatureLevel = GetFeatureLevel();

	// If the platform does not support manual vertex fetching we assume it is a low end platform, and so we don't enable deformation.
	// VertexFactory needs to be able to support max possible shader platform and feature level
	// in case if we switch feature level at runtime.
	FVertexDeclarationElementList Elements;
	SetPrimitiveIdStreamIndex(CurrentFeatureLevel, EVertexInputStreamType::Default, -1);

	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, HAIR_CARDS_VF_PRIMITIVEID_STREAM_INDEX /*AttributeIndex*/, HAIR_CARDS_VF_PRIMITIVEID_STREAM_INDEX /*AttributeIndex_Mobile*/);

	// Note this is a local version of the VF's bSupportsManualVertexFetch, which take into account the feature level
	// When manual fetch is not supported, buffers are bound through input assembly based on vertex declaration. 
	// A vertex declaraction only access FVertexBuffer buffers, so we create wrappers of pooled buffers
	const bool bManualFetch = SupportsManualVertexFetch(CurrentFeatureLevel);
	if (!bManualFetch)
	{
		auto AddUVChannels = [this, &Elements](const FHairCardsVertexBuffer* UVsBuffer, uint32 NumUVs, uint32 StartStreamIndex)
		{
			// Add additional TexCoord
			const uint32 UVSize = sizeof(FVector2f);
			const uint32 NumUVComponentCount = FMath::DivideAndRoundUp(NumUVs, 2u); // Two UVs are packed per Float4
			const EVertexElementType UVComponentVertexElementType = VET_Float2;

			// UV[0] is rootUV which is not part of the regular UVs, hence UV starts at index 1
			for (uint32 UVIt = 1; UVIt < NumUVs; UVIt +=2)
			{
				Elements.Add(this->AccessStreamComponent(FVertexStreamComponent(UVsBuffer, 0, UVIt * UVSize, FHairCardsUVFormat::SizeInByte * NumUVComponentCount, UVComponentVertexElementType, EVertexStreamUsage::Default), StartStreamIndex++));
			}
		};

		if (Data.GeometryType == EHairGeometryType::Cards)
		{
			const FHairGroupInstance::FCards::FLOD& LOD = Data.Instance->Cards.LODs[Data.LODIndex];
			const uint32 NumUVs = LOD.RestResource->BulkData.GetNumUVs();
			const uint32 NumUVComponentCount = FMath::DivideAndRoundUp(NumUVs, 2u); // Two UVs are packed per Float4
			const bool bHasVertexColor = LOD.RestResource->BulkData.Header.bVertexColor;
			const bool bDynamic = LOD.DeformedResource != nullptr;
			const FHairCardsVertexBuffer* VertexColorsBuffer = bHasVertexColor ? &LOD.RestResource->VertexColorsBuffer : &LOD.RestResource->NormalsBuffer;
			if (bDynamic)
			{
				DeformedPositionVertexBuffer[0] = FRDGWrapperVertexBuffer(LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::EFrameType::Current));
				DeformedPositionVertexBuffer[0].InitResource(RHICmdList);

				DeformedPositionVertexBuffer[1] = FRDGWrapperVertexBuffer(LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::EFrameType::Previous));
				DeformedPositionVertexBuffer[1].InitResource(RHICmdList);

				DeformedNormalVertexBuffer = FRDGWrapperVertexBuffer(LOD.DeformedResource->DeformedNormalBuffer);
				DeformedNormalVertexBuffer.InitResource(RHICmdList);

				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&(DeformedPositionVertexBuffer[0]),		0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 0));
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&(DeformedPositionVertexBuffer[1]),		0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 1));

				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&DeformedNormalVertexBuffer,				0, 0,									FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount,		FHairCardsNormalFormat::VertexElementType,		EVertexStreamUsage::Default), 2));
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&DeformedNormalVertexBuffer,				0, FHairCardsNormalFormat::SizeInByte,	FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount,		FHairCardsNormalFormat::VertexElementType,		EVertexStreamUsage::Default), 3));
			}
			else
			{
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->RestPositionBuffer,	0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 0));
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->RestPositionBuffer,	0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 1));

				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->NormalsBuffer,			0, 0,									FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount,		FHairCardsNormalFormat::VertexElementType,		EVertexStreamUsage::Default), 2));
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->NormalsBuffer,			0, FHairCardsNormalFormat::SizeInByte,	FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount,		FHairCardsNormalFormat::VertexElementType,		EVertexStreamUsage::Default), 3));
			}
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->MaterialsBuffer,			0, 0,									FHairCardsMaterialFormat::SizeInByte,												FHairCardsMaterialFormat::VertexElementType,	EVertexStreamUsage::Default), 4));
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(VertexColorsBuffer,							0, 0,									FHairCardsMaterialFormat::SizeInByte,												FHairCardsMaterialFormat::VertexElementType,	EVertexStreamUsage::Default), 5));
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->UVsBuffer,					0, 0,									FHairCardsUVFormat::SizeInByte * NumUVComponentCount,														FHairCardsUVFormat::VertexElementType,			EVertexStreamUsage::Default), 6));

			// Add additional UVs
			AddUVChannels(&LOD.RestResource->UVsBuffer, LOD.RestResource->BulkData.GetNumUVs(), 7);

			// Ensure the rest resources are in correct states for the VF
			const FHairCardsRestResource* RestResource = LOD.RestResource;
			ENQUEUE_RENDER_COMMAND(FHairCardsVertexFactoryTransition)(/*UE::RenderCommandPipe::Groom,*/
			[RestResource, bHasVertexColor](FRHICommandListImmediate& RHICmdImmediateList)
			{
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->RestPositionBuffer.VertexBufferRHI,	ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->NormalsBuffer.VertexBufferRHI, 		ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->UVsBuffer.VertexBufferRHI, 			ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->MaterialsBuffer.VertexBufferRHI, 	ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				if (bHasVertexColor)
				{
					RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->VertexColorsBuffer.VertexBufferRHI, ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				}
			});
		}
		else if (Data.GeometryType == EHairGeometryType::Meshes)
		{
			const FHairGroupInstance::FMeshes::FLOD& LOD = Data.Instance->Meshes.LODs[Data.LODIndex];
			const uint32 NumUVs = LOD.RestResource->BulkData.GetNumUVs();
			const uint32 NumUVComponentCount = FMath::DivideAndRoundUp(NumUVs, 2u); // Two UVs are packed per Float4
			const bool bHasVertexColor = LOD.RestResource->BulkData.Header.bVertexColor;
			const FHairCardsVertexBuffer* VertexColorsBuffer = bHasVertexColor ? &LOD.RestResource->VertexColorsBuffer : &LOD.RestResource->NormalsBuffer;

			// Note: Use the 'Normal' buffer as a dummy input for 'Material' buffer, as Material data is fetched through textures for meshes
			const bool bDynamic = LOD.DeformedResource != nullptr;
			if (bDynamic)
			{
				DeformedPositionVertexBuffer[0] = FRDGWrapperVertexBuffer(LOD.DeformedResource->GetBuffer(FHairMeshesDeformedResource::EFrameType::Current));
				DeformedPositionVertexBuffer[0].InitResource(RHICmdList);

				DeformedPositionVertexBuffer[1] = FRDGWrapperVertexBuffer(LOD.DeformedResource->GetBuffer(FHairMeshesDeformedResource::EFrameType::Previous));
				DeformedPositionVertexBuffer[1].InitResource(RHICmdList);

				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&(DeformedPositionVertexBuffer[0]),		0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 0));
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&(DeformedPositionVertexBuffer[1]),		0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 1));
			}
			else
			{
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->RestPositionBuffer,	0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 0));
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->RestPositionBuffer,	0, 0,									FHairCardsPositionFormat::SizeInByte,												FHairCardsPositionFormat::VertexElementType,	EVertexStreamUsage::Default), 1));
			}
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->NormalsBuffer,				0, 0,									FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount,		FHairCardsNormalFormat::VertexElementType,		EVertexStreamUsage::Default), 2));
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->NormalsBuffer,				0, FHairCardsNormalFormat::SizeInByte,	FHairCardsNormalFormat::SizeInByte * FHairCardsNormalFormat::ComponentCount,		FHairCardsNormalFormat::VertexElementType,		EVertexStreamUsage::Default), 3));

			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->NormalsBuffer,				0, 0,									FHairCardsMaterialFormat::SizeInByte,												FHairCardsMaterialFormat::VertexElementType,	EVertexStreamUsage::Default), 4)); // Dummy
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->VertexColorsBuffer,		0, 0,									FHairCardsMaterialFormat::SizeInByte,												FHairCardsMaterialFormat::VertexElementType,	EVertexStreamUsage::Default), 5));
			Elements.Add(AccessStreamComponent(FVertexStreamComponent(&LOD.RestResource->UVsBuffer,					0, 0,									FHairCardsUVFormat::SizeInByte * NumUVComponentCount,								FHairCardsUVFormat::VertexElementType,			EVertexStreamUsage::Default), 6));

			// Add additional UVs
			AddUVChannels(&LOD.RestResource->UVsBuffer, LOD.RestResource->BulkData.GetNumUVs(), 7);

			// Ensure the rest resources are in correct states for the VF
			const FHairMeshesRestResource* RestResource = LOD.RestResource;
			ENQUEUE_RENDER_COMMAND(FHairCardsVertexFactoryTransition)(/*UE::RenderCommandPipe::Groom,*/
			[RestResource, bHasVertexColor](FRHICommandListImmediate& RHICmdImmediateList)
			{
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->RestPositionBuffer.VertexBufferRHI,	ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->NormalsBuffer.VertexBufferRHI, 		ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->UVsBuffer.VertexBufferRHI, 			ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				if (bHasVertexColor)
				{
					RHICmdImmediateList.Transition(FRHITransitionInfo(RestResource->VertexColorsBuffer.VertexBufferRHI, ERHIAccess::Unknown, ERHIAccess::VertexOrIndexBuffer));
				}
			});
		}

		bNeedsDeclaration = true;
		check(Streams.Num() > 0);
	}
	InitDeclaration(Elements, EVertexInputStreamType::Default);
	check(IsValidRef(GetDeclaration()));

	FHairGroupInstance* HairInstance = Data.Instance;
	check(HairInstance->HairGroupPublicData);

	if (Data.GeometryType == EHairGeometryType::Cards && IsHairStrandsEnabled(EHairStrandsShaderType::Cards, GMaxRHIShaderPlatform))
	{
		if (HairInstance->Cards.LODs.IsValidIndex(Data.LODIndex))
		{
			const FHairGroupInstance::FCards::FLOD& LOD = HairInstance->Cards.LODs[Data.LODIndex];
			HairInstance->Cards.LODs[Data.LODIndex].UniformBuffer = CreateHairCardsVFUniformBuffer(HairInstance, Data.LODIndex, EHairGeometryType::Cards, bManualFetch);
		}
	}
	else if (Data.GeometryType == EHairGeometryType::Meshes && IsHairStrandsEnabled(EHairStrandsShaderType::Meshes, GMaxRHIShaderPlatform))
	{
		if (HairInstance->Meshes.LODs.IsValidIndex(Data.LODIndex))
		{
			const FHairGroupInstance::FMeshes::FLOD& LOD = HairInstance->Meshes.LODs[Data.LODIndex];
			HairInstance->Meshes.LODs[Data.LODIndex].UniformBuffer = CreateHairCardsVFUniformBuffer(HairInstance, Data.LODIndex, EHairGeometryType::Meshes, bManualFetch);
		}
	}
}

void FHairCardsVertexFactory::ReleaseResource()
{
	FVertexFactory::ReleaseResource();
	DeformedPositionVertexBuffer[0].ReleaseResource();
	DeformedPositionVertexBuffer[1].ReleaseResource();
	DeformedNormalVertexBuffer.ReleaseResource();
}

void FHairCardsVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Nothing as the initialization is done when needed
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairCardsVertexFactory, SF_Vertex,		FHairCardsVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairCardsVertexFactory, SF_Pixel,		FHairCardsVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairCardsVertexFactory, SF_Compute,	FHairCardsVertexFactoryShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairCardsVertexFactory, SF_RayHitGroup,	FHairCardsVertexFactoryShaderParameters);
#endif

void FHairCardsVertexFactory::ReleaseRHI()
{
	FVertexFactory::ReleaseRHI();
}

IMPLEMENT_VERTEX_FACTORY_TYPE(FHairCardsVertexFactory,"/Engine/Private/HairStrands/HairCardsVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsManualVertexFetch
);
