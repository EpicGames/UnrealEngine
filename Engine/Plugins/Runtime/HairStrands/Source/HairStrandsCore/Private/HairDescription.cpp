// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairDescription.h"
#include "Misc/SecureHash.h"
#include "Serialization/BulkDataReader.h"
#include "Serialization/BulkDataWriter.h"
#include "Serialization/EditorBulkDataReader.h"
#include "Serialization/EditorBulkDataWriter.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

const FStrandID FStrandID::Invalid(TNumericLimits<uint32>::Max());
const FGroomID FGroomID::Invalid(TNumericLimits<uint32>::Max());

#if WITH_EDITORONLY_DATA
static TConstArrayView<FGuid> GetHairDescriptionCustomVersions()
{
	// Use FMeshDescriptionBulkData's versions as HairDescription uses structures and functions 
	// from FMeshDescription(BulkData) which are subject to serizalization changes
	return FMeshDescriptionBulkData::GetMeshDescriptionCustomVersions();
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// FHairDescription

FHairDescription::FHairDescription()
	: NumVertices(0)
	, NumStrands(0)
{
	// Required attributes
	StrandAttributesSet.RegisterAttribute<int>(HairAttribute::Strand::VertexCount);
	VertexAttributesSet.RegisterAttribute<FVector3f>(HairAttribute::Vertex::Position, 1, FVector3f::ZeroVector);

	// Only one set of groom attributes
	GroomAttributesSet.Initialize(1);
}

void FHairDescription::InitializeVertices(int32 InNumVertices)
{
	NumVertices = InNumVertices;
	VertexAttributesSet.Initialize(InNumVertices);
}

void FHairDescription::InitializeStrands(int32 InNumStrands)
{
	NumStrands = InNumStrands;
	StrandAttributesSet.Initialize(InNumStrands);
}

FVertexID FHairDescription::AddVertex()
{
	FVertexID VertexID = FVertexID(NumVertices++);
	VertexAttributesSet.Insert(VertexID);
	return VertexID;
}

FStrandID FHairDescription::AddStrand()
{
	FStrandID StrandID = FStrandID(NumStrands++);
	StrandAttributesSet.Insert(StrandID);
	return StrandID;
}

void FHairDescription::Reset()
{
	NumVertices = 0;
	NumStrands = 0;

	VertexAttributesSet.Initialize(0);
	StrandAttributesSet.Initialize(0);
	GroomAttributesSet.Initialize(0);
}

bool FHairDescription::IsValid() const
{
	return (NumStrands > 0) && (NumVertices > 0);
}

bool FHairDescription::HasAttribute(EHairAttribute InAttribute) const
{
	// If a new optional attribute is added, please add its UI/text description here
	static_assert(uint32(EHairAttribute::Count) == 8);

	switch (InAttribute)
	{
	case EHairAttribute::Width:			return VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width).IsValid() || StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::Width).IsValid() || GroomAttributes().GetAttributesRef<float>(HairAttribute::Groom::Width).IsValid();
	case EHairAttribute::RootUV:		return StrandAttributes().GetAttributesRef<FVector2f>(HairAttribute::Strand::RootUV).IsValid();
	case EHairAttribute::ClumpID:		return StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::ClumpID).IsValid() || StrandAttributes().GetAttributesRef<FVector3f>(HairAttribute::Strand::ClumpID).IsValid();
	case EHairAttribute::StrandID:		return StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::ID).IsValid();
	case EHairAttribute::Color:			return VertexAttributes().GetAttributesRef<FVector3f>(HairAttribute::Vertex::Color).IsValid();
	case EHairAttribute::Roughness:		return VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Roughness).IsValid();
	case EHairAttribute::AO:			return VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::AO).IsValid();
	case EHairAttribute::PrecomputedGuideWeights:
	{
		return
			// Single
			(StrandAttributes().GetAttributesRef<int>(HairAttribute::Strand::ClosestGuides).IsValid() &&
			 StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::GuideWeights).IsValid())
			||
			// Triplet
			(StrandAttributes().GetAttributesRef<FVector3f>(HairAttribute::Strand::ClosestGuides).IsValid() &&
			 StrandAttributes().GetAttributesRef<FVector3f>(HairAttribute::Strand::GuideWeights).IsValid());
	}
	}
	return false;
}

// Deprecated named accessors. Use HasAttribute() instead
// These accessors are defined here to ease mirror logic with BuildHairDescriptionGroups, if any type/attributes changes
bool FHairDescription::HasRootUV() const { return HasAttribute(EHairAttribute::RootUV); }
bool FHairDescription::HasClumpID() const { return HasAttribute(EHairAttribute::ClumpID); }
bool FHairDescription::HasGuideWeights() const { return HasAttribute(EHairAttribute::PrecomputedGuideWeights); }
bool FHairDescription::HasColorAttributes() const { return HasAttribute(EHairAttribute::Color); }
bool FHairDescription::HasRoughnessAttributes() const { return HasAttribute(EHairAttribute::Roughness); }
bool FHairDescription::HasAOAttributes() const { return HasAttribute(EHairAttribute::AO); }

void FHairDescription::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << NumVertices;
	Ar << NumStrands;

	Ar << VertexAttributesSet;
	Ar << StrandAttributesSet;
	Ar << GroomAttributesSet;
}

FArchive& operator<<(FArchive& Ar, FHairDescriptionVersion& Version)
{
	Ar << Version.bIsValid;
	if (Version.bIsValid)
	{
		Ar << Version.UEVersion;
		Ar << Version.LicenseeVersion;
		Version.CustomVersions.Serialize(Ar);
	}
	else if (Ar.IsLoading())
	{
		Version = FHairDescriptionVersion();
	}
	return Ar;
}

void FHairDescriptionVersion::CopyVersionsFromArchive(const FArchive& Ar)
{
	bIsValid = true;
	UEVersion = Ar.UEVer();
	LicenseeVersion = Ar.LicenseeUEVer();
	CustomVersions = Ar.GetCustomVersions();
}

void FHairDescriptionVersion::CopyVersionsToArchive(FArchive& Ar) const
{
	check(bIsValid);
	Ar.SetUEVer(UEVersion);
	Ar.SetLicenseeUEVer(LicenseeVersion);
	Ar.SetCustomVersions(CustomVersions);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// FHairDescriptionBulkData

#if WITH_EDITORONLY_DATA
void FHairDescriptionBulkData::Serialize(FArchive& Ar, UObject* Owner)
{
	for (const FGuid& CustomVersionGuid : GetHairDescriptionCustomVersions())
	{
		Ar.UsingCustomVersion(CustomVersionGuid);
	}

	if (Ar.IsTransacting())
	{
		// If transacting, keep these members alive the other side of an undo, otherwise their values will get lost
		Ar << BulkDataVersion;
		Ar << bBulkDataUpdated;
	}
	else
	{
		if (Ar.IsSaving())
		{
			// If the bulk data hasn't been updated since this was loaded, there's a possibility that it has old versioning.
			// Explicitly load and resave the FHairDescription so that its version is in sync with the FHairDescriptionBulkData.
			if (!bBulkDataUpdated)
			{
				FHairDescription HairDescription;
				LoadHairDescription(HairDescription);
				SaveHairDescription(HairDescription);
			}
		}
	}

	// Convert legacy hair description bulk data
	if (Ar.IsLoading() && Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::UpdateHairDescriptionBulkData)
	{
		FByteBulkData TempBulkData;
		TempBulkData.Serialize(Ar, Owner);

		FGuid TempGuid;
		Ar << TempGuid;

		BulkData.CreateFromBulkData(TempBulkData, TempGuid, Owner);
	}
	else
	{
	BulkData.Serialize(Ar, Owner);
	}

	if (!Ar.IsTransacting() && Ar.IsLoading())
	{
		// If loading, take the package custom version so it can be applied to the bulk data archive
		// when unpacking HairDescription from it
		BulkDataVersion.bIsValid = true;
		BulkData.GetBulkDataVersions(Ar, BulkDataVersion.UEVersion, BulkDataVersion.LicenseeVersion, BulkDataVersion.CustomVersions);
	}
}

void FHairDescriptionBulkData::SaveHairDescription(FHairDescription& HairDescription)
{
	BulkData.Reset();

	if (HairDescription.IsValid())
	{
		UE::Serialization::FEditorBulkDataWriter Ar(BulkData);
		HairDescription.Serialize(Ar);

		// Preserve CustomVersions at save time so we can reuse the same ones when reloading direct from memory
		BulkDataVersion.CopyVersionsFromArchive(Ar);
	}

	// Mark the HairDescriptionBulkData as having been updated.
	// This means we know that its version is up-to-date.
	bBulkDataUpdated = true;
}

void FHairDescriptionBulkData::LoadHairDescription(FHairDescription& HairDescription)
{
	HairDescription.Reset();

	if (!IsEmpty())
	{
		UE::Serialization::FEditorBulkDataReader Ar(BulkData);

		// Propagate the custom version information from the package to the bulk data, so that the HairDescription
		// is serialized with the same versioning
		checkf(BulkDataVersion.IsValid(), TEXT("If BulkData is non-empty, we should have set the BulkDataVersion when we populated it."));
		BulkDataVersion.CopyVersionsToArchive(Ar);

		HairDescription.Serialize(Ar);
	}
}

void FHairDescriptionBulkData::Empty()
{
	BulkData.Reset();
}

FString FHairDescriptionBulkData::GetIdString() const
{
	FString GuidString = UE::Serialization::IoHashToGuid(BulkData.GetPayloadId()).ToString();
	GuidString += TEXT("X");
	return GuidString;
}

#endif // #if WITH_EDITORONLY_DATA
