// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsClusterCulling.h"
#include "RendererInterface.h"
#include "SceneView.h"
#include "Shader.h"
#include "RenderGraph.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "GroomVisualizationData.h"
#include "GroomInstance.h"
#include "ShaderPrintParameters.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairClusterCullCS: public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairClusterCullCS);
	SHADER_USE_PARAMETER_STRUCT(FHairClusterCullCS, FGlobalShader);

	class FPointPerCurve : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_POINT_PER_CURVE", 4, 8, 16, 32, 64);
	using FPermutationDomain = TShaderPermutationDomain<FPointPerCurve>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER(uint32, ClusterGroupIndex)
		SHADER_PARAMETER(float, LODIndex)
		SHADER_PARAMETER(float, LODBias)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(uint32, DispatchCountX)
		SHADER_PARAMETER(FVector4f, ClusterInfoParameters)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CurveToClusterIdBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PointLODBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairClusterInfo>, ClusterInfoBuffer)
	
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutPointCounter)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutCulledCurveBuffer)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 64u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLUSTER_CULL"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairClusterCullCS, "/Engine/Private/HairStrands/HairStrandsClusterCulling.usf", "Main", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairClusterCullArgsCS: public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairClusterCullArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FHairClusterCullArgsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, ClusterGroupIndex)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PointCounterBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDrawArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsGlobalBuffer)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLUSTER_CULL_ARGS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairClusterCullArgsCS, "/Engine/Private/HairStrands/HairStrandsClusterCulling.usf", "Main", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////
void AddClusterCullingPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FSceneView* View,
	const FShaderPrintData* ShaderPrintData,
	FHairStrandClusterData& ClusterDatas,
	FRDGBufferUAVRef IndirectDispatchArgsGlobalUAV)
{
	check(View);

	static bool bDebug = false;
	if (bDebug)
	{
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForCharacters(2048);
		ShaderPrint::RequestSpaceForLines(2048);
	}

	const uint32 ClusterCount = ClusterDatas.HairGroups.Num();

	// 0. Glogal counter for visible points
	FRDGBufferRef PointCounter 	= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ClusterCount), TEXT("Hair.ClusterPointCounter"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PointCounter, PF_R32_UINT), 0);

	// 1. Build culled index buffer
	FRDGBufferUAVRef PointCounterUAVSkipBarrier = GraphBuilder.CreateUAV(PointCounter, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
	uint32 ClusterGroupIt = 0;
	for (FHairStrandClusterData::FHairGroup& ClusterData : ClusterDatas.HairGroups)
	{
		FHairClusterCullCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairClusterCullCS::FParameters>();
		Parameters->ClusterGroupIndex	 = ClusterGroupIt++;
		Parameters->LODIndex 			 = ClusterData.LODIndex;
		Parameters->LODBias 			 = ClusterData.LODBias;
		Parameters->CurveCount 			 = ClusterData.HairGroupPublicPtr->GetActiveStrandsCurveCount();	
		Parameters->CurveBuffer 		 = RegisterAsSRV(GraphBuilder, *ClusterData.CurveBuffer);
		Parameters->CurveToClusterIdBuffer=RegisterAsSRV(GraphBuilder, *ClusterData.CurveToClusterIdBuffer);
		Parameters->PointLODBuffer 		 = RegisterAsSRV(GraphBuilder, *ClusterData.PointLODBuffer);
		Parameters->ClusterInfoBuffer 	 = RegisterAsSRV(GraphBuilder, *ClusterData.ClusterInfoBuffer);
		Parameters->ClusterInfoParameters= ClusterData.ClusterInfoParameters;
		Parameters->ViewUniformBuffer	 = View->ViewUniformBuffer;
		Parameters->OutPointCounter 	 = PointCounterUAVSkipBarrier;
		Parameters->OutIndexBuffer 		 = RegisterAsUAV(GraphBuilder, *ClusterData.GetCulledVertexIdBuffer());
		Parameters->OutCulledCurveBuffer = RegisterAsUAV(GraphBuilder, *ClusterData.GetCulledCurveBuffer()); // TODO: this could be changed to be transient buffer

		if (bDebug && ShaderPrintData)
		{
			ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
		}

		// Compute the dispatch information for pass dispatching work per curve
		const FPointPerCurveDispatchInfo DispatchInfo = GetPointPerCurveDispatchInfo(ClusterData.MaxPointPerCurve, Parameters->CurveCount, FHairClusterCullCS::GetGroupSize());
		Parameters->DispatchCountX = DispatchInfo.DispatchCount.X;

		FHairClusterCullCS::FPermutationDomain Permutation;
		Permutation.Set<FHairClusterCullCS::FPointPerCurve>(DispatchInfo.PointPerCurve);
		TShaderMapRef<FHairClusterCullCS> ComputeShader(ShaderMap, Permutation);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::ClusterCullPass2"),
			ComputeShader,
			Parameters,
			DispatchInfo.DispatchCount);
	}

	// 2. Prepare indirect draw/dispatch args buffers
	FRDGBufferSRVRef PointCounterSRV = GraphBuilder.CreateSRV(PointCounter, PF_R32_UINT);
	ClusterGroupIt = 0;
	for (FHairStrandClusterData::FHairGroup& ClusterData : ClusterDatas.HairGroups)
	{
		FRDGImportedBuffer DrawIndirectParametersBuffer = Register(GraphBuilder, ClusterData.HairGroupPublicPtr->GetDrawIndirectBuffer(), ERDGImportedBufferFlags::CreateViews);
		FRDGImportedBuffer DrawIndirectParametersRasterComputeBuffer = Register(GraphBuilder, ClusterData.HairGroupPublicPtr->GetDrawIndirectRasterComputeBuffer(), ERDGImportedBufferFlags::CreateViews);

		FHairClusterCullArgsCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairClusterCullArgsCS::FParameters>();
		Parameters->InstanceRegisteredIndex = ClusterData.InstanceRegisteredIndex;
		Parameters->ClusterGroupIndex = ClusterGroupIt++;
		Parameters->PointCounterBuffer = PointCounterSRV;
		Parameters->RWIndirectDrawArgsBuffer = DrawIndirectParametersBuffer.UAV;
		Parameters->RWIndirectDispatchArgsBuffer = DrawIndirectParametersRasterComputeBuffer.UAV;
		Parameters->RWIndirectDispatchArgsGlobalBuffer = IndirectDispatchArgsGlobalUAV;

		TShaderMapRef<FHairClusterCullArgsCS> ComputeShader(ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::ClusterCullPrepareArgs"),
			ComputeShader,
			Parameters,
			FIntVector(1, 1, 1));

		GraphBuilder.SetBufferAccessFinal(DrawIndirectParametersBuffer.Buffer, ERHIAccess::IndirectArgs | ERHIAccess::SRVMask);
		GraphBuilder.SetBufferAccessFinal(DrawIndirectParametersRasterComputeBuffer.Buffer, ERHIAccess::IndirectArgs | ERHIAccess::SRVMask);
		ClusterData.SetCullingResultAvailable(true);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void AddInstanceToClusterData(FHairGroupInstance* In, FHairStrandClusterData& Out)
{
	// Initialize group cluster data for culling by the renderer
	const int32 ClusterDataGroupIndex = Out.HairGroups.Num();
	FHairStrandClusterData::FHairGroup& HairGroupCluster = Out.HairGroups.Emplace_GetRef();
	HairGroupCluster.MaxPointPerCurve = In->Strands.IsValid() ? In->Strands.GetData().Header.MaxPointPerCurve : 0;
	HairGroupCluster.ClusterScale = In->HairGroupPublicData->ClusterScale;

	HairGroupCluster.CurveBuffer = &In->Strands.RestResource->CurveBuffer;
	HairGroupCluster.PointLODBuffer = &In->Strands.ClusterResource->PointLODBuffer;

	HairGroupCluster.ClusterInfoParameters = In->Strands.ClusterResource->BulkData.Header.ClusterInfoParameters;
	HairGroupCluster.ClusterInfoBuffer = &In->Strands.ClusterResource->ClusterInfoBuffer;
	HairGroupCluster.CurveToClusterIdBuffer = &In->Strands.ClusterResource->CurveToClusterIdBuffer;

	HairGroupCluster.InstanceRegisteredIndex = In->RegisteredIndex;
	HairGroupCluster.HairGroupPublicPtr = In->HairGroupPublicData;
	HairGroupCluster.LODBias  = In->HairGroupPublicData->GetLODBias();
	HairGroupCluster.LODIndex = In->HairGroupPublicData->GetLODIndex();
	HairGroupCluster.bVisible = In->HairGroupPublicData->GetLODVisibility();

	// These buffer are create during the culling pass
	// HairGroupCluster.ClusterIdBuffer = nullptr;
	// HairGroupCluster.ClusterIndexOffsetBuffer = nullptr;
	// HairGroupCluster.ClusterIndexCountBuffer = nullptr;

	HairGroupCluster.HairGroupPublicPtr->ClusterDataIndex = ClusterDataGroupIndex;
}