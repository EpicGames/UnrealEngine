// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "HairStrandsInterface.h"

struct FHairGroupInstance;
class FViewInfo;
class FGlobalShaderMap;
class FRDGBuilder;

struct FHairStrandClusterData
{
	struct FHairGroup
	{
		uint32 InstanceRegisteredIndex = ~0;
		float ClusterScale = 0;
		uint32 MaxPointPerCurve = 0;
		FVector4f ClusterInfoParameters = FVector4f::Zero();
		float LODIndex = -1;
		float LODBias = 0.0f;
		bool bVisible = false;

		FRDGExternalBuffer* CurveBuffer = nullptr;
		FRDGExternalBuffer* PointLODBuffer = nullptr;

		// See FHairStrandsClusterResource fro details about those buffers.
		FRDGExternalBuffer* ClusterInfoBuffer = nullptr; // SRV
		FRDGExternalBuffer* CurveToClusterIdBuffer = nullptr; // SRV

		TRefCountPtr<FRDGPooledBuffer> ClusterIdBuffer;
		TRefCountPtr<FRDGPooledBuffer> ClusterIndexOffsetBuffer;
		TRefCountPtr<FRDGPooledBuffer> ClusterIndexCountBuffer;

		// Culling & LOD output
		FRDGExternalBuffer* GetCulledCurveBuffer() const				{ return HairGroupPublicPtr ? &HairGroupPublicPtr->GetCulledCurveBuffer() : nullptr; }
		FRDGExternalBuffer* GetCulledVertexIdBuffer() const				{ return HairGroupPublicPtr ? &HairGroupPublicPtr->GetCulledVertexIdBuffer() : nullptr; }
		bool GetCullingResultAvailable() const							{ return HairGroupPublicPtr ? HairGroupPublicPtr->GetCullingResultAvailable() : false; }
		void SetCullingResultAvailable(bool b)							{ if (HairGroupPublicPtr) HairGroupPublicPtr->SetCullingResultAvailable(b); }

		FHairGroupPublicData* HairGroupPublicPtr = nullptr;
	};

	TArray<FHairGroup> HairGroups;
};

void AddInstanceToClusterData(
	FHairGroupInstance* In,
	FHairStrandClusterData& Out);

void AddClusterCullingPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FSceneView* View,
	const FShaderPrintData* ShaderPrintData,
	FHairStrandClusterData& ClusterDatas,
	FRDGBufferUAVRef IndirectDispatchArgsGlobalUAV);
