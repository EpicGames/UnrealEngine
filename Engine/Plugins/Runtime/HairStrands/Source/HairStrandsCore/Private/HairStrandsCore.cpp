// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsCore.h"
#include "Engine/Texture2D.h"
#include "HairStrandsInterface.h"
#include "Interfaces/IPluginManager.h"
#include "GeometryCache.h"
#include "GroomManager.h"
#include "Engine/StaticMesh.h"
#include "GroomAsset.h"
#include "GroomBindingCompiler.h"
#include "GroomCreateBindingOptions.h"
#include "AssetCompilingManager.h"
#include "UObject/Package.h"
#include "GroomVisualizationData.h"

IMPLEMENT_MODULE(FHairStrandsCore, HairStrandsCore);
DEFINE_LOG_CATEGORY(LogHairStrands);

void ProcessHairStrandsBookmark(
	FRDGBuilder* GraphBuilder,
	EHairStrandsBookmark Bookmark,
	FHairStrandsBookmarkParameters& Parameters);

void FHairStrandsCore::StartupModule()
{
	RegisterBookmarkFunction(ProcessHairStrandsBookmark);

	FAssetCompilingManager::Get().RegisterManager(&FGroomBindingCompilingManager::Get());

	// Maps virtual shader source directory /Plugin/FX/Niagara to the plugin's actual Shaders directory.
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("HairStrands"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/Runtime/HairStrands"), PluginShaderDir);
	SetGroomEnabled(true);
}

void FHairStrandsCore::ShutdownModule()
{
	FAssetCompilingManager::Get().UnregisterManager(&FGroomBindingCompilingManager::Get());

	SetGroomEnabled(false);
}

void FHairStrandsCore::RegisterAssetHelper(const FHairAssetHelper& Helper)
{
#if WITH_EDITOR
	AssetHelper() = Helper;
#endif
}

#if WITH_EDITOR
FHairAssetHelper& FHairStrandsCore::AssetHelper()
{
	static FHairAssetHelper AssetHelper;
	return AssetHelper;
}
#endif

// This is a workaround to be able to create & register UTexture2D from the HairStrandsCore project 
// without requiring editor dependencies. This is used for the hair cards generation, which creates 
// UTexture2D assets when adding a new LOD. Ideally this should be changed and move the logic to the 
// editor part. This is done this way by lack of time and knowledge regarding editor code.
//
// Shared function for allocating and registering UTexture2D
// * TTextureAllocation implements the actual texture/resources allocation
// * TCreateFilename generates a unique filename. It is passed as a function pointer as it uses internally editor dependency, 
//   which we don't want to drag into this runtime module
//
// E.g. PackageName = GroomAsset->GetOutermost()->GetName()
//
// Example of allocator:
//	static void ExampleTextureAllocator(UTexture2D* Out, uint32 Resolution, uint32 MipCount, EPixelFormat Format, ETextureSourceFormat SourceFormat)
//	{
//		FTextureFormatSettings FormatSettings;
//		FormatSettings.SRGB = false;
//	#if WITH_EDITORONLY_DATA
//		Out->Source.Init(Resolution, Resolution, 1, MipCount, SourceFormat, nullptr);
//		Out->SetLayerFormatSettings(0, FormatSettings);
//	#endif // #if WITH_EDITORONLY_DATA
//	
//		Out->PlatformData = new FTexturePlatformData();
//		Out->PlatformData->SizeX = Resolution;
//		Out->PlatformData->SizeY = Resolution;
//		Out->PlatformData->PixelFormat = Format;
//	
//		Out->UpdateResource();
//	}
//

#if WITH_EDITOR
UTexture2D* FHairStrandsCore::CreateTexture(const FString& InPackageName, const FIntPoint& Resolution, const FString& Suffix, TTextureAllocation TextureAllocation)
{
	if (AssetHelper().CreateFilename == nullptr || AssetHelper().RegisterAsset == nullptr)
	{
		return nullptr;
	}

	FString Name;
	FString PackageName;
	AssetHelper().CreateFilename(InPackageName, Suffix, PackageName, Name);

	UObject* InParent = nullptr;
	UPackage* Package = Cast<UPackage>(InParent);
	if (InParent == nullptr && !PackageName.IsEmpty())
	{
		// Then find/create it.
		Package = CreatePackage(*PackageName);
		if (!ensure(Package))
		{
			// There was a problem creating the package
			return nullptr;
		}
	}

	if (UTexture2D* Out = NewObject<UTexture2D>(Package, *Name, RF_Public | RF_Standalone | RF_Transactional))
	{
		const uint32 MipCount = FMath::Max(FMath::FloorLog2(Resolution.X) + 1, FMath::FloorLog2(Resolution.Y) + 1);
		TextureAllocation(Out, Resolution, MipCount);
		Out->MarkPackageDirty();

		// Notify the asset registry
		AssetHelper().RegisterAsset(Out);
		return Out;
	}

	return nullptr;
}

void FHairStrandsCore::ResizeTexture(UTexture2D* Out, const FIntPoint& Resolution, TTextureAllocation TextureAllocation)
{
	if (Out != nullptr && (Out->GetSizeX() != Resolution.X || Out->GetSizeY() != Resolution.Y))
	{
		const uint32 MipCount = FMath::Max(FMath::FloorLog2(Resolution.X) + 1, FMath::FloorLog2(Resolution.Y) + 1);
		TextureAllocation(Out, Resolution, MipCount);
		Out->MarkPackageDirty();
	}
}

UStaticMesh* FHairStrandsCore::CreateStaticMesh(const FString& InPackageName, const FString& Suffix)
{
	if (AssetHelper().CreateFilename == nullptr || AssetHelper().RegisterAsset == nullptr)
	{
		return nullptr;
	}

	FString Name;
	FString PackageName;
	AssetHelper().CreateFilename(InPackageName, Suffix, PackageName, Name);

	UObject* InParent = nullptr;
	UPackage* Package = Cast<UPackage>(InParent);
	if (InParent == nullptr && !PackageName.IsEmpty())
	{
		// Then find/create it.
		Package = CreatePackage(*PackageName);
		if (!ensure(Package))
		{
			// There was a problem creating the package
			return nullptr;
		}
	}

	if (UStaticMesh* Out = NewObject<UStaticMesh>(Package, *Name, RF_Public | RF_Standalone | RF_Transactional))
	{
		// initialize the LOD 0 MeshDescription
		Out->SetNumSourceModels(1);
		Out->GetSourceModel(0).BuildSettings.bRecomputeNormals = false;
		Out->GetSourceModel(0).BuildSettings.bRecomputeTangents = true;
		Out->MarkPackageDirty();
		AssetHelper().RegisterAsset(Out);
		return Out;
	}

	return nullptr;
}

UGroomBindingAsset* FHairStrandsCore::CreateGroomBindingAsset(const FString& InPackageName, UObject* InParent, UGroomAsset* GroomAsset, USkeletalMesh* SourceSkelMesh, USkeletalMesh* TargetSkelMesh, const int32 NumInterpolationPoints, const int32 MatchingSection)
{
	return CreateGroomBindingAsset(EGroomBindingMeshType::SkeletalMesh, InPackageName, InParent, GroomAsset, SourceSkelMesh, TargetSkelMesh, NumInterpolationPoints, MatchingSection, NAME_None);
}

UGroomBindingAsset* FHairStrandsCore::CreateGroomBindingAsset(EGroomBindingMeshType BindingType, const FString& InPackageName, UObject* InParent, UGroomAsset* GroomAsset, UObject* Source, UObject* Target, const int32 NumInterpolationPoints, const int32 MatchingSection, const FName TargetMeshAttribute)
{
	if (!GroomAsset ||
		!((BindingType == EGroomBindingMeshType::SkeletalMesh && Cast<USkeletalMesh>(Target)) || 
		 (BindingType == EGroomBindingMeshType::GeometryCache && Cast<UGeometryCache>(Target)))
		)
	{
		return nullptr;
	}

	// If provided name is empty, then create an auto-generated (unique) filename
	FString Name;
	FString PackageName;
	if (InPackageName.IsEmpty())
	{
		FString Suffix;
		if (Source)
		{
			Suffix += TEXT("_") + Source->GetName();
		}
		if (Target)
		{
			Suffix += TEXT("_") + Target->GetName();
		}
		Suffix += TEXT("_Binding");
		AssetHelper().CreateFilename(GroomAsset->GetOutermost()->GetName(), Suffix, PackageName, Name);		
	}
	else
	{
		AssetHelper().CreateFilename(InPackageName, TEXT(""), PackageName, Name);
	}

	UPackage* Package = Cast<UPackage>(InParent);
	if (InParent == nullptr && !PackageName.IsEmpty())
	{
		// Then find/create it.
		Package = CreatePackage(*PackageName);
		if (!ensure(Package))
		{
			// There was a problem creating the package
			return nullptr;
		}
	}

	if (UGroomBindingAsset* Out = NewObject<UGroomBindingAsset>(Package, *Name, RF_Public | RF_Standalone | RF_Transactional))
	{
		Out->SetGroomBindingType(BindingType);
		Out->SetGroom(GroomAsset);
		if (BindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			Out->SetSourceSkeletalMesh(Cast<USkeletalMesh>(Source));
			Out->SetTargetSkeletalMesh(Cast<USkeletalMesh>(Target));
		}
		else
		{
			Out->SetSourceGeometryCache(Cast<UGeometryCache>(Source));
			Out->SetTargetGeometryCache(Cast<UGeometryCache>(Target));
		}
		Out->GetHairGroupsPlatformData().Reserve(GroomAsset->GetHairGroupsPlatformData().Num());
		Out->SetNumInterpolationPoints(NumInterpolationPoints);
		Out->SetMatchingSection(MatchingSection);
		Out->SetTargetBindingAttribute(TargetMeshAttribute);
		Out->MarkPackageDirty();
		AssetHelper().RegisterAsset(Out);
		return Out;
	}
	return nullptr;
}

UGroomBindingAsset* FHairStrandsCore::CreateGroomBindingAsset(EGroomBindingMeshType BindingType, UGroomAsset* GroomAsset, UObject* Source, UObject* Target, const int32 NumInterpolationPoints, const int32 MatchingSection, const FName TargetMeshAttribute)
{
	return CreateGroomBindingAsset(BindingType, FString(), nullptr, GroomAsset, Source, Target, NumInterpolationPoints, MatchingSection, TargetMeshAttribute);
}

UGroomBindingAsset* FHairStrandsCore::CreateGroomBindingAsset(UGroomAsset* GroomAsset, USkeletalMesh* SourceSkelMesh, USkeletalMesh* TargetSkelMesh, const int32 NumInterpolationPoints, const int32 MatchingSection)
{
	return CreateGroomBindingAsset(EGroomBindingMeshType::SkeletalMesh, FString(), nullptr, GroomAsset, SourceSkelMesh, TargetSkelMesh, NumInterpolationPoints, MatchingSection, NAME_None);
}

void FHairStrandsCore::SaveAsset(UObject* Object)
{
	if (AssetHelper().SaveAsset == nullptr)
	{
		return;
	}

	AssetHelper().SaveAsset(Object);
}
#endif // WITH_EDITOR
