// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsDatas.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "HairAttributes.h"
#include "IO/IoDispatcher.h"
#include "GroomRBFDeformer.h"
#include "Misc/ScopeExit.h"

#if WITH_EDITOR
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper

float GetHairStrandsMaxLength(const FHairStrandsDatas& In)
{
	float MaxLength = 0;
	for (float CurveLength : In.StrandsCurves.CurvesLength)
	{
		MaxLength = FMath::Max(MaxLength, CurveLength);
	}
	return MaxLength;
}

float GetHairStrandsMaxRadius(const TArray<float>& InPointsRadius)
{
	float MaxRadius = 0;
	for (float PointRadius : InPointsRadius)
	{
		MaxRadius = FMath::Max(MaxRadius, PointRadius);
	}
	return MaxRadius;
}

float GetHairStrandsMaxRadius(const FHairStrandsDatas& In)
{
	return GetHairStrandsMaxRadius(In.StrandsPoints.PointsRadius);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Curve/Point/Strands/Interpolation/Culling Data

void FHairStrandsCurves::Reset()
{
	CurvesOffset.Reset();
	CurvesCount.Reset();
	CurvesLength.Reset();
	CurvesRootUV.Reset();
	ClumpIDs.Reset();
	CurvesClosestGuideIDs.Reset();
	CurvesClosestGuideWeights.Reset();
	CurvesMapping.Reset();
	SourceCurveCount = 0;
}

void FHairStrandsCurves::SetNum(const uint32 NumCurves, uint32 InAttributes, bool bHasMapping)
{
	CurvesOffset.SetNum(NumCurves + 1);
	CurvesCount.SetNum(NumCurves);
	CurvesLength.SetNum(NumCurves);

	// Not initialized to track if the data are available
	if (HasHairAttribute(InAttributes, EHairAttribute::RootUV))		{ CurvesRootUV.SetNum(NumCurves); }
	if (HasHairAttribute(InAttributes, EHairAttribute::StrandID))	{ StrandIDs.SetNum(NumCurves); }
	if (HasHairAttribute(InAttributes, EHairAttribute::ClumpID))	{ ClumpIDs.SetNum(NumCurves); }
	if (HasHairAttribute(InAttributes, EHairAttribute::PrecomputedGuideWeights))
	{
		CurvesClosestGuideIDs.SetNum(NumCurves);
		CurvesClosestGuideWeights.SetNum(NumCurves);
	}

	if(bHasMapping)
	{
		CurvesMapping.SetNum(NumCurves);
	}
}

void FHairStrandsPoints::Reset()
{
	PointsPosition.Reset();
	PointsRadius.Reset();
	PointsCoordU.Reset();
	PointsBaseColor.Reset();
	PointsRoughness.Reset();
	PointsAO.Reset();
	PointsMapping.Reset();
	SourcePointCount = 0;
}

void FHairStrandsPoints::SetNum(const uint32 NumPoints, uint32 InAttributes, bool bHasMapping)
{
	PointsPosition.SetNum(NumPoints);
	PointsRadius.SetNum(NumPoints);
	PointsCoordU.SetNum(NumPoints);

	// Not initialized to track if the data are available
	if (HasHairAttribute(InAttributes, EHairAttribute::Color))		{ PointsBaseColor.SetNum(NumPoints); }
	if (HasHairAttribute(InAttributes, EHairAttribute::Roughness))	{ PointsRoughness.SetNum(NumPoints); }
	if (HasHairAttribute(InAttributes, EHairAttribute::AO))			{ PointsAO.SetNum(NumPoints); }

	if(bHasMapping)
	{
		PointsMapping.SetNum(NumPoints);
	}
}

void FHairStrandsRawDatas::Reset()
{
	StrandsCurves.Reset();
	StrandsPoints.Reset();
}

void FHairStrandsDatas::Reset()
{
	StrandsCurves.Reset();
	StrandsPoints.Reset();
	HairDensity = 1;
	BoundingBox = FBox3f(EForceInit::ForceInit);
}

void FHairStrandsInterpolationDatas::Reset()
{
	CurveSimWeights.Reset();
	CurveSimIndices.Reset();
	CurveSimRootPointIndex.Reset();
	PointSimLerps.Reset();
	PointSimIndices.Reset();
}

void FHairStrandsInterpolationDatas::SetNum(const uint32 NumCurves, const uint32 NumPoints)
{
	CurveSimIndices.SetNum(NumCurves);
	CurveSimWeights.SetNum(NumCurves);
	CurveSimRootPointIndex.SetNum(NumCurves);
	PointSimIndices.SetNum(NumPoints);
	PointSimLerps.SetNum(NumPoints);
}

void FHairStrandsClusterData::Reset()
{
	*this = FHairStrandsClusterData();
}

/////////////////////////////////////////////////////////////////////////////////////////
// Common bulk data

namespace HairStrands
{
#if WITH_EDITOR
	const UE::DerivedData::FValueId HairStrandsValueId = UE::DerivedData::FValueId::FromName("HairStrandsStreamingData");
#endif
}

void FHairStrandsBulkCommon::Write_DDC(UObject* Owner, TArray<UE::DerivedData::FCachePutValueRequest>& Out)
{
#if WITH_EDITORONLY_DATA
	FHairStrandsBulkCommon::FQuery Q;
	Q.Type = FHairStrandsBulkCommon::FQuery::WriteDDC;
	Q.OutWriteDDC = &Out;
	Q.DerivedDataKey = &DerivedDataKey;
	Q.Owner = Owner;
	GetResources(Q);
#endif
}

void FHairStrandsBulkCommon::Read_DDC(FHairStreamingRequest* In, TArray<UE::DerivedData::FCacheGetChunkRequest>& Out)
{
#if WITH_EDITORONLY_DATA
	FHairStrandsBulkCommon::FQuery Q;
	Q.Type = FHairStrandsBulkCommon::FQuery::ReadDDC;
	Q.OutReadDDC = &Out;
	Q.DerivedDataKey = &DerivedDataKey;
	Q.StreamingRequest = In;
	Q.StreamingRequest->Chunks.Reserve(GetResourceCount()); // This ensures that Chunks array is never reallocated, which would invalidate pointers to FChunk
	GetResources(Q);
#endif
}

void FHairStrandsBulkCommon::Read_IO(FHairStreamingRequest* In, FBulkDataBatchRequest& Out)
{
	FBulkDataBatchRequest::FBatchBuilder Batch = Out.NewBatch(GetResourceCount());

	FHairStrandsBulkCommon::FQuery Q;
	Q.Type = FHairStrandsBulkCommon::FQuery::ReadIO;
	Q.OutReadIO = &Batch;
	Q.StreamingRequest = In;
	Q.StreamingRequest->Chunks.Reserve(GetResourceCount()); // This ensures that Chunks array is never reallocated, which would invalidate pointers to FChunk
	GetResources(Q);
	Q.OutReadIO->Issue(Out);
}
void FHairStrandsBulkCommon::Write_IO(UObject* Owner, FArchive& Out)
{
	GetResourceVersion(Out);

	FHairStrandsBulkCommon::FQuery Q;
	Q.Type = FHairStrandsBulkCommon::FQuery::ReadWriteIO;
	Q.OutWriteIO = &Out;
	Q.Owner = Owner;
	GetResources(Q);
}
void FHairStrandsBulkCommon::Unload(FHairStreamingRequest* In)
{
	FHairStrandsBulkCommon::FQuery Q;
	Q.Type = FHairStrandsBulkCommon::FQuery::UnloadData;
	Q.StreamingRequest = In;
	Q.StreamingRequest->Chunks.Reserve(GetResourceCount()); // This ensures that Chunks array is never reallocated, which would invalidate pointers to FChunk
	GetResources(Q);
}

void FHairStrandsBulkCommon::Serialize(FArchive& Ar, UObject* Owner)
{
	SerializeHeader(Ar, Owner);
	SerializeData(Ar, Owner);
}

void FHairStrandsBulkCommon::SerializeData(FArchive& Ar, UObject* Owner)
{
	Write_IO(Owner, Ar);
}

void FHairStrandsBulkCommon::FQuery::Add(FHairBulkContainer& In, const TCHAR* InSuffix, uint32& InOffset, uint32 InSize) 
{
	check(Type != None);
#if WITH_EDITORONLY_DATA
	if (Type == WriteDDC)
	{
		const int64 DataSizeInByte = In.Data.GetBulkDataSize();
		TArray<uint8> WriteData;
		WriteData.SetNum(DataSizeInByte);
		FMemory::Memcpy(WriteData.GetData(), In.Data.Lock(LOCK_READ_ONLY), DataSizeInByte);
		In.Data.Unlock();

		using namespace UE::DerivedData;
		FCachePutValueRequest& Out = OutWriteDDC->AddDefaulted_GetRef();
		if (Owner) { Out.Name = Owner->GetPathName(); }
		Out.Key 	= ConvertLegacyCacheKey(*DerivedDataKey + InSuffix);
		Out.Value 	= FValue::Compress(MakeSharedBufferFromArray(MoveTemp(WriteData)));
		Out.Policy 	= ECachePolicy::Default;
		Out.UserData= 0;
	}
	else if (Type == ReadDDC)
	{
		check(StreamingRequest);
		InOffset = StreamingRequest->bSupportOffsetLoad ? InOffset : 0;

		check (InSize >= InOffset);

		// DDC does not support 0-bytes requests, so bypass them.
		const bool bHasValidSize = int32(InSize) - int32(InOffset) > 0;
		if (bHasValidSize)
		{
			// 1. Add chunk request to the streaming request. The chunk will hold the request result.
			FHairStreamingRequest::FChunk& Chunk = StreamingRequest->Chunks.AddDefaulted_GetRef();
			Chunk.Status 	= FHairStreamingRequest::FChunk::EStatus::Pending;
			Chunk.Container = &In;
			Chunk.Size 		= InSize - InOffset;
			Chunk.Offset 	= InOffset;
			Chunk.TotalSize = InSize;
			In.ChunkRequest = &Chunk;

			// 2. Fill in actual DDC request
			check(OutReadDDC);
			using namespace UE::DerivedData;
			FCacheGetChunkRequest& Out = OutReadDDC->AddDefaulted_GetRef();
			Out.Id			= FValueId::Null; 	// HairStrands::HairStrandsValueId : This is only needed for cache record, not cache value.
			Out.Key			= ConvertLegacyCacheKey(*DerivedDataKey + InSuffix);
			Out.RawOffset	= InOffset;
			Out.RawSize		= InSize != 0 ? InSize-InOffset : MAX_uint64;
			Out.RawHash		= FIoHash();
			Out.UserData	= (uint64)&Chunk;
			if (Owner) { Out.Name = Owner->GetPathName(); }
		}
		else
		{
			// Add a (dummy) chunk request, so that the buffer creation path will be identical to the streaming path
			FHairStreamingRequest::FChunk& Chunk = StreamingRequest->Chunks.AddDefaulted_GetRef();
			Chunk.Status 	= FHairStreamingRequest::FChunk::EStatus::Completed;
			Chunk.Container = &In;
			Chunk.Size 		= InSize - InOffset;
			Chunk.Offset 	= InOffset;
			Chunk.TotalSize = InSize;
			In.ChunkRequest = &Chunk;
		}
	}
	else 
#endif
	if (Type == ReadIO)
	{
		// 0. If no size value is provided, use the entire resource
		if (InSize == 0)
		{
			InOffset = 0;
			InSize = In.Data.GetBulkDataSize();
		}

		// If the offset is incorrect (tracking issue?) reload the whole requested size
		if (InSize < InOffset)
		{
			InOffset = 0;
		}

		check(StreamingRequest);

		// IO system does not support 0-bytes requests, so bypass them.
		const bool bHasValidSize = int32(InSize) - int32(InOffset) > 0;
		if (bHasValidSize)
		{
			// 1. Add chunk request to the streaming request. The chunk will hold the request result.
			FHairStreamingRequest::FChunk& Chunk = StreamingRequest->Chunks.AddDefaulted_GetRef();
			Chunk.Status 	= FHairStreamingRequest::FChunk::EStatus::Pending;
			Chunk.Container = &In;
			Chunk.Size 		= InSize - InOffset;
			Chunk.Offset 	= InOffset;
			Chunk.TotalSize = InSize;
			In.ChunkRequest = &Chunk;

			// 2. Fill in actual DDC request
			check(OutReadIO);
			OutReadIO->Read(In.Data, InOffset, Chunk.Size, EAsyncIOPriorityAndFlags::AIOP_Normal, Chunk.Data_IO);
		}
		else
		{
			// Add a (dummy) chunk request, so that the buffer creation path will be identical to the streaming path
			FHairStreamingRequest::FChunk& Chunk = StreamingRequest->Chunks.AddDefaulted_GetRef();
			Chunk.Status = FHairStreamingRequest::FChunk::EStatus::Completed;
			Chunk.Container = &In;
			Chunk.Size = InSize - InOffset;
			Chunk.Offset = InOffset;
			Chunk.TotalSize = InSize;
			In.ChunkRequest = &Chunk;
		}
	}
	else if (Type == ReadWriteIO)
	{
		check(OutWriteIO);

		if (OutWriteIO->IsSaving())
		{
			const uint32 BulkFlags = BULKDATA_Force_NOT_InlinePayload;
			In.Data.SetBulkDataFlags(BulkFlags);
		}
		In.Data.Serialize(*OutWriteIO, Owner, 0/*ChunkIndex*/, false /*bAttemptFileMapping*/);
	}
	else if (Type == UnloadData)
	{
		InOffset = FMath::Min(InOffset, InSize);
		check (StreamingRequest != nullptr);
		FHairStreamingRequest::FChunk& Chunk = StreamingRequest->Chunks.AddDefaulted_GetRef();
		Chunk.Status 	= FHairStreamingRequest::FChunk::EStatus::Unloading;
		Chunk.Container = &In;
		Chunk.Size 		= InSize;
		Chunk.Offset 	= InOffset;
		Chunk.TotalSize = InSize;
		In.ChunkRequest = &Chunk;
	}
	else
	{
		checkNoEntry();
	}
}

/////////////////////////////////////////////////////////////////////////////////////////
// IO/DDC request

bool FHairStreamingRequest::IsNone() const
{ 
#if !WITH_EDITORONLY_DATA
	return IORequest.IsNone();
#else
	return Chunks.Num() == 0;
#endif
}
bool FHairStreamingRequest::IsCompleted()
{ 
	if (IsUnloading())
	{
		return true;
	}

#if !WITH_EDITORONLY_DATA
	if (IORequest.IsCompleted())
	{
		for (FHairStreamingRequest::FChunk& Chunk : Chunks)
		{
			Chunk.Status = FHairStreamingRequest::FChunk::Completed;
			check(Chunk.Container);
			Chunk.Container->LoadedSize = Chunk.TotalSize;
		}
	}
	return Chunks.Num() == 0 || IORequest.IsCompleted(); 
#else
	for (const FHairStreamingRequest::FChunk& Chunk : Chunks)
	{
		if (Chunk.Status != FHairStreamingRequest::FChunk::Completed)
		{
			return false;
		}
	}
	return true; 
#endif
}
bool FHairStreamingRequest::IsUnloading() const
{ 
	for (const FHairStreamingRequest::FChunk& Chunk : Chunks)
	{
		if (Chunk.Status == FHairStreamingRequest::FChunk::Unloading)
		{
			return true;
		}
	}
	return false; 
}
#if WITH_EDITORONLY_DATA
static bool RequestWarmCache(UE::DerivedData::FRequestOwner* RequestOwner, const TArray<UE::DerivedData::FCacheGetChunkRequest>& Requests)
{
	using namespace UE::DerivedData;

	// Currently GetChunk request have long latency if not cached locally. 
	// To reduce cook time, we warming cache by issuing 'Value' request (vs.'Chunk' request)
	const ECachePolicy Policy = ECachePolicy::Default | ECachePolicy::SkipData;
	TArray<FCacheGetValueRequest, TInlineAllocator<16>> WarmRequest;
	for (const FCacheGetChunkRequest& R : Requests)
	{				
		WarmRequest.Add({R.Name, R.Key, Policy, 0});
	}
	bool bHasDataInCache = true;
	GetCache().GetValue(WarmRequest, *RequestOwner, [&bHasDataInCache](FCacheGetValueResponse && Response) { if (Response.Status != EStatus::Ok) { bHasDataInCache = false; } /* If the data are not built the cache query can return false. check(Response.Status == EStatus::Ok);*/ });
	RequestOwner->Wait();
	return bHasDataInCache;
}

bool FHairStreamingRequest::WarmCache(uint32 InRequestedCurveCount, uint32 InRequestedPointCount, FHairStrandsBulkCommon& In)
{
	if (In.GetResourceCount() == 0 || InRequestedCurveCount == 0)
	{
		CurveCount = 0;
		PointCount = 0;
		return true;
	}
	CurveCount = InRequestedCurveCount;
	PointCount = InRequestedPointCount;
	bSupportOffsetLoad = false; // Whole resource loading/caching

	using namespace UE::DerivedData;
	TArray<FCacheGetChunkRequest> Requests;
	In.Read_DDC(this, Requests);

	check(DDCRequestOwner == nullptr);
	DDCRequestOwner = MakeUnique<FRequestOwner>(UE::DerivedData::EPriority::Blocking);
	return RequestWarmCache(DDCRequestOwner.Get(), Requests);
}
#endif

// Request fullfil 2 use cases:
// * Load IO/DDC data and upload them to GPU
// * Load DDC data and store them into bulkdata for serialization
void FHairStreamingRequest::Request(uint32 InRequestedCurveCount, uint32 InRequestedPointCount, FHairStrandsBulkCommon& In,
	bool bWait, bool bFillBulkdata, bool bWarmCache, const FName& InOwnerName, bool* bWaitResult)
{
	bool bLogErrors = bWaitResult == nullptr;
	bool bLocalWaitResult = true;
	ON_SCOPE_EXIT
	{
		if (bWaitResult)
		{
			*bWaitResult = bLocalWaitResult;
		}
	};

	if (In.GetResourceCount() == 0 || InRequestedCurveCount == 0)
	{
		CurveCount = 0;
		PointCount = 0;
		return;
	}
	CurveCount = InRequestedCurveCount;
	PointCount = InRequestedPointCount;

#if !WITH_EDITORONLY_DATA
	{ 
		In.Read_IO(this, IORequest);
		if (bWait)
		{
			IORequest.Wait();
		}
	}
#else
	{
		// When enabled, data can be loaded from an offset. Otherwisee, start from the beginning of the resource
		// This is used when cooking data to force the loading of the entire resource (i.e., bSupportOffsetLoad=false)
		bSupportOffsetLoad = !bFillBulkdata; 

		using namespace UE::DerivedData;
		TArray<FCacheGetChunkRequest> Requests;
		In.Read_DDC(this, Requests);
	
		check(DDCRequestOwner == nullptr);
		DDCRequestOwner = MakeUnique<FRequestOwner>(bWait ? UE::DerivedData::EPriority::Blocking : UE::DerivedData::EPriority::Normal); // <= Move this onto the resource (Buffer, in order to ensure no race condition)
		
		// Currently GetChunk request have long latency if not cached locally. 
		// To reduce cook time, we warming cache by issuing 'Value' request (vs.'Chunk' request)
		if (bWarmCache)
		{
			RequestWarmCache(DDCRequestOwner.Get(), Requests);
		}

		//FRequestBarrier Barrier(*DDCRequestOwner);	// This is a critical section on the owner. It does not constrain ordering
		GetCache().GetChunks(Requests, *DDCRequestOwner,
		[this, &In, bFillBulkdata, InOwnerName, bLogErrors](FCacheGetChunkResponse && Response)
		{
			if (Response.Status == UE::DerivedData::EStatus::Ok)
			{
				FHairStreamingRequest::FChunk& Chunk = *(FHairStreamingRequest::FChunk*)Response.UserData;
				Chunk.Data_DDC 	= MoveTemp(Response.RawData);
				Chunk.Offset 	= Response.RawOffset;
				Chunk.Size 		= Response.RawSize;
				Chunk.TotalSize = Response.RawOffset + Response.RawSize;
				Chunk.Status 	= FHairStreamingRequest::FChunk::Completed;
		
				// Upload the total amount of loaded data
				Chunk.Container->LoadedSize = Chunk.TotalSize;
		
				// Optional fill in of bytebulkdata container
				if (bFillBulkdata)
				{
					check(Chunk.Container);
					FByteBulkData& BulkData = Chunk.Container->Data;
		
					// The buffer is then stored into bulk data
					BulkData.Lock(LOCK_READ_WRITE);
					void* DstData = BulkData.Realloc(Chunk.Size);
					FMemory::Memcpy(DstData, Chunk.GetData(), Chunk.Size);
					BulkData.Unlock();
		
					//Chunk.Release();
				}
			}
			else
			{
				FHairStreamingRequest::FChunk& Chunk = *(FHairStreamingRequest::FChunk*)Response.UserData;
				Chunk.Status = FHairStreamingRequest::FChunk::Failed;
				UE_CLOGF(bLogErrors, LogHairStrands, Error,
					"[Groom] DDC request failed for '%ls' (Key:%ls) ", *InOwnerName.ToString(), *In.DerivedDataKey);
			}
		});

		// Optional wait on DDC response
		if (bWait || bFillBulkdata)
		{
			DDCRequestOwner->Wait();
			if (!IsCompleted())
			{
				if (bLogErrors)
				{
					checkf(IsCompleted(), TEXT("HairStrands fatal error: DDC request failed."));
				}
				bLocalWaitResult = false;
			}
		}
	}
#endif
}

const uint8* FHairStreamingRequest::FChunk::GetData() const 
{ 
	check(Status == Completed); 
#if !WITH_EDITORONLY_DATA
	check(Container);
	return (const uint8*)Data_IO.GetData();
#else
	return (const uint8*)Data_DDC.GetData(); 
#endif
}
void FHairStreamingRequest::FChunk::Release() 
{ 
#if !WITH_EDITORONLY_DATA
	// Take ownership of the buffer
	TIoStatusOr<uint8*> Out = Data_IO.Release();
	// Free the buffer
	FMemory::Free(Out.ConsumeValueOrDie());
#else
	Data_DDC.Reset(); 
#endif
}

/////////////////////////////////////////////////////////////////////////////////////////
// Rest bulk data

uint32 FHairStrandsBulkData::ConvertDataFlagsToHairStrandsFlags(uint32 In)
{
	uint32 Out = 0;
	Out |= !!(In & EDataFlags::DataFlags_HasTrimmedPoint) 		? uint32(EHairStrandsFlags::HasTrimmedPoint)	: 0u;
	Out |= !!(In & EDataFlags::DataFlags_HasTrimmedCurve) 		? uint32(EHairStrandsFlags::HasTrimmedCurve)	: 0u;
	Out |= !!(In & EDataFlags::DataFlags_HasShuffledCurves) 	? uint32(EHairStrandsFlags::HasShuffledCurve)	: 0u;
	Out |= !!(In & EDataFlags::DataFlags_HasExtraEndingPoint) 	? uint32(EHairStrandsFlags::HasExtraEndingPoint): 0u;
	return Out;
}

uint32 FHairStrandsBulkData::ConvertDataFlagsFromHairStrandsFlags(uint32 In)
{
	uint32 Out = 0;
	Out |= !!(In & uint32(EHairStrandsFlags::HasTrimmedPoint)) 		? EDataFlags::DataFlags_HasTrimmedPoint		: 0u;
	Out |= !!(In & uint32(EHairStrandsFlags::HasTrimmedCurve)) 		? EDataFlags::DataFlags_HasTrimmedCurve		: 0u;
	Out |= !!(In & uint32(EHairStrandsFlags::HasShuffledCurve)) 	? EDataFlags::DataFlags_HasShuffledCurves	: 0u;
	Out |= !!(In & uint32(EHairStrandsFlags::HasExtraEndingPoint)) 	? EDataFlags::DataFlags_HasExtraEndingPoint	: 0u;
	return Out;
}

void FHairStrandsBulkData::SerializeHeader(FArchive& Ar, UObject* Owner)
{
	GetResourceVersion(Ar);

	Ar << Header.CurveCount;
	Ar << Header.PointCount;
	Ar << Header.MaxLength;
	Ar << Header.MaxRadius;
	Ar << Header.BoundingBox;
	Ar << Header.MinPointPerCurve;
	Ar << Header.MaxPointPerCurve;
	Ar << Header.AvgPointPerCurve;
	Ar << Header.Flags;
	Ar << Header.Hash.PositionHash;
	Ar << Header.Hash.CurveCountHash;
	Ar << Header.Hash.CurveOffsetHash;
	for (uint8 AttributeIt = 0; AttributeIt < HAIR_CURVE_ATTRIBUTE_COUNT; ++AttributeIt)
	{
		Ar << Header.CurveAttributeOffsets[AttributeIt];
	}
	for (uint8 AttributeIt = 0; AttributeIt < HAIR_POINT_ATTRIBUTE_COUNT; ++AttributeIt)
	{
		Ar << Header.PointAttributeOffsets[AttributeIt];
	}
	Ar << Header.ImportedAttributes;
	Ar << Header.ImportedAttributeFlags;

	Ar << Header.SourceCurveCount;
	Ar << Header.SourcePointCount;

	Ar << Header.CurveToPointCount;
	Ar << Header.CoverageScales;

	Ar << Header.Strides.PositionStride;
	Ar << Header.Strides.CurveStride;
	Ar << Header.Strides.PointToCurveChunkStride;
	Ar << Header.Strides.CurveAttributeChunkStride;
	Ar << Header.Strides.PointAttributeChunkStride;
	Ar << Header.Strides.TranscodedPositionChunkStride;
	Ar << Header.Strides.CurveMappingStride;
	Ar << Header.Strides.PointMappingStride;

	Ar << Header.Strides.PointToCurveChunkElementCount;
	Ar << Header.Strides.CurveAttributeChunkElementCount;
	Ar << Header.Strides.PointAttributeChunkElementCount;
	Ar << Header.Strides.TranscodedPositionChunkElementCount;

	Ar << Header.Transcoding.PositionOffset;
	Ar << Header.Transcoding.PositionScale;
}

void FHairStrandsBulkData::GetResourceVersion(FArchive& Ar) const
{
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
}

uint32 FHairStrandsBulkData::GetResourceCount() const
{
	return 7;
}

void FHairStrandsBulkData::GetResources(FHairStrandsBulkCommon::FQuery& Out)
{
	static_assert(sizeof(FHairStrandsPositionFormat::BulkType) == sizeof(FHairStrandsPositionFormat::Type));
	static_assert(sizeof(FHairStrandsAttributeFormat::BulkType) == sizeof(FHairStrandsAttributeFormat::Type));
	static_assert(sizeof(FHairStrandsPointToCurveFormat::BulkType) == sizeof(FHairStrandsPointToCurveFormat::Type));
	static_assert(sizeof(FHairStrandsRootIndexFormat::BulkType) == sizeof(FHairStrandsRootIndexFormat::Type)); 

	// Translate requested curve count into chunk/offset/size to be read
	uint32 PointCount = 0;
	uint32 CurveCount = 0;
	if (Out.Type == FHairStrandsBulkCommon::FQuery::ReadIO || Out.Type == FHairStrandsBulkCommon::FQuery::ReadDDC || Out.Type == FHairStrandsBulkCommon::FQuery::UnloadData)
	{
		CurveCount = FMath::Min(Header.CurveCount, Out.GetCurveCount());
		PointCount = CurveCount > 0 ? Header.CurveToPointCount[CurveCount -1] : 0;
	}

	const uint32 PointAttributeSize = GetPointAttributeSizeInBytes(PointCount);
	const uint32 CurveAttributeSize = GetCurveAttributeSizeInBytes(CurveCount);
	const uint32 PointToCurveIndexSize = GetPointToCurveSizeInBytes(PointCount);
	const uint32 TranscodedPositionSize = GetTranscodedPositionSizeInBytes(PointCount);

	if (!!(Header.Flags & DataFlags_HasData))
	{
		if (!!(Header.Flags & DataFlags_HasTranscodedPosition))
		{
			Out.Add(Data.TranscodedPositions,TEXT("_TranscodedPositions"), 	Data.TranscodedPositions.LoadedSize,TranscodedPositionSize);
		}
		else
		{
			Out.Add(Data.Positions, 		TEXT("_Positions"), 			Data.Positions.LoadedSize, 			PointCount * Header.Strides.PositionStride);
		}
		Out.Add(Data.CurveAttributes, 		TEXT("_CurveAttributes"), 		Data.CurveAttributes.LoadedSize,	CurveAttributeSize);
		if (Header.Flags & DataFlags_HasPointAttribute)
		{
			Out.Add(Data.PointAttributes, 	TEXT("_PointAttributes"), 		Data.PointAttributes.LoadedSize,	PointAttributeSize);
		}
		Out.Add(Data.PointToCurve, 			TEXT("_PointToCurve"), 			Data.PointToCurve.LoadedSize,		PointToCurveIndexSize);
		Out.Add(Data.Curves, 				TEXT("_Curves"), 				Data.Curves.LoadedSize, 			CurveCount * Header.Strides.CurveStride);

		if(Header.Flags & DataFlags_HasPointMapping)
		{
			Out.Add(Data.PointMapping, TEXT("_PointMapping"), Data.PointMapping.LoadedSize, PointCount * Header.Strides.PointMappingStride);
		}
		if(Header.Flags & DataFlags_HasCurveMapping)
		{
			Out.Add(Data.CurveMapping, TEXT("_CurveMapping"), Data.CurveMapping.LoadedSize, CurveCount * Header.Strides.CurveMappingStride);
		}
	}
}

uint32 FHairStrandsBulkData::GetSize() const
{
	uint32 Out = 0;
	Out += Data.Positions.GetBulkDataSize();
	Out += Data.TranscodedPositions.GetBulkDataSize();
	Out += Data.CurveAttributes.GetBulkDataSize();
	Out += Data.PointAttributes.GetBulkDataSize();
	Out += Data.PointToCurve.GetBulkDataSize();
	Out += Data.Curves.GetBulkDataSize();
	Out += Data.CurveMapping.GetBulkDataSize();
	Out += Data.PointMapping.GetBulkDataSize();
	return Out;
}

void FHairStrandsBulkData::Reset()
{
	Header = FHeader();
	for (uint8 AttributeIt = 0; AttributeIt < HAIR_CURVE_ATTRIBUTE_COUNT; ++AttributeIt)
	{
		Header.CurveAttributeOffsets[AttributeIt] = HAIR_ATTRIBUTE_INVALID_OFFSET;
	}
	for (uint8 AttributeIt = 0; AttributeIt < HAIR_POINT_ATTRIBUTE_COUNT; ++AttributeIt)
	{
		Header.PointAttributeOffsets[AttributeIt] = HAIR_ATTRIBUTE_INVALID_OFFSET;
	}
	// Deallocate memory if needed
	Data.Positions.RemoveBulkData();
	Data.TranscodedPositions.RemoveBulkData();
	Data.CurveAttributes.RemoveBulkData();
	Data.PointAttributes.RemoveBulkData();
	Data.PointToCurve.RemoveBulkData();
	Data.Curves.RemoveBulkData();
	Data.CurveMapping.RemoveBulkData();
	Data.PointMapping.RemoveBulkData();

	// Reset the bulk byte buffer to ensure the (serialize) data size is reset to 0
	Data.Positions 		= FHairBulkContainer();
	Data.TranscodedPositions= FHairBulkContainer();
	Data.CurveAttributes= FHairBulkContainer();
	Data.PointAttributes= FHairBulkContainer();
	Data.PointToCurve	= FHairBulkContainer();
	Data.Curves			= FHairBulkContainer();
	Data.CurveMapping	= FHairBulkContainer();
	Data.PointMapping	= FHairBulkContainer();
}

void FHairStrandsBulkData::ResetLoadedSize()
{
	Data.Positions.LoadedSize		= 0;
	Data.TranscodedPositions.LoadedSize		= 0;
	Data.CurveAttributes.LoadedSize	= 0;
	Data.PointAttributes.LoadedSize	= 0;
	Data.PointToCurve.LoadedSize	= 0;
	Data.Curves.LoadedSize			= 0;
	Data.CurveMapping.LoadedSize	= 0;
	Data.PointMapping.LoadedSize	= 0;
}

float FHairStrandsBulkData::GetCoverageScale(float InCurvePercentage /*[0..1]*/) const
{ 
	float Out = 1.f;
	if (Header.CoverageScales.Num() > 0)
	{
		const uint32 ScaleCount = Header.CoverageScales.Num();
		const float  fScaleIt = FMath::Clamp(InCurvePercentage * ScaleCount, 0u, ScaleCount-1u);
		const uint32 ScaleIt0 = FMath::Floor(fScaleIt);
		const uint32 ScaleIt1 = FMath::Min(ScaleIt0+1, ScaleCount-1);
		const float S = fScaleIt - ScaleIt0;
		Out = FMath::LerpStable(Header.CoverageScales[ScaleIt0], Header.CoverageScales[ScaleIt1], S);
	}
	return Out;
}

// Relies on bulk metadata being populated, which is only in cooked builds
#if !WITH_EDITORONLY_DATA
bool FHairStrandsBulkData::IsCookedBulkDataLoadedForCPUAccess() const
{
	bool bIsAllBulkDataLoaded = true;

	bIsAllBulkDataLoaded &= Data.Positions.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.TranscodedPositions.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.CurveAttributes.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.PointAttributes.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.PointToCurve.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.Curves.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.CurveMapping.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.PointMapping.IsBulkDataEmptyOrLoaded();

	return bIsAllBulkDataLoaded;
}
#endif

/////////////////////////////////////////////////////////////////////////////////////////
// Interpolation bulk data

void FHairStrandsInterpolationBulkData::Reset()
{
	Header = FHeader();
	
	// Deallocate memory if needed
	Data.CurveInterpolation.RemoveBulkData();
	Data.PointInterpolation.RemoveBulkData();

	// Reset the bulk byte buffer to ensure the (serialize) data size is reset to 0
	Data.CurveInterpolation	= FHairBulkContainer();
	Data.PointInterpolation	= FHairBulkContainer();
}

void FHairStrandsInterpolationBulkData::SerializeHeader(FArchive& Ar, UObject* Owner)
{
	Ar << Header.Flags;
	Ar << Header.PointCount;
	Ar << Header.CurveCount;

	Ar << Header.Strides.CurveInterpolationStride;
	Ar << Header.Strides.PointInterpolationStride;
}

uint32 FHairStrandsInterpolationBulkData::GetResourceCount() const
{
	return (Header.Flags & DataFlags_HasData) ? 2 : 0;
}

void FHairStrandsInterpolationBulkData::GetResources(FHairStrandsBulkCommon::FQuery& Out)
{
	static_assert(sizeof(FHairStrandsRootIndexFormat::BulkType) == sizeof(FHairStrandsRootIndexFormat::Type));

	if (Header.Flags & DataFlags_HasData)
	{
		// Translate requested curve count into chunk/offset/size to be read
		uint32 PointCount = 0;
		uint32 CurveCount = 0;
		if (Out.Type == FHairStrandsBulkCommon::FQuery::ReadIO || Out.Type == FHairStrandsBulkCommon::FQuery::ReadDDC || Out.Type == FHairStrandsBulkCommon::FQuery::UnloadData)
		{
			// Aligned requested point to get data aligned on 4-bytes
			// See GroomBuilder.cpp for details
			const uint32 RequestedPointCount = (Header.Flags & DataFlags_HasSingleGuideData) ? FMath::DivideAndRoundUp(Out.GetPointCount(),2u)*2u : Out.GetPointCount();

			CurveCount = FMath::Min(Header.CurveCount, Out.GetCurveCount());
			PointCount = FMath::Min(Header.PointCount, RequestedPointCount);
		}

		Out.Add(Data.CurveInterpolation, 	TEXT("_CurveInterpolation"), 	Data.CurveInterpolation.LoadedSize, 		CurveCount * Header.Strides.CurveInterpolationStride);
		Out.Add(Data.PointInterpolation, 	TEXT("_PointInterpolation"), 	Data.PointInterpolation.LoadedSize, 		PointCount * Header.Strides.PointInterpolationStride);
	}
}

uint32 FHairStrandsInterpolationBulkData::GetSize() const
{
	uint32 Out = 0;
	Out += Data.CurveInterpolation.GetBulkDataSize();
	Out += Data.PointInterpolation.GetBulkDataSize();
	return Out;
}

// Relies on bulk metadata being populated, which is only in cooked builds
#if !WITH_EDITORONLY_DATA
bool FHairStrandsInterpolationBulkData::IsCookedBulkDataLoadedForCPUAccess() const
{
	bool bIsAllBulkDataLoaded = true;

	bIsAllBulkDataLoaded &= Data.CurveInterpolation.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.PointInterpolation.IsBulkDataEmptyOrLoaded();

	return bIsAllBulkDataLoaded;
}
#endif

void FHairStrandsInterpolationBulkData::ResetLoadedSize()
{
	Data.CurveInterpolation.LoadedSize = 0;
	Data.PointInterpolation.LoadedSize = 0;
}

/////////////////////////////////////////////////////////////////////////////////////////
// Cluster culling bulk data

void FHairStrandsClusterBulkData::Reset()
{
	Header.ClusterCount = 0;
	Header.ClusterScale = 0;
	Header.PointCount = 0;
	Header.CurveCount = 0;
	Header.ClusterInfoParameters = FVector4f::Zero();
	Header.LODInfos.Empty();

	Data.CurveToClusterIds.RemoveBulkData();
	Data.PackedClusterInfos.RemoveBulkData();
	Data.PointLODs.RemoveBulkData();

	// Reset the bulk byte buffer to ensure the (serialize) data size is reset to 0
	Data.CurveToClusterIds  = FHairBulkContainer();
	Data.PackedClusterInfos = FHairBulkContainer();
	Data.PointLODs			= FHairBulkContainer();
}

void FHairStrandsClusterBulkData::ResetLoadedSize()
{
	Data.CurveToClusterIds.LoadedSize 	= 0;
	Data.PackedClusterInfos.LoadedSize	= 0;
	Data.PointLODs.LoadedSize			= 0;
}

void FHairStrandsClusterBulkData::SerializeHeader(FArchive& Ar, UObject* Owner)
{
	Ar << Header.ClusterCount;
	Ar << Header.ClusterScale;
	Ar << Header.PointCount;
	Ar << Header.CurveCount;
	Ar << Header.ClusterInfoParameters;
	uint32 LODInfosCount = Header.LODInfos.Num();
	Ar << LODInfosCount;
	if (Ar.IsLoading())
	{
		Header.LODInfos.SetNum(LODInfosCount);
	}
	for (uint32 It = 0; It < LODInfosCount; ++It)
	{
		Ar << Header.LODInfos[It].CurveCount;
		Ar << Header.LODInfos[It].PointCount;
		Ar << Header.LODInfos[It].RadiusScale;
		Ar << Header.LODInfos[It].ScreenSize;
		Ar << Header.LODInfos[It].bIsVisible;
	}

	Ar << Header.Strides.PackedClusterInfoStride;
	Ar << Header.Strides.CurveToClusterIdStride;
	Ar << Header.Strides.PointLODStride;
}

uint32 FHairStrandsClusterBulkData::GetResourceCount() const
{
	return 5;
}

bool ValidateHairBulkData();

void FHairStrandsClusterBulkData::GetResources(FHairStrandsBulkCommon::FQuery & Out)
{
	const bool bHasClusterData = Header.ClusterCount > 0;
	if (bHasClusterData)
	{
		// Translate requested curve count into chunk/offset/size to be read
		uint32 PointCount = 0;
		uint32 CurveCount = 0;
		if (Out.Type == FHairStrandsBulkCommon::FQuery::ReadIO || Out.Type == FHairStrandsBulkCommon::FQuery::ReadDDC || Out.Type == FHairStrandsBulkCommon::FQuery::UnloadData)
		{
			CurveCount = FMath::Min(Header.CurveCount, Out.GetCurveCount());
			PointCount = FMath::Min(Header.PointCount, Out.GetPointCount());
		}

		const uint32 PointLODCount = FMath::DivideAndRoundUp(PointCount, HAIR_POINT_LOD_COUNT_PER_UINT);

		Out.Add(Data.PackedClusterInfos, TEXT("_PackedClusterInfos"), Data.PackedClusterInfos.LoadedSize, Header.ClusterCount * Header.Strides.PackedClusterInfoStride); // Load all data
		Out.Add(Data.CurveToClusterIds, TEXT("_CurveToClusterIds"), Data.CurveToClusterIds.LoadedSize, CurveCount * Header.Strides.CurveToClusterIdStride);
		Out.Add(Data.PointLODs, TEXT("_PointLODs"), Data.PointLODs.LoadedSize, PointLODCount * Header.Strides.PointLODStride);
	}
}

uint32 FHairStrandsClusterBulkData::GetSize() const
{
	uint32 Out = 0;;
	Out += Data.PackedClusterInfos.GetBulkDataSize();
	Out += Data.CurveToClusterIds.GetBulkDataSize();
	Out += Data.PointLODs.GetBulkDataSize();
	return Out;
}

// Relies on bulk metadata being populated, which is only in cooked builds
#if !WITH_EDITORONLY_DATA
bool FHairStrandsClusterBulkData::IsCookedBulkDataLoadedForCPUAccess() const
{
	bool bIsAllBulkDataLoaded = true;

	bIsAllBulkDataLoaded &= Data.PackedClusterInfos.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.CurveToClusterIds.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.PointLODs.IsBulkDataEmptyOrLoaded();

	return bIsAllBulkDataLoaded;
}
#endif

uint32 FHairStrandsClusterBulkData::GetCurveCount(float InLODIndex) const
{
	const TArray<FHairLODInfo>& LODInfos = Header.LODInfos;

	InLODIndex = FMath::Max(InLODIndex, 0.f);

	const int32 iLODIndex = InLODIndex;
	const float S = InLODIndex - iLODIndex;

	const int32 LODIndex0 = FMath::Min(iLODIndex,   LODInfos.Num()-1);
	const int32 LODIndex1 = FMath::Min(iLODIndex+1, LODInfos.Num()-1);
	return FMath::LerpStable<int32>(LODInfos[LODIndex0].CurveCount, LODInfos[LODIndex1].CurveCount, S);
}

/////////////////////////////////////////////////////////////////////////////////////////
// Root data

bool FHairStrandsRootData::HasProjectionData() const
{
	return RootBarycentricBuffer.Num() > 0
	&& RootToUniqueTriangleIndexBuffer.Num() > 0
	&& UniqueTriangleIndexBuffer.Num() > 0
	&& RestUniqueTrianglePositionBuffer.Num() > 0;
}

void FHairStrandsRootData::Reset()
{
	*this = FHairStrandsRootData();
}

/////////////////////////////////////////////////////////////////////////////////////////
// Root bulk data

void FHairStrandsRootBulkData::SerializeHeader(FArchive& Ar, UObject* Owner)
{
	// Header
	{
		Ar << Header.RootCount;
		Ar << Header.PointCount;
		Ar << Header.LODIndex;
		Ar << Header.UniqueTriangleCount;
		Ar << Header.SampleCount;
		Ar << Header.UniqueSectionIndices;
		Ar << Header.MeshSectionCount;
		Ar << Header.Flags;
		Ar << Header.Hash.PositionHash;
		Ar << Header.Hash.CurveCountHash;
		Ar << Header.Hash.CurveOffsetHash;
	}

	// Strides
	{
		Ar << Header.Strides.RootToUniqueTriangleIndexBufferStride;
		Ar << Header.Strides.RootBarycentricBufferStride;
		Ar << Header.Strides.UniqueTriangleIndexBufferStride;
		Ar << Header.Strides.RestUniqueTrianglePositionBufferStride;

		Ar << Header.Strides.MeshInterpolationWeightsBufferStride;
		Ar << Header.Strides.MeshSampleIndicesAndSectionsBufferStride;
		Ar << Header.Strides.RestSamplePositionsBufferStride;
	}
}

const TArray<uint32>& FHairStrandsRootBulkData::GetValidSectionIndices() const
{
	return Header.UniqueSectionIndices;
}

uint32 FHairStrandsRootBulkData::GetResourceCount() const
{
	return 7;
}

void FHairStrandsRootBulkData::GetResources(FQuery& Out)
{
	// Translate requested curve count into chunk/offset/size to be read
	// * RBF points are loaded all at once
	// * Unique triangle data are loaded all at one
	// * Root data are loaded based on the curve request
	uint32 RootCount = Header.RootCount;
	if (Out.Type == FHairStrandsBulkCommon::FQuery::ReadIO || Out.Type == FHairStrandsBulkCommon::FQuery::ReadDDC || Out.Type == FHairStrandsBulkCommon::FQuery::UnloadData)
	{
		RootCount = FMath::Min(Header.RootCount, Out.GetCurveCount());
	}

	Out.Add(Data.RootToUniqueTriangleIndexBuffer, 	TEXT("_RootToUniqueTriangleIndexBuffer"), 		Data.RootToUniqueTriangleIndexBuffer.LoadedSize, 	RootCount * Header.Strides.RootToUniqueTriangleIndexBufferStride);
	Out.Add(Data.RootBarycentricBuffer, 			TEXT("_RootBarycentricBuffer"), 				Data.RootBarycentricBuffer.LoadedSize, 				RootCount * Header.Strides.RootBarycentricBufferStride);
	Out.Add(Data.UniqueTriangleIndexBuffer, 		TEXT("_UniqueTriangleIndexBuffer"), 			Data.UniqueTriangleIndexBuffer.LoadedSize, 			Header.UniqueTriangleCount * Header.Strides.UniqueTriangleIndexBufferStride);		 // Load all data
	Out.Add(Data.RestUniqueTrianglePositionBuffer, 	TEXT("_RestUniqueTrianglePositionBuffer"), 		Data.RestUniqueTrianglePositionBuffer.LoadedSize, 	Header.UniqueTriangleCount * Header.Strides.RestUniqueTrianglePositionBufferStride); // Load all data

	if (Header.SampleCount > 0)
	{
		const uint32 RBFWeightCount = FGroomRBFDeformer::GetWeightCount(Header.SampleCount); 

		Out.Add(Data.MeshInterpolationWeightsBuffer, 	TEXT("_MeshInterpolationWeightsBuffer"), 	Data.MeshInterpolationWeightsBuffer.LoadedSize, 		RBFWeightCount * Header.Strides.MeshInterpolationWeightsBufferStride); 			// Load all data
		Out.Add(Data.MeshSampleIndicesAndSectionsBuffer,TEXT("_MeshSampleIndicesAndSectionsBuffer"),Data.MeshSampleIndicesAndSectionsBuffer.LoadedSize, 	Header.SampleCount * Header.Strides.MeshSampleIndicesAndSectionsBufferStride); 	// Load all data
		Out.Add(Data.RestSamplePositionsBuffer, 		TEXT("_RestSamplePositionsBuffer"), 		Data.RestSamplePositionsBuffer.LoadedSize, 				Header.SampleCount * Header.Strides.RestSamplePositionsBufferStride);			// Load all data
	}
}

uint32 FHairStrandsRootBulkData::GetDataSize() const
{
	uint32 Out = 0;
	Out += Data.RootToUniqueTriangleIndexBuffer.GetBulkDataSize();
	Out += Data.RootBarycentricBuffer.GetBulkDataSize();
	Out += Data.UniqueTriangleIndexBuffer.GetBulkDataSize();
	Out += Data.RestUniqueTrianglePositionBuffer.GetBulkDataSize();

	Out += Data.MeshInterpolationWeightsBuffer.GetBulkDataSize();
	Out += Data.MeshSampleIndicesAndSectionsBuffer.GetBulkDataSize();
	Out += Data.RestSamplePositionsBuffer.GetBulkDataSize();
	return Out;
}

// Relies on bulk metadata being populated, which is only in cooked builds
#if !WITH_EDITORONLY_DATA
bool FHairStrandsRootBulkData::IsCookedBulkDataLoadedForCPUAccess() const
{
	bool bIsAllBulkDataLoaded = true;

	bIsAllBulkDataLoaded &= Data.RootBarycentricBuffer.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.RootToUniqueTriangleIndexBuffer.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.UniqueTriangleIndexBuffer.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.RestUniqueTrianglePositionBuffer.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.MeshInterpolationWeightsBuffer.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.MeshSampleIndicesAndSectionsBuffer.IsBulkDataEmptyOrLoaded();
	bIsAllBulkDataLoaded &= Data.RestSamplePositionsBuffer.IsBulkDataEmptyOrLoaded();

	return bIsAllBulkDataLoaded;
}
#endif

void FHairStrandsRootBulkData::Reset()
{
	// Header
	{
		Header.LODIndex = -1;
		Header.RootCount = 0;
		Header.PointCount = 0;
		Header.SampleCount = 0;
		Header.MeshSectionCount = 0;
		Header.UniqueTriangleCount = 0;
		Header.UniqueSectionIndices.Empty();
	}

	// Data
	{
		// Binding
		Data.RootBarycentricBuffer.RemoveBulkData();
		Data.RootToUniqueTriangleIndexBuffer.RemoveBulkData();
		Data.UniqueTriangleIndexBuffer.RemoveBulkData();
		Data.RestUniqueTrianglePositionBuffer.RemoveBulkData();

		// Reset the bulk byte buffer to ensure the (serialize) data size is reset to 0
		Data.RootBarycentricBuffer 			  = FHairBulkContainer();
		Data.RootToUniqueTriangleIndexBuffer  = FHairBulkContainer();
		Data.UniqueTriangleIndexBuffer		  = FHairBulkContainer();
		Data.RestUniqueTrianglePositionBuffer = FHairBulkContainer();

		// RBF
		Data.MeshInterpolationWeightsBuffer.RemoveBulkData();
		Data.MeshSampleIndicesAndSectionsBuffer.RemoveBulkData();
		Data.RestSamplePositionsBuffer.RemoveBulkData();

		Data.MeshInterpolationWeightsBuffer 	 = FHairBulkContainer();
		Data.MeshSampleIndicesAndSectionsBuffer  = FHairBulkContainer();
		Data.RestSamplePositionsBuffer 			 = FHairBulkContainer();
	}
}

void FHairStrandsRootBulkData::ResetLoadedSize()
{
	// Binding
	Data.RootBarycentricBuffer.LoadedSize 			= 0;
	Data.RootToUniqueTriangleIndexBuffer.LoadedSize = 0;
	Data.UniqueTriangleIndexBuffer.LoadedSize 		= 0;
	Data.RestUniqueTrianglePositionBuffer.LoadedSize= 0;
		
	// RBF
	Data.MeshInterpolationWeightsBuffer.LoadedSize	  = 0;
	Data.MeshSampleIndicesAndSectionsBuffer.LoadedSize= 0;
	Data.RestSamplePositionsBuffer.LoadedSize 		  = 0;
}
