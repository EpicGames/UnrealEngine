// Copyright Epic Games, Inc. All Rights Reserved. 

#include "HairStrandsInterpolation.h"
#include "HairStrandsDatas.h"
#include "HairCardsBuilder.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomCache.h"
#include "HairStrandsInterface.h"
#include "SceneView.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/ResourceArray.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "HAL/ConsoleManager.h"
#include "ShaderPrintParameters.h"
#include "Async/ParallelFor.h"
#include "RenderTargetPool.h"
#include "GroomTextureBuilder.h"
#include "GroomBindingBuilder.h"
#include "SkeletalRenderPublic.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "GroomAsset.h" 
#include "GroomManager.h"
#include "GroomInstance.h"
#include "SystemTextures.h"
#include "ShaderCompilerCore.h"
#include "ShaderPrintParameters.h"
#include "HairStrandsClusterCulling.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "GroomVisualizationData.h"

static float GHairRaytracingRadiusScale = 0;
static FAutoConsoleVariableRef CVarHairRaytracingRadiusScale(TEXT("r.HairStrands.RaytracingRadiusScale"), GHairRaytracingRadiusScale, TEXT("Override the per instance scale factor for raytracing hair strands geometry (0: disabled, >0:enabled)"));

static int GHairRaytracingProceduralSplits = 4;
static FAutoConsoleVariableRef CVarHairRaytracingProceduralScale(TEXT("r.HairStrands.RaytracingProceduralSplits"), GHairRaytracingProceduralSplits, TEXT("Change how many AABBs are used per hair segment to balance between BVH build cost and ray tracing performance. (default: 4)"));

static float GStrandHairWidth = 0.0f;
static FAutoConsoleVariableRef CVarStrandHairWidth(TEXT("r.HairStrands.StrandWidth"), GStrandHairWidth, TEXT("Width of hair strand"));

static int32 GStrandHairInterpolationDebug = 0;
static FAutoConsoleVariableRef CVarStrandHairInterpolationDebug(TEXT("r.HairStrands.Interpolation.Debug"), GStrandHairInterpolationDebug, TEXT("Enable debug rendering for hair interpolation"));

static int32 GHairCardsInterpolationType = 1;
static FAutoConsoleVariableRef CVarHairCardsInterpolationType(TEXT("r.HairStrands.Cards.InterpolationType"), GHairCardsInterpolationType, TEXT("Hair cards interpolation type: 0: None, 1:physics simulation, 2: RBF deformation"));

static int32 GHairStrandsTransferPositionOnLODChange = 0;
static FAutoConsoleVariableRef CVarHairStrandsTransferPositionOnLODChange(TEXT("r.HairStrands.Strands.TransferPrevPos"), GHairStrandsTransferPositionOnLODChange, TEXT("Transfer strands prev. position to current position on LOD switching to avoid large discrepancy causing large motion vector"));

static int32 GHairStrands_Raytracing_ForceRebuildBVH = 1;
static FAutoConsoleVariableRef CVarHairRTGeomForceRebuild(TEXT("r.HairStrands.Strands.Raytracing.ForceRebuildBVH"), GHairStrands_Raytracing_ForceRebuildBVH, TEXT("Force BVH rebuild instead of doing a BVH refit when hair positions changed"));

static int32 GHairStrands_MotionVector_CheckViewID = 1;
static FAutoConsoleVariableRef CVarHairStrands_MotionVector_CheckViewID(TEXT("r.HairStrands.Strands.MotionVectorCheckViewID"), GHairStrands_MotionVector_CheckViewID, TEXT("Issue motion vector on hair strands only if updates happens on the same view"));

static int32 GStrandHairInterpolationForceSingleGuide = 1;
static FAutoConsoleVariableRef CVarHairStrands_Interpolation_ForceSingleGuide(TEXT("r.HairStrands.Strands.Interpolation.ForceSingleGuide"), GStrandHairInterpolationForceSingleGuide, TEXT("Force to use a single guide during the interpolation. This allows to reduce the cost of interpolation, but can create clumpy looks."));

static int32 GVarHairStrands_ScaleHairRadiusByTransform = 1;
static FAutoConsoleVariableRef CVarHairStrands_ScaleHairRadiusByTransform(TEXT("r.HairStrands.Strands.ScaleHairRadiusByTransform"), GVarHairStrands_ScaleHairRadiusByTransform, TEXT("Scale hair strands radius by the instance's transform to make hair radius consistent with the instance scale."));

static int32 GVarHairStrands_AddDeformerStrandsDisplacement = 0;
static FAutoConsoleVariableRef CVarHairStrands_AddDeformerStrandsDisplacement(TEXT("r.HairStrands.Strands.AddDeformerStrandsDisplacement"), GVarHairStrands_AddDeformerStrandsDisplacement, TEXT("Add the deformer displacement to the current position. If set to 0 the deformer position will overwrite the deformed one"));

static int32 GVarHairStrands_BindingDataValidation = 0;
static FAutoConsoleVariableRef CVarVarHairStrands_BindingDataValidation(TEXT("r.HairStrands.Strands.BindingDataValidation"), GVarHairStrands_BindingDataValidation, TEXT("Add binding data validation at runtime (for debugging purpose)"));

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool GetHairStrandsBindingDataValidation()
{
	#if WITH_EDITORONLY_DATA
	return GVarHairStrands_BindingDataValidation > 0;
	#else
	return false;
	#endif
}

bool GetSupportHairStrandsProceduralPrimitive(EShaderPlatform InShaderPlatform);
bool GetSupportHairStrandsLinearSweptSpheres(EShaderPlatform InShaderPlatform);

uint32 GetHairStrandsForceSingleGuideInterpolation()
{
	return GStrandHairInterpolationForceSingleGuide > 0;
}

uint32 GetHairCardsInterpolationType()
{
	return FMath::Clamp(GHairCardsInterpolationType, 0, 2);
}

bool IsHairStrandsTransferPositionOnLODChange()
{
	return GHairStrandsTransferPositionOnLODChange > 0;
}

bool IsHairStrandsForceRebuildBVH()
{
	return GHairStrands_Raytracing_ForceRebuildBVH > 0;
}

uint32 GetHairRaytracingProceduralSplits()
{
	return FMath::Clamp(GHairRaytracingProceduralSplits, 1, STRANDS_PROCEDURAL_INTERSECTOR_MAX_SPLITS);
}

inline uint32 ComputeGroupSize()
{
	const uint32 GroupSize = IsRHIDeviceAMD() ? 64 : (IsRHIDeviceNVIDIA() ? 32 : 64);
	check(GroupSize == 64 || GroupSize == 32);
	return GroupSize;
}

void GetHairStrandsAttributeParameter(const FHairStrandsBulkData& In, FHairStrandsInstanceAttributeParameters& Out)
{
	check(FMath::IsPowerOfTwo(In.Header.Strides.CurveAttributeChunkElementCount));
	Out.CurveAttributeIndexToChunkDivAsShift	= FPlatformMath::FloorLog2(In.Header.Strides.CurveAttributeChunkElementCount);
	Out.CurveAttributeChunkElementCount			= In.Header.Strides.CurveAttributeChunkElementCount;
	Out.CurveAttributeChunkStrideInBytes		= In.Header.Strides.CurveAttributeChunkStride;

	check(FMath::IsPowerOfTwo(In.Header.Strides.PointAttributeChunkElementCount));
	Out.PointAttributeIndexToChunkDivAsShift	= FPlatformMath::FloorLog2(In.Header.Strides.PointAttributeChunkElementCount);
	Out.PointAttributeChunkElementCount     	= In.Header.Strides.PointAttributeChunkElementCount;
	Out.PointAttributeChunkStrideInBytes    	= In.Header.Strides.PointAttributeChunkStride;

	PACK_CURVE_HAIR_ATTRIBUTE_OFFSETS(Out.CurveAttributeOffsets, In.Header.CurveAttributeOffsets);
	PACK_POINT_HAIR_ATTRIBUTE_OFFSETS(Out.PointAttributeOffsets, In.Header.PointAttributeOffsets);
}

FGroomCacheResources CreateGroomCacheBuffer(FRDGBuilder& GraphBuilder, FGroomCacheVertexData& InVertexData)
{
	FGroomCacheResources Out;
	
	if (InVertexData.PointsPosition.Num() > 0)
	{
		FRDGBufferRef PositionBuffer = nullptr;
		if (InVertexData.PositionBuffer == nullptr)
		{
			const uint32 DataCount = InVertexData.PointsPosition.Num();
			const uint32 DataSizeInBytes = sizeof(FVector3f) * DataCount;
			// Deformation are upload into a Buffer<float> as the original position are float3 which is is both
			// 1) incompatible with structure buffer alignment (128bits), and 2) incompatible with vertex buffer 
			// as R32G32B32_FLOAT format is not well supported for SRV across HW.
			// So instead the positions are uploaded into vertex buffer Buffer<float>
			PositionBuffer = CreateVertexBuffer(
				GraphBuilder,
				TEXT("GroomCache_PositionBuffer"),
				FRDGBufferDesc::CreateBufferDesc(sizeof(float), InVertexData.PointsPosition.Num() * 3),
				InVertexData.PointsPosition.GetData(),
				DataSizeInBytes,
				ERDGInitialDataFlags::None);
			GraphBuilder.QueueBufferExtraction(PositionBuffer, &InVertexData.PositionBuffer);
		}
		else
		{
			PositionBuffer = GraphBuilder.RegisterExternalBuffer(InVertexData.PositionBuffer);
		}
		Out.PositionBuffer = GraphBuilder.CreateSRV(PositionBuffer, PF_R32_FLOAT);
	}

	const uint32 RadiusDataCount = InVertexData.PointsRadius.Num();
	if (RadiusDataCount > 0)
	{
		FRDGBufferRef RadiusBuffer = nullptr;
		if (InVertexData.RadiusBuffer == nullptr)
		{
			const uint32 RadiusDataSizeInBytes = sizeof(float) * RadiusDataCount;
			RadiusBuffer = CreateVertexBuffer(
				GraphBuilder,
				TEXT("GroomCache_RadiusBuffer"),
				FRDGBufferDesc::CreateBufferDesc(sizeof(float), InVertexData.PointsRadius.Num()),
				InVertexData.PointsRadius.GetData(),
				RadiusDataSizeInBytes,
				ERDGInitialDataFlags::None);
			GraphBuilder.QueueBufferExtraction(RadiusBuffer, &InVertexData.RadiusBuffer);
		}
		else
		{
			RadiusBuffer = GraphBuilder.RegisterExternalBuffer(InVertexData.RadiusBuffer);
		}
		Out.bHasRadiusData = true;
		Out.RadiusBuffer = GraphBuilder.CreateSRV(RadiusBuffer, PF_R32_FLOAT);
	}
	else
	{
		Out.RadiusBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4), PF_R32_FLOAT);
	}

	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
class FTransferVelocityPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTransferVelocityPassCS);
	SHADER_USE_PARAMETER_STRUCT(FTransferVelocityPassCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(uint32, PointOffset)
		SHADER_PARAMETER(uint32, TotalPointCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, InBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HAIRTRANSFER_PREV_POSITION"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FTransferVelocityPassCS, "/Engine/Private/HairStrands/HairStrandsInterpolation.usf", "MainCS", SF_Compute);

void AddTransferPositionPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 PointOffset,
	const uint32 PointCount,
	const uint32 TotalPointCount,
	FRDGBufferSRVRef InBuffer,
	FRDGBufferUAVRef OutBuffer)
{
	if (PointCount == 0 || TotalPointCount == 0) return;

	FTransferVelocityPassCS::FParameters* Parameters = GraphBuilder.AllocParameters<FTransferVelocityPassCS::FParameters>();
	Parameters->PointCount = PointCount;
	Parameters->PointOffset = PointOffset;
	Parameters->TotalPointCount = TotalPointCount;
	Parameters->InBuffer = InBuffer;
	Parameters->OutBuffer = OutBuffer;

	const FIntVector DispatchCount(FMath::DivideAndRoundUp(PointCount, FTransferVelocityPassCS::GetGroupSize()), 1, 1);
	TShaderMapRef<FTransferVelocityPassCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::TransferPositionForVelocity"),
		ComputeShader,
		Parameters,
		DispatchCount);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FGroomCacheUpdatePassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGroomCacheUpdatePassCS);
	SHADER_USE_PARAMETER_STRUCT(FGroomCacheUpdatePassCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(uint32, CachePointCount)
		SHADER_PARAMETER(uint32, bHasRadiusData)
		SHADER_PARAMETER(uint32, bHasAddedControlPoint)
		SHADER_PARAMETER(float, InterpolationFactor)
		SHADER_PARAMETER(float, MaxHairRadius)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InPosition0Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InPosition1Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InRadius0Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InRadius1Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, InRestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, InRestCurveBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, InRestPointToCurveBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InDeformedOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutDeformedBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_GROOMCACHE_UPDATE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FGroomCacheUpdatePassCS, "/Engine/Private/HairStrands/HairStrandsInterpolation.usf", "MainCS", SF_Compute);

void AddGroomCacheUpdatePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 InstanceRegisteredIndex,
	uint32 PointCount,
	uint32 CurveCount,
	float InterpolationFactor,
	float InMaxHairRadius,
	bool bHasAddedControlPoint,
	FGroomCacheResources CacheResources0,
	FGroomCacheResources CacheResources1,
	FRDGBufferSRVRef InRestPositionBuffer,
	FRDGBufferSRVRef InRestCurveBuffer,
	FRDGBufferSRVRef InRestPointToCurveBuffer,
	FRDGBufferSRVRef InDeformedOffsetBuffer,
	FRDGBufferUAVRef OutBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AddGroomCacheUpdatePass);

	if (PointCount == 0 || !CacheResources0.PositionBuffer || !CacheResources1.PositionBuffer) return;
	
	// Retrive the cache point count from the SRV.
	const uint32 CachePointCount = CacheResources0.PositionBuffer->GetParent() ? CacheResources0.PositionBuffer->GetParent()->Desc.NumElements / 3u : PointCount;

	FGroomCacheUpdatePassCS::FParameters* Parameters = GraphBuilder.AllocParameters<FGroomCacheUpdatePassCS::FParameters>();
	Parameters->InstanceRegisteredIndex = InstanceRegisteredIndex;
	Parameters->PointCount = PointCount;
	Parameters->CurveCount = CurveCount;
	Parameters->CachePointCount = CachePointCount;
	Parameters->InPosition0Buffer = CacheResources0.PositionBuffer;
	Parameters->InPosition1Buffer = CacheResources1.PositionBuffer;
	Parameters->InRadius0Buffer = CacheResources0.RadiusBuffer;
	Parameters->InRadius1Buffer = CacheResources1.RadiusBuffer;
	Parameters->InRestPositionBuffer = InRestPositionBuffer;
	Parameters->InRestCurveBuffer = InRestCurveBuffer;
	Parameters->InRestPointToCurveBuffer = InRestPointToCurveBuffer;
	Parameters->InDeformedOffsetBuffer = InDeformedOffsetBuffer;
	Parameters->OutDeformedBuffer = OutBuffer;
	Parameters->InterpolationFactor = InterpolationFactor;
	Parameters->MaxHairRadius = InMaxHairRadius;
	Parameters->bHasRadiusData = CacheResources0.bHasRadiusData ? 1u : 0u;
	Parameters->bHasAddedControlPoint = bHasAddedControlPoint ? 1u : 0u;

	const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(PointCount, FGroomCacheUpdatePassCS::GetGroupSize()), 1, 1);
	TShaderMapRef<FGroomCacheUpdatePassCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::GroomCacheUpdate"),
		ComputeShader,
		Parameters,
		DispatchCount);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FDeformGuideCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDeformGuideCS);
	SHADER_USE_PARAMETER_STRUCT(FDeformGuideCS, FGlobalShader);

	class FDeformationType : SHADER_PERMUTATION_INT("PERMUTATION_DEFORMATION", 5);
	using FPermutationDomain = TShaderPermutationDomain<FDeformationType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(FVector3f, SimRestOffset)
		
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SimRestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SimDeformedPositionBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SimRootBarycentricBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SimPointToCurveBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SimRootToUniqueTriangleIndexBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, SimDeformedOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SimRestPosePositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutSimDeformedPositionBuffer)

		SHADER_PARAMETER(uint32, SampleCount)
		SHADER_PARAMETER(uint32, RBFLocalSpace)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, RestSamplePositionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, MeshSampleWeightsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeformedSamplePositionsBuffer)

		SHADER_PARAMETER_SRV(Buffer<float4>, BoneDeformedPositionBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_DEFORM_GUIDE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeformGuideCS, "/Engine/Private/HairStrands/HairStrandsGuideDeform.usf", "MainCS", SF_Compute);

void AddDeformSimHairStrandsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 InstanceRegisteredIndex,
	const uint32 MeshLODIndex,
	const uint32 VertexCount,
	FHairStrandsRestRootResource* SimRestRootResources,
	FHairStrandsDeformedRootResource* SimDeformedRootResources,
	FRDGBufferSRVRef SimRestPosePositionBuffer,
	FRDGBufferSRVRef SimPointToCurveBuffer,
	FRDGImportedBuffer& OutSimDeformedPositionBuffer,
	const FVector& SimRestOffset,
	FRDGBufferSRVRef SimDeformedOffsetBuffer,
	const bool bHasGlobalInterpolation,
	FRHIShaderResourceView* BoneBufferSRV)
{
	enum EInternalDeformationType
	{
		InternalDeformationType_ByPass = 0,
		InternalDeformationType_Offset = 1,
		InternalDeformationType_Skinned = 2,
		InternalDeformationType_RBF = 3,
		InternalDeformationType_Bones = 4,
		InternalDeformationTypeCount
	};

	EInternalDeformationType InternalDeformationType = InternalDeformationType_Offset;

	FDeformGuideCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDeformGuideCS::FParameters>();
	Parameters->InstanceRegisteredIndex = InstanceRegisteredIndex;
	Parameters->SimRestPosePositionBuffer = SimRestPosePositionBuffer;
	Parameters->OutSimDeformedPositionBuffer = OutSimDeformedPositionBuffer.UAV;
	Parameters->VertexCount = VertexCount;
	Parameters->SimDeformedOffsetBuffer = SimDeformedOffsetBuffer;
	Parameters->SimRestOffset = (FVector3f)SimRestOffset;
	{
		const bool bIsPointToCurveBuffersValid = SimPointToCurveBuffer != nullptr;
		if (bIsPointToCurveBuffersValid)
		{
			Parameters->SimPointToCurveBuffer = SimPointToCurveBuffer;
		}

		const uint32 RootCount = SimRestRootResources ? SimRestRootResources->GetRootCount() : 0;

		if(BoneBufferSRV)
		{
			InternalDeformationType = InternalDeformationType_Bones;
			Parameters->BoneDeformedPositionBuffer = BoneBufferSRV;
		}
		else
		{
			const bool bSupportDynamicMesh = 
				RootCount > 0 && 
				MeshLODIndex >= 0 && 
				SimRestRootResources->IsValid(MeshLODIndex) &&
				SimDeformedRootResources->IsValid(MeshLODIndex) &&
				bIsPointToCurveBuffersValid;
			
			bool bSupportGlobalInterpolation = false;
			if (bSupportDynamicMesh)
			{
				FHairStrandsLODRestRootResource& RestLODDatas = *SimRestRootResources->GetLOD(MeshLODIndex);
				FHairStrandsLODDeformedRootResource& DeformedLODDatas = *SimDeformedRootResources->GetLOD(MeshLODIndex);

				bSupportGlobalInterpolation = bHasGlobalInterpolation && (RestLODDatas.SampleCount > 0);
				if (!bSupportGlobalInterpolation) 
				{
					InternalDeformationType = InternalDeformationType_Skinned;
					Parameters->SimRootToUniqueTriangleIndexBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RootToUniqueTriangleIndexBuffer);
					Parameters->SimRestPositionBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RestUniqueTrianglePositionBuffer);
					Parameters->SimDeformedPositionBuffer = RegisterAsSRV(GraphBuilder, DeformedLODDatas.GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current));
					Parameters->SimRootBarycentricBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RootBarycentricBuffer);
				}
				else
				{
					InternalDeformationType = InternalDeformationType_RBF;
					Parameters->MeshSampleWeightsBuffer = RegisterAsSRV(GraphBuilder, DeformedLODDatas.GetMeshSampleWeightsBuffer(FHairStrandsLODDeformedRootResource::Current));
					Parameters->RestSamplePositionsBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RestSamplePositionsBuffer);
					Parameters->DeformedSamplePositionsBuffer = RegisterAsSRV(GraphBuilder, DeformedLODDatas.GetDeformedSamplePositionsBuffer(FHairStrandsLODDeformedRootResource::Current));
					Parameters->SampleCount = RestLODDatas.SampleCount;
					Parameters->RBFLocalSpace = UE::Groom::IsRBFLocalSpaceEnabled();
;
				}
			}
		}
	}

	FDeformGuideCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FDeformGuideCS::FDeformationType>(InternalDeformationType);

	const FIntVector DispatchCount(FMath::DivideAndRoundUp(VertexCount,FDeformGuideCS::GetGroupSize()), 1, 1);
	TShaderMapRef<FDeformGuideCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::DeformSimHairStrands"),
		ComputeShader,
		Parameters,
		DispatchCount);

	GraphBuilder.SetBufferAccessFinal(OutSimDeformedPositionBuffer.Buffer, ERHIAccess::SRVMask);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FRDGHairStrandsCullingData ImportCullingData(FRDGBuilder& GraphBuilder, FHairGroupPublicData* In)
{
	FRDGHairStrandsCullingData Out;
	Out.bCullingResultAvailable = In->GetCullingResultAvailable();

	if (Out.bCullingResultAvailable)
	{
		Out.HairStrandsVF_CullingIndirectBuffer		= Register(GraphBuilder, In->GetDrawIndirectRasterComputeBuffer(), ERDGImportedBufferFlags::CreateViews);
		Out.HairStrandsVF_CullingIndexBuffer		= Register(GraphBuilder, In->GetCulledVertexIdBuffer(), ERDGImportedBufferFlags::CreateViews);
	}

	return Out;
}

class FHairInterpolationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairInterpolationCS);
	SHADER_USE_PARAMETER_STRUCT(FHairInterpolationCS, FGlobalShader);

	class FPointPerCurve : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_POINT_PER_CURVE", 4, 8, 16, 32);
	class FDynamicGeometry : SHADER_PERMUTATION_INT("PERMUTATION_DYNAMIC_GEOMETRY", 4); /* These is actually 5 DynamicGeometry permutations, but PERMUTATION_DYNAMIC_GEOMETRY==4 is never used (experimental) */
	class FSimulation : SHADER_PERMUTATION_BOOL("PERMUTATION_SIMULATION");
	class FSingleGuide : SHADER_PERMUTATION_BOOL("PERMUTATION_USE_SINGLE_GUIDE");
	class FDeformer : SHADER_PERMUTATION_INT("PERMUTATION_DEFORMER", 3);
	class FWaveOps : SHADER_PERMUTATION_BOOL("PERMUTATION_WAVEOPS");
	using FPermutationDomain = TShaderPermutationDomain<FDynamicGeometry, FSimulation, FSingleGuide, FDeformer, FPointPerCurve, FWaveOps>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderDrawParameters)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(uint32, SimPointCount)
		SHADER_PARAMETER(uint32, SimCurveCount)

		SHADER_PARAMETER(float, HairLengthScale)
		SHADER_PARAMETER(FVector3f, InRenHairPositionOffset)
		SHADER_PARAMETER(FVector3f, InSimHairPositionOffset)
		SHADER_PARAMETER(FMatrix44f, LocalToWorldMatrix)
		SHADER_PARAMETER(uint32, DispatchCountX)
		SHADER_PARAMETER(uint32, bUseSingleGuide)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenCurveBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenRestPosePositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenDeformerPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutRenDeformedPositionBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SimRestPosePositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SimDeformedPositionBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveInterpolationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointInterpolationBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, RenRootRestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, RenRootDeformedPositionBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RenRootBarycentricBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RenRootToUniqueTriangleIndexBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SimRootRestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SimRootDeformedPositionBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SimRootBarycentricBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SimRootToUniqueTriangleIndexBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, OutSimHairPositionOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, OutRenHairPositionOffsetBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairStrandsVFTODO_CullingIndirectBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairStrandsVFTODO_CullingIndexBuffer)
		RDG_BUFFER_ACCESS(HairStrandsVFTODO_CullingIndirectBufferArgs, ERHIAccess::IndirectArgs)
		
		END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 32;}

	static bool DoesPlatformSupportWaveOps(EShaderPlatform InPlatform, uint32 InPointPerCurve)
	{
		// D3D11 / SM5 or preview do not support, or work well with, wave-ops by default (or SM5 preview has issues with wave intrinsics too)
		if (InPointPerCurve != 32 || InPlatform == SP_PCD3D_SM5 || FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(InPlatform))
		{
			return false;
		}
		return FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(InPlatform) != ERHIFeatureSupport::Unsupported && FDataDrivenShaderPlatformInfo::GetMinimumWaveSize(InPlatform) <= 32 && FDataDrivenShaderPlatformInfo::GetMaximumWaveSize(InPlatform) >= 32;
	}

	// Runtime check for platforms that have varying wave size support depending on GPU.
	static bool DoesSupportWaveOps(EShaderPlatform InPlatform, uint32 InPointPerCurve)
	{
		return DoesPlatformSupportWaveOps(InPlatform, InPointPerCurve) && GRHIMinimumWaveSize <= 32 && GRHIMaximumWaveSize >= 32;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>() && !DoesPlatformSupportWaveOps(Parameters.Platform, PermutationVector.Get<FPointPerCurve>()))
		{
			return false;
		}
		return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); 
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>() && !DoesSupportWaveOps(Parameters.Platform, PermutationVector.Get<FPointPerCurve>()))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		// single guide can't be disabled when GetHairStrandsForceSingleGuideInterpolation() is set
		if (!PermutationVector.Get<FSingleGuide>() && GetHairStrandsForceSingleGuideInterpolation())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached; 
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HAIRINTERPOLATION"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairInterpolationCS, "/Engine/Private/HairStrands/HairStrandsInterpolation.usf", "MainCS", SF_Compute);

// AddHairStrandsInterpolationPass takes in input
// * Guide resources (optional)
// * Guide root triangles (optional)
// * Strands resources
// * Strands root triangles (optional)
// 
// This functions is in charge of deforming the rendering strands (OutRenderPositionBuffer) based on the guides
// The guides are either the result of a simulation deformation, or a global interpolation deformation (aka RBF deformation), or both
void AddHairStrandsInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	EShaderPlatform InPlatform,
	const FShaderPrintData* ShaderPrintData,
	const FHairGroupInstance* Instance,
	const uint32 VertexCount,
	const uint32 CurveCount,
	const FHairStrandsRestResource* RenRestResource,
	const FHairStrandsRestResource* SimRestResource,
	const bool bSimDataValid,
	const uint32 MaxPointPerCurve,
	const int32 MeshLODIndex,
	const float HairLengthScale,
	const EHairInterpolationType HairInterpolationType,
	const EHairGeometryType InstanceGeometryType,
	const FRDGHairStrandsCullingData& CullingData,
	const FVector& InRenHairWorldOffset,
	const FVector& InSimHairWorldOffset,
	const FRDGBufferSRVRef& OutRenHairPositionOffsetBuffer,
	const FRDGBufferSRVRef& OutSimHairPositionOffsetBuffer,
	const FHairStrandsRestRootResource* RenRestRootResources,
	const FHairStrandsRestRootResource* SimRestRootResources,
	const FHairStrandsDeformedRootResource* RenDeformedRootResources,
	const FHairStrandsDeformedRootResource* SimDeformedRootResources,
	const FRDGBufferSRVRef& RenRestPosePositionBuffer,
	const FRDGBufferSRVRef& RenCurveBuffer,
	const bool bUseSingleGuide,
	const FRDGBufferSRVRef& CurveInterpolationBuffer,
	const FRDGBufferSRVRef& PointInterpolationBuffer,
	const FRDGBufferSRVRef& SimRestPosePositionBuffer,
	const FRDGBufferSRVRef& SimDeformedPositionBuffer,
	const FRDGBufferSRVRef& RenDeformerPositionBuffer,
	FRDGBufferUAVRef& OutRenPositionBuffer,
	const FHairStrandsLODDeformedRootResource::EFrameType DeformedFrame)
{
	FHairInterpolationCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairInterpolationCS::FParameters>();
	Parameters->RenRestPosePositionBuffer = RenRestPosePositionBuffer;
	Parameters->SimRestPosePositionBuffer = SimRestPosePositionBuffer;
	Parameters->SimDeformedPositionBuffer = SimDeformedPositionBuffer;
	Parameters->CurveInterpolationBuffer = CurveInterpolationBuffer;
	Parameters->PointInterpolationBuffer = PointInterpolationBuffer;
	Parameters->OutRenDeformedPositionBuffer = OutRenPositionBuffer;

	Parameters->VertexCount = VertexCount;
	Parameters->CurveCount = CurveCount;
	Parameters->InRenHairPositionOffset = (FVector3f)InRenHairWorldOffset;
	Parameters->InSimHairPositionOffset = (FVector3f)InSimHairWorldOffset;

	Parameters->OutSimHairPositionOffsetBuffer = OutSimHairPositionOffsetBuffer;
	Parameters->OutRenHairPositionOffsetBuffer = OutRenHairPositionOffsetBuffer;
	
	Parameters->LocalToWorldMatrix = FMatrix44f(Instance->LocalToWorld.ToMatrixWithScale());		// LWC_TODO: Precision loss
	Parameters->HairLengthScale = HairLengthScale;

	Parameters->SimPointCount = 0;
	Parameters->SimCurveCount = 0;
	if (Instance->Guides.RestResource)
	{		
		Parameters->SimPointCount = Instance->Guides.RestResource->BulkData.Header.PointCount;
		Parameters->SimCurveCount = Instance->Guides.RestResource->BulkData.Header.CurveCount;
	}

	Parameters->RenCurveBuffer = RenCurveBuffer;
	Parameters->bUseSingleGuide = bUseSingleGuide;

	if (ShaderPrintData)
	{
		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderDrawParameters);
	}
	
	const bool bSupportDynamicMesh_Ren = 
		RenRestRootResources &&
		RenRestRootResources->GetRootCount() > 0 && 
		MeshLODIndex >= 0 && 
		RenRestRootResources->IsValid(MeshLODIndex) &&
		RenDeformedRootResources->IsValid(MeshLODIndex);

	const bool bSupportDynamicMesh_Sim =
		SimRestRootResources &&
		SimDeformedRootResources && 
		SimRestRootResources->GetRootCount() > 0 && 
		MeshLODIndex >= 0 &&
		SimRestRootResources->IsValid(MeshLODIndex) &&
		SimDeformedRootResources->IsValid(MeshLODIndex);

	// Guides
	// Guide interpolation with dynamic mesh geometry is now support with guide cache. It implicitly assumes that the guide-cache follows correctly the binding surface
	bool bSupportGlobalInterpolation = false;
	if (bSupportDynamicMesh_Sim && (Instance->Guides.bIsSimulationEnable || Instance->Guides.bHasGlobalInterpolation || Instance->Guides.bIsDeformationEnable || Instance->Guides.bIsSimulationCacheEnable))
	{
		const FHairStrandsLODRestRootResource& Sim_RestLODDatas = *SimRestRootResources->GetLOD(MeshLODIndex);
		const FHairStrandsLODDeformedRootResource& Sim_DeformedLODDatas = *SimDeformedRootResources->GetLOD(MeshLODIndex);
		bSupportGlobalInterpolation = Instance->Guides.bHasGlobalInterpolation && (Sim_RestLODDatas.SampleCount > 0);
		{
			Parameters->SimRootRestPositionBuffer = RegisterAsSRV(GraphBuilder, Sim_RestLODDatas.RestUniqueTrianglePositionBuffer);
			Parameters->SimRootBarycentricBuffer = RegisterAsSRV(GraphBuilder, Sim_RestLODDatas.RootBarycentricBuffer);
			Parameters->SimRootToUniqueTriangleIndexBuffer = RegisterAsSRV(GraphBuilder, Sim_RestLODDatas.RootToUniqueTriangleIndexBuffer);
		}
		{
			Parameters->SimRootDeformedPositionBuffer = RegisterAsSRV(GraphBuilder, Sim_DeformedLODDatas.GetDeformedUniqueTrianglePositionBuffer(DeformedFrame));
		}

		if (bSimDataValid && GetHairStrandsBindingDataValidation())
		{
			check(SimRestResource->BulkData.Header.Hash == Sim_RestLODDatas.GetBulkData().Header.Hash);
		}
	}

	// Strands
	if (bSupportDynamicMesh_Ren)
	{
		const FHairStrandsLODRestRootResource& Ren_RestLODDatas = *RenRestRootResources->GetLOD(MeshLODIndex);
		const FHairStrandsLODDeformedRootResource& Ren_DeformedLODDatas = *RenDeformedRootResources->GetLOD(MeshLODIndex);
		{
			Parameters->RenRootRestPositionBuffer = RegisterAsSRV(GraphBuilder, Ren_RestLODDatas.RestUniqueTrianglePositionBuffer);
			Parameters->RenRootBarycentricBuffer = RegisterAsSRV(GraphBuilder, Ren_RestLODDatas.RootBarycentricBuffer);
			Parameters->RenRootToUniqueTriangleIndexBuffer = RegisterAsSRV(GraphBuilder, Ren_RestLODDatas.RootToUniqueTriangleIndexBuffer);
		}
		{
			Parameters->RenRootDeformedPositionBuffer = RegisterAsSRV(GraphBuilder, Ren_DeformedLODDatas.GetDeformedUniqueTrianglePositionBuffer(DeformedFrame));
		}

		if (GetHairStrandsBindingDataValidation())
		{
			check(RenRestResource->BulkData.Header.Hash == Ren_RestLODDatas.GetBulkData().Header.Hash);
		}
	}

	const bool bSupportDeformer = RenDeformerPositionBuffer != nullptr;
	if (bSupportDeformer)
	{
		Parameters->RenDeformerPositionBuffer = RenDeformerPositionBuffer;
	}

	const bool bHasLocalDeformation = Instance->Guides.bIsSimulationEnable || bSupportGlobalInterpolation || Instance->Guides.bIsDeformationEnable || Instance->Guides.bIsSimulationCacheEnable;

	// Enable support for dynamic mesh (skinning) only if roots data are valids
	// * If there is no need of Sim data, only rely on Ren resources
	// * If there is need of Sim data, rely on both Sim and Ren resources
	const bool bSupportDynamicMesh = bHasLocalDeformation ? (bSupportDynamicMesh_Ren && bSupportDynamicMesh_Sim) : bSupportDynamicMesh_Ren;

	// Select dynamic geometry permutation, based on the Simulation/RBF/InterpolationType
	int32 DynamicGeometryType = 0;
	{
		switch (HairInterpolationType)
		{
		case EHairInterpolationType::NoneSkinning			: DynamicGeometryType = 0; break; // INTERPOLATION_RIGID
		case EHairInterpolationType::RigidSkinning			: DynamicGeometryType = 1; break; // INTERPOLATION_SKINNING_OFFSET
		case EHairInterpolationType::OffsetSkinning			: DynamicGeometryType = 2; break; // INTERPOLATION_SKINNING_TRANSLATION
		case EHairInterpolationType::SmoothSkinning			: DynamicGeometryType = 3; break; // INTERPOLATION_SKINNING_ROTATION
		//case EHairInterpolationType::SmoothOffsetSkinning	: DynamicGeometryType = 4; break; // INTERPOLATION_SKINNING_TRANSLATION_AND_ROTATION - Experimental, not used
		}		

		if ( bSupportDynamicMesh && !bHasLocalDeformation) DynamicGeometryType = 1; // INTERPOLATION_SKINNING_OFFSET
		if (!bSupportDynamicMesh && !bHasLocalDeformation) DynamicGeometryType = 0; // INTERPOLATION_RIGID
		if (!bSupportDynamicMesh &&  bHasLocalDeformation) DynamicGeometryType = 0; // INTERPOLATION_RIGID
	}

	// Compute the dispatch information for pass dispatching work per curve
	const FPointPerCurveDispatchInfo DispatchInfo = GetPointPerCurveDispatchInfo(MaxPointPerCurve, CurveCount, FHairInterpolationCS::GetGroupSize());
	Parameters->DispatchCountX = DispatchInfo.DispatchCount.X;

	const bool bSingleGuidePermutation = bUseSingleGuide || GetHairStrandsForceSingleGuideInterpolation();
	const bool bWaveOps = FHairInterpolationCS::DoesSupportWaveOps(InPlatform, DispatchInfo.PointPerCurve);

	// Deformer type : 0 no deformer, 1 add deformer displacement, 2 overwrite deformed position with the deformer one
	const int32 DeformerType = !bSupportDeformer ? 0 : GVarHairStrands_AddDeformerStrandsDisplacement ? 1 : 2;
		
	FHairInterpolationCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairInterpolationCS::FDynamicGeometry>(DynamicGeometryType);
	PermutationVector.Set<FHairInterpolationCS::FSimulation>(bHasLocalDeformation);
	PermutationVector.Set<FHairInterpolationCS::FSingleGuide>(bSingleGuidePermutation);
	PermutationVector.Set<FHairInterpolationCS::FDeformer>(DeformerType);
	PermutationVector.Set<FHairInterpolationCS::FPointPerCurve>(DispatchInfo.PointPerCurve);
	PermutationVector.Set<FHairInterpolationCS::FWaveOps>(bWaveOps);
	TShaderMapRef<FHairInterpolationCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::Interpolation(culling=off, PerCurve=%d, SingleGuide=%d, DynGeom:%d, WaveOps:%d)", DispatchInfo.PointPerCurve, bSingleGuidePermutation ? 1u : 0u, DynamicGeometryType > 0 ? 1u : 0u, bWaveOps ? 1u : 0u),
		ComputeShader,
		Parameters,
		DispatchInfo.DispatchCount);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Patch hair attribute (For debug visualization)

class FHairPatchAttributeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairPatchAttributeCS);
	SHADER_USE_PARAMETER_STRUCT(FHairPatchAttributeCS, FGlobalShader);

	class FSimulation : SHADER_PERMUTATION_INT("PERMUTATION_SIMULATION", 3); // No simulation, Simlation with 1 guide, Simulation with 3 guides
	using FPermutationDomain = TShaderPermutationDomain<FSimulation>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(uint32, bUseSingleGuide)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceAttributeParameters, RenAttributes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenCurveBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveInterpolationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointInterpolationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RenCurveToClusterIdBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutRenAttributeBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 1024u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_PATCHATTRIBUTE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairPatchAttributeCS, "/Engine/Private/HairStrands/HairStrandsInterpolation.usf", "MainCS", SF_Compute);

void AddPatchAttributePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 CurveCount,
	const EHairPatchAttribute Mode,
	const bool bSimulation,
	const bool bUseSingleGuide,
	const FHairStrandsBulkData& RenBulkData,
	const FRDGBufferRef& RenAttributeBuffer,
	const FRDGBufferSRVRef& RenCurveBuffer,
	const FRDGBufferSRVRef& RenCurveToClusterIdBuffer,
	const FRDGBufferSRVRef& CurveInterpolationBuffer,
	const FRDGBufferSRVRef& PointInterpolationBuffer,
	FRDGImportedBuffer& OutRenAttributeBuffer)
{
	// Sanity check
	check(Mode == EHairPatchAttribute::ClusterInfluence || Mode == EHairPatchAttribute::GuideInflucence)
	if (Mode == EHairPatchAttribute::GuideInflucence && !bSimulation) return;

	const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(CurveCount, FHairPatchAttributeCS::GetGroupSize()), 1, 1);

	// First copy all the rendering attributes into the new bufffer, before overriding some of them
	AddCopyBufferPass(GraphBuilder, OutRenAttributeBuffer.Buffer, RenAttributeBuffer);

	FHairPatchAttributeCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairPatchAttributeCS::FParameters>();
	Parameters->bUseSingleGuide = bUseSingleGuide ? 1u : 0u;
	Parameters->CurveCount = CurveCount;
	Parameters->RenCurveBuffer = RenCurveBuffer;
	Parameters->RenCurveToClusterIdBuffer = RenCurveToClusterIdBuffer;
	GetHairStrandsAttributeParameter(RenBulkData, Parameters->RenAttributes);

	if (bSimulation)
	{
		Parameters->CurveInterpolationBuffer  = CurveInterpolationBuffer;
		Parameters->PointInterpolationBuffer  = PointInterpolationBuffer;
	}

	Parameters->OutRenAttributeBuffer = OutRenAttributeBuffer.UAV;

	FHairPatchAttributeCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairPatchAttributeCS::FSimulation>(Mode == EHairPatchAttribute::ClusterInfluence ? 0 : (bUseSingleGuide ? 1 : 2));

	TShaderMapRef<FHairPatchAttributeCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::PatchAttribute"),
		ComputeShader,
		Parameters,
		DispatchCount);

	GraphBuilder.SetBufferAccessFinal(OutRenAttributeBuffer.Buffer, ERHIAccess::SRVMask);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairClusterAABBCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairClusterAABBCS);
	SHADER_USE_PARAMETER_STRUCT(FHairClusterAABBCS, FGlobalShader);

	class FCPUAABB : SHADER_PERMUTATION_BOOL("PERMUTATION_USE_CPU_AABB");
	using FPermutationDomain = TShaderPermutationDomain<FCPUAABB>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderDrawParameters)
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(float, LODIndex)
		SHADER_PARAMETER(float, ClusterScale)
		SHADER_PARAMETER(uint32, ClusterOffset)
		SHADER_PARAMETER(uint32, ClusterCount)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(FVector3f, CPUBoundMin)
		SHADER_PARAMETER(FVector3f, CPUBoundMax)
		SHADER_PARAMETER(FMatrix44f, LocalToTranslatedWorldMatrix)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenCurveBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RenPointLODBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenderDeformedPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, RenderDeformedOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutClusterAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutGroupAABBBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 64; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLUSTERAABB"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairClusterAABBCS, "/Engine/Private/HairStrands/HairStrandsClusterCulling.usf", "ClusterAABBEvaluationCS", SF_Compute);

void AddHairClusterAABBPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 ActiveCurveCount,
	const FVector& TranslatedWorldOffset,
	const FShaderPrintData* ShaderPrintData,
	const EHairAABBUpdateType UpdateType,
	FHairGroupInstance* Instance,
	FRDGBufferSRVRef RenderDeformedOffsetBuffer,
	FHairStrandClusterData::FHairGroup* ClusterData,
	const uint32 ClusterOffset,
	const uint32 ClusterCount,
	FRDGBufferSRVRef RenderPositionBufferSRV,
	FRDGBufferSRVRef& DrawIndirectRasterComputeBuffer,
	FRDGBufferUAVRef ClusterAABBUAV,
	FRDGBufferUAVRef GroupAABBBUAV)
{
	// Clusters AABB are only update if the groom is deformed.
	const FBoxSphereBounds& Bounds = Instance->Strands.GetData().Header.BoundingBox;
	FTransform InRenLocalToTranslatedWorld = Instance->LocalToWorld;
	InRenLocalToTranslatedWorld.AddToTranslation(TranslatedWorldOffset);
	const FBoxSphereBounds TransformedBounds = Bounds.TransformBy(InRenLocalToTranslatedWorld);

	FHairClusterAABBCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairClusterAABBCS::FParameters>();
	Parameters->InstanceRegisteredIndex = Instance->RegisteredIndex;
	Parameters->LODIndex = ClusterData ? ClusterData->LODIndex : 1;;
	Parameters->CPUBoundMin = (FVector3f)TransformedBounds.GetBox().Min;
	Parameters->CPUBoundMax = (FVector3f)TransformedBounds.GetBox().Max;
	Parameters->LocalToTranslatedWorldMatrix = FMatrix44f(InRenLocalToTranslatedWorld.ToMatrixWithScale());
	Parameters->RenderDeformedPositionBuffer = RenderPositionBufferSRV;
	Parameters->RenderDeformedOffsetBuffer = RenderDeformedOffsetBuffer;
	Parameters->CurveCount = ActiveCurveCount;
	Parameters->ClusterOffset = ClusterData ? ClusterOffset : 0;
	Parameters->ClusterCount = ClusterData ? ClusterCount : 1;
	Parameters->ClusterScale = ClusterData ? ClusterData->ClusterScale : 1.f;
	Parameters->RenCurveBuffer = RegisterAsSRV(GraphBuilder, Instance->Strands.RestResource->CurveBuffer);
	Parameters->RenPointLODBuffer = ClusterData ? RegisterAsSRV(GraphBuilder, *ClusterData->PointLODBuffer) : nullptr;
	Parameters->OutClusterAABBBuffer = ClusterAABBUAV;
	Parameters->OutGroupAABBBuffer = GroupAABBBUAV;

	if (ShaderPrintData)
	{
		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderDrawParameters);		
	}

	FHairClusterAABBCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairClusterAABBCS::FCPUAABB>(UpdateType == EHairAABBUpdateType::UpdateClusterAABB ? 0u : 1u);
	TShaderMapRef<FHairClusterAABBCS> ComputeShader(ShaderMap, PermutationVector);

	if (UpdateType == EHairAABBUpdateType::UpdateClusterAABB)
	{
		FIntVector DispatchCount(Parameters->ClusterCount, 1, 1);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::ClusterAABB(Cluster,Group)"),
			ComputeShader,
			Parameters,
			DispatchCount);
	}
	else
	{
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::ClusterAABB(Group Only - CPU bound)"),
			ComputeShader,
			Parameters,
			FIntVector(1,1,1));
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairCardsDeformationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairCardsDeformationCS);
	SHADER_USE_PARAMETER_STRUCT(FHairCardsDeformationCS, FGlobalShader);

	class FDynamicGeometry : SHADER_PERMUTATION_INT("PERMUTATION_DYNAMIC_GEOMETRY", 2);
	using FPermutationDomain = TShaderPermutationDomain<FDynamicGeometry>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(uint32, CardsVertexCount)
		SHADER_PARAMETER(uint32, GuideVertexCount)
		SHADER_PARAMETER(FVector3f, GuideRestPositionOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, GuideRestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, GuideDeformedPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, GuideDeformedPositionOffsetBuffer)
		SHADER_PARAMETER_SRV(Buffer, CardsRestPositionBuffer)
		SHADER_PARAMETER_SRV(Buffer, CardsRestTangentBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CardsInterpolationBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TriangleRestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TriangleDeformedPositionBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GuideRootToUniqueTriangleIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GuideRootBarycentricBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, GuideVertexToRootIndexBuffer)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, CardsDeformedPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, CardsDeformedTangentBuffer)

		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderDrawParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 256u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Cards, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_DEFORM_CARDS"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairCardsDeformationCS, "/Engine/Private/HairStrands/HairCardsDeformation.usf", "MainCS", SF_Compute);

void AddHairCardsDeformationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const ERHIFeatureLevel::Type FeatureLevel,
	const FShaderPrintData* ShaderPrintData,
	FHairGroupInstance* Instance,
	const int32 HairLODIndex,
	const int32 MeshLODIndex,
	ERHIPipeline CardPositionExternalAccessPipeline)
{
	if (!Instance->Cards.IsValid(HairLODIndex))
		return;

	FHairGroupInstance::FCards::FLOD& LOD = Instance->Cards.LODs[HairLODIndex];
	
	FRDGImportedBuffer CardsDeformedPositionBuffer_Curr = Register(GraphBuilder, LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::Current), ERDGImportedBufferFlags::CreateViews);
	FRDGImportedBuffer CardsDeformedPositionBuffer_Prev = Register(GraphBuilder, LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::Previous), ERDGImportedBufferFlags::CreateViews);

	FRDGImportedBuffer CardsDeformedNormalBuffer = Register(GraphBuilder, LOD.DeformedResource->DeformedNormalBuffer, ERDGImportedBufferFlags::CreateViews);
	GraphBuilder.UseInternalAccessMode({
		CardsDeformedPositionBuffer_Curr.Buffer,
		CardsDeformedPositionBuffer_Prev.Buffer,
		CardsDeformedNormalBuffer.Buffer
	});

	// If LOD hasn't switched, copy the current buffer into previous buffer to have correct motion vectors
	const bool bHasLODSwitch = Instance->HairGroupPublicData->VFInput.bHasLODSwitch;
	if (!bHasLODSwitch)
	{
		AddCopyBufferPass(GraphBuilder, CardsDeformedPositionBuffer_Prev.Buffer, CardsDeformedPositionBuffer_Curr.Buffer);
	}
	
	FHairCardsDeformationCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairCardsDeformationCS::FParameters>();
	Parameters->InstanceRegisteredIndex		= Instance->RegisteredIndex;
	Parameters->LocalToWorld				= FMatrix44f(Instance->GetCurrentLocalToWorld().ToMatrixWithScale());		// LWC_TODO: Precision loss
	Parameters->GuideVertexCount			= LOD.Guides.RestResource->GetPointCount();
	Parameters->GuideRestPositionOffset		= (FVector3f)LOD.Guides.RestResource->GetPositionOffset();
	Parameters->GuideRestPositionBuffer		= RegisterAsSRV(GraphBuilder, LOD.Guides.RestResource->PositionBuffer);
	Parameters->GuideDeformedPositionBuffer = RegisterAsSRV(GraphBuilder, LOD.Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current));
	Parameters->GuideDeformedPositionOffsetBuffer = RegisterAsSRV(GraphBuilder, LOD.Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current));

	Parameters->CardsVertexCount			= LOD.RestResource->GetVertexCount();
	Parameters->CardsRestPositionBuffer		= LOD.RestResource->RestPositionBuffer.ShaderResourceViewRHI;
	Parameters->CardsRestTangentBuffer		= LOD.RestResource->NormalsBuffer.ShaderResourceViewRHI;
	Parameters->CardsDeformedPositionBuffer = CardsDeformedPositionBuffer_Curr.UAV;
	Parameters->CardsDeformedTangentBuffer	= CardsDeformedNormalBuffer.UAV;

	Parameters->CardsInterpolationBuffer	= RegisterAsSRV(GraphBuilder, LOD.InterpolationResource->InterpolationBuffer);

	const FHairStrandsRestResource* RestResources = LOD.Guides.RestResource;
	const FHairStrandsRestRootResource* RestRootResources = LOD.Guides.RestRootResource;
	const FHairStrandsDeformedRootResource* DeformedRootResources = LOD.Guides.DeformedRootResource;

	const bool bSupportDynamicMesh =
		Instance->BindingType == EHairBindingType::Skinning &&
		RestRootResources &&
		RestRootResources->GetRootCount() > 0 &&
		MeshLODIndex >= 0 &&
		RestRootResources->IsValid(MeshLODIndex) &&
		DeformedRootResources->IsValid(MeshLODIndex);
	if (bSupportDynamicMesh)
	{
		Parameters->GuideVertexToRootIndexBuffer = RegisterAsSRV(GraphBuilder, RestResources->PointToCurveBuffer);
		const FHairStrandsLODRestRootResource& RestLODDatas = *RestRootResources->GetLOD(MeshLODIndex);
		const FHairStrandsLODDeformedRootResource& DeformedLODDatas = *DeformedRootResources->GetLOD(MeshLODIndex);

		Parameters->GuideRootBarycentricBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RootBarycentricBuffer);
		Parameters->GuideRootToUniqueTriangleIndexBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RootToUniqueTriangleIndexBuffer);
		Parameters->TriangleRestPositionBuffer = RegisterAsSRV(GraphBuilder, RestLODDatas.RestUniqueTrianglePositionBuffer);
		Parameters->TriangleDeformedPositionBuffer = RegisterAsSRV(GraphBuilder, DeformedLODDatas.GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current));
	}

	if (ShaderPrintData)
	{
		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderDrawParameters);
	}

	FHairCardsDeformationCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairCardsDeformationCS::FDynamicGeometry>(bSupportDynamicMesh ? 1 : 0);
	TShaderMapRef<FHairCardsDeformationCS> ComputeShader(ShaderMap, PermutationVector);

	const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(Parameters->CardsVertexCount, FHairCardsDeformationCS::GetGroupSize()), 1, 1);

	const bool bManualFetch = FVertexFactoryType::CheckManualVertexFetchSupport(FeatureLevel);
	FBufferRHIRef CardsRestPositionBuffer = LOD.RestResource->RestPositionBuffer.VertexBufferRHI;
	FBufferRHIRef CardsRestTangentBuffer  = LOD.RestResource->NormalsBuffer.VertexBufferRHI;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::CardsDeformation(%s)", bSupportDynamicMesh ? TEXT("Dynamic") : TEXT("Static")),
		Parameters,
		ERDGPassFlags::Compute,
		[Parameters, ComputeShader, DispatchCount, CardsRestPositionBuffer, CardsRestTangentBuffer, bManualFetch](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			// On platforms not supporting manual vertex fetching, ensure the resources are in 'VerteOrIndexBuffer' state after position/normals update
			if (!bManualFetch)
			{
				RHICmdList.Transition(FRHITransitionInfo(CardsRestPositionBuffer, ERHIAccess::Unknown, ERHIAccess::SRVMask));
				RHICmdList.Transition(FRHITransitionInfo(CardsRestTangentBuffer, ERHIAccess::Unknown, ERHIAccess::SRVMask));
			}
			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, DispatchCount);
			if (!bManualFetch)
			{
				RHICmdList.Transition(FRHITransitionInfo(CardsRestPositionBuffer, ERHIAccess::SRVMask, ERHIAccess::VertexOrIndexBuffer));
				RHICmdList.Transition(FRHITransitionInfo(CardsRestTangentBuffer, ERHIAccess::SRVMask, ERHIAccess::VertexOrIndexBuffer));
			}
		});


	// If LOD has switched, copy the current buffer, so that we don't get incorrect motion vector
	if (bHasLODSwitch)
	{
		AddCopyBufferPass(GraphBuilder, CardsDeformedPositionBuffer_Prev.Buffer, CardsDeformedPositionBuffer_Curr.Buffer);
	}

	GraphBuilder.UseExternalAccessMode(CardsDeformedPositionBuffer_Curr.Buffer, ERHIAccess::SRVMask, CardPositionExternalAccessPipeline);

	GraphBuilder.UseExternalAccessMode({
		CardsDeformedPositionBuffer_Prev.Buffer,
		CardsDeformedNormalBuffer.Buffer
	}, ERHIAccess::SRVMask);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairTangentCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairTangentCS);
	SHADER_USE_PARAMETER_STRUCT(FHairTangentCS, FGlobalShader);

	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	using FPermutationDomain = TShaderPermutationDomain<FCulling>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(uint32, HairStrandsVF_bCullingEnable)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,	HairStrandsVF_CullingIndirectBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,	HairStrandsVF_CullingIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputTangentBuffer)
		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TANGENT"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairTangentCS, "/Engine/Private/HairStrands/HairStrandsTangent.usf", "MainCS", SF_Compute);

void AddHairTangentPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 PointCount,
	FHairGroupPublicData* HairGroupPublicData,
	FRDGBufferSRVRef PositionBuffer,
	FRDGBufferUAVRef OutTangentBuffer)
{
	const bool bCullingEnable = IsHairStrandContinuousDecimationReorderingEnabled() ? false : (HairGroupPublicData && HairGroupPublicData->GetCullingResultAvailable());

	FHairTangentCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairTangentCS::FParameters>();
	Parameters->PositionBuffer = PositionBuffer;
	Parameters->OutputTangentBuffer = OutTangentBuffer;
	Parameters->PointCount = PointCount;
	Parameters->HairStrandsVF_bCullingEnable = bCullingEnable ? 1 : 0;

	FHairTangentCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairTangentCS::FCulling>(bCullingEnable);

	TShaderMapRef<FHairTangentCS> ComputeShader(ShaderMap, PermutationVector);

	if (bCullingEnable)
	{
		FRDGImportedBuffer DrawIndirectRasterComputeBuffer  = Register(GraphBuilder, HairGroupPublicData->GetDrawIndirectRasterComputeBuffer(), ERDGImportedBufferFlags::CreateSRV);
		Parameters->HairStrandsVF_CullingIndirectBuffer		= DrawIndirectRasterComputeBuffer.SRV;
		Parameters->HairStrandsVF_CullingIndexBuffer		= RegisterAsSRV(GraphBuilder, HairGroupPublicData->GetCulledVertexIdBuffer());
		Parameters->IndirectBufferArgs						= DrawIndirectRasterComputeBuffer.Buffer;
		
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::Tangent(culling=on)"),
			ComputeShader,
			Parameters,
			DrawIndirectRasterComputeBuffer.Buffer, 0);
	}
	else
	{
		const FIntVector DispatchCount(FMath::DivideAndRoundUp(PointCount, FHairTangentCS::GetGroupSize()), 1, 1);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::Tangent(culling=off)"),
			ComputeShader,
			Parameters,
			DispatchCount);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairRaytracingGeometryCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairRaytracingGeometryCS);
	SHADER_USE_PARAMETER_STRUCT(FHairRaytracingGeometryCS, FGlobalShader);

	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	class FPrimitiveType : SHADER_PERMUTATION_ENUM_CLASS("PRIMITIVE_TYPE", EHairStrandsRayTracingPrimitiveType);
	using FPermutationDomain = TShaderPermutationDomain<FCulling, FPrimitiveType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(uint32, HairStrandsVF_RegisteredIndex)
		SHADER_PARAMETER(float, HairStrandsVF_HairRadius)
		SHADER_PARAMETER(float, HairStrandsVF_HairRootScale)
		SHADER_PARAMETER(float, HairStrandsVF_HairTipScale)

		SHADER_PARAMETER(uint32, HairStrandsVF_bCullingEnable)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer,	HairStrandsVF_CullingIndirectBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer,	HairStrandsVF_CullingIndexBuffer)
		RDG_BUFFER_ACCESS(HairStrandsVF_CullingIndirectBufferArgs, ERHIAccess::IndirectArgs)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, PositionOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TangentBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutputIndexBuffer)
		SHADER_PARAMETER(uint32, RaytracingProceduralSplits)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		const EHairStrandsRayTracingPrimitiveType PrimitiveType = PermutationVector.Get<FPrimitiveType>();

		if ((PrimitiveType == EHairStrandsRayTracingPrimitiveType::LinearSweptSphere) && !GetSupportHairStrandsLinearSweptSpheres(Parameters.Platform))
		{
			return false;
		}

		if ((PrimitiveType == EHairStrandsRayTracingPrimitiveType::Procedural) && !GetSupportHairStrandsProceduralPrimitive(Parameters.Platform))
		{
			return false;
		}

		if ((PrimitiveType == EHairStrandsRayTracingPrimitiveType::Triangle) && GetSupportHairStrandsProceduralPrimitive(Parameters.Platform))
		{
			return false;
		}

		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); 
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RT_GEOMETRY"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

BEGIN_SHADER_PARAMETER_STRUCT(FStrandsResourceBLASParameters, )
	RDG_BUFFER_ACCESS(PositionBuffer, ERHIAccess::SRVCompute)
	RDG_BUFFER_ACCESS(IndexBuffer, ERHIAccess::SRVCompute)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FCardsOrMeshesResourceBLASParameters, )
	RDG_BUFFER_ACCESS(PositionBuffer, ERHIAccess::SRVCompute)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER(FHairRaytracingGeometryCS, "/Engine/Private/HairStrands/HairStrandsRaytracingGeometry.usf", "MainCS", SF_Compute);

void AddGenerateRaytracingGeometryPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FShaderPrintData* ShaderPrintData,
	uint32 InstanceRegisteredIndex,
	uint32 ActivePointCount,
	uint32 TotalPointCount,
	EHairStrandsRayTracingPrimitiveType RayTracingPrimitiveType,
	int ProceduralSplits,
	float HairRadius,
	float RootScale,
	float TipScale,
	const FRDGBufferSRVRef& HairWorldOffsetBuffer,
	FRDGHairStrandsCullingData& CullingData,
	const FRDGBufferSRVRef& PositionBuffer,
	const FRDGBufferSRVRef& TangentBuffer,
	const FRDGBufferUAVRef& OutPositionBuffer,
	const FRDGBufferUAVRef& OutIndexBuffer)
{
	FHairRaytracingGeometryCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairRaytracingGeometryCS::FParameters>();
	Parameters->PointCount = ActivePointCount;
	Parameters->PositionOffsetBuffer = HairWorldOffsetBuffer;
	Parameters->HairStrandsVF_RegisteredIndex = InstanceRegisteredIndex;
	Parameters->HairStrandsVF_HairRadius = HairRadius;
	Parameters->HairStrandsVF_HairRootScale = RootScale;
	Parameters->HairStrandsVF_HairTipScale = TipScale;
	Parameters->PositionBuffer = PositionBuffer;
	Parameters->TangentBuffer = TangentBuffer;
	Parameters->OutputPositionBuffer = OutPositionBuffer;
	Parameters->RaytracingProceduralSplits = GetHairRaytracingProceduralSplits();
	if (ShaderPrintData)
	{
		ShaderPrint::SetParameters(GraphBuilder, *ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
	}
	Parameters->OutputIndexBuffer = OutIndexBuffer;
	Parameters->RaytracingProceduralSplits = ProceduralSplits;

	const bool bCullingEnable = CullingData.bCullingResultAvailable;
	Parameters->HairStrandsVF_bCullingEnable = bCullingEnable ? 1 : 0;
	if (bCullingEnable)
	{
		Parameters->HairStrandsVF_CullingIndirectBuffer = CullingData.HairStrandsVF_CullingIndirectBuffer.SRV;
		Parameters->HairStrandsVF_CullingIndexBuffer = CullingData.HairStrandsVF_CullingIndexBuffer.SRV;
		Parameters->HairStrandsVF_CullingIndirectBufferArgs = CullingData.HairStrandsVF_CullingIndirectBuffer.Buffer;
	}

	FHairRaytracingGeometryCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairRaytracingGeometryCS::FCulling>(bCullingEnable);
	PermutationVector.Set<FHairRaytracingGeometryCS::FPrimitiveType>(RayTracingPrimitiveType);
	TShaderMapRef<FHairRaytracingGeometryCS> ComputeShader(ShaderMap, PermutationVector);

	const FIntVector DispatchCount(FMath::DivideAndRoundUp(TotalPointCount, FHairRaytracingGeometryCS::GetGroupSize()), 1, 1);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::GenerateRaytracingGeometry(bCulling:%d, PrimitiveType:%d)", bCullingEnable, (int32)RayTracingPrimitiveType),
		ComputeShader,
		Parameters,
		DispatchCount);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FClearAABBCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearAABBCS);
	SHADER_USE_PARAMETER_STRUCT(FClearAABBCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutAABBBuffer)
		SHADER_PARAMETER(uint32, AABBCount)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLEARAABB"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearAABBCS, "/Engine/Private/HairStrands/HairStrandsClusterCulling.usf", "MainClearAABBCS", SF_Compute);

void AddClearAABBPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 AABBCount,
	FRDGBufferUAVRef& OutAABBUAV)
{
	check(OutAABBUAV);

	FClearAABBCS::FParameters* Parameters = GraphBuilder.AllocParameters<FClearAABBCS::FParameters>();
	Parameters->AABBCount = AABBCount;
	Parameters->OutAABBBuffer = OutAABBUAV;

	TShaderMapRef<FClearAABBCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::ClearAABBs"),
		ComputeShader,
		Parameters,
		FIntVector(FMath::DivideAndRoundUp(AABBCount * 6u, 64u), 1, 1));
}

///////////////////////////////////////////////////////////////////////////////////////////////////

#if RHI_RAYTRACING
static void UpdateHairAccelerationStructure(FRHICommandList& RHICmdList, FRayTracingGeometry* RayTracingGeometry, EAccelerationStructureBuildMode InMode)
{
	SCOPED_DRAW_EVENT(RHICmdList, CommitHairRayTracingGeometryUpdates);

	FRayTracingGeometryBuildParams Params;
	Params.BuildMode = InMode;
	Params.Geometry = RayTracingGeometry->GetRHI();
	Params.Segments = RayTracingGeometry->GetInitializer().Segments;

	RHICmdList.BuildAccelerationStructures(MakeArrayView(&Params, 1));
}

static void BuildHairAccelerationStructure_Strands(
	FRHICommandList& RHICmdList, 
	uint32 RaytracingVertexCount, 
	uint32 RayTracingIndexCount,
	FBufferRHIRef& PositionBuffer, 
	FBufferRHIRef& IndexBuffer,
	FRayTracingGeometry* OutRayTracingGeometry,
	const FString& AssetDebugName,
	const FString& MeshComponentName,
	uint32 LODIndex,
	EHairStrandsRayTracingPrimitiveType RayTracingPrimitiveType,
	int ProceduralSplits)
{
	FRayTracingGeometryInitializer Initializer;
#if HAIR_RESOURCE_DEBUG_NAME
	FString DebugName(TEXT("AS_HairStrands_LOD") + FString::FromInt(LODIndex) + TEXT("_") + AssetDebugName);
	Initializer.DebugName = FName(DebugName);
#else
	static const FName DebugName("AS_HairStrands");	
	static int32 DebugNumber = 0;
	Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
#endif
#if RHI_ENABLE_RESOURCE_INFO
	const FName OwnerName(FString::Printf(TEXT("%s [LOD%d]"), *MeshComponentName, LODIndex));
	Initializer.OwnerName = OwnerName;
#endif
	
	if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::LinearSweptSphere)
	{
		Initializer.IndexBuffer = IndexBuffer;
		Initializer.IndexBufferOffset = 0;
		Initializer.GeometryType = RTGT_LinearSweptSpheres;
		Initializer.TotalPrimitiveCount = RayTracingIndexCount / 2;
	}
	else if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::Procedural)
	{
		Initializer.GeometryType = RTGT_Procedural;
		Initializer.TotalPrimitiveCount = (RaytracingVertexCount / (2 * STRANDS_PROCEDURAL_INTERSECTOR_MAX_SPLITS)) * ProceduralSplits;
	}
	else
	{
		Initializer.IndexBuffer = IndexBuffer;
		Initializer.IndexBufferOffset = 0;
		Initializer.GeometryType = RTGT_Triangles;
		Initializer.TotalPrimitiveCount = RayTracingIndexCount / 3;
	}
	Initializer.bFastBuild = true;
	Initializer.bAllowUpdate = false; // we won't be using update, so specify this to allow degenerate primitives to be discarded

	FRayTracingGeometrySegment Segment;
	Segment.VertexBuffer = PositionBuffer;
	Segment.VertexBufferElementType = FHairStrandsRaytracingFormat::VertexElementType;

	if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::LinearSweptSphere)
	{
		Segment.VertexBufferStride = FHairStrandsRaytracingFormat::SizeInByte;
		Segment.MaxVertices = RaytracingVertexCount;
		Segment.NumPrimitives = Initializer.TotalPrimitiveCount;
	}
	else if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::Procedural)
	{
		// stride covers one AABB (which is made up of two vertices)
		Segment.VertexBufferStride = FHairStrandsRaytracingFormat::SizeInByte * 2;
		Segment.MaxVertices = RaytracingVertexCount;
		Segment.NumPrimitives = Initializer.TotalPrimitiveCount;
	}
	else
	{
		Segment.VertexBufferStride = FHairStrandsRaytracingFormat::SizeInByte;
		Segment.MaxVertices = RaytracingVertexCount;
		Segment.NumPrimitives = RayTracingIndexCount / 3;
	}
	Initializer.Segments.Add(Segment);

	OutRayTracingGeometry->SetInitializer(Initializer);

	// Build inline rather than queuing through the geometry manager so the same-frame dynamic RT instance gather sees this geometry as built.
	OutRayTracingGeometry->CreateRayTracingGeometry(RHICmdList, ERTAccelerationStructureBuildPriority::Skip);
	UpdateHairAccelerationStructure(RHICmdList, OutRayTracingGeometry, EAccelerationStructureBuildMode::Build);
}

static void BuildHairAccelerationStructure_Cards(FRHICommandList& RHICmdList,
	FHairCardsRestResource* RestResource,
	FHairCardsDeformedResource* DeformedResource,
	FRayTracingGeometry* OutRayTracingGeometry,
	const FString& AssetDebugName,
	const FString& MeshComponentName,
	uint32 LODIndex)
{
	FRayTracingGeometryInitializer Initializer;
#if HAIR_RESOURCE_DEBUG_NAME
	FString DebugName(TEXT("AS_HairCards_LOD") + FString::FromInt(LODIndex) + TEXT("_") + AssetDebugName);
	Initializer.DebugName = FName(DebugName);
#else
	static const FName DebugName("AS_HairCards");
	static int32 DebugNumber = 0;
	Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
#endif
#if RHI_ENABLE_RESOURCE_INFO
	const FName OwnerName(FString::Printf(TEXT("%s [LOD%d]"), *MeshComponentName, LODIndex));
	Initializer.OwnerName = OwnerName;
#endif
	Initializer.IndexBuffer = RestResource->RestIndexBuffer.IndexBufferRHI;
	Initializer.IndexBufferOffset = 0;
	Initializer.GeometryType = RTGT_Triangles;
	Initializer.TotalPrimitiveCount = RestResource->GetPrimitiveCount();
	Initializer.bFastBuild = true;
	Initializer.bAllowUpdate = true;

	FBufferRHIRef PositionBuffer = RestResource->RestPositionBuffer.VertexBufferRHI;
	if (DeformedResource)
	{
		PositionBuffer = DeformedResource->GetBuffer(FHairCardsDeformedResource::Current).Buffer->GetRHI(); // This will likely flicker result in half speed motion as everyother frame will use the wrong buffer
	}

	FRayTracingGeometrySegment Segment;
	Segment.VertexBuffer = PositionBuffer;
	Segment.VertexBufferStride = FHairCardsPositionFormat::SizeInByte;
	Segment.VertexBufferElementType = FHairCardsPositionFormat::VertexElementType;
	Segment.NumPrimitives = RestResource->GetPrimitiveCount();
	Segment.MaxVertices = RestResource->GetVertexCount();
	Initializer.Segments.Add(Segment);

	OutRayTracingGeometry->SetInitializer(Initializer);

	// Build inline rather than queuing through the geometry manager so the same-frame dynamic RT instance gather sees this geometry as built.
	OutRayTracingGeometry->CreateRayTracingGeometry(RHICmdList, ERTAccelerationStructureBuildPriority::Skip);
	UpdateHairAccelerationStructure(RHICmdList, OutRayTracingGeometry, EAccelerationStructureBuildMode::Build);
}

static void BuildHairAccelerationStructure_Meshes(FRHICommandList& RHICmdList,
	FHairMeshesRestResource* RestResource,
	FHairMeshesDeformedResource* DeformedResource,
	FRayTracingGeometry* OutRayTracingGeometry,
	const FString& AssetDebugName,
	uint32 LODIndex)
{
	FRayTracingGeometryInitializer Initializer;
#if HAIR_RESOURCE_DEBUG_NAME
	FString DebugName(TEXT("AS_HairMeshes_LOD") + FString::FromInt(LODIndex) + TEXT("_") + AssetDebugName);
	Initializer.DebugName = FName(DebugName);
#else
	static const FName DebugName("AS_HairMeshes");
	static int32 DebugNumber = 0;
	Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
#endif
	Initializer.IndexBuffer = RestResource->IndexBuffer.IndexBufferRHI;
	Initializer.IndexBufferOffset = 0;
	Initializer.GeometryType = RTGT_Triangles;
	Initializer.TotalPrimitiveCount = RestResource->GetPrimitiveCount();
	Initializer.bFastBuild = true;
	Initializer.bAllowUpdate = true;

	FBufferRHIRef PositionBuffer = RestResource->RestPositionBuffer.VertexBufferRHI;
	if (DeformedResource)
	{	
		PositionBuffer = DeformedResource->GetBuffer(FHairMeshesDeformedResource::Current).Buffer->GetRHI(); // This will likely flicker result in half speed motion as everyother frame will use the wrong buffer
	}

	FRayTracingGeometrySegment Segment;
	Segment.VertexBuffer = PositionBuffer;
	Segment.VertexBufferStride = FHairCardsPositionFormat::SizeInByte;
	Segment.VertexBufferElementType = FHairCardsPositionFormat::VertexElementType;
	Segment.NumPrimitives = RestResource->GetPrimitiveCount();
	Segment.MaxVertices = RestResource->GetVertexCount();
	Initializer.Segments.Add(Segment);

	OutRayTracingGeometry->SetInitializer(Initializer);

	// Build inline rather than queuing through the geometry manager so the same-frame dynamic RT instance gather sees this geometry as built.
	OutRayTracingGeometry->CreateRayTracingGeometry(RHICmdList, ERTAccelerationStructureBuildPriority::Skip);
	UpdateHairAccelerationStructure(RHICmdList, OutRayTracingGeometry, EAccelerationStructureBuildMode::Build);
}

void AddBuildStrandsAccelerationStructurePass(
	FRDGBuilder& GraphBuilder,
	FHairGroupInstance* Instance,
	uint32 ProceduralSplits,
	bool bNeedUpdate,
	FRDGBufferRef Raytracing_PositionBuffer,
	FRDGBufferRef Raytracing_IndexBuffer)
{
	const uint32 HairLODIndex = Instance->HairGroupPublicData->GetIntLODIndex();
	const EHairStrandsRayTracingPrimitiveType RayTracingPrimitiveType = Instance->Strands.RenRaytracingResource->RayTracingPrimitiveType;

	FStrandsResourceBLASParameters* Parameters = GraphBuilder.AllocParameters < FStrandsResourceBLASParameters > ();
	Parameters->PositionBuffer = Raytracing_PositionBuffer;
	Parameters->IndexBuffer = Raytracing_IndexBuffer;
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::UpdateBLAS(Strands)"),
		Parameters,
		ERDGPassFlags::NeverCull | ERDGPassFlags::Compute,
		[Instance, HairLODIndex, bNeedUpdate, RayTracingPrimitiveType, ProceduralSplits](FRHICommandListImmediate& RHICmdList)
		{
			SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());
	
			const bool bLocalNeedBuild = !Instance->Strands.RenRaytracingResource->bIsRTGeometryInitialized || (Instance->Strands.CachedProceduralSplits != ProceduralSplits);
			if (bLocalNeedBuild)
			{
				FBufferRHIRef PositionBuffer(Instance->Strands.RenRaytracingResource->PositionBuffer.Buffer->GetRHI());
	
				// no index buffer needed for procedural primitive
				FBufferRHIRef IndexBuffer = nullptr;
				if (RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::Triangle ||
				    RayTracingPrimitiveType == EHairStrandsRayTracingPrimitiveType::LinearSweptSphere)
				{
					IndexBuffer = Instance->Strands.RenRaytracingResource->IndexBuffer.Buffer->GetRHI();
				}

				Instance->Strands.CachedProceduralSplits = ProceduralSplits;
				BuildHairAccelerationStructure_Strands(
					RHICmdList,
					Instance->Strands.RenRaytracingResource->MaxVertexCount,
					Instance->Strands.RenRaytracingResource->MaxIndexCount,
					PositionBuffer,
					IndexBuffer,
					&Instance->Strands.RenRaytracingResource->RayTracingGeometry,
					Instance->Debug.GroomAssetName,
					Instance->Debug.MeshComponentName,
					HairLODIndex,
					RayTracingPrimitiveType,
					ProceduralSplits
				);
			}
			else if (bNeedUpdate)
			{
				// hair strands can move chaotically during simulation which can really tank ray tracing performance
				// even though a rebuild is more expensive, we can still come out ahead overall, even with a single ray cast per pixel
				UpdateHairAccelerationStructure(RHICmdList, &Instance->Strands.RenRaytracingResource->RayTracingGeometry, IsHairStrandsForceRebuildBVH() ? EAccelerationStructureBuildMode::Build : EAccelerationStructureBuildMode::Update);
			}
			Instance->Strands.RenRaytracingResource->bIsRTGeometryInitialized = true;
		});
}

void AddBuildHairCardAccelerationStructurePass(
	FRDGBuilder& GraphBuilder,
	FHairGroupInstance* Instance,
	int32 HairLODIndex,
	bool bNeedUpdate)
{
	check(Instance);
	check(Instance->Cards.LODs.IsValidIndex(HairLODIndex));

	FHairGroupInstance::FCards::FLOD& LOD = Instance->Cards.LODs[HairLODIndex];
	FCardsOrMeshesResourceBLASParameters* Parameters = GraphBuilder.AllocParameters<FCardsOrMeshesResourceBLASParameters>();
	Parameters->PositionBuffer = LOD.DeformedResource ? Register(GraphBuilder, LOD.DeformedResource->GetBuffer(FHairCardsDeformedResource::Current), ERDGImportedBufferFlags::None).Buffer : nullptr;

	// * When cards are dynamic, RT geometry is built once and then update.
	// * When cards are static (i.e., no sim, not attached to  skinning()), in which case RT geometry needs only to be built once. This static geometry is shared between all the instances, 
	//   and owned by the asset, rathter than the component. In this case the RT geometry will be built only once for all the instances
	const bool bNeedBuild = !LOD.RaytracingResource->bIsRTGeometryInitialized;
	if (bNeedBuild || bNeedUpdate)
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::UpdateBLAS(Cards)"),
			Parameters,
			ERDGPassFlags::NeverCull | ERDGPassFlags::Compute,
			[Instance, HairLODIndex, bNeedUpdate](FRHICommandListImmediate& RHICmdList)
		{
			SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());

			FHairGroupInstance::FCards::FLOD& LocalLOD = Instance->Cards.LODs[HairLODIndex];

			const bool bLocalNeedBuild = !LocalLOD.RaytracingResource->bIsRTGeometryInitialized;
			if (bLocalNeedBuild)
			{
				BuildHairAccelerationStructure_Cards(RHICmdList, LocalLOD.RestResource, LocalLOD.DeformedResource, &LocalLOD.RaytracingResource->RayTracingGeometry, Instance->Debug.GroomAssetName, Instance->Debug.MeshComponentName, HairLODIndex);
				LocalLOD.RaytracingResource->bIsRTGeometryInitialized = true;
			}
			else if (bNeedUpdate)
			{
				// TODO: evaluate perf tradeoff of rebuild vs refit
				UpdateHairAccelerationStructure(RHICmdList, &LocalLOD.RaytracingResource->RayTracingGeometry, EAccelerationStructureBuildMode::Update);
			}
		});
	}
}

void AddBuildHairMeshAccelerationStructurePass(
	FRDGBuilder& GraphBuilder,
	FHairGroupInstance* Instance,
	int32 HairLODIndex,
	bool bNeedUpdate)
{
	check(Instance);
	check(Instance->Meshes.LODs.IsValidIndex(HairLODIndex));

	FHairGroupInstance::FMeshes::FLOD& LOD = Instance->Meshes.LODs[HairLODIndex];
	FCardsOrMeshesResourceBLASParameters* Parameters = GraphBuilder.AllocParameters<FCardsOrMeshesResourceBLASParameters>();
	Parameters->PositionBuffer = LOD.DeformedResource ? Register(GraphBuilder, LOD.DeformedResource->GetBuffer(FHairMeshesDeformedResource::Current), ERDGImportedBufferFlags::None).Buffer : nullptr;

	const bool bNeedBuid = !LOD.RaytracingResource->bIsRTGeometryInitialized;
	if (bNeedBuid || bNeedUpdate)
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::UpdateBLAS(Meshes)"),
			Parameters,
			ERDGPassFlags::NeverCull | ERDGPassFlags::Compute,
			[Instance, HairLODIndex, bNeedUpdate](FRHICommandListImmediate& RHICmdList)
			{
				SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());

				FHairGroupInstance::FMeshes::FLOD& LocalLOD = Instance->Meshes.LODs[HairLODIndex];	
				const bool bLocalNeedBuild = !LocalLOD.RaytracingResource->bIsRTGeometryInitialized;
				if (bLocalNeedBuild)
				{
					BuildHairAccelerationStructure_Meshes(RHICmdList, LocalLOD.RestResource, LocalLOD.DeformedResource,  &LocalLOD.RaytracingResource->RayTracingGeometry, Instance->Debug.GroomAssetName, HairLODIndex);
					LocalLOD.RaytracingResource->bIsRTGeometryInitialized = true;
				}
				else if (bNeedUpdate)
				{
					// TODO: evaluate perf tradeoff of rebuild vs refit
					UpdateHairAccelerationStructure(RHICmdList, &LocalLOD.RaytracingResource->RayTracingGeometry, EAccelerationStructureBuildMode::Update);
				}
			});
	}
}
#endif // RHI_RAYTRACING

static void ConvertHairStrandsVFParameters(
	FRDGBuilder* GraphBuilder,
	FRDGImportedBuffer& OutBuffer,
	FRDGExternalBuffer& OutExternalBuffer,
	FShaderResourceViewRHIRef& OutBufferRHISRV,
	const FRDGExternalBuffer& ExternalBuffer)
{
	if (GraphBuilder)
	{
		OutBuffer = Register(*GraphBuilder, ExternalBuffer, ERDGImportedBufferFlags::CreateSRV);
	}
	OutBufferRHISRV = ExternalBuffer.SRV;
	OutExternalBuffer = ExternalBuffer;
}
#define CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutName, ExternalBuffer) ConvertHairStrandsVFParameters(GraphBuilder, OutName, OutName##External, OutName##RHISRV, ExternalBuffer);

// Compute/Update the hair strands description which will be used for rendering (VF) / voxelization & co.
FHairGroupPublicData::FVertexFactoryInput InternalComputeHairStrandsVertexInputData(FRDGBuilder* GraphBuilder, const FHairGroupInstance* Instance, EGroomViewMode ViewMode)
{
	FHairGroupPublicData::FVertexFactoryInput OutVFInput;
	if (!Instance || Instance->GeometryType != EHairGeometryType::Strands)
	{
		const FUintVector4 InvalidOffset = FUintVector4(HAIR_ATTRIBUTE_INVALID_OFFSET,HAIR_ATTRIBUTE_INVALID_OFFSET,HAIR_ATTRIBUTE_INVALID_OFFSET,HAIR_ATTRIBUTE_INVALID_OFFSET);
		for (uint32 OffsetIt = 0; OffsetIt < HAIR_CURVE_ATTRIBUTE_OFFSET_COUNT; ++OffsetIt)
		{
			OutVFInput.Strands.Common.Attributes.CurveAttributeOffsets[OffsetIt] = InvalidOffset;
		}
		for (uint32 OffsetIt = 0; OffsetIt < HAIR_POINT_ATTRIBUTE_OFFSET_COUNT; ++OffsetIt)
		{
			OutVFInput.Strands.Common.Attributes.PointAttributeOffsets[OffsetIt] = InvalidOffset;
		}
		return OutVFInput;
	}

	const bool bSupportHasGuides = Instance->Guides.NeedsGuides();
	const bool bSupportDeformation = Instance->Strands.DeformedResource != nullptr;
	bool bUseGuideAttributeOffsets = false;

	// 1. Guides
	if (ViewMode == EGroomViewMode::SimHairStrands && bSupportHasGuides)
	{
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionBuffer,			Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionBuffer,		Instance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Previous));
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.TangentBuffer,			Instance->Guides.DeformedResource->TangentBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveAttributeBuffer,		Instance->Guides.RestResource->CurveAttributeBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointAttributeBuffer,		Instance->Guides.RestResource->PointAttributeBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveBuffer,				Instance->Guides.RestResource->CurveBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointToCurveBuffer,		Instance->Guides.RestResource->PointToCurveBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionOffsetBuffer,		Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionOffsetBuffer,	Instance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Previous));

		OutVFInput.Strands.Common.PositionOffset = FVector3f(Instance->Guides.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current)); // LWC precision loss
		OutVFInput.Strands.Common.PrevPositionOffset = FVector3f(Instance->Guides.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Previous)); // LWC precision loss
		OutVFInput.Strands.Common.PointCount = Instance->Guides.RestResource->GetPointCount();
		OutVFInput.Strands.Common.CurveCount = Instance->Guides.RestResource->GetCurveCount();
		bUseGuideAttributeOffsets = true;
	}
	// 2. Render Strands with deformation
	else if (bSupportDeformation)
	{
		const bool bSameUniqueView = 
			Instance->Strands.DeformedResource->GetUniqueViewID(FHairStrandsDeformedResource::Current) ==
			Instance->Strands.DeformedResource->GetUniqueViewID(FHairStrandsDeformedResource::Previous);
		const bool bHasValidMotionVector = GHairStrands_MotionVector_CheckViewID ? bSameUniqueView : true;
		if (bHasValidMotionVector)
		{
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionBuffer,			Instance->Strands.DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionBuffer,		Instance->Strands.DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Previous));
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionOffsetBuffer,		Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionOffsetBuffer, Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Previous));

			OutVFInput.Strands.Common.PositionOffset	 = FVector3f(Instance->Strands.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current)); // LWC precision loss
			OutVFInput.Strands.Common.PrevPositionOffset = FVector3f(Instance->Strands.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Previous)); // LWC precision loss
		}
		else
		{
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionBuffer,			Instance->Strands.DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionBuffer,		Instance->Strands.DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionOffsetBuffer,		Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current));
			CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionOffsetBuffer,	Instance->Strands.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current));

			OutVFInput.Strands.Common.PositionOffset	 = FVector3f(Instance->Strands.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current)); // LWC precision loss
			OutVFInput.Strands.Common.PrevPositionOffset = FVector3f(Instance->Strands.DeformedResource->GetPositionOffset(FHairStrandsDeformedResource::EFrameType::Current)); // LWC precision loss
		}
		const bool bUseDebugCurveBuffer = (ViewMode == EGroomViewMode::RenderHairStrands || ViewMode == EGroomViewMode::Cluster || ViewMode == EGroomViewMode::ClusterAABB);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.TangentBuffer,			Instance->Strands.DeformedResource->TangentBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveAttributeBuffer,		bUseDebugCurveBuffer ? Instance->Strands.DebugCurveAttributeBuffer : Instance->Strands.RestResource->CurveAttributeBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointAttributeBuffer,		Instance->Strands.RestResource->PointAttributeBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointToCurveBuffer,		Instance->Strands.RestResource->PointToCurveBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveBuffer,				Instance->Strands.RestResource->CurveBuffer);

		// If using groom deformer, replace point/curve attribute with their deformer version
		if (!bUseDebugCurveBuffer)
		{
			if (Instance->Strands.DeformedResource->DeformerCurveAttributeBuffer.Buffer) { CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveAttributeBuffer,	Instance->Strands.DeformedResource->DeformerCurveAttributeBuffer); }
			if (Instance->Strands.DeformedResource->DeformerPointAttributeBuffer.Buffer) { CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointAttributeBuffer,	Instance->Strands.DeformedResource->DeformerPointAttributeBuffer); }
		}

		OutVFInput.Strands.Common.PointCount = Instance->HairGroupPublicData->GetActiveStrandsPointCount();
		OutVFInput.Strands.Common.CurveCount = Instance->HairGroupPublicData->GetActiveStrandsCurveCount();
	}
	// 3. Render Strands in rest position: used when there are no skinning (rigid binding or no binding), no simulation, and no RBF
	else
	{
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionBuffer,			Instance->Strands.RestResource->PositionBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionBuffer,		Instance->Strands.RestResource->PositionBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.TangentBuffer,			Instance->Strands.RestResource->TangentBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveAttributeBuffer,		(ViewMode == EGroomViewMode::Cluster || ViewMode == EGroomViewMode::ClusterAABB) ? Instance->Strands.DebugCurveAttributeBuffer : Instance->Strands.RestResource->CurveAttributeBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointAttributeBuffer,		Instance->Strands.RestResource->PointAttributeBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PointToCurveBuffer,		Instance->Strands.RestResource->PointToCurveBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.CurveBuffer,				Instance->Strands.RestResource->CurveBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PositionOffsetBuffer,		Instance->Strands.RestResource->PositionOffsetBuffer);
		CONVERT_HAIRSSTRANDS_VF_PARAMETERS(OutVFInput.Strands.PrevPositionOffsetBuffer, Instance->Strands.RestResource->PositionOffsetBuffer);

		OutVFInput.Strands.Common.PositionOffset 	= FVector3f(Instance->Strands.RestResource->GetPositionOffset()); // LWC Precision loss
		OutVFInput.Strands.Common.PrevPositionOffset= FVector3f(Instance->Strands.RestResource->GetPositionOffset()); // LWC Precision loss
		OutVFInput.Strands.Common.PointCount 		= Instance->HairGroupPublicData->GetActiveStrandsPointCount();
		OutVFInput.Strands.Common.CurveCount 		= Instance->HairGroupPublicData->GetActiveStrandsCurveCount();
	}

	
	bool bRaytracingGeometry = false;
#if RHI_RAYTRACING
	// Flag bUseRaytracingGeometry only if RT geometry has been allocated for RayTracing view (not for PathTracing view). 
	// This flag is used later for selecting if voxelization needs to flags voxel has shadow caster or if shadow casting is handled 
	// by the RT geometry.
	bRaytracingGeometry = Instance->Strands.RenRaytracingResource != nullptr && EnumHasAnyFlags(Instance->Strands.ViewRayTracingMask, EHairViewRayTracingMask::RayTracing);
#endif

	// Scale hair radius based on the max. scaling factor from the instance's transform
	float InstanceScale = 1.f;
	if (GVarHairStrands_ScaleHairRadiusByTransform > 0)
	{
		const FVector3d S = Instance->LocalToWorld.GetScale3D();
		InstanceScale = FMath::Max3(S.X, S.Y, S.Z);
	}

	OutVFInput.Strands.Common.RegisteredIndex = Instance->RegisteredIndex;
	OutVFInput.Strands.Common.Radius = (GStrandHairWidth > 0 ? GStrandHairWidth : Instance->Strands.Modifier.HairWidth) * 0.5f;
	OutVFInput.Strands.Common.Radius *= Instance->HairGroupPublicData->GetActiveStrandsRadiusScale();
	OutVFInput.Strands.Common.Radius *= InstanceScale;
	OutVFInput.Strands.Common.RootScale = Instance->Strands.Modifier.HairRootScale;
	OutVFInput.Strands.Common.TipScale = Instance->Strands.Modifier.HairTipScale;
	OutVFInput.Strands.Common.RaytracingRadiusScale = (GHairRaytracingRadiusScale > 0 ? GHairRaytracingRadiusScale : Instance->Strands.Modifier.HairRaytracingRadiusScale);
	OutVFInput.Strands.Common.RaytracingRadiusScale /= InstanceScale; // In raytracing, the BLAS is automatically scaled by the instance transform, so cancel out the InstanceScale factor above to avoid applying it twice
	OutVFInput.Strands.Common.Length = Instance->Strands.Modifier.HairLength;
	OutVFInput.Strands.Common.Density = Instance->Strands.Modifier.HairShadowDensity;
	OutVFInput.Strands.Common.Flags = 0;
	OutVFInput.Strands.Common.Flags |= Instance->Strands.Modifier.bScatterSceneLighting ? HAIR_FLAGS_SCATTER_SCENE_LIGHT : 0u;
	OutVFInput.Strands.Common.Flags |= Instance->Strands.Modifier.bUseStableRasterization ? HAIR_FLAGS_STABLE_RASTER : 0u;
	OutVFInput.Strands.Common.Flags |= bRaytracingGeometry ? HAIR_FLAGS_RAYTRACING_GEOMETRY : 0u;
	OutVFInput.Strands.Common.Flags |= (Instance->Debug.Proxy && Instance->Debug.Proxy->Holdout()) ? HAIR_FLAGS_HOLDOUT : 0u;
	OutVFInput.Strands.Common.Flags |= (Instance->Debug.Proxy && Instance->Debug.Proxy->CastsContactShadow()) ? HAIR_FLAGS_CAST_CONTACT_SHADOW : 0u;
	OutVFInput.Strands.Common.RaytracingProceduralSplits = GetHairRaytracingProceduralSplits();
	OutVFInput.Strands.Common.GroupIndex = Instance->Debug.GroupIndex;
	OutVFInput.Strands.Common.GroupCount = Instance->Debug.GroupCount;
	OutVFInput.Strands.Common.bSimulation = (Instance->Guides.bIsSimulationEnable || Instance->Guides.bIsSimulationCacheEnable) ? 1 : 0;
	OutVFInput.Strands.Common.bSingleGuide = (Instance->Strands.InterpolationResource && (Instance->Strands.InterpolationResource->BulkData.Header.Flags & FHairStrandsInterpolationBulkData::DataFlags_HasSingleGuideData) != 0) ? 1 : 0;
	OutVFInput.Strands.Common.bCurveRemapping = (Instance->Strands.RestResource && (Instance->Strands.RestResource->BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasCurveMapping) != 0) ? 1 : 0;
	OutVFInput.Strands.Common.bPointRemapping = (Instance->Strands.RestResource && (Instance->Strands.RestResource->BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasPointMapping) != 0) ? 1 : 0;
	OutVFInput.Strands.Common.SourcePointCount = bUseGuideAttributeOffsets ? Instance->Guides.RestResource->GetSourcePointCount() : Instance->Strands.RestResource->GetSourcePointCount();
	OutVFInput.Strands.Common.SourceCurveCount = bUseGuideAttributeOffsets ? Instance->Guides.RestResource->GetSourceCurveCount() : Instance->Strands.RestResource->GetSourceCurveCount();

	// Add local transform assignement here ...

	if (bUseGuideAttributeOffsets)
	{
		GetHairStrandsAttributeParameter(Instance->Guides.GetData(), OutVFInput.Strands.Common.Attributes);
	}
	else
	{
		GetHairStrandsAttributeParameter(Instance->Strands.GetData(), OutVFInput.Strands.Common.Attributes);
	}

	return OutVFInput;
}

FHairGroupPublicData::FVertexFactoryInput ComputeHairStrandsVertexInputData(const FHairGroupInstance* Instance, EGroomViewMode ViewMode)
{
	return InternalComputeHairStrandsVertexInputData(nullptr, Instance, ViewMode);
}

void CreateHairStrandsDebugAttributeBuffer(FRDGBuilder& GraphBuilder, FRDGExternalBuffer* DebugAttributeBuffer, uint32 VertexCount, const FName& OwnerName);

EGroomCacheType GetHairInstanceCacheType(const FHairGroupInstance* Instance)
{
	const bool bHasValidGroomCacheBuffers = Instance->Debug.GroomCacheBuffers.IsValid() && 
		Instance->Debug.GroomCacheBuffers->GetCurrentFrameBuffer().GroupsData.IsValidIndex(Instance->Debug.GroupIndex) && 
		Instance->Debug.GroomCacheBuffers->GetNextFrameBuffer().GroupsData.IsValidIndex(Instance->Debug.GroupIndex);
	return bHasValidGroomCacheBuffers ? Instance->Debug.GroomCacheType : EGroomCacheType::None;
}