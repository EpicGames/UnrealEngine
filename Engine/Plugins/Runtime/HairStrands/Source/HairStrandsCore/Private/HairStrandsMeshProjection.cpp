// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsMeshProjection.h"
#include "MeshMaterialShader.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "CommonRenderResources.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "SkeletalRenderPublic.h"
#include "RenderGraphUtils.h"
#include "GroomResources.h"
#include "SystemTextures.h"
#include "GroomBindingBuilder.h"

static int32 GHairProjectionMaxTrianglePerProjectionIteration = 8;
static FAutoConsoleVariableRef CVarHairProjectionMaxTrianglePerProjectionIteration(TEXT("r.HairStrands.Projection.MaxTrianglePerIteration"), GHairProjectionMaxTrianglePerProjectionIteration, TEXT("Change the number of triangles which are iterated over during one projection iteration step. In kilo triangle (e.g., 8 == 8000 triangles). Default is 8."));

static int32 GHairStrandsUseGPUPositionOffset = 1;
static FAutoConsoleVariableRef CVarHairStrandsUseGPUPositionOffset(TEXT("r.HairStrands.UseGPUPositionOffset"), GHairStrandsUseGPUPositionOffset, TEXT("Use GPU position offset to improve hair strands position precision."));

///////////////////////////////////////////////////////////////////////////////////////////////////

#define MAX_HAIRSTRANDS_SECTION_COUNT 255
#define MAX_HAIRSTRANDS_SECTION_BITOFFSET 24

uint32 GetHairStrandsMaxSectionCount()
{
	return MAX_HAIRSTRANDS_SECTION_COUNT;
}

uint32 GetHairStrandsMaxTriangleCount()
{
	return (1 << MAX_HAIRSTRANDS_SECTION_BITOFFSET) - 1;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FSkinUpdateCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSkinUpdateCS);
	SHADER_USE_PARAMETER_STRUCT(FSkinUpdateCS, FGlobalShader);

	
	class FPrevious : SHADER_PERMUTATION_BOOL("PERMUTATION_PREV");
	class FUnlimitedBoneInfluence : SHADER_PERMUTATION_BOOL("GPUSKIN_UNLIMITED_BONE_INFLUENCE");
	class FUseExtraInfluence : SHADER_PERMUTATION_BOOL("GPUSKIN_USE_EXTRA_INFLUENCES");
	class FBoneIndexUint16 : SHADER_PERMUTATION_BOOL("GPUSKIN_BONE_INDEX_UINT16");
	class FBoneWeightUint16 : SHADER_PERMUTATION_BOOL("GPUSKIN_BONE_WEIGHTS_UINT16");	
	using FPermutationDomain = TShaderPermutationDomain<FUnlimitedBoneInfluence, FUseExtraInfluence, FBoneIndexUint16, FBoneWeightUint16, FPrevious>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, bWriteTangent)
		SHADER_PARAMETER(uint32, NumVertexToProcess)
		SHADER_PARAMETER(uint32, NumTotalVertices)
		SHADER_PARAMETER(uint32, SectionVertexBaseIndex)
		SHADER_PARAMETER(uint32, WeightIndexSize)
		SHADER_PARAMETER(uint32, WeightStride)
		SHADER_PARAMETER(uint32, BoneTransformOffset)
		SHADER_PARAMETER(uint32, PrevBoneTransformOffset)
		SHADER_PARAMETER_SRV(Buffer<uint>, WeightLookup)
		SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(BoneMatrices)
		SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(PrevBoneMatrices)
		SHADER_PARAMETER_SRV(Buffer<uint>, VertexWeights)
		SHADER_PARAMETER_SRV(Buffer<float>, RestPositions)
		SHADER_PARAMETER_SRV(Buffer<float4>, RestTangents)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, DeformedPositions)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, PrevDeformedPositions)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, DeformedTangents)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 64; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SKIN_UPDATE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FSkinUpdateCS, "/Engine/Private/HairStrands/HairStrandsSkinUpdate.usf", "UpdateSkinPositionCS", SF_Compute);

void AddSkinUpdatePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FSkeletalMeshLODRenderData& RenderData,
	const TArray<FSkinUpdateSection>& Sections,
	FRDGBufferRef OutDeformedPositionBuffer,
	FRDGBufferRef OutPrevDeformedPositionBuffer,
	FRDGBufferRef OutDeformedTangentBuffer)
{
	check(Sections.Num() > 0);
	const FSkinWeightVertexBuffer* SkinWeight = &RenderData.SkinWeightVertexBuffer;
	const bool bPrevPosition = OutPrevDeformedPositionBuffer != nullptr && Sections[0].BonePrevBuffer;
	
	bool bHasTangentSRV = false;
	FRDGBufferUAVRef DeformedTangentsUAV = nullptr;
	if (OutDeformedTangentBuffer)
	{
		const FRHIShaderResourceView* TangentSRV = RenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
		check(TangentSRV);
		const EPixelFormat TangentFormat = TangentSRV->GetDesc().Buffer.SRV.Format;
		DeformedTangentsUAV = GraphBuilder.CreateUAV(OutDeformedTangentBuffer, TangentFormat, ERDGUnorderedAccessViewFlags::SkipBarrier);
		bHasTangentSRV = true;
	}
	else
	{
		// Dummy output
		DeformedTangentsUAV = GraphBuilder.CreateUAV(GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4u, 1), TEXT("DummyUAV")), PF_R8G8B8A8_SNORM, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	FSkinUpdateCS::FParameters* Parameters = GraphBuilder.AllocParameters<FSkinUpdateCS::FParameters>();
	Parameters->bWriteTangent			= bHasTangentSRV ? 1 : 0;
	Parameters->WeightIndexSize 		= SkinWeight->GetBoneIndexByteSize() | (SkinWeight->GetBoneWeightByteSize() << 8);
	Parameters->NumVertexToProcess 		= 0;
	Parameters->NumTotalVertices		= RenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	Parameters->SectionVertexBaseIndex 	= 0;
	Parameters->WeightStride 			= SkinWeight->GetConstantInfluencesVertexStride();
	Parameters->WeightLookup 			= SkinWeight->GetLookupVertexBuffer()->GetSRV();
	Parameters->BoneMatrices 			= Sections[0].BoneBuffer;
	Parameters->PrevBoneMatrices 		= Sections[0].BonePrevBuffer;
	Parameters->VertexWeights 			= SkinWeight->GetDataVertexBuffer()->GetSRV();
	Parameters->RestPositions 			= RenderData.StaticVertexBuffers.PositionVertexBuffer.GetSRV();
	Parameters->RestTangents 			= RenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentsSRV();
	Parameters->DeformedPositions 		= GraphBuilder.CreateUAV(OutDeformedPositionBuffer, PF_R32_FLOAT, ERDGUnorderedAccessViewFlags::SkipBarrier);
	Parameters->DeformedTangents 		= DeformedTangentsUAV;
	if (bPrevPosition)
	{
		Parameters->PrevDeformedPositions = GraphBuilder.CreateUAV(OutPrevDeformedPositionBuffer, PF_R32_FLOAT, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	// Set dummy tangent if not available and not used by the shaders
	if (!Parameters->bWriteTangent && Parameters->RestTangents == nullptr)
	{
		Parameters->RestTangents = Parameters->RestPositions;
	}

	FSkinUpdateCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSkinUpdateCS::FUnlimitedBoneInfluence>(SkinWeight->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence);
	PermutationVector.Set<FSkinUpdateCS::FUseExtraInfluence>(SkinWeight->GetMaxBoneInfluences() > MAX_INFLUENCES_PER_STREAM);
	PermutationVector.Set<FSkinUpdateCS::FBoneIndexUint16 >(SkinWeight->Use16BitBoneIndex());
	PermutationVector.Set<FSkinUpdateCS::FBoneWeightUint16 >(SkinWeight->Use16BitBoneWeight());
	PermutationVector.Set<FSkinUpdateCS::FPrevious>(bPrevPosition);
	TShaderMapRef<FSkinUpdateCS> ComputeShader(ShaderMap, PermutationVector);

	const FShaderParametersMetadata* ParametersMetadata = FSkinUpdateCS::FParameters::FTypeInfo::GetStructMetadata();
	ClearUnusedGraphResources(ComputeShader, ParametersMetadata, Parameters);
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::UpdateSkinPosition(%s)", bPrevPosition ? TEXT("Curr,Prev") : TEXT("Curr")),
		ParametersMetadata,
		Parameters,
		ERDGPassFlags::Compute,
		[ParametersMetadata, Parameters, ComputeShader, Sections, bPrevPosition](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			for (const FSkinUpdateSection& Section : Sections)
			{
				Parameters->NumVertexToProcess 		= Section.NumVertexToProcess;
				Parameters->SectionVertexBaseIndex 	= Section.SectionVertexBaseIndex;
				Parameters->BoneMatrices 			= Section.BoneBuffer;
				Parameters->BoneTransformOffset		= Section.BoneTransformOffset;
				check(Parameters->BoneMatrices);
				if (bPrevPosition)
				{
					Parameters->PrevBoneMatrices = Section.BonePrevBuffer;
					Parameters->PrevBoneTransformOffset = Section.PrevBoneTransformOffset;
					check(Parameters->PrevBoneMatrices);
				}

				const FIntVector DispatchGroupCount = FComputeShaderUtils::GetGroupCount(Parameters->NumVertexToProcess, FSkinUpdateCS::GetGroupSize());
				check(DispatchGroupCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, ParametersMetadata, *Parameters, DispatchGroupCount);
			}
		});
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairUpdateMeshTriangleCS : public FGlobalShader
{
private:
	DECLARE_GLOBAL_SHADER(FHairUpdateMeshTriangleCS);
	SHADER_USE_PARAMETER_STRUCT(FHairUpdateMeshTriangleCS, FGlobalShader);

	class FPositionType : SHADER_PERMUTATION_BOOL("PERMUTATION_POSITION_TYPE");
	class FPrevious : SHADER_PERMUTATION_BOOL("PERMUTATION_PREVIOUS"); 
	using FPermutationDomain = TShaderPermutationDomain<FPositionType, FPrevious>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, TangentFormat)
		SHADER_PARAMETER(uint32, MaxSectionCount)
		SHADER_PARAMETER(uint32, MaxUniqueTriangleCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, MeshSectionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RDGMeshPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RDGMeshPreviousPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RDGMeshTangentBuffer)
		SHADER_PARAMETER_SRV(Buffer, MeshPositionBuffer)
		SHADER_PARAMETER_SRV(Buffer, MeshPreviousPositionBuffer)
		SHADER_PARAMETER_SRV(Buffer, MeshIndexBuffer)
		SHADER_PARAMETER_SRV(Buffer, MeshTangentBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, UniqueTriangleIndices)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutUniqueTrianglePrevPosition)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutUniqueTriangleCurrPosition)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 128; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_MESH_UPDATE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairUpdateMeshTriangleCS, "/Engine/Private/HairStrands/HairStrandsMesh.usf", "MainCS", SF_Compute);

static bool AddHairStrandUpdateMeshTrianglesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 UniqueTriangleCount, 
	const uint32 TotalSectionCount, // Total section count of the underlying mesh
	const TArray<uint32>& UniqueMeshSectionIndices,
	const FCachedGeometry& MeshLODData,
	FRDGBufferSRVRef UniqueTriangleIndexSRV,
	FRDGBufferUAVRef OutputCurrUAV,
	FRDGBufferUAVRef OutputPrevUAV)
{
	const uint32 EffectiveSectionCount 	= MeshLODData.Sections.Num(); // Section count which are valid & used by groom data

	// If a skel. mesh is not streaming yet, its SRV will be null
	const bool bValid = EffectiveSectionCount > 0 && TotalSectionCount < GetHairStrandsMaxSectionCount();
	const bool bReady = MeshLODData.Sections[0].IndexBuffer != nullptr && (MeshLODData.Sections[0].RDGPositionBuffer != nullptr || MeshLODData.Sections[0].PositionBuffer != nullptr);
	if (!bReady || !bValid)
	{
		return false;
	}

	FHairUpdateMeshTriangleCS::FParameters CommonParameters;
	CommonParameters.MaxUniqueTriangleCount 		= UniqueTriangleCount;
	CommonParameters.MaxSectionCount 				= TotalSectionCount;
	CommonParameters.TangentFormat					= MeshLODData.Sections[0].TangentFormat;
	CommonParameters.RDGMeshPositionBuffer			= MeshLODData.Sections[0].RDGPositionBuffer;
	CommonParameters.RDGMeshPreviousPositionBuffer 	= MeshLODData.Sections[0].RDGPreviousPositionBuffer;
	CommonParameters.RDGMeshTangentBuffer			= MeshLODData.Sections[0].RDGTangentBuffer;
	CommonParameters.MeshPositionBuffer				= MeshLODData.Sections[0].PositionBuffer;
	CommonParameters.MeshPreviousPositionBuffer		= MeshLODData.Sections[0].PreviousPositionBuffer;
	CommonParameters.MeshIndexBuffer				= MeshLODData.Sections[0].IndexBuffer;
	CommonParameters.MeshTangentBuffer				= MeshLODData.Sections[0].TangentBuffer;
	CommonParameters.UniqueTriangleIndices 			= UniqueTriangleIndexSRV;
	CommonParameters.OutUniqueTriangleCurrPosition	= OutputCurrUAV;
	CommonParameters.OutUniqueTrianglePrevPosition	= OutputPrevUAV;

	const bool bUseRDGPositionBuffer = CommonParameters.RDGMeshPositionBuffer != nullptr;

	struct FSectionData
	{
		uint32 TotalIndexCount;
		uint32 TotalVertexCount;
		uint32 IndexBaseIndex;
		uint32 UVsChannelOffset : 8;
		uint32 UVsChannelCount : 8;
		uint32 bIsSwapped : 1;
		uint32 bUseFaceNormal : 1;
		uint32 Pad : 14;
	};

	FSectionData Default;
	Default.TotalIndexCount = 0;
	Default.TotalVertexCount = 0;
	Default.IndexBaseIndex = 0;
	Default.UVsChannelOffset  = 0;
	Default.UVsChannelCount  = 0;
	Default.bIsSwapped = 0;
	Default.bUseFaceNormal = 0;
	Default.Pad = 0;

	// Allocate data for *all* sections, but only fill in the used/valid sections
	TArray<FSectionData> SectionDatas;
	SectionDatas.Init(Default, TotalSectionCount); 
	for (uint32 MeshSectionIndex : UniqueMeshSectionIndices)
	{
		// MeshLODData.Sections can contains only a sparse set of sections (vs. all sections). 
		// Thus, the section's index needs to be remapped
		const int32 RemappedSectionIndex = MeshLODData.Sections.FindLastByPredicate(
			[MeshSectionIndex](const FCachedGeometry::Section& In) 
			{ 
				return In.SectionIndex == MeshSectionIndex; 
			});
		if(RemappedSectionIndex != INDEX_NONE)
		{ 
			const FCachedGeometry::Section& MeshSectionData = MeshLODData.Sections[RemappedSectionIndex];
			const uint32 SectionIndex = MeshSectionData.SectionIndex;
			SectionDatas[SectionIndex].TotalIndexCount	= MeshSectionData.TotalIndexCount;
			SectionDatas[SectionIndex].TotalVertexCount	= MeshSectionData.TotalVertexCount;
			SectionDatas[SectionIndex].IndexBaseIndex	= MeshSectionData.IndexBaseIndex;
			SectionDatas[SectionIndex].UVsChannelOffset	= MeshSectionData.UVsChannelOffset;
			SectionDatas[SectionIndex].UVsChannelCount	= MeshSectionData.UVsChannelCount;
			SectionDatas[SectionIndex].Pad				= 0u;
			SectionDatas[SectionIndex].bIsSwapped		= bUseRDGPositionBuffer ? 
				(MeshSectionData.RDGPositionBuffer != CommonParameters.RDGMeshPositionBuffer ? 1u : 0u) :
				(MeshSectionData.PositionBuffer    != CommonParameters.MeshPositionBuffer    ? 1u : 0u);
			SectionDatas[SectionIndex].bUseFaceNormal	= 0u;

			// Sanity check
			check(MeshSectionData.UVsChannelOffset < 255);
			check(MeshSectionData.UVsChannelCount < 255);
			check(CommonParameters.RDGMeshPositionBuffer == MeshSectionData.RDGPositionBuffer || CommonParameters.RDGMeshPositionBuffer == MeshSectionData.RDGPreviousPositionBuffer);
			check(CommonParameters.MeshPositionBuffer    == MeshSectionData.PositionBuffer    || CommonParameters.MeshPositionBuffer    == MeshSectionData.PreviousPositionBuffer);
			check(CommonParameters.MeshIndexBuffer		 == MeshSectionData.IndexBuffer);
			check(CommonParameters.MeshTangentBuffer	 == MeshSectionData.TangentBuffer);
		}
	}

	// If no previous position buffer available, reusing the current position buffers
	if (CommonParameters.MeshPreviousPositionBuffer == nullptr)
	{
		CommonParameters.MeshPreviousPositionBuffer = CommonParameters.MeshPositionBuffer;
	}
	if (CommonParameters.RDGMeshPreviousPositionBuffer == nullptr)
	{
		CommonParameters.RDGMeshPreviousPositionBuffer = CommonParameters.RDGMeshPositionBuffer;
	}

	// On some platform the TangentSRV can be null as it is not created by default (requires CPU access flags).
	// For the RDG path the TangentSRV can also be null on platforms that do not support manual vertex fetch or enable gpu skin cache.
	// In such cases we bind any valid buffer as a dummy data (not sampled), and we use face normal.
	if (CommonParameters.MeshTangentBuffer == nullptr || CommonParameters.RDGMeshTangentBuffer == nullptr)
	{
		bool bUseFaceNormal = false;

		if (CommonParameters.MeshTangentBuffer == nullptr)
		{
			CommonParameters.MeshTangentBuffer = CommonParameters.MeshIndexBuffer;
			bUseFaceNormal = bUseFaceNormal || (bUseRDGPositionBuffer == false);
		}
		if (CommonParameters.RDGMeshTangentBuffer == nullptr)
		{
			CommonParameters.RDGMeshTangentBuffer = CommonParameters.UniqueTriangleIndices;
			bUseFaceNormal = bUseFaceNormal || (bUseRDGPositionBuffer == true);
		}

		if (bUseFaceNormal)
		{
			for (FSectionData& SectionData : SectionDatas)
			{
				SectionData.bUseFaceNormal = 1u;
			}
		}
	}

	FRDGBufferRef SectionBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("Hair.SkelMeshSectionBuffer"), sizeof(FSectionData),  SectionDatas.Num(), SectionDatas.GetData(), sizeof(FSectionData) * SectionDatas.Num());
	CommonParameters.MeshSectionBuffer = GraphBuilder.CreateSRV(SectionBuffer);

	const bool bComputePreviousDeformedPosition = OutputPrevUAV != nullptr;
	{
		FHairUpdateMeshTriangleCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairUpdateMeshTriangleCS::FParameters>();
		*PassParameters = CommonParameters;

		FHairUpdateMeshTriangleCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHairUpdateMeshTriangleCS::FPrevious>(bComputePreviousDeformedPosition);
		PermutationVector.Set<FHairUpdateMeshTriangleCS::FPositionType>(bUseRDGPositionBuffer ? 1 : 0);

		const FIntVector DispatchGroupCount = FComputeShaderUtils::GetGroupCount(UniqueTriangleCount, FHairUpdateMeshTriangleCS::GetGroupSize());
		check(DispatchGroupCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
		TShaderMapRef<FHairUpdateMeshTriangleCS> ComputeShader(ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::TriangleMeshUpdate(%s,%s)", bComputePreviousDeformedPosition ? TEXT("Previous/Current, Current") : TEXT("Current"), bUseRDGPositionBuffer ? TEXT("RDGBuffer") : TEXT("SkinCacheBuffer")),
			ComputeShader,
			PassParameters,
			DispatchGroupCount);
	}

	return true;
}

void AddHairStrandUpdateMeshTrianglesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	const FCachedGeometry& MeshLODData,
	FHairStrandsRestRootResource* RestResources,
	FHairStrandsDeformedRootResource* DeformedResources)
{	
	if (RestResources->GetRootCount() == 0 || !RestResources->IsValid(MeshLODIndex))
	{
		return;
	}

	FHairStrandsLODRestRootResource& RestLODData = *RestResources->GetLOD(MeshLODIndex);
	check(RestLODData.GetBulkData().Header.LODIndex == MeshLODIndex);
	const uint32 UniqueTriangleCount = RestLODData.GetBulkData().Header.UniqueTriangleCount;
	const uint32 TotalMeshSectionCount = RestLODData.GetBulkData().Header.MeshSectionCount;
	if (UniqueTriangleCount == 0)
	{
		return;
	}

	FRDGImportedBuffer OutputCurrBuffer = Register(GraphBuilder, DeformedResources->GetLOD(MeshLODIndex)->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateUAV, ERDGUnorderedAccessViewFlags::SkipBarrier);
	FRDGImportedBuffer OutputPrevBuffer;
	const bool bComputePreviousDeformedPosition = IsHairStrandContinuousDecimationReorderingEnabled();
	if (bComputePreviousDeformedPosition)
	{
		OutputPrevBuffer = Register(GraphBuilder, DeformedResources->GetLOD(MeshLODIndex)->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Previous), ERDGImportedBufferFlags::CreateUAV, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	if (AddHairStrandUpdateMeshTrianglesPass(
			GraphBuilder,
			ShaderMap,
			UniqueTriangleCount,
			TotalMeshSectionCount,
			RestLODData.GetBulkData().Header.UniqueSectionIndices,
			MeshLODData,
			RegisterAsSRV(GraphBuilder, RestLODData.UniqueTriangleIndexBuffer),
			OutputCurrBuffer.UAV,
			OutputPrevBuffer.UAV))
	{

		GraphBuilder.SetBufferAccessFinal(OutputCurrBuffer.Buffer, ERHIAccess::SRVMask);
		if (OutputPrevBuffer.Buffer)
		{
			GraphBuilder.SetBufferAccessFinal(OutputPrevBuffer.Buffer, ERHIAccess::SRVMask);
		}
	
		// Update the last known mesh LOD for which the root resources has been updated
		DeformedResources->GetLOD(MeshLODIndex)->Status = FHairStrandsLODDeformedRootResource::EStatus::Completed;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairMeshesInterpolateCS : public FGlobalShader
{
private:
	DECLARE_GLOBAL_SHADER(FHairMeshesInterpolateCS);
	SHADER_USE_PARAMETER_STRUCT(FHairMeshesInterpolateCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER(uint32, RBFLocalSpace)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, RestSamplePositionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, MeshSampleWeightsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeformedSamplePositionsBuffer)

		SHADER_PARAMETER_SRV(Buffer, RestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutDeformedPositionBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HAIRMESHES"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairMeshesInterpolateCS, "/Engine/Private/HairStrands/HairStrandsMesh.usf", "MainHairMeshesCS", SF_Compute);

template<typename TRestResource, typename TDeformedResource>
void InternalAddHairRBFInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const bool bCards,
	const int32 MeshLODIndex,
	TRestResource* RestResources,
	TDeformedResource* DeformedResources,
	FHairStrandsRestRootResource* RestRootResources,
	FHairStrandsDeformedRootResource* DeformedRootResources)
{
	const uint32 VertexCount = RestResources ? RestResources->GetVertexCount() : 0;
	if (!RestResources || !DeformedResources || !RestRootResources || !DeformedRootResources || VertexCount == 0 || MeshLODIndex >= RestRootResources->LODs.Num() || MeshLODIndex < 0)
	{
		return;
	}

	// Copy current to previous position before update the current position. This allows to have correct motion vectors.
	FRDGImportedBuffer DeformedPositionBuffer_Curr = Register(GraphBuilder, DeformedResources->GetBuffer(TDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateUAV);
	FRDGImportedBuffer DeformedPositionBuffer_Prev = Register(GraphBuilder, DeformedResources->GetBuffer(TDeformedResource::EFrameType::Previous), ERDGImportedBufferFlags::CreateUAV);

	// Take buffers out of external access mode so the copy + dispatch passes below can manage transitions.
	GraphBuilder.UseInternalAccessMode({ DeformedPositionBuffer_Curr.Buffer, DeformedPositionBuffer_Prev.Buffer });

	AddCopyBufferPass(GraphBuilder, DeformedPositionBuffer_Prev.Buffer, DeformedPositionBuffer_Curr.Buffer);

	FHairStrandsLODRestRootResource& RestLODData = *RestRootResources->GetLOD(MeshLODIndex);
	FHairStrandsLODDeformedRootResource& DeformedLODData = *DeformedRootResources->GetLOD(MeshLODIndex);
	FHairMeshesInterpolateCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairMeshesInterpolateCS::FParameters>();
	
	Parameters->VertexCount = VertexCount;
	Parameters->MaxSampleCount = RestLODData.SampleCount;
	Parameters->RBFLocalSpace = UE::Groom::IsRBFLocalSpaceEnabled();

	Parameters->RestPositionBuffer			= RestResources->RestPositionBuffer.ShaderResourceViewRHI;
	Parameters->OutDeformedPositionBuffer	= DeformedPositionBuffer_Curr.UAV;

	Parameters->RestSamplePositionsBuffer	= RegisterAsSRV(GraphBuilder, RestLODData.RestSamplePositionsBuffer);
	Parameters->MeshSampleWeightsBuffer		= RegisterAsSRV(GraphBuilder, DeformedLODData.GetMeshSampleWeightsBuffer(FHairStrandsLODDeformedRootResource::Current));
	Parameters->DeformedSamplePositionsBuffer	= RegisterAsSRV(GraphBuilder, DeformedLODData.GetDeformedSamplePositionsBuffer(FHairStrandsLODDeformedRootResource::Current));

	const FIntVector DispatchGroupCount = FComputeShaderUtils::GetGroupCount(VertexCount, 128);
	check(DispatchGroupCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
	TShaderMapRef<FHairMeshesInterpolateCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::HairInterpolationRBF(%s)", bCards ? TEXT("Cards") : TEXT("Meshes")),
		ComputeShader,
		Parameters,
		DispatchGroupCount);

	// Return buffers to external access mode so subsequent non-RDG vertex-factory binds (raw SRVs in static UBs) see them in SRV state.
	GraphBuilder.UseExternalAccessMode({ DeformedPositionBuffer_Curr.Buffer, DeformedPositionBuffer_Prev.Buffer }, ERHIAccess::SRVMask);
}

void AddHairMeshesRBFInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	FHairMeshesRestResource* RestResources,
	FHairMeshesDeformedResource* DeformedResources,
	FHairStrandsRestRootResource* RestRootResources,
	FHairStrandsDeformedRootResource* DeformedRootResources)
{
	InternalAddHairRBFInterpolationPass(
		GraphBuilder,
		ShaderMap,
		false,
		MeshLODIndex,
		RestResources,
		DeformedResources,
		RestRootResources,
		DeformedRootResources);
}

void AddHairCardsRBFInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	FHairCardsRestResource* RestResources,
	FHairCardsDeformedResource* DeformedResources,
	FHairStrandsRestRootResource* RestRootResources,
	FHairStrandsDeformedRootResource* DeformedRootResources)
{
	InternalAddHairRBFInterpolationPass(
		GraphBuilder,
		ShaderMap,
		true,
		MeshLODIndex,
		RestResources,
		DeformedResources,
		RestRootResources,
		DeformedRootResources);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
//
class FHairInitMeshSamplesCS : public FGlobalShader
{
public:
private:
	DECLARE_GLOBAL_SHADER(FHairInitMeshSamplesCS);
	SHADER_USE_PARAMETER_STRUCT(FHairInitMeshSamplesCS, FGlobalShader);

	class FPositionType : SHADER_PERMUTATION_INT("PERMUTATION_POSITION_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FPositionType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MaxSectionCount)
		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER(uint32, MaxVertexCount)

		SHADER_PARAMETER_SRV(Buffer, MeshPositionBuffer0)
		SHADER_PARAMETER_SRV(Buffer, MeshPositionBuffer1)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RDGMeshPositionBuffer0)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RDGMeshPositionBuffer1)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SampleIndicesAndSectionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, MeshSectionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutSamplePositionsBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 128; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SAMPLE_INIT"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairInitMeshSamplesCS, "/Engine/Private/HairStrands/HairStrandsMesh.usf", "MainCS", SF_Compute);

void AddHairStrandInitMeshSamplesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 LODIndex,
	const FCachedGeometry& MeshData,
	FHairStrandsRestRootResource* RestResources,
	FHairStrandsDeformedRootResource* DeformedResources)
{
	if (!RestResources->IsValid(LODIndex))
	{
		return;
	}
	
	FHairStrandsLODRestRootResource& RestLODData = *RestResources->GetLOD(LODIndex);

	const uint32 TotalSectionCount = RestLODData.GetBulkData().Header.MeshSectionCount;
	const uint32 UniqueSectionIndicesCount = RestLODData.GetBulkData().Header.UniqueSectionIndices.Num();
	const uint32 EffectiveSectionCount = MeshData.Sections.Num();
	if (EffectiveSectionCount == 0 || RestLODData.SampleCount == 0 || UniqueSectionIndicesCount == 0)
	{
		return;
	}

	FRDGImportedBuffer OutBuffer = Register(GraphBuilder, DeformedResources->GetLOD(LODIndex)->GetDeformedSamplePositionsBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateUAV);
	
	FHairInitMeshSamplesCS::FParameters CommonParameters;
	CommonParameters.MaxSampleCount 				= RestLODData.SampleCount;
	CommonParameters.MaxSectionCount 				= TotalSectionCount;
	CommonParameters.MaxVertexCount 				= MeshData.Sections[0].TotalVertexCount;
	CommonParameters.RDGMeshPositionBuffer0			= MeshData.Sections[0].RDGPositionBuffer;
	CommonParameters.RDGMeshPositionBuffer1		 	= MeshData.Sections[0].RDGPreviousPositionBuffer;
	CommonParameters.MeshPositionBuffer0			= MeshData.Sections[0].PositionBuffer;
	CommonParameters.MeshPositionBuffer1			= MeshData.Sections[0].PreviousPositionBuffer;
	CommonParameters.SampleIndicesAndSectionsBuffer = RegisterAsSRV(GraphBuilder, RestLODData.MeshSampleIndicesAndSectionsBuffer);
	CommonParameters.OutSamplePositionsBuffer 		= OutBuffer.UAV;

	const bool bUseRDGPositionBuffer = CommonParameters.RDGMeshPositionBuffer0 != nullptr;

	// Create buffer with section data for look up in shader
	struct FSectionData
	{
		uint32 bIsSwapped = 0;
	};
	TArray<FSectionData> SectionDatas;
	SectionDatas.Init(FSectionData(), FMath::Max(uint32(MeshData.Sections.Num()), TotalSectionCount));
	for (const FCachedGeometry::Section& Section : MeshData.Sections)
	{
		const uint32 SectionIndex = Section.SectionIndex;
		SectionDatas[SectionIndex].bIsSwapped = 
			bUseRDGPositionBuffer ?
			(Section.RDGPositionBuffer != CommonParameters.RDGMeshPositionBuffer0 ? 1u : 0u) :
			(Section.PositionBuffer    != CommonParameters.MeshPositionBuffer0    ? 1u : 0u);

		// Sanity check
		check(CommonParameters.RDGMeshPositionBuffer0 == Section.RDGPositionBuffer || CommonParameters.RDGMeshPositionBuffer0 == Section.RDGPreviousPositionBuffer);
		check(CommonParameters.MeshPositionBuffer0    == Section.PositionBuffer    || CommonParameters.MeshPositionBuffer0    == Section.PreviousPositionBuffer);
	}
	CommonParameters.MeshSectionBuffer = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("Hair.SkelMeshSectionBuffer"), sizeof(FSectionData),  SectionDatas.Num(), SectionDatas.GetData(), sizeof(FSectionData) * SectionDatas.Num()));

	// If no previous position buffer available, reusing the current position buffers
	if (CommonParameters.MeshPositionBuffer1 == nullptr)
	{
		CommonParameters.MeshPositionBuffer1 = CommonParameters.MeshPositionBuffer0;
	}
	if (CommonParameters.RDGMeshPositionBuffer1 == nullptr)
	{
		CommonParameters.RDGMeshPositionBuffer1 = CommonParameters.RDGMeshPositionBuffer0;
	}

	FHairInitMeshSamplesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairInitMeshSamplesCS::FParameters>();
	*PassParameters = CommonParameters;

	FHairInitMeshSamplesCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairInitMeshSamplesCS::FPositionType>(bUseRDGPositionBuffer ? 1 : 0);

	const FIntVector DispatchGroupCount = FComputeShaderUtils::GetGroupCount(CommonParameters.MaxSampleCount, FHairInitMeshSamplesCS::GetGroupSize());
	check(DispatchGroupCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
	TShaderMapRef<FHairInitMeshSamplesCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::InitMeshSamples"),
		ComputeShader,
		PassParameters,
		DispatchGroupCount);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
//
class FHairUpdateMeshSamplesCS : public FGlobalShader
{
private:
	DECLARE_GLOBAL_SHADER(FHairUpdateMeshSamplesCS);
	SHADER_USE_PARAMETER_STRUCT(FHairUpdateMeshSamplesCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER(uint32, RBFLocalSpace)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InterpolationWeightsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, SampleRestPositionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, SampleDeformedPositionsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutSampleDeformationsBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SAMPLE_UPDATE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairUpdateMeshSamplesCS, "/Engine/Private/HairStrands/HairStrandsMesh.usf", "MainCS", SF_Compute);

void AddHairStrandUpdateMeshSamplesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 LODIndex,
	const FCachedGeometry& MeshData,
	FHairStrandsRestRootResource* RestResources,
	FHairStrandsDeformedRootResource* DeformedResources)
{
	if (LODIndex < 0 || !RestResources->IsValid(LODIndex) || !DeformedResources->IsValid(LODIndex))
	{
		return;
	}

	FHairStrandsLODRestRootResource& RestLODData = *RestResources->GetLOD(LODIndex);
	FHairStrandsLODDeformedRootResource& DeformedLODData = *DeformedResources->GetLOD(LODIndex);

	const uint32 SectionCount = MeshData.Sections.Num();
	if (SectionCount > 0 && RestLODData.SampleCount > 0)
	{
		FHairUpdateMeshSamplesCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairUpdateMeshSamplesCS::FParameters>();

		FRDGImportedBuffer OutWeightsBuffer = Register(GraphBuilder, DeformedLODData.GetMeshSampleWeightsBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateUAV);

		Parameters->MaxSampleCount					= RestLODData.SampleCount;
		Parameters->InterpolationWeightsBuffer		= RegisterAsSRV(GraphBuilder, RestLODData.MeshInterpolationWeightsBuffer);
		Parameters->SampleRestPositionsBuffer		= RegisterAsSRV(GraphBuilder, RestLODData.RestSamplePositionsBuffer);
		Parameters->SampleDeformedPositionsBuffer	= RegisterAsSRV(GraphBuilder, DeformedLODData.GetDeformedSamplePositionsBuffer(FHairStrandsLODDeformedRootResource::Current));
		Parameters->OutSampleDeformationsBuffer		= OutWeightsBuffer.UAV;
		Parameters->RBFLocalSpace = UE::Groom::IsRBFLocalSpaceEnabled();

		const FIntVector DispatchGroupCount = FComputeShaderUtils::GetGroupCount(RestLODData.SampleCount+4, 128);
		check(DispatchGroupCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
		TShaderMapRef<FHairUpdateMeshSamplesCS> ComputeShader(ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::UpdateMeshSamples"),
			ComputeShader,
			Parameters,
			DispatchGroupCount);

		GraphBuilder.SetBufferAccessFinal(OutWeightsBuffer.Buffer, ERHIAccess::SRVMask);
	}
}


///////////////////////////////////////////////////////////////////////////////////////////////////

// Generate follicle mask texture
BEGIN_SHADER_PARAMETER_STRUCT(FHairFollicleMaskParameters, )
	SHADER_PARAMETER(FVector2f, OutputResolution)
	SHADER_PARAMETER(uint32, MaxRootCount)
	SHADER_PARAMETER(uint32, MaxUniqueTriangleIndex)
	SHADER_PARAMETER(uint32, Channel)
	SHADER_PARAMETER(uint32, KernelSizeInPixels)

	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, UniqueTrianglePositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootToUniqueTriangleIndexBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootBarycentricBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootUVsBuffer)

	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FHairFollicleMask : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_FOLLICLE_MASK"), 1);
	}

	FHairFollicleMask() = default;
	FHairFollicleMask(const CompiledShaderInitializerType& Initializer) : FGlobalShader(Initializer) {}
};

class FHairFollicleMaskVS : public FHairFollicleMask
{
	DECLARE_GLOBAL_SHADER(FHairFollicleMaskVS);
	SHADER_USE_PARAMETER_STRUCT(FHairFollicleMaskVS, FHairFollicleMask);

	class FUVType : SHADER_PERMUTATION_INT("PERMUTATION_UV_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FUVType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairFollicleMaskParameters, Pass)
	END_SHADER_PARAMETER_STRUCT()
};

class FHairFollicleMaskPS : public FHairFollicleMask
{
	DECLARE_GLOBAL_SHADER(FHairFollicleMaskPS);
	SHADER_USE_PARAMETER_STRUCT(FHairFollicleMaskPS, FHairFollicleMask);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairFollicleMaskParameters, Pass)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FHairFollicleMaskPS, "/Engine/Private/HairStrands/HairStrandsFollicleMask.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FHairFollicleMaskVS, "/Engine/Private/HairStrands/HairStrandsFollicleMask.usf", "MainVS", SF_Vertex);

static void AddFollicleMaskPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const bool bNeedClear,
	const uint32 KernelSizeInPixels,
	const uint32 Channel,
	const uint32 LODIndex,
	FHairStrandsRestRootResource* RestResources,
	FRDGTextureRef OutTexture)
{
	const uint32 RootCount = RestResources->GetRootCount();
	if (!RestResources->IsValid(LODIndex) || RootCount == 0)
		return;

	FHairStrandsLODRestRootResource& LODData = *RestResources->GetLOD(LODIndex);
	if (!LODData.RootBarycentricBuffer.Buffer || !LODData.RestUniqueTrianglePositionBuffer.Buffer)
		return;

	const FIntPoint OutputResolution = OutTexture->Desc.Extent;
	FHairFollicleMaskParameters* Parameters = GraphBuilder.AllocParameters<FHairFollicleMaskParameters>();
	Parameters->UniqueTrianglePositionBuffer = RegisterAsSRV(GraphBuilder, LODData.RestUniqueTrianglePositionBuffer);
	Parameters->RootToUniqueTriangleIndexBuffer = RegisterAsSRV(GraphBuilder, LODData.RootToUniqueTriangleIndexBuffer);
	Parameters->RootBarycentricBuffer   = RegisterAsSRV(GraphBuilder, LODData.RootBarycentricBuffer);
	Parameters->RootUVsBuffer = nullptr;
	Parameters->OutputResolution = OutputResolution;
	Parameters->MaxRootCount = RootCount;
	Parameters->MaxUniqueTriangleIndex = LODData.GetBulkData().Header.UniqueTriangleCount;
	Parameters->Channel = FMath::Min(Channel, 3u);
	Parameters->KernelSizeInPixels = FMath::Clamp(KernelSizeInPixels, 2u, 200u);
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutTexture, bNeedClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad, 0);

	FHairFollicleMaskVS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairFollicleMaskVS::FUVType>(0); // Mesh UVs

	TShaderMapRef<FHairFollicleMaskVS> VertexShader(ShaderMap, PermutationVector);
	TShaderMapRef<FHairFollicleMaskPS> PixelShader(ShaderMap);
	FHairFollicleMaskVS::FParameters ParametersVS;
	FHairFollicleMaskPS::FParameters ParametersPS;
	ParametersVS.Pass = *Parameters;
	ParametersPS.Pass = *Parameters;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::FollicleMask"),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, ParametersVS, ParametersPS, VertexShader, PixelShader, OutputResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{

		RHICmdList.SetViewport(0, 0, 0.0f, OutputResolution.X, OutputResolution.Y, 1.0f);

		// Apply additive blending pipeline state.
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Max, BF_SourceColor, BF_DestColor, BO_Max, BF_One, BF_One>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), ParametersPS);

		// Emit an instanced quad draw call on the order of the number of pixels on the screen.	
		RHICmdList.DrawPrimitive(0, Parameters->MaxRootCount, 1);
	});
}

static void AddFollicleMaskPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const bool bNeedClear,
	const uint32 KernelSizeInPixels,
	const uint32 Channel,
	const uint32 RootCount,
	const FRDGBufferRef RootUVBuffer,
	FRDGTextureRef OutTexture)
{
	const FIntPoint OutputResolution = OutTexture->Desc.Extent;
	FHairFollicleMaskParameters* Parameters = GraphBuilder.AllocParameters<FHairFollicleMaskParameters>();
	Parameters->UniqueTrianglePositionBuffer = nullptr;
	Parameters->RootBarycentricBuffer = nullptr;
	Parameters->RootUVsBuffer = GraphBuilder.CreateSRV(RootUVBuffer, PF_G32R32F);
	Parameters->OutputResolution = OutputResolution;
	Parameters->MaxRootCount = RootCount;
	Parameters->MaxUniqueTriangleIndex = 0;
	Parameters->Channel = FMath::Min(Channel, 3u);
	Parameters->KernelSizeInPixels = FMath::Clamp(KernelSizeInPixels, 2u, 200u);
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutTexture, bNeedClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad, 0);

	FHairFollicleMaskVS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairFollicleMaskVS::FUVType>(1); // Groom root's UV

	TShaderMapRef<FHairFollicleMaskVS> VertexShader(ShaderMap, PermutationVector);
	TShaderMapRef<FHairFollicleMaskPS> PixelShader(ShaderMap);
	FHairFollicleMaskVS::FParameters ParametersVS;
	FHairFollicleMaskPS::FParameters ParametersPS;
	ParametersVS.Pass = *Parameters;
	ParametersPS.Pass = *Parameters;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::FollicleMask"),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, ParametersVS, ParametersPS, VertexShader, PixelShader, OutputResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{

			RHICmdList.SetViewport(0, 0, 0.0f, OutputResolution.X, OutputResolution.Y, 1.0f);

			// Apply additive blending pipeline state.
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Max, BF_SourceColor, BF_DestColor, BO_Max, BF_One, BF_One>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), ParametersPS);

			// Emit an instanced quad draw call on the order of the number of pixels on the screen.	
			RHICmdList.DrawPrimitive(0, Parameters->MaxRootCount, 1);
		});
}

void GenerateFolliculeMask(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const EPixelFormat Format,
	const FIntPoint Resolution,
	const uint32 MipCount,
	const uint32 KernelSizeInPixels,
	const uint32 Channel,
	const int32 LODIndex,
	FHairStrandsRestRootResource* RestResources,
	FRDGTextureRef& OutTexture)
{
	const FLinearColor ClearColor(0.0f, 0.f, 0.f, 0.f);

	bool bClear = OutTexture == nullptr;
	if (OutTexture == nullptr)
	{
		FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(Resolution, Format, FClearValueBinding(ClearColor), TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV, MipCount);
		OutTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("Hair.FollicleMask"));
	}

	AddFollicleMaskPass(GraphBuilder, ShaderMap, bClear, KernelSizeInPixels, Channel, LODIndex, RestResources, OutTexture);
}

void GenerateFolliculeMask(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const EPixelFormat Format,
	const FIntPoint Resolution,
	const uint32 MipCount,
	const uint32 KernelSizeInPixels,
	const uint32 Channel,
	const TArray<FRDGBufferRef>& RootUVBuffers,
	FRDGTextureRef& OutTexture)
{
	const FLinearColor ClearColor(0.0f, 0.f, 0.f, 0.f);

	bool bClear = OutTexture == nullptr;
	if (OutTexture == nullptr)
	{
		FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(Resolution, Format, FClearValueBinding(ClearColor), TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV, MipCount);
		OutTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("Hair.FollicleMask"));
	}

	for (const FRDGBufferRef& RootUVBuffer : RootUVBuffers)
	{
		const uint32 RootCount = RootUVBuffer->Desc.NumElements;
		AddFollicleMaskPass(GraphBuilder, ShaderMap, bClear, KernelSizeInPixels, Channel, RootCount, RootUVBuffer, OutTexture);
		bClear = false;
	}
}

class FGenerateMipCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateMipCS);
	SHADER_USE_PARAMETER_STRUCT(FGenerateMipCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, Resolution)
		SHADER_PARAMETER(uint32, SourceMip)
		SHADER_PARAMETER(uint32, TargetMip)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_GENERATE_MIPS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMipCS, "/Engine/Private/HairStrands/HairStrandsFollicleMask.usf", "MainCS", SF_Compute);

void AddComputeMipsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FRDGTextureRef& OutTexture)
{
	check(OutTexture->Desc.Extent.X == OutTexture->Desc.Extent.Y);
	const uint32 Resolution = OutTexture->Desc.Extent.X;
	const uint32 MipCount = OutTexture->Desc.NumMips;
	for (uint32 MipIt = 0; MipIt < MipCount - 1; ++MipIt)
	{
		const uint32 SourceMipIndex = MipIt;
		const uint32 TargetMipIndex = MipIt + 1;
		const uint32 TargetResolution = Resolution >> TargetMipIndex;

		FGenerateMipCS::FParameters* Parameters = GraphBuilder.AllocParameters<FGenerateMipCS::FParameters>();
		Parameters->InTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(OutTexture, SourceMipIndex));
		Parameters->OutTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutTexture, TargetMipIndex));
		Parameters->Resolution = Resolution;
		Parameters->SourceMip = SourceMipIndex;
		Parameters->TargetMip = TargetMipIndex;
		Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		TShaderMapRef<FGenerateMipCS> ComputeShader(ShaderMap);
		ClearUnusedGraphResources(ComputeShader, Parameters);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrandsComputeVoxelMip"),
			Parameters,
			ERDGPassFlags::Compute,
			[Parameters, ComputeShader, TargetResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(FIntPoint(TargetResolution, TargetResolution), FIntPoint(8, 8));
			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, GroupCount);
		});
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairUpdatePositionOffsetCS : public FGlobalShader
{
public:
private:
	DECLARE_GLOBAL_SHADER(FHairUpdatePositionOffsetCS);
	SHADER_USE_PARAMETER_STRUCT(FHairUpdatePositionOffsetCS, FGlobalShader);

	class FUseGPUOffset : SHADER_PERMUTATION_BOOL("PERMUTATION_USE_GPU_OFFSET");
	class FPrevious : SHADER_PERMUTATION_BOOL("PERMUTATION_PREVIOUS");
	using FPermutationDomain = TShaderPermutationDomain<FUseGPUOffset, FPrevious>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, UpdateType)
		SHADER_PARAMETER(uint32, CardLODIndex)
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(FVector3f, CPUCurrPositionOffset)
		SHADER_PARAMETER(FVector3f, CPUPrevPositionOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootTriangleCurrPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RootTrianglePrevPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutCurrOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutPrevOffsetBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_OFFSET_UPDATE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairUpdatePositionOffsetCS, "/Engine/Private/HairStrands/HairStrandsMesh.usf", "MainCS", SF_Compute);

void AddHairStrandUpdatePositionOffsetPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	EHairPositionUpdateType UpdateType,
	const int32 InstanceRegisteredIndex,
	const int32 HairLODIndex,
	const int32 MeshLODIndex,
	FHairStrandsDeformedRootResource* DeformedRootResources,
	FHairStrandsDeformedResource* DeformedResources)
{
	if ((DeformedRootResources && MeshLODIndex < 0) || DeformedResources == nullptr)
	{
		return;
	}

	const bool bComputePreviousDeformedPosition = IsHairStrandContinuousDecimationReorderingEnabled();

	FRDGImportedBuffer OutCurrPositionOffsetBuffer = Register(GraphBuilder, DeformedResources->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Current), ERDGImportedBufferFlags::CreateUAV);
	FRDGImportedBuffer OutPrevPositionOffsetBuffer = Register(GraphBuilder, DeformedResources->GetPositionOffsetBuffer(FHairStrandsDeformedResource::Previous), ERDGImportedBufferFlags::CreateUAV);

	FRDGImportedBuffer RootTriangleCurrPositionBuffer;
	FRDGImportedBuffer RootTrianglePrevPositionBuffer;
	if (DeformedRootResources)
	{
		RootTriangleCurrPositionBuffer = Register(GraphBuilder, DeformedRootResources->GetLOD(MeshLODIndex)->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current), ERDGImportedBufferFlags::CreateSRV);
		RootTrianglePrevPositionBuffer = Register(GraphBuilder, DeformedRootResources->GetLOD(MeshLODIndex)->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Previous), ERDGImportedBufferFlags::CreateSRV);
	}

	const bool bUseGPUOffset = DeformedRootResources != nullptr && DeformedRootResources->IsValid(MeshLODIndex) && DeformedRootResources->GetLOD(MeshLODIndex)->IsInitialized() && GHairStrandsUseGPUPositionOffset > 0;
	const uint32 CurrOffsetIndex = DeformedResources->GetIndex(FHairStrandsDeformedResource::Current);
	const uint32 PrevOffsetIndex = DeformedResources->GetIndex(FHairStrandsDeformedResource::Previous);

	FHairUpdatePositionOffsetCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairUpdatePositionOffsetCS::FParameters>();
	PassParameters->UpdateType 				= uint32(UpdateType);
	PassParameters->CardLODIndex			= HairLODIndex;
	PassParameters->InstanceRegisteredIndex = InstanceRegisteredIndex;

	PassParameters->CPUCurrPositionOffset			= (FVector3f)DeformedResources->PositionOffset[CurrOffsetIndex];
	PassParameters->CPUPrevPositionOffset			= (FVector3f)DeformedResources->PositionOffset[PrevOffsetIndex];
	PassParameters->RootTriangleCurrPositionBuffer	= RootTriangleCurrPositionBuffer.SRV;
	PassParameters->RootTrianglePrevPositionBuffer	= RootTrianglePrevPositionBuffer.SRV;
	PassParameters->OutCurrOffsetBuffer				= OutCurrPositionOffsetBuffer.UAV;
	PassParameters->OutPrevOffsetBuffer				= OutPrevPositionOffsetBuffer.UAV;

	FHairUpdatePositionOffsetCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairUpdatePositionOffsetCS::FUseGPUOffset>(bUseGPUOffset);
	PermutationVector.Set<FHairUpdatePositionOffsetCS::FPrevious>(bComputePreviousDeformedPosition);
	TShaderMapRef<FHairUpdatePositionOffsetCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::UpdatePositionOffset(%s,%s)", bUseGPUOffset ? TEXT("GPU") : TEXT("CPU"), bComputePreviousDeformedPosition ? TEXT("Current/Previous") : TEXT("Current")),
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, 1));

	GraphBuilder.SetBufferAccessFinal(OutCurrPositionOffsetBuffer.Buffer, ERHIAccess::SRVMask);
	GraphBuilder.SetBufferAccessFinal(OutPrevPositionOffsetBuffer.Buffer, ERHIAccess::SRVMask);
}