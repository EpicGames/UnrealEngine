// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StrandHairVertexFactory.cpp: Strand hair vertex factory implementation
=============================================================================*/

#include "HairStrandsVertexFactory.h"
#include "SceneView.h"
#include "MeshBatch.h"
#include "MeshDrawShaderBindings.h"
#include "ShaderParameterUtils.h"
#include "Rendering/ColorVertexBuffer.h"
#include "MaterialDomain.h"
#include "MeshMaterialShader.h"
#include "HairStrandsInterface.h"
#include "GroomInstance.h"
#include "GroomVisualizationData.h"
#include "DataDrivenShaderPlatformInfo.h"

#define VF_STRANDS_SUPPORT_GPU_SCENE 1
#define VF_STRANDS_PROCEDURAL_INTERSECTOR 1
#define VF_STRANDS_LINEAR_SWEPT_SPHERES_INTERSECTOR 1

bool GetSupportHairStrandsProceduralPrimitive(EShaderPlatform InShaderPlatform)
{
	return VF_STRANDS_PROCEDURAL_INTERSECTOR && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(InShaderPlatform);
}

bool GetSupportHairStrandsLinearSweptSpheres(EShaderPlatform InShaderPlatform)
{
	return VF_STRANDS_LINEAR_SWEPT_SPHERES_INTERSECTOR && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingLinearSweptSpheres(InShaderPlatform);
}

/////////////////////////////////////////////////////////////////////////////////////////

template<typename T> inline void VFS_BindParam(FMeshDrawSingleShaderBindings& ShaderBindings, const FShaderResourceParameter& Param, T* Value)	{ if (Param.IsBound() && Value) ShaderBindings.Add(Param, Value); }
template<typename T> inline void VFS_BindParam(FMeshDrawSingleShaderBindings& ShaderBindings, const FShaderParameter& Param, const T& Value)	{ if (Param.IsBound()) ShaderBindings.Add(Param, Value); }

class FDummyCulledDispatchVertexIdsBuffer : public FVertexBuffer
{
public:
	FShaderResourceViewRHIRef SRVUint;
	FShaderResourceViewRHIRef SRVFloat;
	FShaderResourceViewRHIRef SRVRGBA;
	FShaderResourceViewRHIRef SRVRGBA_Uint;

	FShaderResourceViewRHIRef SRVByteAddress;
	FBufferRHIRef ByteAddressBufferRHI;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const static FLazyName ClassName(TEXT("FDummyCulledDispatchVertexIdsBuffer"));
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("FDummyCulledDispatchVertexIdsBuffer"), 4)
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask)
				.SetClassName(ClassName)
				.SetInitActionZeroData();

			VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		}

		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateByteAddress(TEXT("FDummyByteAddressBuffer"), sizeof(uint32) * 4, 0)
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.SetInitialState(ERHIAccess::SRVMask)
				.SetClassName(ClassName)
				.SetInitActionZeroData();

			ByteAddressBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		}

		SRVUint = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
		SRVFloat = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_FLOAT));
		SRVRGBA = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8));
		SRVRGBA_Uint = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R8G8B8A8_UINT));

		SRVByteAddress = RHICmdList.CreateShaderResourceView(ByteAddressBufferRHI, FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Raw));
	}

	virtual void ReleaseRHI() override
	{
		ByteAddressBufferRHI.SafeRelease();
		SRVByteAddress.SafeRelease();

		VertexBufferRHI.SafeRelease();
		SRVUint.SafeRelease();
		SRVFloat.SafeRelease();
		SRVRGBA.SafeRelease();
		SRVRGBA_Uint.SafeRelease();
	}
};
TGlobalResource<FDummyCulledDispatchVertexIdsBuffer> GDummyCulledDispatchVertexIdsBuffer;

/////////////////////////////////////////////////////////////////////////////////////////

FHairGroupPublicData::FVertexFactoryInput ComputeHairStrandsVertexInputData(const FHairGroupInstance* Instance, EGroomViewMode ViewMode);
uint32 GetHairRaytracingProceduralSplits();
FHairStrandsVertexFactoryUniformShaderParameters FHairGroupInstance::GetHairStandsUniformShaderParameters(EGroomViewMode ViewMode) const
{
	const FHairGroupPublicData::FVertexFactoryInput VFInput = ComputeHairStrandsVertexInputData(this, ViewMode);

	FHairStrandsVertexFactoryUniformShaderParameters Out = {};
	Out.Common = VFInput.Strands.Common;

	Out.Resources.PositionBuffer					= VFInput.Strands.PositionBufferRHISRV;
	Out.Resources.PositionOffsetBuffer 				= VFInput.Strands.PositionOffsetBufferRHISRV;
	Out.Resources.PointAttributeBuffer				= VFInput.Strands.PointAttributeBufferRHISRV;
	Out.Resources.CurveAttributeBuffer				= VFInput.Strands.CurveAttributeBufferRHISRV;
	Out.Resources.CurveBuffer						= VFInput.Strands.CurveBufferRHISRV;
	Out.Resources.PointToCurveBuffer				= VFInput.Strands.PointToCurveBufferRHISRV;
	Out.Resources.TangentBuffer 					= VFInput.Strands.TangentBufferRHISRV;
	Out.PrevResources.PreviousPositionBuffer		= VFInput.Strands.PrevPositionBufferRHISRV;
	Out.PrevResources.PreviousPositionOffsetBuffer	= VFInput.Strands.PrevPositionOffsetBufferRHISRV;

	// swap in some default data for those buffers that are not valid yet
	if (!Out.Resources.PositionBuffer) 						{ Out.Resources.PositionBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVByteAddress; }
	if (!Out.Resources.PositionOffsetBuffer) 				{ Out.Resources.PositionOffsetBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVFloat; }
	if (!Out.Resources.CurveAttributeBuffer) 				{ Out.Resources.CurveAttributeBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVByteAddress; }
	if (!Out.Resources.PointAttributeBuffer) 				{ Out.Resources.PointAttributeBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVByteAddress; }
	if (!Out.Resources.CurveBuffer) 						{ Out.Resources.CurveBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVByteAddress; }
	if (!Out.Resources.PointToCurveBuffer) 					{ Out.Resources.PointToCurveBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVByteAddress; }
	if (!Out.Resources.TangentBuffer) 						{ Out.Resources.TangentBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVFloat; }

	if (!Out.PrevResources.PreviousPositionBuffer) 			{ Out.PrevResources.PreviousPositionBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVByteAddress; }
	if (!Out.PrevResources.PreviousPositionOffsetBuffer) 	{ Out.PrevResources.PreviousPositionOffsetBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVFloat; }

	Out.Culling.bCullingEnable = HairGroupPublicData->GetCullingResultAvailable();
	if (Out.Culling.bCullingEnable)
	{
		Out.Culling.CullingIndexBuffer = HairGroupPublicData->GetCulledVertexIdBuffer().SRV;
	}
	else
	{
		Out.Culling.CullingIndexBuffer = GDummyCulledDispatchVertexIdsBuffer.SRVUint;
	}
	return Out;
}


IMPLEMENT_UNIFORM_BUFFER_STRUCT(FHairStrandsVertexFactoryUniformShaderParameters, "HairStrandsVF");

template<bool bSupportLinearSweptSpheres>
class HAIRSTRANDSCORE_API FHairStrandsVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FHairStrandsVertexFactoryShaderParameters, NonVirtual);
public:

	void Bind(const FShaderParameterMap& ParameterMap)
	{
	}

	void GetElementShaderBindings(
		const FSceneInterface* Scene,
		const FSceneView* View,
		const FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
	) const
	{
		const THairStrandsVertexFactory<bSupportLinearSweptSpheres>* VF = static_cast<const THairStrandsVertexFactory<bSupportLinearSweptSpheres>*>(VertexFactory);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FHairStrandsVertexFactoryUniformShaderParameters>(), VF->Data.Instance->Strands.UniformBuffer);
	}
};

IMPLEMENT_TEMPLATE_TYPE_LAYOUT(template<>, FHairStrandsVertexFactoryShaderParameters<true>);
IMPLEMENT_TEMPLATE_TYPE_LAYOUT(template<>, FHairStrandsVertexFactoryShaderParameters<false>);

/**
 * Should we cache the material's shadertype on this platform with this vertex factory? 
 */
template<bool bSupportLinearSweptSpheres>
bool THairStrandsVertexFactory<bSupportLinearSweptSpheres>::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	if constexpr (bSupportLinearSweptSpheres)
	{
		if (!GetSupportHairStrandsLinearSweptSpheres(Parameters.Platform))
		{
			return false;
		}
	}

	return (Parameters.MaterialParameters.MaterialDomain == MD_Surface && Parameters.MaterialParameters.bIsUsedWithHairStrands && IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform)) || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	bool bUseGPUSceneAndPrimitiveIdStream = false;
#if VF_STRANDS_SUPPORT_GPU_SCENE
	bUseGPUSceneAndPrimitiveIdStream = 
		Parameters.VertexFactoryType->SupportsPrimitiveIdStream() 
		&& UseGPUScene(Parameters.Platform)
		// TODO: support GPUScene on mobile
		&& !IsMobilePlatform(Parameters.Platform);
#endif
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bUseGPUSceneAndPrimitiveIdStream);
	OutEnvironment.SetDefine(TEXT("HAIR_STRAND_MESH_FACTORY"), 1);

	if constexpr (bSupportLinearSweptSpheres)
	{
		OutEnvironment.SetDefine(TEXT("ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR"), 1);
		OutEnvironment.SetDefine(TEXT("ALLOW_CUSTOM_RAYTRACING_HIT_TYPE"), 1);
	}
	else
	{
		const bool bUseProceduralIntersection = GetSupportHairStrandsProceduralPrimitive(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR"), bUseProceduralIntersection);

		// This is not strictly needed since it always return front face hit in IS.
		// However, hard coding front face check may allow compiler to optimize code more aggressively.
		if (bUseProceduralIntersection)
		{
			OutEnvironment.SetDefine(TEXT("ALLOW_CUSTOM_RAYTRACING_HIT_TYPE"), 1);
		}
	}
}

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors)
{
#if VF_STRANDS_SUPPORT_GPU_SCENE
	if (Type->SupportsPrimitiveIdStream()
		&& UseGPUScene(Platform) 
		&& !IsMobilePlatform(Platform) // On mobile VS may use PrimtiveUB while GPUScene is enabled
		&& ParameterMap.ContainsParameterAllocation(FPrimitiveUniformShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
	{
		OutErrors.AddUnique(*FString::Printf(TEXT("Shader attempted to bind the Primitive uniform buffer even though Vertex Factory %s computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(PrimitiveId).Member instead of Primitive.Member."), Type->GetName()));
	}
#endif
}

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::SetData(const FDataType& InData)
{
	Data = InData;
	UpdateRHI(FRHICommandListImmediate::Get());
}

/**
* Copy the data from another vertex factory
* @param Other - factory to copy from
*/
template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::Copy(const THairStrandsVertexFactory& Other)
{
	THairStrandsVertexFactory* VertexFactory = this;
	const FDataType* DataCopy = &Other.Data;
	ENQUEUE_RENDER_COMMAND(FHairStrandsVertexFactoryCopyData)(/*UE::RenderCommandPipe::Groom,*/
		[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
		{
			VertexFactory->Data = *DataCopy;
		});
	BeginUpdateResourceRHI(this);
}

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::InitResources(FRHICommandListBase& RHICmdList)
{
	if (bIsInitialized)
		return;

	FVertexFactory::InitResource(RHICmdList); //Call VertexFactory/RenderResources::InitResource() to mark the resource as initialized();

	bIsInitialized = true;
	bNeedsDeclaration = false;

	// We create different streams based on feature level
	check(HasValidFeatureLevel());

	FVertexDeclarationElementList Elements;
#if VF_STRANDS_SUPPORT_GPU_SCENE
	if (AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, 13, 0xff))
	{
		bNeedsDeclaration = true;
	}
#endif

	if (bNeedsDeclaration)
	{
		check(Streams.Num() > 0);
	}
	InitDeclaration(Elements);
	check(IsValidRef(GetDeclaration()));

	// create the buffer
	FHairStrandsVertexFactoryUniformShaderParameters Parameters = Data.Instance->GetHairStandsUniformShaderParameters(EGroomViewMode::None);
	Data.Instance->Strands.UniformBuffer = FHairStrandsUniformBuffer::CreateUniformBufferImmediate(Parameters, UniformBuffer_MultiFrame);
}

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
#if VF_STRANDS_SUPPORT_GPU_SCENE
	Elements.Add(FVertexElement(0, 0, VET_UInt, 13, sizeof(uint32), true));
#endif
}

template<bool bSupportLinearSweptSpheres>
EPrimitiveIdMode THairStrandsVertexFactory<bSupportLinearSweptSpheres>::GetPrimitiveIdMode(ERHIFeatureLevel::Type In) const
{
	return PrimID_ForceZero;
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairStrandsVertexFactory, SF_Vertex,      FHairStrandsVertexFactoryShaderParameters<false>);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairStrandsVertexFactory, SF_Pixel,       FHairStrandsVertexFactoryShaderParameters<false>);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairStrandsVertexFactory, SF_Compute,     FHairStrandsVertexFactoryShaderParameters<false>);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FHairStrandsVertexFactory, SF_RayHitGroup, FHairStrandsVertexFactoryShaderParameters<false>);
#endif

#if VF_STRANDS_LINEAR_SWEPT_SPHERES_INTERSECTOR
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLinearSweptSpheresHairStrandsVertexFactory, SF_Vertex,     FHairStrandsVertexFactoryShaderParameters<true>);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLinearSweptSpheresHairStrandsVertexFactory, SF_Pixel,      FHairStrandsVertexFactoryShaderParameters<true>);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLinearSweptSpheresHairStrandsVertexFactory, SF_Compute,    FHairStrandsVertexFactoryShaderParameters<true>);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLinearSweptSpheresHairStrandsVertexFactory, SF_RayHitGroup, FHairStrandsVertexFactoryShaderParameters<true>);
#endif
#endif

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Nothing as the initialize runs only on first use
}

template<bool bSupportLinearSweptSpheres>
void THairStrandsVertexFactory<bSupportLinearSweptSpheres>::ReleaseRHI()
{
	FVertexFactory::ReleaseRHI();
}

// Explicit instantiations
template class THairStrandsVertexFactory<true>;
template class THairStrandsVertexFactory<false>;

IMPLEMENT_TEMPLATE_VERTEX_FACTORY_TYPE(template<>, FHairStrandsVertexFactory, "/Engine/Private/HairStrands/HairStrandsVertexFactory.ush",
	EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| (VF_STRANDS_SUPPORT_GPU_SCENE ? EVertexFactoryFlags::SupportsPrimitiveIdStream : EVertexFactoryFlags::None)
	| EVertexFactoryFlags::SupportsRayTracing
	| (VF_STRANDS_PROCEDURAL_INTERSECTOR ? EVertexFactoryFlags::SupportsRayTracingProceduralPrimitive : EVertexFactoryFlags::None)
	| EVertexFactoryFlags::SupportsManualVertexFetch
	| EVertexFactoryFlags::SupportsPSOPrecaching
);

IMPLEMENT_TEMPLATE_VERTEX_FACTORY_TYPE(template<>, FLinearSweptSpheresHairStrandsVertexFactory, "/Engine/Private/HairStrands/HairStrandsVertexFactory.ush",
	EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrecisePrevWorldPos
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| (VF_STRANDS_SUPPORT_GPU_SCENE ? EVertexFactoryFlags::SupportsPrimitiveIdStream : EVertexFactoryFlags::None)
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsManualVertexFetch
	| EVertexFactoryFlags::SupportsPSOPrecaching
);

