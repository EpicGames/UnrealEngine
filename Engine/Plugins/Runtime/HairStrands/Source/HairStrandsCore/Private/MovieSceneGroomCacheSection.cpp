// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGroomCacheSection.h"
#include "GroomCache.h"
#include "MovieScene.h"
#include "MovieSceneGroomCacheTemplate.h"
#include "MovieSceneTimeHelpers.h"
#include "Misc/QualifiedFrameTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGroomCacheSection)

FMovieSceneGroomCacheParams::FMovieSceneGroomCacheParams()
{
	GroomCache = nullptr;
	PlayRate = 1.f;
	bReverse = false;
}

UMovieSceneGroomCacheSection::UMovieSceneGroomCacheSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BlendType = EMovieSceneBlendType::Absolute;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR

	PreviousPlayRate = Params.PlayRate;

#endif
}

TOptional<FFrameTime> UMovieSceneGroomCacheSection::GetOffsetTime() const
{
	return TOptional<FFrameTime>(Params.FirstLoopStartFrameOffset);
}

TOptional<TRange<FFrameNumber> > UMovieSceneGroomCacheSection::GetAutoSizeRange() const
{
	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameTime AnimationLength = Params.GetSequenceLength() * FrameRate;
	int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f);

	return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + IFrameNumber + 1);
}


void UMovieSceneGroomCacheSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	SetFlags(RF_Transactional);

	if (TryModify())
	{
		if (bTrimLeft)
		{
			FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

			Params.FirstLoopStartFrameOffset = HasStartFrame() ? GetFirstLoopStartOffsetAtTrimTime(TrimTime, Params, GetInclusiveStartFrame(), FrameRate) : 0;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
	}
}

UMovieSceneSection* UMovieSceneGroomCacheSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	const FFrameNumber InitialFirstLoopStartFrameOffset = Params.FirstLoopStartFrameOffset;

	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	const FFrameNumber NewOffset = HasStartFrame() ? GetFirstLoopStartOffsetAtTrimTime(SplitTime, Params, GetInclusiveStartFrame(), FrameRate) : 0;

	UMovieSceneSection* NewSection = Super::SplitSection(SplitTime, bDeleteKeys);
	if (NewSection != nullptr)
	{
		UMovieSceneGroomCacheSection* NewGroomCacheSection = Cast<UMovieSceneGroomCacheSection>(NewSection);
		NewGroomCacheSection->Params.FirstLoopStartFrameOffset = NewOffset;
	}

	// Restore original offset modified by splitting
	Params.FirstLoopStartFrameOffset = InitialFirstLoopStartFrameOffset;

	return NewSection;
}


void UMovieSceneGroomCacheSection::GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
{
	Super::GetSnapTimes(OutSnapTimes, bGetSectionBorders);

	const FFrameRate   FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	const FFrameNumber StartFrame = GetInclusiveStartFrame();
	const FFrameNumber EndFrame = GetExclusiveEndFrame() - 1; // -1 because we don't need to add the end frame twice

	const float AnimPlayRate = FMath::IsNearlyZero(Params.PlayRate) ? 1.0f : Params.PlayRate;
	const float SeqLengthSeconds = Params.GetSequenceLength() - FrameRate.AsSeconds(Params.StartFrameOffset + Params.EndFrameOffset) / AnimPlayRate;
	const float FirstLoopSeqLengthInSeconds = SeqLengthSeconds - FrameRate.AsSeconds(Params.FirstLoopStartFrameOffset) / AnimPlayRate;

	const FFrameTime SequenceFrameLength = SeqLengthSeconds * FrameRate;
	const FFrameTime FirstLoopSequenceFrameLength = FirstLoopSeqLengthInSeconds * FrameRate;
	if (SequenceFrameLength.FrameNumber > 1)
	{
		// Snap to the repeat times
		bool IsFirstLoop = true;
		FFrameTime CurrentTime = StartFrame;
		while (CurrentTime < EndFrame)
		{
			OutSnapTimes.Add(CurrentTime.FrameNumber);
			if (IsFirstLoop)
			{
				CurrentTime += FirstLoopSequenceFrameLength;
				IsFirstLoop = false;
			}
			else
			{
				CurrentTime += SequenceFrameLength;
			}
		}
	}
}

float UMovieSceneGroomCacheSection::MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const
{
	FMovieSceneGroomCacheSectionTemplateParameters TemplateParams(Params, GetInclusiveStartFrame(), GetExclusiveEndFrame());
	return TemplateParams.MapTimeToAnimation(ComponentDuration, InPosition, InTickResolution);
}


#if WITH_EDITOR
void UMovieSceneGroomCacheSection::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Store the current play rate so that we can compute the amount to compensate the section end time when the play rate changes
	PreviousPlayRate = Params.PlayRate;

	Super::PreEditChange(PropertyAboutToChange);
}

void UMovieSceneGroomCacheSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Adjust the duration automatically if the play rate changes
	if (PropertyChangedEvent.Property != nullptr &&
		PropertyChangedEvent.Property->GetFName() == TEXT("PlayRate"))
	{
		float NewPlayRate = Params.PlayRate;

		if (!FMath::IsNearlyZero(NewPlayRate))
		{
			float CurrentDuration = UE::MovieScene::DiscreteSize(GetRange());
			float NewDuration = CurrentDuration * (PreviousPlayRate / NewPlayRate);
			SetEndFrame(GetInclusiveStartFrame() + FMath::FloorToInt(NewDuration));

			PreviousPlayRate = NewPlayRate;
		}
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

float FMovieSceneGroomCacheParams::GetSequenceLength() const
{
	return GroomCache != nullptr ? GroomCache->GetDuration() : 0.f;
}

