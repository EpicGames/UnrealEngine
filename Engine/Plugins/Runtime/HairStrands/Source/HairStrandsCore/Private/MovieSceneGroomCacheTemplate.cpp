// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGroomCacheTemplate.h"
#include "GroomComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGroomCacheTemplate)

DECLARE_CYCLE_STAT(TEXT("Groom Cache Evaluate"), MovieSceneEval_GroomCache_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Groom Cache Token Execute"), MovieSceneEval_GroomCache_TokenExecute, STATGROUP_MovieSceneEval);

/** Used to set Manual Tick back to previous when outside section */
struct FPreAnimatedGroomCacheTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			FToken(UGroomComponent* InComponent)
			{
				// Cache this object's current update flag and animation mode
				bInManualTick = InComponent->GetManualTick();
			}

			virtual void RestoreState(UObject& ObjectToRestore, const UE::MovieScene::FRestoreStateParams& Params)
			{
				UGroomComponent* Component = CastChecked<UGroomComponent>(&ObjectToRestore);
				Component->SetManualTick(bInManualTick);
				Component->ResetAnimationTime();
			}
			bool bInManualTick;
		};

		return FToken(CastChecked<UGroomComponent>(&Object));
	}
	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FPreAnimatedGroomCacheTokenProducer>();
	}
};

/** A movie scene execution token that executes a GroomCache */
struct FGroomCacheExecutionToken
	: IMovieSceneExecutionToken

{
	FGroomCacheExecutionToken(const FMovieSceneGroomCacheSectionTemplateParameters &InParams) :
		Params(InParams)
	{}

	static UGroomComponent* GroomComponentFromObject(UObject* BoundObject)
	{
		if (AActor* Actor = Cast<AActor>(BoundObject))
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UGroomComponent* GroomComp = Cast<UGroomComponent>(Component))
				{
					return GroomComp;
				}
			}
		}
		else if (UGroomComponent* GroomComp = Cast<UGroomComponent>(BoundObject))
		{
			if (GroomComp->GetGroomCache())
			{
				return GroomComp;
			}
		}
		return nullptr;
	}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_GroomCache_TokenExecute)
		if (Operand.ObjectBindingID.IsValid())
		{
			for (TWeakObjectPtr<> WeakObj : Player.FindBoundObjects(Operand))
			{
				if (UObject* Obj = WeakObj.Get())
				{
					UGroomComponent* GroomComp = GroomComponentFromObject(Obj);
					if (GroomComp && GroomComp->IsRegistered())
					{
						// Set the GroomCache on the component only if it's set and valid in the Params
						if (Params.GroomCache && Params.GroomCache != GroomComp->GetGroomCache())
						{
							UGroomCache* GroomCache = Params.GroomCache;
							{
								GroomComp->SetGroomCache(GroomCache);
							}
						}
						else
						{
							// It could be unset if the Params was referencing a transient GroomCache
							// In that case, use the GroomCache that is set on the component
							Params.GroomCache = GroomComp->GetGroomCache();
						}
						Player.SavePreAnimatedState(*GroomComp, FPreAnimatedGroomCacheTokenProducer::GetAnimTypeID(), FPreAnimatedGroomCacheTokenProducer());
						GroomComp->SetManualTick(true);
						// calculate the time at which to evaluate the animation
						float EvalTime = Params.MapTimeToAnimation(GroomComp->GetGroomCacheDuration(), Context.GetTime(), Context.GetFrameRate());
						GroomComp->TickAtThisTime(EvalTime, true, Params.bReverse, true);
					}
				}
			}
		}
	}

	FMovieSceneGroomCacheSectionTemplateParameters Params;
};

FMovieSceneGroomCacheSectionTemplate::FMovieSceneGroomCacheSectionTemplate(const UMovieSceneGroomCacheSection& InSection)
	: Params(const_cast<FMovieSceneGroomCacheParams&> (InSection.Params), InSection.GetInclusiveStartFrame(), InSection.GetExclusiveEndFrame())
{
}

//We use a token here so we can set the manual tick state back to what it was previously when outside this section.
//This is similar to how Skeletal Animation evaluation also works.
void FMovieSceneGroomCacheSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_GroomCache_Evaluate)
	ExecutionTokens.Add(FGroomCacheExecutionToken(Params));
}

float FMovieSceneGroomCacheSectionTemplateParameters::MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InFrameRate) const
{
	const float SequenceLength = ComponentDuration;
	const FFrameTime AnimationLength = SequenceLength * InFrameRate;
	const int32 LengthInFrames = AnimationLength.FrameNumber.Value + (int)(AnimationLength.GetSubFrame() + 0.5f) + 1;
	//we only play end if we are not looping, and assuming we are looping if Length is greater than default length;
	const bool bLooping = (SectionEndTime.Value - SectionStartTime.Value + StartFrameOffset + EndFrameOffset) > LengthInFrames;

	InPosition = FMath::Clamp(InPosition, FFrameTime(SectionStartTime), FFrameTime(SectionEndTime - 1));

	const float SectionPlayRate = PlayRate;
	const float AnimPlayRate = FMath::IsNearlyZero(SectionPlayRate) ? 1.0f : SectionPlayRate;

	const float FirstLoopSeqLength = SequenceLength - InFrameRate.AsSeconds(FirstLoopStartFrameOffset + StartFrameOffset + EndFrameOffset);
	const float SeqLength = SequenceLength - InFrameRate.AsSeconds(StartFrameOffset + EndFrameOffset);

	float AnimPosition = FFrameTime::FromDecimal((InPosition - SectionStartTime).AsDecimal() * AnimPlayRate) / InFrameRate;
	AnimPosition += InFrameRate.AsSeconds(FirstLoopStartFrameOffset);
	if (SeqLength > 0.f && (bLooping || !FMath::IsNearlyEqual(AnimPosition, SeqLength, 1e-4f)))
	{
		AnimPosition = FMath::Fmod(AnimPosition, SeqLength);
	}
	AnimPosition += InFrameRate.AsSeconds(StartFrameOffset);
	if (bReverse)
	{
		AnimPosition = SequenceLength - AnimPosition;
	}

	return AnimPosition;
}

