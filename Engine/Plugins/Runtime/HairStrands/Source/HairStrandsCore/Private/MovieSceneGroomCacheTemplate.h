// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneGroomCacheSection.h"
#include "MovieSceneGroomCacheTemplate.generated.h"

USTRUCT()
struct FMovieSceneGroomCacheSectionTemplateParameters : public FMovieSceneGroomCacheParams
{
	GENERATED_BODY()

	FMovieSceneGroomCacheSectionTemplateParameters() {}

	FMovieSceneGroomCacheSectionTemplateParameters(const FMovieSceneGroomCacheParams& BaseParams, FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime)
		: FMovieSceneGroomCacheParams(BaseParams)
		, SectionStartTime(InSectionStartTime)
		, SectionEndTime(InSectionEndTime)
	{}

	float MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const;

	UPROPERTY()
	FFrameNumber SectionStartTime;

	UPROPERTY()
	FFrameNumber SectionEndTime;
};

USTRUCT()
struct FMovieSceneGroomCacheSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneGroomCacheSectionTemplate() {}
	FMovieSceneGroomCacheSectionTemplate(const UMovieSceneGroomCacheSection& Section);

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneGroomCacheSectionTemplateParameters Params;
};
