// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGroomCacheTrack.h"
#include "GroomComponent.h"
#include "MovieScene.h"
#include "MovieSceneGroomCacheSection.h"
#include "MovieSceneGroomCacheTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGroomCacheTrack)

#define LOCTEXT_NAMESPACE "MovieSceneGroomCacheTrack"

/* UMovieSceneGroomCacheTrack constructor
 *****************************************************************************/

UMovieSceneGroomCacheTrack::UMovieSceneGroomCacheTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(96, 128, 24, 65);
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bCanEvaluateNearestSection = true;
	EvalOptions.bEvaluateInPreroll = true;
}

/* UMovieSceneGroomCacheTrack interface
 *****************************************************************************/

UMovieSceneSection* UMovieSceneGroomCacheTrack::AddNewAnimation(FFrameNumber KeyTime, UGroomComponent* GroomComp)
{
	UMovieSceneGroomCacheSection* NewSection = Cast<UMovieSceneGroomCacheSection>(CreateNewSection());
	{
		FFrameTime AnimationLength = GroomComp->GetGroomCacheDuration() * GetTypedOuter<UMovieScene>()->GetTickResolution();
		int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f) + 1;
		NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, IFrameNumber, INDEX_NONE);
		
		NewSection->Params.GroomCache = GroomComp->GetGroomCache();
	}

	AddSection(*NewSection);

	return NewSection;
}

TArray<UMovieSceneSection*> UMovieSceneGroomCacheTrack::GetAnimSectionsAtTime(FFrameNumber Time)
{
	TArray<UMovieSceneSection*> Sections;
	for (auto Section : AnimationSections)
	{
		if (Section->IsTimeWithinSection(Time))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}

FMovieSceneEvalTemplatePtr UMovieSceneGroomCacheTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneGroomCacheSectionTemplate(*CastChecked<UMovieSceneGroomCacheSection>(&InSection));
}

/* UMovieSceneTrack interface
 *****************************************************************************/

const TArray<UMovieSceneSection*>& UMovieSceneGroomCacheTrack::GetAllSections() const
{
	return AnimationSections;
}

bool UMovieSceneGroomCacheTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneGroomCacheSection::StaticClass();
}

UMovieSceneSection* UMovieSceneGroomCacheTrack::CreateNewSection()
{
	return NewObject<UMovieSceneGroomCacheSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneGroomCacheTrack::RemoveAllAnimationData()
{
	AnimationSections.Empty();
}

bool UMovieSceneGroomCacheTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}

void UMovieSceneGroomCacheTrack::AddSection(UMovieSceneSection& Section)
{
	AnimationSections.Add(&Section);
}

void UMovieSceneGroomCacheTrack::RemoveSection(UMovieSceneSection& Section)
{
	AnimationSections.Remove(&Section);
}

void UMovieSceneGroomCacheTrack::RemoveSectionAt(int32 SectionIndex)
{
	AnimationSections.RemoveAt(SectionIndex);
}

bool UMovieSceneGroomCacheTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneGroomCacheTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Groom Cache");
}

#endif

#undef LOCTEXT_NAMESPACE

