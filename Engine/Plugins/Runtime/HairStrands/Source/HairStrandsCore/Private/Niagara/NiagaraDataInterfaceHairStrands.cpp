// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/NiagaraDataInterfaceHairStrands.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShader.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderer.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraConstants.h"

#include "Components/SkeletalMeshComponent.h"
#include "ShaderParameterUtils.h"
#include "ClearQuad.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "Components/SkeletalMeshComponent.h"

#include "GroomComponent.h"
#include "GroomAsset.h"
#include "GroomBindingBuilder.h"
#include "SystemTextures.h"
#include "ShaderCompilerCore.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceHairStrands)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceHairStrands"

//------------------------------------------------------------------------------------------------------------
namespace NDIHairStrandsLocal
{
BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer,DeformedPositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer,	RestPositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,		CurvesOffsetsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>,		RestTrianglePositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>,		DeformedTrianglePositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,		RootBarycentricCoordinatesBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,		RootToUniqueTriangleIndexBuffer)
	SHADER_PARAMETER(FMatrix44f,						WorldTransform)
	SHADER_PARAMETER(FMatrix44f,						WorldInverse)
	SHADER_PARAMETER(FQuat4f,							WorldRotation)
	SHADER_PARAMETER(FMatrix44f,						BoneTransform)
	SHADER_PARAMETER(FMatrix44f,						BoneInverse)
	SHADER_PARAMETER(FQuat4f,							BoneRotation)
	SHADER_PARAMETER(int,								NumStrands)
	SHADER_PARAMETER(int,								StrandSize)
	SHADER_PARAMETER(int,								InterpolationMode)
	SHADER_PARAMETER(FVector3f,							RestRootOffset)
	SHADER_PARAMETER(FVector3f,							DeformedRootOffset)
	SHADER_PARAMETER(FVector3f,							RestPositionOffset)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, DeformedPositionOffset)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>,		BoundingBoxBuffer)
	SHADER_PARAMETER(uint32,							ResetSimulation)
	SHADER_PARAMETER(uint32,							RestUpdate)
	SHADER_PARAMETER(uint32,							LocalSimulation)
	SHADER_PARAMETER(int,								SampleCount)
	SHADER_PARAMETER(int,								RBFLocalSpace)
	SHADER_PARAMETER(FIntVector4,						BoundingBoxOffsets)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, RestSamplePositionsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, MeshSampleWeightsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, DeformedSamplePositionsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,		ParamsScaleBuffer)
	SHADER_PARAMETER(FVector3f,							BoneLinearVelocity)
	SHADER_PARAMETER(FVector3f,							BoneAngularVelocity)
	SHADER_PARAMETER(FVector3f,							BoneLinearAcceleration)
	SHADER_PARAMETER(FVector3f,							BoneAngularAcceleration)
END_SHADER_PARAMETER_STRUCT()

static const TCHAR* CommonShaderFiles[] =
{
	TEXT("/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"),
	//OutHLSL += TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDirectSolver.ush");
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraStrandsExternalForce.ush"),
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraHookeSpringMaterial.ush"),
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraAngularSpringMaterial.ush"),
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraConstantVolumeMaterial.ush"),
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraCosseratRodMaterial.ush"),
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraStaticCollisionConstraint.ush"),
	TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDataInterfaceHairStrands.ush"),
};

static const TCHAR* TemplateShaderFile = TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDataInterfaceHairStrandsTemplate.ush");

//------------------------------------------------------------------------------------------------------------

static const FName GetPointPositionName(TEXT("GetPointPosition"));

static const FName GetStrandSizeName(TEXT("GetStrandSize"));
static const FName GetNumStrandsName(TEXT("GetNumStrands"));

static const FName GetWorldTransformName(TEXT("GetWorldTransform"));
static const FName GetWorldInverseName(TEXT("GetWorldInverse"));

static const FName GetSubStepsName("GetSubSteps");
static const FName GetIterationCountName("GetIterationCount");

static const FName GetGravityVectorName("GetGravityVector");
static const FName GetGravityPreloadingName("GetGravityPreloading");
static const FName GetAirDragName("GetAirDrag");
static const FName GetAirVelocityName("GetAirVelocity");

static const FName GetSolveBendName("GetSolveBend");
static const FName GetProjectBendName("GetProjectBend");
static const FName GetBendDampingName("GetBendDamping");
static const FName GetBendStiffnessName("GetBendStiffness");
static const FName GetBendScaleName("GetBendScale");

static const FName GetSolveStretchName("GetSolveStretch");
static const FName GetProjectStretchName("GetProjectStretch");
static const FName GetStretchDampingName("GetStretchDamping");
static const FName GetStretchStiffnessName("GetStretchStiffness");
static const FName GetStretchScaleName("GetStretchScale");

static const FName GetSolveCollisionName("GetSolveCollision");
static const FName GetProjectCollisionName("GetProjectCollision");
static const FName GetStaticFrictionName("GetStaticFriction");
static const FName GetKineticFrictionName("GetKineticFriction");
static const FName GetStrandsViscosityName("GetStrandsViscosity");
static const FName GetGridDimensionName("GetGridDimension");
static const FName GetCollisionRadiusName("GetCollisionRadius");
static const FName GetRadiusScaleName("GetRadiusScale");

static const FName GetStrandsDensityName("GetStrandsDensity");
static const FName GetStrandsSmoothingName("GetStrandsSmoothing");
static const FName GetStrandsThicknessName("GetStrandsThickness");
static const FName GetThicknessScaleName("GetThicknessScale");
static const FName GetTimeDilationName("GetTimeDilation");

//------------------------------------------------------------------------------------------------------------

static const FName ComputeNodePositionName(TEXT("ComputeNodePosition"));
static const FName ComputeNodeOrientationName(TEXT("ComputeNodeOrientation"));
static const FName ComputeNodeMassName(TEXT("ComputeNodeMass"));
static const FName ComputeNodeInertiaName(TEXT("ComputeNodeInertia"));

//------------------------------------------------------------------------------------------------------------

static const FName ComputeEdgeLengthName(TEXT("ComputeEdgeLength"));
static const FName ComputeEdgeRotationName(TEXT("ComputeEdgeRotation"));
static const FName ComputeEdgeDirectionName(TEXT("ComputeEdgeDirection"));

//------------------------------------------------------------------------------------------------------------

static const FName ComputeRestPositionName(TEXT("ComputeRestPosition"));
static const FName ComputeRestOrientationName(TEXT("ComputeRestOrientation"));
static const FName ComputeLocalStateName(TEXT("ComputeLocalState"));

//------------------------------------------------------------------------------------------------------------

static const FName AdvectNodePositionName(TEXT("AdvectNodePosition"));
static const FName AdvectNodeOrientationName(TEXT("AdvectNodeOrientation"));
static const FName UpdateLinearVelocityName(TEXT("UpdateLinearVelocity"));
static const FName UpdateAngularVelocityName(TEXT("UpdateAngularVelocity"));

//------------------------------------------------------------------------------------------------------------

static const FName GetLocalVectorName(TEXT("GetLocalVector"));
static const FName GetWorldVectorName(TEXT("GetWorldVector"));

static const FName AttachNodePositionName(TEXT("AttachNodePosition"));
static const FName AttachNodeOrientationName(TEXT("AttachNodeOrientation"));

static const FName AttachNodeStateName(TEXT("AttachNodeState"));
static const FName UpdateNodeStateName(TEXT("UpdateNodeState"));

//------------------------------------------------------------------------------------------------------------

static const FName UpdatePointPositionName(TEXT("UpdatePointPosition"));
static const FName ResetPointPositionName(TEXT("ResetPointPosition"));

//------------------------------------------------------------------------------------------------------------

static const FName GetBoundingBoxName(TEXT("GetBoundingBox"));
static const FName ResetBoundingBoxName(TEXT("ResetBoundingBox"));
static const FName BuildBoundingBoxName(TEXT("BuildBoundingBox"));

//------------------------------------------------------------------------------------------------------------

static const FName SetupDistanceSpringMaterialName(TEXT("SetupDistanceSpringMaterial"));
static const FName SolveDistanceSpringMaterialName(TEXT("SolveDistanceSpringMaterial"));
static const FName ProjectDistanceSpringMaterialName(TEXT("ProjectDistanceSpringMaterial"));

//------------------------------------------------------------------------------------------------------------

static const FName SetupAngularSpringMaterialName(TEXT("SetupAngularSpringMaterial"));
static const FName SolveAngularSpringMaterialName(TEXT("SolveAngularSpringMaterial"));
static const FName ProjectAngularSpringMaterialName(TEXT("ProjectAngularSpringMaterial"));

//------------------------------------------------------------------------------------------------------------

static const FName SetupStretchRodMaterialName(TEXT("SetupStretchRodMaterial"));
static const FName SolveStretchRodMaterialName(TEXT("SolveStretchRodMaterial"));
static const FName ProjectStretchRodMaterialName(TEXT("ProjectStretchRodMaterial"));

//------------------------------------------------------------------------------------------------------------

static const FName SetupBendRodMaterialName(TEXT("SetupBendRodMaterial"));
static const FName SolveBendRodMaterialName(TEXT("SolveBendRodMaterial"));
static const FName ProjectBendRodMaterialName(TEXT("ProjectBendRodMaterial"));

//------------------------------------------------------------------------------------------------------------

static const FName SolveHardCollisionConstraintName(TEXT("SolveHardCollisionConstraint"));
static const FName ProjectHardCollisionConstraintName(TEXT("ProjectHardCollisionConstraint"));

static const FName SetupSoftCollisionConstraintName(TEXT("SetupSoftCollisionConstraint"));
static const FName SolveSoftCollisionConstraintName(TEXT("SolveSoftCollisionConstraint"));
static const FName ProjectSoftCollisionConstraintName(TEXT("ProjectSoftCollisionConstraint"));

//------------------------------------------------------------------------------------------------------------

static const FName UpdateMaterialFrameName(TEXT("UpdateMaterialFrame"));
static const FName ComputeMaterialFrameName(TEXT("ComputeMaterialFrame"));

//------------------------------------------------------------------------------------------------------------

static const FName ComputeAirDragForceName(TEXT("ComputeAirDragForce"));

//------------------------------------------------------------------------------------------------------------

static const FName NeedSimulationResetName(TEXT("NeedSimulationReset"));
static const FName HasGlobalInterpolationName(TEXT("HasGlobalInterpolation"));
static const FName NeedRestUpdateName(TEXT("NeedRestUpdate"));

//------------------------------------------------------------------------------------------------------------

static const FName InitGridSamplesName(TEXT("InitGridSamples"));
static const FName GetSampleStateName(TEXT("GetSampleState"));

//------------------------------------------------------------------------------------------------------------

} //namespace NDIHairStrandsLocal

//------------------------------------------------------------------------------------------------------------

enum class EHairSimulationInterpolationMode : uint8
{
	Rigid = 0,
	Skinned = 1,
	RBF = 2
};

//------------------------------------------------------------------------------------------------------------

static int32 GHairSimulationMaxDelay = 4;
static FAutoConsoleVariableRef CVarHairSimulationMaxDelay(TEXT("r.HairStrands.SimulationMaxDelay"), GHairSimulationMaxDelay, TEXT("Maximum tick Delay before starting the simulation"));

static int32 GHairSimulationRestUpdate = false;
static FAutoConsoleVariableRef CVarHairSimulationRestUpdate(TEXT("r.HairStrands.SimulationRestUpdate"), GHairSimulationRestUpdate, TEXT("Update the simulation rest pose"));

// Temporary cvar just in case using the latest hair group instance is changing the behavior 
static bool GEnableProxyInstanceTransform = false;
static FAutoConsoleVariableRef CVarEnableProxyInstanceTransform(TEXT("r.HairStrands.Debug.EnableProxyInstanceTransform"), GEnableProxyInstanceTransform, TEXT("Enable the use of the niagara proxy instance to compute the bone/world transform. For debug only"));

//------------------------------------------------------------------------------------------------------------

bool UsesNanite(const UGroomComponent* In)
{
	return In && In->GroomAsset && In->GroomAsset->UsesNanite();
}

// Forward declaration of the GroomComponent function that is retrieving a ref counted group instance ptr given an index
TRefCountPtr<FHairGroupInstance> GetRefCountedHairGroupInstance(const UGroomComponent* GroomComponent, const int32 GroupIndex);
TRefCountPtr<FHairGroupInstance> GetRefCountedHairGroupInstance(const FNDIHairStrandsData* ProxyData)
{
	// When a groom asset use Nanite geometry, we don't support simulation and can't retrieve the HairInstance data.
	if (ProxyData && !ProxyData->bUsesNanite)
	{
		return GetRefCountedHairGroupInstance(ProxyData->HairGroupInstSource.Get(), ProxyData->HairGroupIndex);
	}
	return nullptr;
}

//------------------------------------------------------------------------------------------------------------
// Helper struct/function to extracting used resource prior to dispatching

struct FHairGroupInstanceRDG
{
	bool bIsValid = false;
	bool bIsRootValid = false;
	bool bIsDeformedValid = false;
	int32 MeshLODIndex = -1;
	EHairGeometryType GeometryType = EHairGeometryType::NoneGeometry;
	EHairBindingType BindingType = EHairBindingType::NoneBinding;
	uint32 NumPoints = 0;
	uint32 NumCurves = 0;
	uint32 SampleCount = 0;

	// Position / Curve
	FVector3f RestPositionOffsetValue = FVector3f::ZeroVector;
	FRDGBufferSRVRef RestPositionBuffer = nullptr;
	FRDGBufferUAVRef DeformedPositionBufferUAV = nullptr;
	FRDGBufferSRVRef DeformedPositionOffsetSRV = nullptr;
	FRDGBufferSRVRef CurvesOffsetsBuffer = nullptr;
	FRDGBufferSRVRef PointToCurveIndexBuffer = nullptr;

	FRDGBufferRef DeformedPositionBuffer = nullptr;
	FRDGBufferRef DeformedPositionOffset = nullptr;
	FRDGBufferRef PreviousDeformedPositionBuffer = nullptr;
	FRDGBufferRef PreviousDeformedPositionOffset = nullptr;

	// Skinning
	FRDGBufferSRVRef RestTrianglePositionBuffer = nullptr;
	FRDGBufferSRVRef DeformedTrianglePositionBuffer = nullptr;
	FRDGBufferSRVRef RootBarycentricCoordinatesBuffer = nullptr;
	FRDGBufferSRVRef RootToUniqueTriangleIndexBuffer = nullptr;

	// RBF
	FRDGBufferSRVRef RestSamplePositionsBuffer = nullptr;
	FRDGBufferSRVRef MeshSampleWeightsBuffer = nullptr;
	FRDGBufferSRVRef DeformedSamplePositionsBuffer = nullptr;

	bool IsValid() const 		{ return bIsValid && GeometryType != EHairGeometryType::NoneGeometry; }
	bool IsDeformedValid() const{ return bIsDeformedValid && GeometryType != EHairGeometryType::NoneGeometry; }
	bool IsRootValid() const	{ return bIsRootValid && IsValid(); }
};

static FHairGroupInstanceRDG Convert(FRDGBuilder& GraphBuilder, const TRefCountPtr<FHairGroupInstance>& In)
{
	FHairGroupInstanceRDG Out;
	if (In.IsValid() && In->Guides.IsValid())
	{
		const int32 MeshLODIndex = In->HairGroupPublicData ? In->HairGroupPublicData->GetMeshLODIndex() : -1;

		Out.MeshLODIndex 					= MeshLODIndex;
		Out.GeometryType 					= In->GeometryType;
		Out.BindingType						= In->BindingType;

		if (FHairStrandsRestResource* RestResource = In->Guides.RestResource)
		{
			Out.NumPoints						= RestResource->GetPointCount();
			Out.NumCurves						= RestResource->GetCurveCount();
			Out.RestPositionOffsetValue			= (FVector3f)RestResource->GetPositionOffset();
			Out.RestPositionBuffer				= RegisterAsSRV(GraphBuilder, RestResource->PositionBuffer);
			Out.CurvesOffsetsBuffer				= RegisterAsSRV(GraphBuilder, RestResource->CurveBuffer);
			Out.PointToCurveIndexBuffer			= RegisterAsSRV(GraphBuilder, RestResource->PointToCurveBuffer);
		}

		if (FHairStrandsDeformedResource* DeformedResource = In->Guides.DeformedResource)
		{
			FRDGImportedBuffer CurrDeformedPositionBuffer = Register(GraphBuilder, DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateUAV);
			FRDGImportedBuffer PrevDeformedPositionBuffer = Register(GraphBuilder, DeformedResource->GetBuffer(FHairStrandsDeformedResource::EFrameType::Previous), ERDGImportedBufferFlags::None);

			FRDGImportedBuffer CurrDeformedPositionOffset = Register(GraphBuilder, DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateSRV);
			FRDGImportedBuffer PrevDeformedPositionOffset = Register(GraphBuilder, DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Previous), ERDGImportedBufferFlags::None);

			Out.DeformedPositionBufferUAV = CurrDeformedPositionBuffer.UAV;
			Out.DeformedPositionBuffer    = CurrDeformedPositionBuffer.Buffer;
			Out.DeformedPositionOffsetSRV = CurrDeformedPositionOffset.SRV;
			Out.DeformedPositionOffset    = CurrDeformedPositionOffset.Buffer;

			Out.PreviousDeformedPositionBuffer = PrevDeformedPositionBuffer.Buffer;
			Out.PreviousDeformedPositionOffset = PrevDeformedPositionOffset.Buffer;
		}

		const bool bHasSkinnedInterpolation = In->Guides.DeformedRootResource && In->Guides.RestRootResource && 
			In->Guides.DeformedRootResource->IsValid(MeshLODIndex) && In->Guides.RestRootResource->IsValid(MeshLODIndex);
		if (bHasSkinnedInterpolation)
		{
			if (const FHairStrandsLODRestRootResource* RootResource = In->Guides.RestRootResource->GetLOD(MeshLODIndex))
			{
				Out.RestTrianglePositionBuffer		= RegisterAsSRV(GraphBuilder, RootResource->RestUniqueTrianglePositionBuffer);
				Out.RootBarycentricCoordinatesBuffer= RegisterAsSRV(GraphBuilder, RootResource->RootBarycentricBuffer);
				Out.RootToUniqueTriangleIndexBuffer	= RegisterAsSRV(GraphBuilder, RootResource->RootToUniqueTriangleIndexBuffer);
				Out.RestSamplePositionsBuffer		= RegisterAsSRV(GraphBuilder, RootResource->RestSamplePositionsBuffer);
				Out.SampleCount						= RootResource->SampleCount;
			}

			if (const FHairStrandsLODDeformedRootResource* RootResource = In->Guides.DeformedRootResource->GetLOD(MeshLODIndex))
			{
				Out.DeformedTrianglePositionBuffer	= RegisterAsSRV(GraphBuilder, RootResource->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current));
				Out.MeshSampleWeightsBuffer			= RegisterAsSRV(GraphBuilder, RootResource->GetMeshSampleWeightsBuffer((FHairStrandsLODDeformedRootResource::Current)));
				Out.DeformedSamplePositionsBuffer	= RegisterAsSRV(GraphBuilder, RootResource->GetDeformedSamplePositionsBuffer((FHairStrandsLODDeformedRootResource::Current)));
				Out.bIsRootValid					= Out.BindingType == EHairBindingType::Skinning;
			}
		}
	}

	// Set Shader SRV
	FRDGBufferSRVRef DummyStructuredBuffer   = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 16u, FUintVector4(0,0,0,0)));
	FRDGBufferSRVRef DummyByteAddressBuffer  = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 16u));
	FRDGBufferSRVRef DummyVertexBuffer16byte = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 16u), FHairStrandsMeshTrianglePositionFormat::Format);
	FRDGBufferSRVRef DummyVertexBuffer4byte  = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4u), FHairStrandsRootBarycentricFormat::Format);

	check(FHairStrandsMeshTrianglePositionFormat::SizeInByte == 16u);
	check(FHairStrandsRootBarycentricFormat::SizeInByte == 4u);
	check(FHairStrandsRootToUniqueTriangleIndexFormat::SizeInByte == 4u);

	Out.bIsValid = Out.RestPositionBuffer && In->Guides.RestResource->IsInitialized();
	Out.bIsDeformedValid = Out.DeformedPositionBufferUAV && In->Guides.DeformedResource->IsInitialized();

	// Fallback
	if (!Out.RestPositionBuffer) 				{ Out.RestPositionBuffer 				= DummyByteAddressBuffer; };
	if (!Out.DeformedPositionBufferUAV)			{ Out.DeformedPositionBufferUAV 		= GraphBuilder.CreateUAV(GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(16u), TEXT("Niagara.Hair.DummyUAV")), ERDGUnorderedAccessViewFlags::SkipBarrier); };
	if (!Out.DeformedPositionOffsetSRV)			{ Out.DeformedPositionOffsetSRV			= DummyStructuredBuffer; };
	if (!Out.CurvesOffsetsBuffer)				{ Out.CurvesOffsetsBuffer 				= DummyByteAddressBuffer; };
	if (!Out.RestTrianglePositionBuffer)		{ Out.RestTrianglePositionBuffer 		= DummyVertexBuffer16byte; };
	if (!Out.DeformedTrianglePositionBuffer)	{ Out.DeformedTrianglePositionBuffer 	= DummyVertexBuffer16byte; };
	if (!Out.RootBarycentricCoordinatesBuffer)	{ Out.RootBarycentricCoordinatesBuffer 	= DummyVertexBuffer4byte; };
	if (!Out.RootToUniqueTriangleIndexBuffer)	{ Out.RootToUniqueTriangleIndexBuffer 	= DummyVertexBuffer4byte; };
	if (!Out.RestSamplePositionsBuffer)			{ Out.RestSamplePositionsBuffer 		= DummyStructuredBuffer; };
	if (!Out.MeshSampleWeightsBuffer)			{ Out.MeshSampleWeightsBuffer 			= DummyStructuredBuffer; };
	if (!Out.DeformedSamplePositionsBuffer)		{ Out.DeformedSamplePositionsBuffer 	= DummyStructuredBuffer; };
	return Out;
}

//------------------------------------------------------------------------------------------------------------

struct FNDIHairStrandsInfo
{
	int32  GroupIndex = 0;
	int32  LODIndex = 0;
	uint32 NumControlPoints = 0;
	uint32 NumCurves = 0;
	FTransform LocalToWorld = FTransform::Identity;
	bool bHasValidResources = false;

	bool IsValid() const { return bHasValidResources; }
};

//------------------------------------------------------------------------------------------------------------

void FNDIHairStrandsBuffer::Initialize(
	const FNDIHairStrandsInfo& In,
	const TStaticArray<float, 32 * NumScales>& InParamsScale)
{
	bNeedResouces = In.IsValid();
	ParamsScale = InParamsScale;
	bValidGeometryType = false;
}

void FNDIHairStrandsBuffer::Transfer(FRDGBuilder& GraphBuilder, const TStaticArray<float, 32 * NumScales>& InParamsScale)
{
	if (bNeedResouces && ParamsScaleBuffer.IsValid())
	{
		const uint32 ScaleCount = 32 * NumScales;
		const uint32 ScaleBytes = sizeof(float) * ScaleCount;
		
		ParamsScaleBuffer.Initialize(GraphBuilder, TEXT("ParamsScaleBuffer"), EPixelFormat::PF_R32_FLOAT, sizeof(float), ScaleCount);
		GraphBuilder.QueueBufferUpload(ParamsScaleBuffer.GetOrCreateBuffer(GraphBuilder), ParamsScale.GetData(), ScaleBytes);
	}
}

void FNDIHairStrandsBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	//ReadbackBuffer = new FRHIGPUBufferReadback(TEXT("Hair.PositionOffsetBuffer"));
	
	if (bNeedResouces)
	{
		FRDGBuilder GraphBuilder(RHICmdList.GetAsImmediate());
		{
			static const uint32 ZeroData[] = {
				UINT_MAX,UINT_MAX,UINT_MAX,0,0,0,
				UINT_MAX,UINT_MAX,UINT_MAX,0,0,0,
				UINT_MAX,UINT_MAX,UINT_MAX,0,0,0,
				UINT_MAX,UINT_MAX,UINT_MAX,0,0,0
			};
			const uint32 BoundCount = UE_ARRAY_COUNT(ZeroData);

			BoundingBoxBuffer.Initialize(GraphBuilder, TEXT("BoundingBoxBuffer"), EPixelFormat::PF_R32_UINT, sizeof(uint32), BoundCount);
			GraphBuilder.QueueBufferUpload(BoundingBoxBuffer.GetOrCreateBuffer(GraphBuilder), ZeroData, sizeof(ZeroData), ERDGInitialDataFlags::NoCopy);
			BoundingBoxBuffer.EndGraphUsage();
		}
		{
			const uint32 ScaleCount = 32 * NumScales;
			const uint32 ScaleBytes = sizeof(float) * ScaleCount;

			ParamsScaleBuffer.Initialize(GraphBuilder, TEXT("ParamsScaleBuffer"), EPixelFormat::PF_R32_FLOAT, sizeof(float), ScaleCount);
			GraphBuilder.QueueBufferUpload(ParamsScaleBuffer.GetOrCreateBuffer(GraphBuilder), ParamsScale.GetData(), ScaleBytes);
			ParamsScaleBuffer.EndGraphUsage();
		}
		GraphBuilder.Execute();
	}
}

void FNDIHairStrandsBuffer::ReleaseRHI()
{
	//delete ReadbackBuffer;
	//ReadbackBuffer = nullptr;
	
	if (bNeedResouces)
	{
		BoundingBoxBuffer.Release();
		ParamsScaleBuffer.Release();
	}
}

//------------------------------------------------------------------------------------------------------------

ETickingGroup ComputeTickingGroup(const TWeakObjectPtr<UGroomComponent> GroomComponent)
{
	ETickingGroup TickingGroup = NiagaraFirstTickGroup;
	
	if (GroomComponent.Get() != nullptr)
	{
		const ETickingGroup ComponentTickGroup = FMath::Max(GroomComponent->PrimaryComponentTick.TickGroup, GroomComponent->PrimaryComponentTick.EndTickGroup);
		const ETickingGroup ClampedTickGroup = FMath::Clamp(ETickingGroup(ComponentTickGroup + 1), NiagaraFirstTickGroup, NiagaraLastTickGroup);

		TickingGroup = FMath::Max(TickingGroup, ClampedTickGroup);
	}
	return TickingGroup;
}


void FNDIHairStrandsData::Release()
{
	if (HairStrandsBuffer)
	{
		ENQUEUE_RENDER_COMMAND(DeleteResource)(
			[ParamPointerToRelease = HairStrandsBuffer](FRHICommandListImmediate& RHICmdList)
		{
			ParamPointerToRelease->ReleaseResource();
			delete ParamPointerToRelease;
		});
		HairStrandsBuffer = nullptr;
	}
}

void FNDIHairStrandsData::Update(
	UNiagaraDataInterfaceHairStrands* Interface, 
	const FNDIHairStrandsInfo& InData,
	const float DeltaSeconds)
{
	if (Interface != nullptr && InData.IsValid())
	{
		const UGroomAsset* GroomAsset = Interface->SourceComponent->GroomAsset;

		WorldTransform = InData.LocalToWorld;

		const bool bHasValidBindingAsset = (Interface->IsComponentValid() && Interface->SourceComponent->BindingAsset && GroomAsset);

		GlobalInterpolation = bHasValidBindingAsset ? GroomAsset->IsGlobalInterpolationEnable(InData.GroupIndex, InData.LODIndex) : false;
		bSkinningTransfer = bHasValidBindingAsset ?
			(Interface->SourceComponent->BindingAsset->GetSourceSkeletalMesh() && Interface->SourceComponent->BindingAsset->GetTargetSkeletalMesh() &&
			 Interface->SourceComponent->BindingAsset->GetSourceSkeletalMesh() != Interface->SourceComponent->BindingAsset->GetTargetSkeletalMesh()) : false;
		
		TickingGroup = Interface->IsComponentValid() ? ComputeTickingGroup(Interface->SourceComponent) : NiagaraFirstTickGroup;

		const bool bIsSimulationEnable = Interface->IsComponentValid() ? Interface->SourceComponent->IsSimulationEnable(InData.GroupIndex, InData.LODIndex) : 
			GroomAsset ? GroomAsset->IsSimulationEnable(InData.GroupIndex, InData.LODIndex) : false;

		if (GroomAsset != nullptr && GroomAsset->GetHairGroupsPhysics().IsValidIndex(InData.GroupIndex) && bIsSimulationEnable)
		{
			const FHairGroupsPhysics& HairPhysics = GroomAsset->GetHairGroupsPhysics()[InData.GroupIndex];
			StrandsSize = static_cast<uint8>(HairPhysics.StrandsParameters.StrandsSize);

			HairGroupIndex = Interface->IsComponentValid() ? InData.GroupIndex : -1;
			HairGroupInstSource = Interface->IsComponentValid() ? Interface->SourceComponent : nullptr;

			SubSteps = HairPhysics.SolverSettings.SubSteps;
			IterationCount = HairPhysics.SolverSettings.IterationCount;

			GravityVector = HairPhysics.ExternalForces.GravityVector;
			GravityPreloading = HairPhysics.SolverSettings.GravityPreloading;
			AirDrag = HairPhysics.ExternalForces.AirDrag;
			AirVelocity = HairPhysics.ExternalForces.AirVelocity;

			SolveBend = HairPhysics.MaterialConstraints.BendConstraint.SolveBend;
			ProjectBend = HairPhysics.MaterialConstraints.BendConstraint.ProjectBend;
			BendDamping = HairPhysics.MaterialConstraints.BendConstraint.BendDamping;
			BendStiffness = HairPhysics.MaterialConstraints.BendConstraint.BendStiffness;

			SolveStretch = HairPhysics.MaterialConstraints.StretchConstraint.SolveStretch;
			ProjectStretch = HairPhysics.MaterialConstraints.StretchConstraint.ProjectStretch;
			StretchDamping = HairPhysics.MaterialConstraints.StretchConstraint.StretchDamping;
			StretchStiffness = HairPhysics.MaterialConstraints.StretchConstraint.StretchStiffness;

			SolveCollision = HairPhysics.MaterialConstraints.CollisionConstraint.SolveCollision;
			ProjectCollision = HairPhysics.MaterialConstraints.CollisionConstraint.ProjectCollision;
			StaticFriction = HairPhysics.MaterialConstraints.CollisionConstraint.StaticFriction;
			KineticFriction = HairPhysics.MaterialConstraints.CollisionConstraint.KineticFriction;
			StrandsViscosity = HairPhysics.MaterialConstraints.CollisionConstraint.StrandsViscosity;
			GridDimension = HairPhysics.MaterialConstraints.CollisionConstraint.GridDimension;
			CollisionRadius = HairPhysics.MaterialConstraints.CollisionConstraint.CollisionRadius;

			StrandsDensity = HairPhysics.StrandsParameters.StrandsDensity;
			StrandsSmoothing = HairPhysics.StrandsParameters.StrandsSmoothing;
			StrandsThickness = HairPhysics.StrandsParameters.StrandsThickness;

			for (int32 i = 0; i < StrandsSize; ++i)
			{
				const float VertexCoord = static_cast<float>(i) / (StrandsSize-1.0);
				ParamsScale[32 * BendOffset + i] = HairPhysics.MaterialConstraints.BendConstraint.BendScale.GetRichCurveConst()->Eval(VertexCoord);
				ParamsScale[32 * StretchOffset + i] = HairPhysics.MaterialConstraints.StretchConstraint.StretchScale.GetRichCurveConst()->Eval(VertexCoord);
				ParamsScale[32 * RadiusOffset + i] = HairPhysics.MaterialConstraints.CollisionConstraint.RadiusScale.GetRichCurveConst()->Eval(VertexCoord);
				ParamsScale[32 * ThicknessOffset + i] = HairPhysics.StrandsParameters.ThicknessScale.GetRichCurveConst()->Eval(VertexCoord);
			}

			NumStrands = InData.NumCurves;
			LocalSimulation = false;
			BoneTransform = FTransform::Identity;
			TimeDilation = 1.0f;

			if (Interface->IsComponentValid())
			{
				if(UWorld* World = Interface->SourceComponent->GetWorld())
				{ 
					if(AWorldSettings* WorldSettings = World->GetWorldSettings())
					{ 
						TimeDilation = WorldSettings->GetEffectiveTimeDilation();
					}
				}

				const FHairSimulationSettings& SimulationSettings = Interface->SourceComponent->SimulationSettings;
				LocalSimulation = SimulationSettings.SimulationSetup.bLocalSimulation;
				Interface->SourceComponent->BuildSimulationTransform(BoneTransform);
				BoneTransform.NormalizeRotation();

				// Convert to double for LWC
				FMatrix44d BoneTransformDouble = BoneTransform.ToMatrixWithScale();
				const FMatrix44d WorldTransformDouble = WorldTransform.ToMatrixWithScale();

				if(DeltaSeconds != 0.0f && !ForceReset)
				{
					const FMatrix44d PreviousBoneTransformDouble = PreviousBoneTransform.ToMatrixWithScale();
					const FMatrix44d DeltaTransformDouble =  BoneTransformDouble * PreviousBoneTransformDouble.Inverse();
					
					const FTransform DeltaTransform = FTransform(FMatrix(DeltaTransformDouble));
					const FQuat DeltaRotation = DeltaTransform.GetRotation();
					
					// Apply linear velocity scale
					BoneLinearVelocity = FVector3f(FMath::Clamp(SimulationSettings.SimulationSetup.LinearVelocityScale, 0.f, 1.f) * DeltaTransform.GetTranslation() / DeltaSeconds);
					BoneLinearAcceleration = (BoneLinearVelocity-PreviousBoneLinearVelocity) / DeltaSeconds;

					
					// Apply angular velocity scale
					BoneAngularVelocity = (FVector3f)BoneTransform.TransformVector(DeltaRotation.GetRotationAxis() * DeltaRotation.GetAngle() *
						FMath::Clamp(SimulationSettings.SimulationSetup.AngularVelocityScale, 0.f, 1.f)) / DeltaSeconds;
					BoneAngularAcceleration = (BoneAngularVelocity-PreviousBoneAngularVelocity) / DeltaSeconds;
				}
				else
				{
					BoneLinearVelocity = FVector3f::Zero();
					BoneAngularVelocity = FVector3f::Zero();

					BoneLinearAcceleration = FVector3f::Zero();
					BoneAngularAcceleration = FVector3f::Zero();
				}
				
				PreviousBoneTransform = BoneTransform;
				PreviousBoneLinearVelocity = BoneLinearVelocity;
				PreviousBoneAngularVelocity = BoneAngularVelocity;

				BoneTransformDouble = BoneTransformDouble * WorldTransformDouble.Inverse();
				const FMatrix44d WorldTransformFloat = BoneTransformDouble;
				BoneTransform = FTransform(WorldTransformFloat);
				BoneTransform.NormalizeRotation();
				
				if (SimulationSettings.bOverrideSettings)
				{
					GravityVector = SimulationSettings.ExternalForces.GravityVector;
					AirDrag = SimulationSettings.ExternalForces.AirDrag;
					AirVelocity = SimulationSettings.ExternalForces.AirVelocity;

					BendDamping = SimulationSettings.MaterialConstraints.BendDamping;
					BendStiffness = SimulationSettings.MaterialConstraints.BendStiffness;

					StretchDamping = SimulationSettings.MaterialConstraints.StretchDamping;
					StretchStiffness = SimulationSettings.MaterialConstraints.StretchStiffness;

					StaticFriction = SimulationSettings.MaterialConstraints.StaticFriction;
					KineticFriction = SimulationSettings.MaterialConstraints.KineticFriction;
					StrandsViscosity = SimulationSettings.MaterialConstraints.StrandsViscosity;
					CollisionRadius = SimulationSettings.MaterialConstraints.CollisionRadius;
				}
			}
		}
		else
		{
			ResetDatas();
		}
	}
}

bool FNDIHairStrandsData::Init(UNiagaraDataInterfaceHairStrands* Interface, FNiagaraSystemInstance* SystemInstance)
{
	HairStrandsBuffer = nullptr;

	if (Interface != nullptr)
	{
		{
			FNDIHairStrandsInfo InfoData;
			Interface->ExtractDatasAndResources(SystemInstance, InfoData);
			Update(Interface, InfoData, 0.0f);

			HairStrandsBuffer = new FNDIHairStrandsBuffer();
			HairStrandsBuffer->Initialize(InfoData, ParamsScale);

			BeginInitResource(HairStrandsBuffer);

			ForceReset = true;
		}
	}

	return true;
}

//------------------------------------------------------------------------------------------------------------

void FNDIHairStrandsProxy::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNDIHairStrandsData* SourceData = static_cast<FNDIHairStrandsData*>(PerInstanceData);
	FNDIHairStrandsData* TargetData = &(SystemInstancesToProxyData.FindOrAdd(Instance));

	ensure(TargetData);
	if (TargetData)
	{
		TargetData->CopyDatas(SourceData);
	}
	else
	{
		UE_LOGF(LogHairStrands, Log, "ConsumePerInstanceDataFromGameThread() ... could not find %ls", *FNiagaraUtilities::SystemInstanceIDToString(Instance));
	}
	SourceData->~FNDIHairStrandsData();
}

void FNDIHairStrandsProxy::InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	check(!SystemInstancesToProxyData.Contains(SystemInstance));

	FNDIHairStrandsData* TargetData = SystemInstancesToProxyData.Find(SystemInstance);
	TargetData = &SystemInstancesToProxyData.Add(SystemInstance);
}

void FNDIHairStrandsProxy::DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	//check(SystemInstancesToProxyData.Contains(SystemInstance));
	SystemInstancesToProxyData.Remove(SystemInstance);
}

//------------------------------------------------------------------------------------------------------------

FORCEINLINE bool RequiresSimulationReset(FNiagaraSystemInstance* SystemInstance, uint32& OldSkeletalMeshes)
{
	uint32 NewSkeletalMeshes = 0;
	if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
	{
		if (AActor* RootActor = AttachComponent->GetAttachmentRootActor())
		{
			for (UActorComponent* ActorComp : RootActor->GetComponents())
			{
				USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(ActorComp);
				if (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset())
				{
					NewSkeletalMeshes += GetTypeHash(SkelMeshComp->GetSkeletalMeshAsset()->GetName());
				}
			}
		}
	}
	bool bNeedReset = NewSkeletalMeshes != OldSkeletalMeshes;
	OldSkeletalMeshes = NewSkeletalMeshes;
	return bNeedReset;
}

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfaceHairStrands::UNiagaraDataInterfaceHairStrands(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultSource(nullptr)
	, SourceActor(nullptr)
	, SourceComponent(nullptr)
{

	Proxy.Reset(new FNDIHairStrandsProxy());
}

bool UNiagaraDataInterfaceHairStrands::IsComponentValid() const
{
	return (SourceComponent.IsValid() && SourceComponent != nullptr);
}

void UNiagaraDataInterfaceHairStrands::ExtractSourceComponent(FNiagaraSystemInstance* SystemInstance)
{
	SourceComponent = nullptr;
	if (SourceActor)
	{
		AGroomActor* HairStrandsActor = Cast<AGroomActor>(SourceActor);
		if (HairStrandsActor != nullptr)
		{
			SourceComponent = HairStrandsActor->GetGroomComponent();
		}
		else
		{
			SourceComponent = SourceActor->FindComponentByClass<UGroomComponent>();
		}
	}
	else if (SystemInstance)
	{
		if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
		{
			// First, look to our attachment hierarchy for the source component
			for (USceneComponent* Curr = AttachComponent; Curr; Curr = Curr->GetAttachParent())
			{
				UGroomComponent* SourceComp = Cast<UGroomComponent>(Curr);
				if (SourceComp && SourceComp->GroomAsset)
				{
					SourceComponent = SourceComp;
					break;
				}
			}

			if (!SourceComponent.IsValid())
			{
				// Next, check out outer chain to look for the component
				if (UGroomComponent* OuterComp = AttachComponent->GetTypedOuter<UGroomComponent>())
				{
					SourceComponent = OuterComp;
				}
				else if (AActor* Owner = AttachComponent->GetAttachmentRootActor())
				{
					// Lastly, look through all our root actor's components for a sibling component
					for (UActorComponent* ActorComp : Owner->GetComponents())
					{
						UGroomComponent* SourceComp = Cast<UGroomComponent>(ActorComp);
						if (SourceComp && SourceComp->GroomAsset)
						{
							SourceComponent = SourceComp;
							break;
						}
					}
				}
			}
		}
	}
}

void UNiagaraDataInterfaceHairStrands::ExtractDatasAndResources(
	FNiagaraSystemInstance* SystemInstance,
	FNDIHairStrandsInfo& Out)
{
	ExtractSourceComponent(SystemInstance);

	Out.NumControlPoints = 0;
	Out.NumCurves = 0;
	Out.GroupIndex = -1;
	Out.LODIndex = -1;
	Out.LocalToWorld = FTransform::Identity;
	Out.bHasValidResources = false;

	if (IsComponentValid() && SystemInstance)
	{
		for (int32 NiagaraIndex = 0, NiagaraCount = SourceComponent->NiagaraComponents.Num(); NiagaraIndex < NiagaraCount; ++NiagaraIndex)
		{
			if (UNiagaraComponent* NiagaraComponent = SourceComponent->NiagaraComponents[NiagaraIndex])
			{
				if (FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController())
				{
					if (SystemInstanceController->GetSystemInstanceID() == SystemInstance->GetId())
					{
						Out.GroupIndex = NiagaraIndex;
						break;
					}
				}
			}
		}
		if (Out.GroupIndex >= 0 && Out.GroupIndex < SourceComponent->NiagaraComponents.Num())
		{
			Out.LODIndex = SourceComponent->GetForcedLOD();
			Out.LocalToWorld = SourceComponent->GetComponentTransform();
			if (SourceComponent->GroomAsset && SourceComponent->GroomAsset->GetHairGroupsPlatformData().IsValidIndex(Out.GroupIndex))
			{
				const FHairGroupPlatformData::FGuides& Guides = SourceComponent->GroomAsset->GetHairGroupsPlatformData()[Out.GroupIndex].Guides;
				Out.NumControlPoints = Guides.BulkData.GetNumPoints();
				Out.NumCurves = Guides.BulkData.GetNumCurves();
				Out.bHasValidResources = SourceComponent->GetGuideStrandsRestResource(Out.GroupIndex) != nullptr;
			}
		}
	}
	else if (DefaultSource != nullptr)
	{
		Out.NumControlPoints = 0;
		Out.GroupIndex = 0;
		Out.LODIndex = 0;
		Out.LocalToWorld = SystemInstance ? SystemInstance->GetWorldTransform() : FTransform::Identity;
	}
}

ETickingGroup UNiagaraDataInterfaceHairStrands::CalculateTickGroup(const void* PerInstanceData) const
{
	const FNDIHairStrandsData* InstanceData = static_cast<const FNDIHairStrandsData*>(PerInstanceData);

	if (InstanceData)
	{
		return InstanceData->TickingGroup;
	}
	return NiagaraFirstTickGroup;
}

bool UNiagaraDataInterfaceHairStrands::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIHairStrandsData* InstanceData = new (PerInstanceData) FNDIHairStrandsData();
	check(InstanceData);

	return InstanceData->Init(this, SystemInstance);
}

void UNiagaraDataInterfaceHairStrands::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIHairStrandsData* InstanceData = static_cast<FNDIHairStrandsData*>(PerInstanceData);

	InstanceData->Release();
	InstanceData->~FNDIHairStrandsData();

	FNDIHairStrandsProxy* ThisProxy = GetProxyAs<FNDIHairStrandsProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			ThisProxy->SystemInstancesToProxyData.Remove(InstanceID);
		}
	);
}

bool UNiagaraDataInterfaceHairStrands::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDIHairStrandsData* InstanceData = static_cast<FNDIHairStrandsData*>(PerInstanceData);

	FNDIHairStrandsInfo InfoData;
	ExtractDatasAndResources(SystemInstance, InfoData);

	if (SourceComponent != nullptr)
	{
		InstanceData->ForceReset = SourceComponent->bResetSimulation || RequiresSimulationReset(SystemInstance, InstanceData->SkeletalMeshes);
		if (InstanceData->ForceReset)
		{
			FNDIHairStrandsBuffer* LocalStrandsBuffer = InstanceData->HairStrandsBuffer;
			ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
				[LocalStrandsBuffer](FRHICommandListImmediate& CmdList)
				{
					if(LocalStrandsBuffer)
					{
						LocalStrandsBuffer->bShouldReset = true;
					}
				}
			);
		}
	}
	InstanceData->Update(this, InfoData, InDeltaSeconds);
	return false;
}

//------------------------------------------------------------------------------------------------------------

#define NIAGARA_HAIR_STRANDS_THREAD_COUNT_INTERPOLATE 32

/** Compute shader to interpolate the groom position from the sim cache */
class FInterpolateGroomGuidesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInterpolateGroomGuidesCS)
	SHADER_USE_PARAMETER_STRUCT(FInterpolateGroomGuidesCS, FGlobalShader);

	class FInterpolationType : SHADER_PERMUTATION_INT("PERMUTATION_INTERPOLATION", 2);
	using FPermutationDomain = TShaderPermutationDomain<FInterpolationType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>,	RestTrianglePositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>,	DeformedTrianglePositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,	RootBarycentricCoordinatesBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,	RootToUniqueTriangleIndexBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer,	PointToCurveIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer,DeformedPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer,	RestPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,		CurvesOffsetsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, DeformedPositionOffset)
		SHADER_PARAMETER(FVector3f,				RestPositionOffset)
		SHADER_PARAMETER(FMatrix44f,			WorldToLocal)

		SHADER_PARAMETER(int,					StrandsSize)
		SHADER_PARAMETER(int,					NumPoints)

		SHADER_PARAMETER_SRV(Buffer<float>,		NiagaraFloatBuffer)
		SHADER_PARAMETER(int,					NiagaraFloatStride)

		SHADER_PARAMETER(int,					NodePositionComponent)
		SHADER_PARAMETER(int,					RestPositionComponent)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_HAIR_STRANDS_THREAD_COUNT_INTERPOLATE);
	}
};

IMPLEMENT_SHADER_TYPE(, FInterpolateGroomGuidesCS, TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraInterpolateGroomGuides.usf"), TEXT("MainCS"), SF_Compute);

//------------------------------------------------------------------------------------------------------------

FMatrix44f ComputeBoneTransform(const FNDIHairStrandsData* InstanceData, const TRefCountPtr<FHairGroupInstance>& GroupInstance)
{
	if(InstanceData)
	{
		const FTransform RigidTransform = GroupInstance.IsValid() ? GroupInstance->Debug.RigidCurrentLocalToWorld : InstanceData->WorldTransform;
		if(RigidTransform.IsValid() && InstanceData->BoneTransform.IsValid())
		{
			return  FMatrix44f((InstanceData->BoneTransform * RigidTransform).ToMatrixWithScale());
		}
	}
	return FMatrix44f::Identity;
}

FMatrix44f ComputeWorldTransform(const FNDIHairStrandsData* InstanceData, const TRefCountPtr<FHairGroupInstance>& GroupInstance)
{
	if(InstanceData)
	{
		const FMatrix44f BoneTransformFloat = ComputeBoneTransform(InstanceData, GroupInstance);

		const FTransform WorldTransform = GroupInstance.IsValid() ? GroupInstance->GetCurrentLocalToWorld() : InstanceData->WorldTransform;
		FMatrix44f WorldTransformFloat = WorldTransform.IsValid() ? FMatrix44f(WorldTransform.ToMatrixWithScale()) : FMatrix44f::Identity;
	
		if (InstanceData->LocalSimulation)
		{
			const FMatrix44d WorldTransformDouble(WorldTransformFloat);
			const FMatrix44d BoneTransformDouble(BoneTransformFloat);

			// Due to large world coordinate we compute the relative world transform in double precision 
			WorldTransformFloat = FMatrix44f(WorldTransformDouble * BoneTransformDouble.Inverse());
		}
		
		if (WorldTransformFloat.ContainsNaN() && InstanceData->WorldTransform.IsValid())
		{
			WorldTransformFloat = FMatrix44f(InstanceData->WorldTransform.ToMatrixWithScale());
		}
		return WorldTransformFloat;
	}
	return FMatrix44f::Identity;
}

static void InterpolateGroomGuides(
	FRDGBuilder& GraphBuilder, 
	FNiagaraDataBuffer& ParticlesBuffer,
	const uint32 NodePositionComponent,  
	const uint32 RestPositionComponent, 
	FNDIHairStrandsBuffer* HairStrandsBuffer,
	const TRefCountPtr<FHairGroupInstance>& HairGroupInstance,
	const uint32 StrandsSize, 
	const FMatrix44f& WorldToLocal)
{
	FHairGroupInstanceRDG InstanceRDG = Convert(GraphBuilder, HairGroupInstance);

	const bool bIsHairValid = HairStrandsBuffer && HairStrandsBuffer->IsInitialized();
	const bool bIsRestValid = bIsHairValid && InstanceRDG.IsValid();
	const bool bIsDeformedValid = bIsHairValid && InstanceRDG.IsDeformedValid();
	const bool bIsRootValid = bIsHairValid && InstanceRDG.IsRootValid();

	if(bIsRestValid && bIsDeformedValid)
	{
		FInterpolateGroomGuidesCS::FPermutationDomain InterpolationDomain;
		InterpolationDomain.Set<FInterpolateGroomGuidesCS::FInterpolationType>(!bIsRootValid);

		TShaderMapRef<FInterpolateGroomGuidesCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), InterpolationDomain);

		FInterpolateGroomGuidesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInterpolateGroomGuidesCS::FParameters>();
		PassParameters->StrandsSize = StrandsSize;
		PassParameters->NumPoints = InstanceRDG.NumPoints;

		PassParameters->NiagaraFloatBuffer = ParticlesBuffer.GetGPUBufferFloat().SRV;
		PassParameters->NiagaraFloatStride = ParticlesBuffer.GetFloatStride() / sizeof(float);
		PassParameters->NodePositionComponent = NodePositionComponent;
		PassParameters->RestPositionComponent = RestPositionComponent;
		PassParameters->WorldToLocal = WorldToLocal;

		PassParameters->DeformedPositionOffset = InstanceRDG.DeformedPositionOffsetSRV;
		PassParameters->RestPositionOffset = InstanceRDG.RestPositionOffsetValue;
		PassParameters->DeformedPositionBuffer = InstanceRDG.DeformedPositionBufferUAV;
		PassParameters->RestPositionBuffer = InstanceRDG.RestPositionBuffer;
		
		PassParameters->PointToCurveIndexBuffer = InstanceRDG.PointToCurveIndexBuffer;
		PassParameters->CurvesOffsetsBuffer = InstanceRDG.CurvesOffsetsBuffer;
		PassParameters->RestTrianglePositionBuffer = InstanceRDG.RestTrianglePositionBuffer;
		PassParameters->DeformedTrianglePositionBuffer = InstanceRDG.DeformedTrianglePositionBuffer;
		PassParameters->RootBarycentricCoordinatesBuffer = InstanceRDG.RootBarycentricCoordinatesBuffer;
		PassParameters->RootToUniqueTriangleIndexBuffer = InstanceRDG.RootToUniqueTriangleIndexBuffer;
		
		const uint32 GroupSize = NIAGARA_HAIR_STRANDS_THREAD_COUNT_INTERPOLATE;
		const uint32 DispatchCount = FMath::DivideAndRoundUp(InstanceRDG.NumPoints, GroupSize);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("InterpolateGroomGuidesCS"),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FIntVector(DispatchCount, 1, 1)
		);

		if (InstanceRDG.DeformedPositionBuffer && InstanceRDG.PreviousDeformedPositionBuffer)
		{
			AddCopyBufferPass(GraphBuilder, InstanceRDG.PreviousDeformedPositionBuffer, InstanceRDG.DeformedPositionBuffer);
			AddCopyBufferPass(GraphBuilder, InstanceRDG.PreviousDeformedPositionOffset, InstanceRDG.DeformedPositionOffset);
		}
	}
}

void UNiagaraDataInterfaceHairStrands::SimCachePostReadFrame(void* OptionalPerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIHairStrandsData* PerIntanceData = static_cast<FNDIHairStrandsData*>(OptionalPerInstanceData);
	PerInstanceTick(PerIntanceData, SystemInstance, 0.0f);

	const FNiagaraVariable NodePositionVariable(FNiagaraTypeDefinition::GetVec3Def(), FName("NodePosition"));
	const FNiagaraVariable RestPositionVariable(FNiagaraTypeDefinition::GetVec3Def(), FName("RestPosition"));

	for (TSharedRef<FNiagaraEmitterInstance, ESPMode::ThreadSafe> EmitterInst : SystemInstance->GetEmitters())
	{
		FNiagaraDataSet& EmitterDataSet = EmitterInst->GetParticleData();
		const FNiagaraDataSetCompiledData& CompiledDataSet = EmitterDataSet.GetCompiledData();

		// Validate we have the components
		const int32 NodePositionIndex = CompiledDataSet.Variables.IndexOfByKey(NodePositionVariable);
		const int32 RestPositionIndex = CompiledDataSet.Variables.IndexOfByKey(RestPositionVariable);
		if (!CompiledDataSet.VariableLayouts.IsValidIndex(NodePositionIndex) || !CompiledDataSet.VariableLayouts.IsValidIndex(RestPositionIndex))
		{
			continue;
		}

		// Skip compressed systems
		if (CompiledDataSet.VariableLayouts[NodePositionIndex].GetNumFloatComponents() != 3 || CompiledDataSet.VariableLayouts[RestPositionIndex].GetNumFloatComponents() != 3)
		{
			continue;
		}

		// Queue command to interpolate the data
		const int NodePositionComponent = CompiledDataSet.VariableLayouts[NodePositionIndex].GetFloatComponentStart();
		const int RestPositionComponent = CompiledDataSet.VariableLayouts[RestPositionIndex].GetFloatComponentStart();

		FNDIHairStrandsData ProxyData;
		ProxyData.CopyDatas(PerIntanceData);
				
		ENQUEUE_RENDER_COMMAND(NiagaraInterpolateGroomSimCache) (
			[ProxyData, DataSet_RT=&EmitterDataSet, NodePositionComponent, RestPositionComponent](FRHICommandListImmediate& RHICmdList)
			{
				FMemMark MemMark(FMemStack::Get());
				FRDGBuilder GraphBuilder(RHICmdList);

				const TRefCountPtr<FHairGroupInstance> HairGroupInstance = GetRefCountedHairGroupInstance(&ProxyData);
				
				FNiagaraDataBuffer& ParticlesBuffer = DataSet_RT->GetCurrentDataChecked();
				if(!GEnableProxyInstanceTransform)
				{
					const FMatrix44f WorldToLocal = ComputeWorldTransform(&ProxyData, HairGroupInstance).Inverse();
					InterpolateGroomGuides(GraphBuilder, ParticlesBuffer, NodePositionComponent, RestPositionComponent,
						ProxyData.HairStrandsBuffer, HairGroupInstance, ProxyData.StrandsSize, WorldToLocal);
				}
				else
				{
					const FMatrix44f WorldToLocal = ComputeWorldTransform(&ProxyData, ProxyData.HairGroupInstance).Inverse();
					InterpolateGroomGuides(GraphBuilder, ParticlesBuffer, NodePositionComponent, RestPositionComponent,
						ProxyData.HairStrandsBuffer, ProxyData.HairGroupInstance, ProxyData.StrandsSize, WorldToLocal);
				}
				GraphBuilder.Execute();
			}
		);
	}
}

TArray<FNiagaraVariableBase> UNiagaraDataInterfaceHairStrands::GetSimCacheRendererAttributes(const UObject* UsageContext) const
{
	TArray<FNiagaraVariableBase> HairStrandsCachedVariables;
	if (const UNiagaraEmitter* UsageEmitter = Cast<UNiagaraEmitter>(UsageContext))
	{
		HairStrandsCachedVariables.Emplace(FNiagaraTypeDefinition::GetVec3Def(), FName(UsageEmitter->GetUniqueEmitterName() + TEXT(".") + FNiagaraConstants::ParticleAttributeNamespaceString + TEXT(".") + TEXT("NodePosition")));
		HairStrandsCachedVariables.Emplace(FNiagaraTypeDefinition::GetVec3Def(), FName(UsageEmitter->GetUniqueEmitterName() + TEXT(".") + FNiagaraConstants::ParticleAttributeNamespaceString + TEXT(".") + TEXT("RestPosition")));
	}

	return HairStrandsCachedVariables;
}

bool UNiagaraDataInterfaceHairStrands::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceHairStrands* OtherTyped = CastChecked<UNiagaraDataInterfaceHairStrands>(Destination);
	OtherTyped->SourceActor= SourceActor;
	OtherTyped->SourceComponent = SourceComponent;
	OtherTyped->DefaultSource = DefaultSource;

	return true;
}

bool UNiagaraDataInterfaceHairStrands::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceHairStrands* OtherTyped = CastChecked<const UNiagaraDataInterfaceHairStrands>(Other);

	return  (OtherTyped->SourceActor == SourceActor) && (OtherTyped->SourceComponent == SourceComponent)
		&& (OtherTyped->DefaultSource == DefaultSource);
}

void UNiagaraDataInterfaceHairStrands::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA

// Codegen optimization degenerates for very long functions like GetFunctions when combined with the invokation of lots of FORCEINLINE methods.
// We don't need this code to be particularly fast anyway. The other way to improve this code compilation time would be to split it in multiple functions.
BEGIN_FUNCTION_BUILD_OPTIMIZATION 

void UNiagaraDataInterfaceHairStrands::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIHairStrandsLocal;
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNumStrandsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Strands")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStrandSizeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Strand Size")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSubStepsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Sub Steps")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetIterationCountName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Iteration Count")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetGravityVectorName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Gravity Vector")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetGravityPreloadingName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Gravity Preloading")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetAirDragName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Air Drag")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetAirVelocityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Air Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSolveBendName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Solve Bend")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetProjectBendName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Project Bend")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBendDampingName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Damping")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBendStiffnessName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Stiffness")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBendScaleName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Scale")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSolveStretchName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Solve Stretch")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetProjectStretchName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Project Stretch")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStretchDampingName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Damping")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStretchStiffnessName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Stiffness")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStretchScaleName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Scale")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSolveCollisionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Solve Collision")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetProjectCollisionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Project Collision")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStaticFrictionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Static Fraction")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetKineticFrictionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Kinetic Friction")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStrandsViscosityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Strands Viscosity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetGridDimensionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Dimension")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetCollisionRadiusName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Collision Radius")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetRadiusScaleName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Radius Scale")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStrandsDensityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Strands Density")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetTimeDilationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Dilation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStrandsSmoothingName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Strands Smoothing")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetStrandsThicknessName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Strands Thickness")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetThicknessScaleName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Thickness Scale")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetWorldTransformName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("World Transform")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetWorldInverseName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("World Inverse")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPointPositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Vertex Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Vertex Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeNodePositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Smoothing Filter")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeNodeOrientationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeNodeMassName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Strands Density")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Mass")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeNodeInertiaName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Strands Density")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Inertia")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeEdgeLengthName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Node Offset")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Edge Length")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeEdgeRotationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Edge Rotation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeRestPositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeRestOrientationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rest Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeLocalStateName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rest Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Local Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AdvectNodePositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Mass")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Position Mobile")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("External Force")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Force Gradient")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Linear Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Linear Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AdvectNodeOrientationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Inertia")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Orientation Mobile")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("External Torque")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Torque Gradient")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Angular Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Angular Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdateLinearVelocityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Previous Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Linear Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdateAngularVelocityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Previous Orientation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Angular Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetLocalVectorName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("World Vector")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Vector")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetWorldVectorName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Vector")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Is Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("World Vector")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AttachNodePositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AttachNodeOrientationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rest Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = AttachNodeStateName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Local Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Local Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdateNodeStateName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdatePointPositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Report Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ResetPointPositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Report Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBoundingBoxName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Box Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Box Center")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Box Extent")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ResetBoundingBoxName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = BuildBoundingBoxName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetupDistanceSpringMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Node Offset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveDistanceSpringMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Node Offset")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Multiplier")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ProjectDistanceSpringMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Node Offset")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetupAngularSpringMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveAngularSpringMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Direction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ProjectAngularSpringMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Direction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetupStretchRodMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveStretchRodMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ProjectStretchRodMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Stretch Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetupBendRodMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveBendRodMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rest Rotation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ProjectBendRodMaterialName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rest Rotation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveHardCollisionConstraintName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Penetration Depth")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Normal")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Static Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Kinetic Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Constraint Multiplier")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ProjectHardCollisionConstraintName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Penetration Depth")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Normal")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Static Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Kinetic Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetupSoftCollisionConstraintName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Collision Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveSoftCollisionConstraintName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Penetration Depth")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Normal")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Static Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Kinetic Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Damping")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Compliance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Material Weight")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Material Multiplier")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ProjectSoftCollisionConstraintName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Enable Constraint")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Collision Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Penetration Depth")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Collision Normal")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Static Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Kinetic Friction")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeEdgeDirectionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Mass")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Gravity Vector")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Gravity Preloading")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Bend Stiffness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Rest Length")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Rest Direction")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdateMaterialFrameName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeMaterialFrameName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Node Orientation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeAirDragForceName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Air Density")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Air Viscosity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Air Drag")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Air Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Thickness")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Drag Force")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Drag Gradient")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = InitGridSamplesName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Linear Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Node Mass")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Samples")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Delta Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Delta Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Sample Mass")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSampleStateName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Node Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Linear Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Delta Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Delta Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Num Samples")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Sample Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Sample Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Sample Velocity")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NeedSimulationResetName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Reset Simulation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = HasGlobalInterpolationName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Global Interpolation")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NeedRestUpdateName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Hair Strands")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Rest Update")));

		OutFunctions.Add(Sig);
	}
}
END_FUNCTION_BUILD_OPTIMIZATION

#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetNumStrands);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandSize);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSubSteps);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetIterationCount);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetGravityPreloading);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetGravityVector);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetAirDrag);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetAirVelocity);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSolveBend);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetProjectBend);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBendDamping);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBendStiffness);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBendScale);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSolveStretch);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetProjectStretch);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStretchDamping);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStretchStiffness);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStretchScale);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSolveCollision);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetProjectCollision);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStaticFriction);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetKineticFriction);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsViscosity);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetGridDimension);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetCollisionRadius);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetRadiusScale);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsDensity);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetTimeDilation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsSmoothing);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsThickness);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetThicknessScale);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetWorldTransform);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetWorldInverse);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetPointPosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodePosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodeOrientation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodeMass);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodeInertia);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeEdgeLength);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeEdgeRotation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeRestPosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeRestOrientation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeLocalState);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AttachNodePosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AttachNodeOrientation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AttachNodeState);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateNodeState);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdatePointPosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ResetPointPosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, EvalSkinnedPosition);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBoundingBox);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ResetBoundingBox);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, BuildBoundingBox);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AdvectNodePosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AdvectNodeOrientation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateLinearVelocity);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateAngularVelocity);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupDistanceSpringMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveDistanceSpringMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectDistanceSpringMaterial);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupAngularSpringMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveAngularSpringMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectAngularSpringMaterial);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupStretchRodMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveStretchRodMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectStretchRodMaterial);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupBendRodMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveBendRodMaterial);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectBendRodMaterial);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveHardCollisionConstraint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectHardCollisionConstraint);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupSoftCollisionConstraint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectSoftCollisionConstraint);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveSoftCollisionConstraint);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeEdgeDirection);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateMaterialFrame);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeMaterialFrame);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeAirDragForce);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, NeedSimulationReset);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, HasGlobalInterpolation);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, NeedRestUpdate);

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, InitGridSamples);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSampleState);

void UNiagaraDataInterfaceHairStrands::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIHairStrandsLocal;

	if (BindingInfo.Name == GetNumStrandsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetNumStrands)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStrandSizeName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandSize)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetSubStepsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSubSteps)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetIterationCountName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetIterationCount)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetGravityVectorName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetGravityVector)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetGravityPreloadingName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetGravityPreloading)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetAirDragName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetAirDrag)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetAirVelocityName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetAirVelocity)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetSolveBendName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSolveBend)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetProjectBendName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetProjectBend)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetBendDampingName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBendDamping)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetBendStiffnessName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBendStiffness)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetBendScaleName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBendScale)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetSolveStretchName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSolveStretch)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetProjectStretchName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetProjectStretch)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStretchDampingName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStretchDamping)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStretchStiffnessName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStretchStiffness)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStretchScaleName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStretchScale)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetSolveCollisionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSolveCollision)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetProjectCollisionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetProjectCollision)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStaticFrictionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStaticFriction)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetKineticFrictionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetKineticFriction)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStrandsViscosityName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsViscosity)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetGridDimensionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetGridDimension)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetCollisionRadiusName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetCollisionRadius)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetRadiusScaleName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetRadiusScale)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStrandsDensityName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsDensity)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetTimeDilationName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetTimeDilation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStrandsSmoothingName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsSmoothing)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetStrandsThicknessName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetStrandsThickness)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetThicknessScaleName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetThicknessScale)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetWorldTransformName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 16);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetWorldTransform)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetWorldInverseName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 16);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetWorldInverse)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetPointPositionName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetPointPosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeNodePositionName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodePosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeNodeOrientationName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodeOrientation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeNodeMassName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodeMass)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeNodeInertiaName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeNodeInertia)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeEdgeLengthName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeEdgeLength)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeEdgeRotationName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeEdgeRotation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeRestPositionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeRestPosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeRestOrientationName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeRestOrientation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeLocalStateName)
	{
		check(BindingInfo.GetNumInputs() == 8 && BindingInfo.GetNumOutputs() == 7);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeLocalState)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == AttachNodePositionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AttachNodePosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == AttachNodeOrientationName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AttachNodeOrientation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == AttachNodeStateName)
	{
		check(BindingInfo.GetNumInputs() == 8 && BindingInfo.GetNumOutputs() == 7);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AttachNodeState)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdateNodeStateName)
	{
		check(BindingInfo.GetNumInputs() == 11 && BindingInfo.GetNumOutputs() == 7);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateNodeState)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdatePointPositionName)
	{
		check(BindingInfo.GetNumInputs() == 7 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdatePointPosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ResetPointPositionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ResetPointPosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == AdvectNodePositionName)
	{
		check(BindingInfo.GetNumInputs() == 16 && BindingInfo.GetNumOutputs() == 6);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AdvectNodePosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == AdvectNodeOrientationName)
	{
		check(BindingInfo.GetNumInputs() == 19 && BindingInfo.GetNumOutputs() == 7);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, AdvectNodeOrientation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdateLinearVelocityName)
	{
		check(BindingInfo.GetNumInputs() == 8 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateLinearVelocity)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdateAngularVelocityName)
	{
		check(BindingInfo.GetNumInputs() == 10 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateAngularVelocity)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetBoundingBoxName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 6);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetBoundingBox)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ResetBoundingBoxName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ResetBoundingBox)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == BuildBoundingBoxName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, BuildBoundingBox)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetupDistanceSpringMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 7 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupDistanceSpringMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveDistanceSpringMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 9 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveDistanceSpringMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ProjectDistanceSpringMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 7 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectDistanceSpringMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetupAngularSpringMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 5);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupAngularSpringMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveAngularSpringMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 13 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveAngularSpringMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ProjectAngularSpringMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 9 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectAngularSpringMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetupStretchRodMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 5);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupStretchRodMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveStretchRodMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 10 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveStretchRodMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ProjectStretchRodMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectStretchRodMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetupBendRodMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 5);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupBendRodMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveBendRodMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 14 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveBendRodMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ProjectBendRodMaterialName)
	{
		check(BindingInfo.GetNumInputs() == 10 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectBendRodMaterial)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveHardCollisionConstraintName)
	{
		check(BindingInfo.GetNumInputs() == 15 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveHardCollisionConstraint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ProjectHardCollisionConstraintName)
	{
		check(BindingInfo.GetNumInputs() == 15 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectHardCollisionConstraint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveSoftCollisionConstraintName)
	{
		check(BindingInfo.GetNumInputs() == 21 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SolveSoftCollisionConstraint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ProjectSoftCollisionConstraintName)
	{
		check(BindingInfo.GetNumInputs() == 16 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ProjectSoftCollisionConstraint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetupSoftCollisionConstraintName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 5);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, SetupSoftCollisionConstraint)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeEdgeDirectionName)
	{
		check(BindingInfo.GetNumInputs() == 16 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeEdgeDirection)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdateMaterialFrameName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, UpdateMaterialFrame)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeMaterialFrameName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeMaterialFrame)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeAirDragForceName)
	{
		check(BindingInfo.GetNumInputs() == 14 && BindingInfo.GetNumOutputs() == 6);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, ComputeAirDragForce)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == InitGridSamplesName)
	{
		check(BindingInfo.GetNumInputs() == 9 && BindingInfo.GetNumOutputs() == 8);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, InitGridSamples)::Bind(this, OutFunc);
		}
	else if (BindingInfo.Name == GetSampleStateName)
	{
		check(BindingInfo.GetNumInputs() == 15 && BindingInfo.GetNumOutputs() == 6);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, GetSampleState)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == NeedSimulationResetName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, NeedSimulationReset)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == HasGlobalInterpolationName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, HasGlobalInterpolation)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == NeedRestUpdateName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceHairStrands, NeedRestUpdate)::Bind(this, OutFunc);
	}
}

void WriteTransform(const FMatrix& ToWrite, FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FExternalFuncRegisterHandler<float> Out00(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out01(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out02(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out03(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out04(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out05(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out06(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out07(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out08(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out09(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out10(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out11(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out12(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out13(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out14(Context);
	VectorVM::FExternalFuncRegisterHandler<float> Out15(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*Out00.GetDest() = ToWrite.M[0][0]; Out00.Advance();
		*Out01.GetDest() = ToWrite.M[0][1]; Out01.Advance();
		*Out02.GetDest() = ToWrite.M[0][2]; Out02.Advance();
		*Out03.GetDest() = ToWrite.M[0][3]; Out03.Advance();
		*Out04.GetDest() = ToWrite.M[1][0]; Out04.Advance();
		*Out05.GetDest() = ToWrite.M[1][1]; Out05.Advance();
		*Out06.GetDest() = ToWrite.M[1][2]; Out06.Advance();
		*Out07.GetDest() = ToWrite.M[1][3]; Out07.Advance();
		*Out08.GetDest() = ToWrite.M[2][0]; Out08.Advance();
		*Out09.GetDest() = ToWrite.M[2][1]; Out09.Advance();
		*Out10.GetDest() = ToWrite.M[2][2]; Out10.Advance();
		*Out11.GetDest() = ToWrite.M[2][3]; Out11.Advance();
		*Out12.GetDest() = ToWrite.M[3][0]; Out12.Advance();
		*Out13.GetDest() = ToWrite.M[3][1]; Out13.Advance();
		*Out14.GetDest() = ToWrite.M[3][2]; Out14.Advance();
		*Out15.GetDest() = ToWrite.M[3][3]; Out15.Advance();
	}
}

void UNiagaraDataInterfaceHairStrands::GetNumStrands(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutNumStrands(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutNumStrands.GetDestAndAdvance() = InstData->NumStrands;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStrandSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutStrandSize(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStrandSize.GetDestAndAdvance() = InstData->StrandsSize;
	}
}

void UNiagaraDataInterfaceHairStrands::GetSubSteps(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutSubSteps(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutSubSteps.GetDestAndAdvance() = InstData->SubSteps;
	}
}

void UNiagaraDataInterfaceHairStrands::GetIterationCount(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutIterationCount(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutIterationCount.GetDestAndAdvance() = InstData->IterationCount;
	}
}

void UNiagaraDataInterfaceHairStrands::GetGravityVector(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGravityVectorX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGravityVectorY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGravityVectorZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutGravityVectorX.GetDestAndAdvance() = InstData->GravityVector.X;
		*OutGravityVectorY.GetDestAndAdvance() = InstData->GravityVector.Y;
		*OutGravityVectorZ.GetDestAndAdvance() = InstData->GravityVector.Z;
	}
}

void UNiagaraDataInterfaceHairStrands::GetGravityPreloading(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGravityPreloading(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutGravityPreloading.GetDestAndAdvance() = InstData->GravityPreloading;
	}
}

void UNiagaraDataInterfaceHairStrands::GetAirDrag(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutAirDrag(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutAirDrag.GetDestAndAdvance() = InstData->AirDrag;
	}
}

void UNiagaraDataInterfaceHairStrands::GetAirVelocity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutAirVelocityX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutAirVelocityY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutAirVelocityZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutAirVelocityX.GetDestAndAdvance() = InstData->AirVelocity.X;
		*OutAirVelocityY.GetDestAndAdvance() = InstData->AirVelocity.Y;
		*OutAirVelocityZ.GetDestAndAdvance() = InstData->AirVelocity.Z;
	}
}

void UNiagaraDataInterfaceHairStrands::GetSolveBend(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutSolveBend(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutSolveBend.GetDestAndAdvance() = InstData->SolveBend;
	}
}

void UNiagaraDataInterfaceHairStrands::GetProjectBend(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutProjectBend(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutProjectBend.GetDestAndAdvance() = InstData->ProjectBend;
	}
}

void UNiagaraDataInterfaceHairStrands::GetBendDamping(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutBendDamping(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutBendDamping.GetDestAndAdvance() = InstData->BendDamping;
	}
}

void UNiagaraDataInterfaceHairStrands::GetBendStiffness(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutBendStiffness(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutBendStiffness.GetDestAndAdvance() = InstData->BendStiffness;
	}
}

void UNiagaraDataInterfaceHairStrands::GetBendScale(FVectorVMExternalFunctionContext& Context)
{}

void UNiagaraDataInterfaceHairStrands::GetSolveStretch(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutSolveStretch(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutSolveStretch.GetDestAndAdvance() = InstData->SolveStretch;
	}
}

void UNiagaraDataInterfaceHairStrands::GetProjectStretch(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutProjectStretch(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutProjectStretch.GetDestAndAdvance() = InstData->ProjectStretch;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStretchDamping(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStretchDamping(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStretchDamping.GetDestAndAdvance() = InstData->StretchDamping;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStretchStiffness(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStretchStiffness(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStretchStiffness.GetDestAndAdvance() = InstData->StretchStiffness;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStretchScale(FVectorVMExternalFunctionContext& Context)
{
}


void UNiagaraDataInterfaceHairStrands::GetSolveCollision(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutSolveCollision(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutSolveCollision.GetDestAndAdvance() = InstData->SolveCollision;
	}
}

void UNiagaraDataInterfaceHairStrands::GetProjectCollision(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutProjectCollision(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutProjectCollision.GetDestAndAdvance() = InstData->ProjectCollision;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStaticFriction(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStaticFriction(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStaticFriction.GetDestAndAdvance() = InstData->StaticFriction;
	}
}

void UNiagaraDataInterfaceHairStrands::GetKineticFriction(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutKineticFriction(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutKineticFriction.GetDestAndAdvance() = InstData->KineticFriction;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStrandsViscosity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStrandsViscosity(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStrandsViscosity.GetDestAndAdvance() = InstData->StrandsViscosity;
	}
}

void UNiagaraDataInterfaceHairStrands::GetGridDimension(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGridDimensionX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGridDimensionY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutGridDimensionZ(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutGridDimensionX.GetDestAndAdvance() = InstData->GridDimension.X;
		*OutGridDimensionY.GetDestAndAdvance() = InstData->GridDimension.Y;
		*OutGridDimensionZ.GetDestAndAdvance() = InstData->GridDimension.Z;
	}
}
void UNiagaraDataInterfaceHairStrands::GetCollisionRadius(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutCollisionRadius(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutCollisionRadius.GetDestAndAdvance() = InstData->CollisionRadius;
	}
}

void UNiagaraDataInterfaceHairStrands::GetRadiusScale(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfaceHairStrands::GetStrandsSmoothing(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStrandsSmoothing(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStrandsSmoothing.GetDestAndAdvance() = InstData->StrandsSmoothing;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStrandsDensity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStrandsDensity(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStrandsDensity.GetDestAndAdvance() = InstData->StrandsDensity;
	}
}

void UNiagaraDataInterfaceHairStrands::GetTimeDilation(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutTimeDilation(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutTimeDilation.GetDestAndAdvance() = InstData->TimeDilation;
	}
}

void UNiagaraDataInterfaceHairStrands::GetStrandsThickness(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutStrandsThickness(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutStrandsThickness.GetDestAndAdvance() = InstData->StrandsThickness;
	}
}

void UNiagaraDataInterfaceHairStrands::GetThicknessScale(FVectorVMExternalFunctionContext& Context)
{
}

void UNiagaraDataInterfaceHairStrands::GetWorldTransform(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	FMatrix WorldTransform = InstData->WorldTransform.ToMatrixWithScale();

	WriteTransform(WorldTransform, Context);
}

void UNiagaraDataInterfaceHairStrands::GetWorldInverse(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	FMatrix WorldInverse = InstData->WorldTransform.ToMatrixWithScale().Inverse();

	WriteTransform(WorldInverse, Context);
}

void UNiagaraDataInterfaceHairStrands::GetBoundingBox(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ResetBoundingBox(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::BuildBoundingBox(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::GetPointPosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeNodePosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeNodeOrientation(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeNodeMass(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeNodeInertia(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeEdgeLength(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeEdgeRotation(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeRestPosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeRestOrientation(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeLocalState(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::UpdatePointPosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ResetPointPosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::AttachNodePosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::EvalSkinnedPosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::AttachNodeOrientation(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::AttachNodeState(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::UpdateNodeState(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::AdvectNodePosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::AdvectNodeOrientation(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::UpdateLinearVelocity(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::UpdateAngularVelocity(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SetupDistanceSpringMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SolveDistanceSpringMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ProjectDistanceSpringMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SetupAngularSpringMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SolveAngularSpringMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ProjectAngularSpringMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}


void UNiagaraDataInterfaceHairStrands::SetupStretchRodMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SolveStretchRodMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ProjectStretchRodMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SetupBendRodMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SolveBendRodMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ProjectBendRodMaterial(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeEdgeDirection(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::UpdateMaterialFrame(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeMaterialFrame(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SolveHardCollisionConstraint(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ProjectHardCollisionConstraint(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::SolveSoftCollisionConstraint(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ProjectSoftCollisionConstraint(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}
void UNiagaraDataInterfaceHairStrands::SetupSoftCollisionConstraint(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::ComputeAirDragForce(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::NeedSimulationReset(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::InitGridSamples(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::GetSampleState(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceHairStrands::HasGlobalInterpolation(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIHairStrandsData> InstData(Context);
	VectorVM::FExternalFuncRegisterHandler<int32> OutGlobalInterpolation(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		*OutGlobalInterpolation.GetDestAndAdvance() = InstData->GlobalInterpolation;
	}
}

void UNiagaraDataInterfaceHairStrands::NeedRestUpdate(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceHairStrands::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIHairStrandsLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		GetStrandSizeName,
		GetNumStrandsName,
		GetWorldTransformName,
		GetWorldInverseName,
		GetStretchScaleName,
		GetBendScaleName,
		GetRadiusScaleName,
		GetThicknessScaleName,
		GetPointPositionName,
		ComputeNodePositionName,
		ComputeNodeOrientationName,
		ComputeNodeMassName,
		ComputeNodeInertiaName,
		ComputeEdgeLengthName,
		ComputeEdgeRotationName,
		ComputeRestPositionName,
		ComputeRestOrientationName,
		ComputeLocalStateName,
		GetLocalVectorName,
		GetWorldVectorName,
		AttachNodePositionName,
		AttachNodeOrientationName,
		AttachNodeStateName,
		UpdateNodeStateName,
		UpdatePointPositionName,
		ResetPointPositionName,
		AdvectNodePositionName,
		AdvectNodeOrientationName,
		UpdateLinearVelocityName,
		UpdateAngularVelocityName,
		GetBoundingBoxName,
		ResetBoundingBoxName,
		BuildBoundingBoxName,
		SetupDistanceSpringMaterialName,
		SolveDistanceSpringMaterialName,
		ProjectDistanceSpringMaterialName,
		SetupAngularSpringMaterialName,
		SolveAngularSpringMaterialName,
		ProjectAngularSpringMaterialName,
		SetupStretchRodMaterialName,
		SolveStretchRodMaterialName,
		ProjectStretchRodMaterialName,
		SetupBendRodMaterialName,
		SolveBendRodMaterialName,
		ProjectBendRodMaterialName,
		SolveHardCollisionConstraintName,
		ProjectHardCollisionConstraintName,
		SolveSoftCollisionConstraintName,
		ProjectSoftCollisionConstraintName,
		SetupSoftCollisionConstraintName,
		ComputeEdgeDirectionName,
		UpdateMaterialFrameName,
		ComputeMaterialFrameName,
		ComputeAirDragForceName,
		InitGridSamplesName,
		GetSampleStateName,
		NeedSimulationResetName,
		HasGlobalInterpolationName,
		NeedRestUpdateName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

void UNiagaraDataInterfaceHairStrands::GetCommonHLSL(FString& OutHLSL)
{
	for (const TCHAR* CommonFile : NDIHairStrandsLocal::CommonShaderFiles)
	{
		OutHLSL.Appendf(TEXT("#include \"%s\"\n"), CommonFile);
	}
}

bool UNiagaraDataInterfaceHairStrands::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
		return false;

	for (const TCHAR* CommonFile : NDIHairStrandsLocal::CommonShaderFiles)
	{
		InVisitor->UpdateString(TEXT("NiagaraDataInterfaceHairStrandsHLSLSource"), GetShaderFileHash(CommonFile, EShaderPlatform::SP_PCD3D_SM5).ToString());
	}
	InVisitor->UpdateString(TEXT("NiagaraDataInterfaceHairStrandsTemplateHLSLSource"), GetShaderFileHash(NDIHairStrandsLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5).ToString());

	return true;
}

void UNiagaraDataInterfaceHairStrands::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	FString TemplateFile;
	LoadShaderSourceFile(NDIHairStrandsLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceHairStrands::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIHairStrandsLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceHairStrands::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDIHairStrandsProxy& DIProxy = Context.GetProxy<FNDIHairStrandsProxy>();
	FNDIHairStrandsData* ProxyData = DIProxy.SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	
	const TRefCountPtr<FHairGroupInstance> HairGroupInstance = GetRefCountedHairGroupInstance(ProxyData);

	FHairGroupInstanceRDG InstanceRDG = Convert(GraphBuilder, HairGroupInstance);
	
	const int32 MeshLODIndex = InstanceRDG.MeshLODIndex;
	const bool bIsHairValid = ProxyData != nullptr && ProxyData->HairStrandsBuffer && ProxyData->HairStrandsBuffer->IsInitialized();
	const bool bIsRestValid = bIsHairValid && InstanceRDG.IsValid() &&
		// TEMP: These check are only temporary for avoiding crashes while we find the bottom of the issue.
		ProxyData->HairStrandsBuffer->ParamsScaleBuffer.IsValid() && ProxyData->HairStrandsBuffer->BoundingBoxBuffer.IsValid();

	NDIHairStrandsLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIHairStrandsLocal::FShaderParameters>();
	if (bIsHairValid && bIsRestValid)
	{
		check(ProxyData);

		FNDIHairStrandsBuffer* HairStrandsBuffer = ProxyData->HairStrandsBuffer;

		// Projection Buffers
		const bool bHasSkinnedInterpolation = InstanceRDG.IsRootValid();
		const EHairSimulationInterpolationMode InterpolationModeValue = bHasSkinnedInterpolation ? (ProxyData->GlobalInterpolation ?
			EHairSimulationInterpolationMode::RBF : EHairSimulationInterpolationMode::Skinned) : EHairSimulationInterpolationMode::Rigid;

		// Simulation setup (we update the rest configuration based on the deformed positions 
		// if in restupdate mode or if we are resetting the sim and using RBF transfer since the rest positions are not matching the physics asset)
		const bool bShouldReset = HairStrandsBuffer->bShouldReset || !HairStrandsBuffer->bValidGeometryType;
		HairStrandsBuffer->ResetCount = bShouldReset ? 0 : FMath::Min(GHairSimulationMaxDelay + 1, HairStrandsBuffer->ResetCount+ 1);
		
		const int32 NeedResetValue = HairStrandsBuffer->ResetCount <= GHairSimulationMaxDelay;
		const int32 RestUpdateValue = GHairSimulationRestUpdate || (NeedResetValue && ProxyData->bSkinningTransfer) ? 1 : 0;
		const int32 LocalSimulationValue = ProxyData->LocalSimulation;
		
		HairStrandsBuffer->bShouldReset = false;
		HairStrandsBuffer->bValidGeometryType = true;
		HairStrandsBuffer->CurrentMeshLOD = MeshLODIndex;

		// Offsets / Transforms

		FMatrix44f WorldTransformFloat = FMatrix44f::Identity;
		FMatrix44f BoneTransformFloat = FMatrix44f::Identity;
		if(!GEnableProxyInstanceTransform)
		{
			WorldTransformFloat =  ComputeWorldTransform(ProxyData, HairGroupInstance);
			BoneTransformFloat = ComputeBoneTransform(ProxyData, HairGroupInstance);
		}
		else
		{
			WorldTransformFloat =  ComputeWorldTransform(ProxyData, ProxyData->HairGroupInstance);
			BoneTransformFloat = ComputeBoneTransform(ProxyData, ProxyData->HairGroupInstance);
		}
		
		if (ProxyData->BoneLinearVelocity.ContainsNaN() || ProxyData->BoneAngularVelocity.ContainsNaN() || ProxyData->BoneLinearAcceleration.ContainsNaN() || ProxyData->BoneAngularAcceleration.ContainsNaN())
		{
			UE_LOGF(LogHairStrands, Log, "Bad bones state");
		}

		if (InstanceRDG.IsDeformedValid() &&  !InstanceRDG.IsRootValid() && InstanceRDG.BindingType == EHairBindingType::Skinning)
		{
			UE_LOGF(LogHairStrands, Log, "FNDIHairStrandsParametersCS() Groom Asset %ls from component %ls is set to use skinning interpolation but the skin resources are not valid",
				*HairGroupInstance->Debug.GroomAssetName, *HairGroupInstance->Debug.MeshComponentName);
		}

		// Set shader constants
		ShaderParameters->BoundingBoxOffsets = HairStrandsBuffer->BoundingBoxOffsets;
		ShaderParameters->WorldTransform = WorldTransformFloat;
		ShaderParameters->WorldInverse = WorldTransformFloat.Inverse();
		ShaderParameters->WorldRotation = WorldTransformFloat.GetMatrixWithoutScale().ToQuat();
		ShaderParameters->NumStrands = ProxyData->NumStrands;
		ShaderParameters->StrandSize = ProxyData->StrandsSize;
		ShaderParameters->BoneTransform = BoneTransformFloat;
		ShaderParameters->BoneInverse = BoneTransformFloat.Inverse();
		ShaderParameters->BoneRotation = BoneTransformFloat.GetMatrixWithoutScale().ToQuat();
		ShaderParameters->BoneLinearVelocity = ProxyData->BoneLinearVelocity;
		ShaderParameters->BoneAngularVelocity = ProxyData->BoneAngularVelocity;
		ShaderParameters->BoneLinearAcceleration = ProxyData->BoneLinearAcceleration;
		ShaderParameters->BoneAngularAcceleration = ProxyData->BoneAngularAcceleration;
		ShaderParameters->ResetSimulation = NeedResetValue;
		ShaderParameters->InterpolationMode = int32(InterpolationModeValue);
		ShaderParameters->RestUpdate = RestUpdateValue;
		ShaderParameters->LocalSimulation = LocalSimulationValue;
		ShaderParameters->RestRootOffset = FVector3f::ZeroVector;
		ShaderParameters->DeformedRootOffset = FVector3f::ZeroVector;
		ShaderParameters->RestPositionOffset = InstanceRDG.RestPositionOffsetValue;
		ShaderParameters->SampleCount = InstanceRDG.SampleCount;
		ShaderParameters->RBFLocalSpace = UE::Groom::IsRBFLocalSpaceEnabled();

		ShaderParameters->DeformedPositionBuffer = InstanceRDG.DeformedPositionBufferUAV;
		ShaderParameters->CurvesOffsetsBuffer = InstanceRDG.CurvesOffsetsBuffer;
		ShaderParameters->RestPositionBuffer = InstanceRDG.RestPositionBuffer;
		ShaderParameters->DeformedPositionOffset = InstanceRDG.DeformedPositionOffsetSRV;
		ShaderParameters->RestTrianglePositionBuffer = InstanceRDG.RestTrianglePositionBuffer;
		ShaderParameters->DeformedTrianglePositionBuffer = InstanceRDG.DeformedTrianglePositionBuffer;
		ShaderParameters->RestSamplePositionsBuffer = InstanceRDG.RestSamplePositionsBuffer;
		ShaderParameters->MeshSampleWeightsBuffer = InstanceRDG.MeshSampleWeightsBuffer;
		ShaderParameters->DeformedSamplePositionsBuffer = InstanceRDG.DeformedSamplePositionsBuffer;
		ShaderParameters->RootBarycentricCoordinatesBuffer = InstanceRDG.RootBarycentricCoordinatesBuffer;
		ShaderParameters->RootToUniqueTriangleIndexBuffer = InstanceRDG.RootToUniqueTriangleIndexBuffer;

		ShaderParameters->BoundingBoxBuffer = HairStrandsBuffer->BoundingBoxBuffer.GetOrCreateUAV(GraphBuilder);
		ShaderParameters->ParamsScaleBuffer = HairStrandsBuffer->ParamsScaleBuffer.GetOrCreateSRV(GraphBuilder);
	}
	else
	{
		if (bIsHairValid)
		{
			ProxyData->HairStrandsBuffer->bValidGeometryType = false;
		}
		// Set shader constants
		ShaderParameters->BoundingBoxOffsets = FIntVector4(0, 1, 2, 3);
		ShaderParameters->WorldTransform = FMatrix44f::Identity;
		ShaderParameters->WorldInverse = FMatrix44f::Identity;
		ShaderParameters->WorldRotation = FQuat4f::Identity;
		ShaderParameters->NumStrands = 1;
		ShaderParameters->StrandSize = 1;
		ShaderParameters->BoneTransform = FMatrix44f::Identity;
		ShaderParameters->BoneInverse = FMatrix44f::Identity;
		ShaderParameters->BoneRotation = FQuat4f::Identity;
		ShaderParameters->BoneLinearVelocity = FVector3f::ZeroVector;
		ShaderParameters->BoneAngularVelocity = FVector3f::ZeroVector;
		ShaderParameters->BoneLinearAcceleration = FVector3f::ZeroVector;
		ShaderParameters->BoneAngularAcceleration = FVector3f::ZeroVector;
		ShaderParameters->ResetSimulation = 0;
		ShaderParameters->InterpolationMode = 0;
		ShaderParameters->RestUpdate = 0;
		ShaderParameters->LocalSimulation = 0;
		ShaderParameters->RestRootOffset = FVector3f::ZeroVector;
		ShaderParameters->DeformedRootOffset = FVector3f::ZeroVector;
		ShaderParameters->RestPositionOffset = FVector3f::ZeroVector;
		ShaderParameters->SampleCount = 0;
		ShaderParameters->RBFLocalSpace = 0;

		ShaderParameters->DeformedPositionBuffer = InstanceRDG.DeformedPositionBufferUAV;
		ShaderParameters->CurvesOffsetsBuffer = InstanceRDG.CurvesOffsetsBuffer;
		ShaderParameters->RestPositionBuffer = InstanceRDG.RestPositionBuffer;
		ShaderParameters->DeformedPositionOffset = InstanceRDG.DeformedPositionOffsetSRV;
		ShaderParameters->RestTrianglePositionBuffer = InstanceRDG.RestTrianglePositionBuffer;
		ShaderParameters->DeformedTrianglePositionBuffer = InstanceRDG.DeformedTrianglePositionBuffer;
		ShaderParameters->RestSamplePositionsBuffer = InstanceRDG.RestSamplePositionsBuffer;
		ShaderParameters->MeshSampleWeightsBuffer = InstanceRDG.MeshSampleWeightsBuffer;
		ShaderParameters->DeformedSamplePositionsBuffer = InstanceRDG.DeformedSamplePositionsBuffer;
		ShaderParameters->RootBarycentricCoordinatesBuffer = InstanceRDG.RootBarycentricCoordinatesBuffer;
		ShaderParameters->RootToUniqueTriangleIndexBuffer = InstanceRDG.RootToUniqueTriangleIndexBuffer;

		ShaderParameters->BoundingBoxBuffer = Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, PF_R32_UINT);
		ShaderParameters->ParamsScaleBuffer = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, PF_R32_FLOAT);
	}
}

void UNiagaraDataInterfaceHairStrands::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	check(Proxy);

	FNDIHairStrandsData* GameThreadData = static_cast<FNDIHairStrandsData*>(PerInstanceData);
	FNDIHairStrandsData* RenderThreadData = static_cast<FNDIHairStrandsData*>(DataForRenderThread);

	if (GameThreadData && RenderThreadData)
	{
		RenderThreadData->CopyDatas(GameThreadData);
	}
}

void FNDIHairStrandsProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	if (Context.GetSimStageData().bFirstStage)
	{
		FNDIHairStrandsData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());

		if (ProxyData != nullptr && ProxyData->HairStrandsBuffer != nullptr)
		{
			FIntVector4& BoundingBoxOffsets = ProxyData->HairStrandsBuffer->BoundingBoxOffsets;
			const int32 FirstOffset = BoundingBoxOffsets[0];

			BoundingBoxOffsets[0] = BoundingBoxOffsets[1];
			BoundingBoxOffsets[1] = BoundingBoxOffsets[2];
			BoundingBoxOffsets[2] = BoundingBoxOffsets[3];
			BoundingBoxOffsets[3] = FirstOffset;

			ProxyData->HairStrandsBuffer->Transfer(Context.GetGraphBuilder(), ProxyData->ParamsScale);
		}
	}
}

void FNDIHairStrandsProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	// Check we have valid hair data
	FNDIHairStrandsData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	FNDIHairStrandsBuffer* HairStrandsBuffer = ProxyData ? ProxyData->HairStrandsBuffer : nullptr;
	if (!HairStrandsBuffer || !HairStrandsBuffer->IsInitialized())
	{
		return;
	}

	const TRefCountPtr<FHairGroupInstance> HairGroupInstance = GetRefCountedHairGroupInstance(ProxyData);

	// MGPU DeformedPositionBuffer copy after simulation
	FHairStrandsDeformedResource* GuideDeformedBuffer = HairGroupInstance.IsValid() ? HairGroupInstance->Guides.DeformedResource : nullptr;
	if (GuideDeformedBuffer && GuideDeformedBuffer->IsInitialized())
	{
		const FRDGExternalBuffer& DeformedBuffer = GuideDeformedBuffer->GetBuffer(FHairStrandsDeformedResource::EFrameType::Current);
		if (DeformedBuffer.Buffer.IsValid())
		{
			if (FRHIBuffer* DeformedPositionBuffer = DeformedBuffer.Buffer->GetRHI())
			{
				Context.GetComputeDispatchInterface().MultiGPUResourceModified(Context.GetGraphBuilder(), DeformedPositionBuffer, false, true);
			}
		}
	}

	if (Context.IsFinalPostSimulate())
	{
		HairStrandsBuffer->BoundingBoxBuffer.EndGraphUsage();
		HairStrandsBuffer->ParamsScaleBuffer.EndGraphUsage();
	}
}

#undef LOCTEXT_NAMESPACE

