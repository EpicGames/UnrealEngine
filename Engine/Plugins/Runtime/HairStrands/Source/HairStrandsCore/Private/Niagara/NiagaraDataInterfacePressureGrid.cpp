// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/NiagaraDataInterfacePressureGrid.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShader.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderer.h"
#include "NiagaraSimStageData.h"
#include "NiagaraSystemInstance.h"

#include "ShaderParameterUtils.h"
#include "ClearQuad.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfacePressureGrid)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfacePressureGrid"
DEFINE_LOG_CATEGORY_STATIC(LogPressureGrid, Log, All);

//------------------------------------------------------------------------------------------------------------

namespace NDIPressureGridLocal
{
	static const TCHAR* CommonShaderFile = TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDataInterfacePressureGrid.ush");
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDataInterfacePressureGridTemplate.ush");

	static const FName BuildDistanceFieldName(TEXT("BuildDistanceField"));
	static const FName BuildDensityFieldName(TEXT("BuildDensityField"));
	static const FName SolveGridPressureName(TEXT("SolveGridPressure"));
	static const FName ScaleCellFieldsName(TEXT("ScaleCellFields"));
	static const FName SetSolidBoundaryName(TEXT("SetSolidBoundary"));
	static const FName ComputeBoundaryWeightsName(TEXT("ComputeBoundaryWeights"));
	static const FName GetNodePositionName(TEXT("GetNodePosition"));
	static const FName GetDensityFieldName(TEXT("GetDensityField"));
	static const FName UpdateDeformationGradientName(TEXT("UpdateDeformationGradient"));

} //namespace NDIPressureGridLocal

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfacePressureGrid::UNiagaraDataInterfacePressureGrid(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Proxy.Reset(new FNDIPressureGridProxy());
	NumAttributes = 18;
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfacePressureGrid::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIPressureGridLocal;

	Super::GetFunctionsInternal(OutFunctions);
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNodePositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Grid Cell")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Position")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetDensityFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particle Density")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdateDeformationGradientName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Velocity Gradient")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Deformation Gradient")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Deformation Gradient")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Gradient Determinant")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = BuildDistanceFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = BuildDensityFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particle Mass")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particle Density")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SolveGridPressureName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Grid Cell")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Init Stage")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Project Status")));
		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetSolidBoundaryName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Grid Cell")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Cell Distance")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Cell Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Boundary Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeBoundaryWeightsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Grid Cell")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Weights Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ScaleCellFieldsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Pressure Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Grid Cell")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Transfer Status")));

		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, BuildDistanceField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, BuildDensityField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, SolveGridPressure);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, SetSolidBoundary);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, ScaleCellFields);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, ComputeBoundaryWeights);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, GetNodePosition);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, GetDensityField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, UpdateDeformationGradient);

void UNiagaraDataInterfacePressureGrid::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIPressureGridLocal;

	Super::GetVMExternalFunction(BindingInfo, InstanceData, OutFunc);

	if (BindingInfo.Name == BuildDistanceFieldName)
	{
		check(BindingInfo.GetNumInputs() == 8 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, BuildDistanceField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == BuildDensityFieldName)
	{
		check(BindingInfo.GetNumInputs() == 10 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, BuildDensityField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdateDeformationGradientName)
	{
		check(BindingInfo.GetNumInputs() == 34 && BindingInfo.GetNumOutputs() == 17);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, UpdateDeformationGradient)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetNodePositionName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, GetNodePosition)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetDensityFieldName)
	{
		check(BindingInfo.GetNumInputs() == 8 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, GetDensityField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SolveGridPressureName)
	{
		check(BindingInfo.GetNumInputs() == 3 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, SolveGridPressure)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetSolidBoundaryName)
	{
		check(BindingInfo.GetNumInputs() == 6 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, SetSolidBoundary)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeBoundaryWeightsName)
	{
		check(BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, ComputeBoundaryWeights)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ScaleCellFieldsName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePressureGrid, ScaleCellFields)::Bind(this, OutFunc);
	}
}

void UNiagaraDataInterfacePressureGrid::BuildDistanceField(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::BuildDensityField(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::SolveGridPressure(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::ComputeBoundaryWeights(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::SetSolidBoundary(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::ScaleCellFields(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::GetNodePosition(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfacePressureGrid::GetDensityField(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}
void UNiagaraDataInterfacePressureGrid::UpdateDeformationGradient(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfacePressureGrid::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIPressureGridLocal;

	if (Super::GetFunctionHLSL(ParamInfo, FunctionInfo, FunctionInstanceIndex, OutHLSL))
	{
		return true;
	}

	static const TSet<FName> ValidGpuFunctions =
	{
		BuildDistanceFieldName,
		BuildDensityFieldName,
		UpdateDeformationGradientName,
		SolveGridPressureName,
		GetNodePositionName,
		GetDensityFieldName,
		SetSolidBoundaryName,
		ComputeBoundaryWeightsName,
		ScaleCellFieldsName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

void UNiagaraDataInterfacePressureGrid::GetCommonHLSL(FString& OutHLSL)
{
	Super::GetCommonHLSL(OutHLSL);
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), NDIPressureGridLocal::CommonShaderFile);
}

bool UNiagaraDataInterfacePressureGrid::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
		return false;

	InVisitor->UpdateString(TEXT("NiagaraDataInterfacePressureGridHLSLSource"), GetShaderFileHash(NDIPressureGridLocal::CommonShaderFile, EShaderPlatform::SP_PCD3D_SM5).ToString());
	InVisitor->UpdateString(TEXT("NiagaraDataInterfacePressureGridTemplateHLSLSource"), GetShaderFileHash(NDIPressureGridLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5).ToString());

	return true;
}

void UNiagaraDataInterfacePressureGrid::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	FString TemplateFile;
	LoadShaderSourceFile(NDIPressureGridLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}
#endif

//------------------------------------------------------------------------------------------------------------

#define NIAGARA_HAIR_STRANDS_THREAD_COUNT_PRESSURE 4

class FClearPressureGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearPressureGridCS)
	SHADER_USE_PARAMETER_STRUCT(FClearPressureGridCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_HAIR_STRANDS_THREAD_COUNT_PRESSURE);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector,					GridSize)
		SHADER_PARAMETER(int,							CopyPressure)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	GridCurrentBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	GridDestinationBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_SHADER_TYPE(, FClearPressureGridCS, TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraClearPressureGrid.usf"), TEXT("MainCS"), SF_Compute);

//------------------------------------------------------------------------------------------------------------

inline void ClearBuffer(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FNDIVelocityGridBuffer* CurrentGridBuffer, FNDIVelocityGridBuffer* DestinationGridBuffer, const FIntVector& GridSize, const bool CopyPressure)
{
	if (CurrentGridBuffer->GridDataBuffer.IsValid() == false || DestinationGridBuffer->GridDataBuffer.IsValid() == false)
	{
		return;
	}

	const FIntVector ExtendedSize = GridSize + FIntVector(1, 1, 1);
	const uint32 ThreadGroupSize = NIAGARA_HAIR_STRANDS_THREAD_COUNT_PRESSURE;
	const FIntVector ThreadGroupCount(
		FMath::DivideAndRoundUp((uint32)ExtendedSize.X, ThreadGroupSize),
		FMath::DivideAndRoundUp((uint32)ExtendedSize.Y, ThreadGroupSize),
		FMath::DivideAndRoundUp((uint32)ExtendedSize.Z, ThreadGroupSize)
	);

	TShaderMapRef<FClearPressureGridCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	FClearPressureGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearPressureGridCS::FParameters>();
	PassParameters->GridSize = ExtendedSize;
	PassParameters->CopyPressure = CopyPressure;
	PassParameters->GridCurrentBuffer = CurrentGridBuffer->GridDataBuffer.GetOrCreateSRV(GraphBuilder);
	PassParameters->GridDestinationBuffer = DestinationGridBuffer->GridDataBuffer.GetOrCreateUAV(GraphBuilder);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("VelocityGrid::ClearTexture"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		ThreadGroupCount
	);
}

//------------------------------------------------------------------------------------------------------------

void FNDIPressureGridProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	FNDIVelocityGridData* ProxyData = FNDIVelocityGridProxy::SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());

	if (ProxyData != nullptr)
	{
		if (Context.GetSimStageData().bFirstStage)
		{
			ClearBuffer(Context.GetGraphBuilder(), Context.GetComputeDispatchInterface().GetFeatureLevel(), ProxyData->CurrentGridBuffer, ProxyData->DestinationGridBuffer, ProxyData->GridSize, true);
		}
	}
}

#undef LOCTEXT_NAMESPACE
