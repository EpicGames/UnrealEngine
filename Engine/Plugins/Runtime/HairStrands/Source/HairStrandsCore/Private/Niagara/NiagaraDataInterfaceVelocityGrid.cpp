// Copyright Epic Games, Inc. All Rights Reserved.

#include "Niagara/NiagaraDataInterfaceVelocityGrid.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShader.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRenderer.h"
#include "NiagaraSimStageData.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "ShaderParameterUtils.h"
#include "ClearQuad.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceVelocityGrid)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceVelocityGrid"
DEFINE_LOG_CATEGORY_STATIC(LogVelocityGrid, Log, All);

//------------------------------------------------------------------------------------------------------------

namespace NDIVelocityGridLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters,)
		SHADER_PARAMETER(FMatrix44f,					WorldTransform)
		SHADER_PARAMETER(FMatrix44f,					WorldInverse)
		SHADER_PARAMETER(FIntVector,					GridSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	GridCurrentBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	GridDestinationBuffer)
	END_SHADER_PARAMETER_STRUCT();

	static const TCHAR* CommonShaderFile = TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDataInterfaceVelocityGrid.ush");
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraDataInterfaceVelocityGridTemplate.ush");

	static const FName BuildVelocityFieldName(TEXT("BuildVelocityField"));
	static const FName SampleVelocityFieldName(TEXT("SampleVelocityField"));
	static const FName ComputeGridSizeName(TEXT("ComputeGridSize"));
	static const FName UpdateGridTransformName(TEXT("UpdateGridTransform"));
	static const FName SetGridDimensionName(TEXT("SetGridDimension"));

} //namespace NDIVelocityGridLocal

//------------------------------------------------------------------------------------------------------------

void FNDIVelocityGridBuffer::Initialize(const FIntVector InGridSize, const int32 InNumAttributes)
{
	GridSize = InGridSize;
	NumAttributes = InNumAttributes;
}

void FNDIVelocityGridBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (GridSize.X != 0 && GridSize.Y != 0 && GridSize.Z != 0)
	{
		static const uint32 NumComponents = NumAttributes;

		FMemMark MemMark(FMemStack::Get());
		FRDGBuilder GraphBuilder(RHICmdList.GetAsImmediate());
		GridDataBuffer.Initialize(GraphBuilder, TEXT("FNDIVelocityGridBuffer"), EPixelFormat::PF_R32_SINT, sizeof(int32), (GridSize.X + 1) * NumComponents * (GridSize.Y + 1) * (GridSize.Z + 1));
		GridDataBuffer.EndGraphUsage();
		GraphBuilder.Execute();
	}
}

void FNDIVelocityGridBuffer::ReleaseRHI()
{
	GridDataBuffer.Release();
}

//------------------------------------------------------------------------------------------------------------

void FNDIVelocityGridData::Swap()
{
	FNDIVelocityGridBuffer* StoredBufferPointer = CurrentGridBuffer;
	CurrentGridBuffer = DestinationGridBuffer;
	DestinationGridBuffer = StoredBufferPointer;
}

void FNDIVelocityGridData::Release()
{
	if (CurrentGridBuffer)
	{
		ENQUEUE_RENDER_COMMAND(DeleteResourceA)(
			[ParamPointerToRelease = CurrentGridBuffer](FRHICommandListImmediate& RHICmdList)
		{
			ParamPointerToRelease->ReleaseResource();
			delete ParamPointerToRelease;
		});
		CurrentGridBuffer = nullptr;
	}
	if (DestinationGridBuffer)
	{

		ENQUEUE_RENDER_COMMAND(DeleteResourceB)(
			[ParamPointerToRelease = DestinationGridBuffer](FRHICommandListImmediate& RHICmdList)
		{
			ParamPointerToRelease->ReleaseResource();
			delete ParamPointerToRelease;
		});
		DestinationGridBuffer = nullptr;
	}
}

void FNDIVelocityGridData::Resize()
{
	if (NeedResize)
	{
		if (CurrentGridBuffer)
		{
			CurrentGridBuffer->Initialize(GridSize, NumAttributes);
			BeginInitResource(CurrentGridBuffer);
		}
		if (DestinationGridBuffer)
		{
			DestinationGridBuffer->Initialize(GridSize, NumAttributes);
			BeginInitResource(DestinationGridBuffer);
		}
		NeedResize = false;
	}
}

bool FNDIVelocityGridData::Init(const FIntVector& InGridSize, const int32 InNumAttributes, FNiagaraSystemInstance* SystemInstance)
{
	CurrentGridBuffer = nullptr;
	DestinationGridBuffer = nullptr;

	GridSize = FIntVector(1, 1, 1);
	NeedResize = true;
	WorldTransform = WorldInverse = FMatrix::Identity;

	if (InGridSize[0] != 0 && InGridSize[1] != 0 && InGridSize[2] != 0)
	{
		GridSize = InGridSize;
		NumAttributes = InNumAttributes;

		CurrentGridBuffer = new FNDIVelocityGridBuffer();
		DestinationGridBuffer = new FNDIVelocityGridBuffer();

		Resize();
	}

	return true;
}

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfaceVelocityGrid::UNiagaraDataInterfaceVelocityGrid(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, GridSize(10)
{
	Proxy.Reset(new FNDIVelocityGridProxy());
	NumAttributes = 6;
}

bool UNiagaraDataInterfaceVelocityGrid::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	const FIntVector ClampedSize = FIntVector(FMath::Clamp(GridSize.X, 0, 50), FMath::Clamp(GridSize.Y, 0, 50), FMath::Clamp(GridSize.Z, 0, 50));

	if (GridSize != ClampedSize)
	{
		UE_LOGF(LogVelocityGrid, Warning, "The grid size is beyond its maximum value (50)");
	}
	GridSize = ClampedSize;

	FNDIVelocityGridData* InstanceData = new (PerInstanceData) FNDIVelocityGridData();
	check(InstanceData);

	return InstanceData->Init(this->GridSize, NumAttributes, SystemInstance);
}

void UNiagaraDataInterfaceVelocityGrid::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIVelocityGridData* InstanceData = static_cast<FNDIVelocityGridData*>(PerInstanceData);

	InstanceData->Release();
	InstanceData->~FNDIVelocityGridData();

	FNDIVelocityGridProxy* ThisProxy = GetProxyAs<FNDIVelocityGridProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		ThisProxy->SystemInstancesToProxyData.Remove(InstanceID);
	}
	);
}

bool UNiagaraDataInterfaceVelocityGrid::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDIVelocityGridData* InstanceData = static_cast<FNDIVelocityGridData*>(PerInstanceData);

	bool RequireReset = false;
	if (InstanceData)
	{
		InstanceData->WorldTransform = SystemInstance->GetWorldTransform().ToMatrixWithScale();

		if (InstanceData->NeedResize)
		{
			InstanceData->Resize();
		}
	}
	return RequireReset;
}

bool UNiagaraDataInterfaceVelocityGrid::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceVelocityGrid* OtherTyped = CastChecked<UNiagaraDataInterfaceVelocityGrid>(Destination);
	OtherTyped->GridSize = GridSize;

	return true;
}

bool UNiagaraDataInterfaceVelocityGrid::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceVelocityGrid* OtherTyped = CastChecked<const UNiagaraDataInterfaceVelocityGrid>(Other);

	return (OtherTyped->GridSize == GridSize);
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVelocityGrid::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!Super::AppendCompileHash(InVisitor))
	{
		return false;
	}

	InVisitor->UpdateString(TEXT("NiagaraDataInterfaceVelocityGridHLSLSource"), GetShaderFileHash(NDIVelocityGridLocal::CommonShaderFile, EShaderPlatform::SP_PCD3D_SM5).ToString());
	InVisitor->UpdateString(TEXT("NiagaraDataInterfaceVelocityGridTemplateHLSLSource"), GetShaderFileHash(NDIVelocityGridLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5).ToString());
	InVisitor->UpdateShaderParameters<NDIVelocityGridLocal::FShaderParameters>();
	
	return true;
}
#endif

void UNiagaraDataInterfaceVelocityGrid::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceVelocityGrid::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIVelocityGridLocal;

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = BuildVelocityFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT(" Velocity Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particle Mass")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Velocity")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Velocity Gradient")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SampleVelocityFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT(" Velocity Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Scaled Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particle Mass")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Particle Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Velocity Gradient")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = ComputeGridSizeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT(" Velocity Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Center")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Extent")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Origin")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Grid Length")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = UpdateGridTransformName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = false;
		Sig.bSupportsCPU = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT(" Velocity Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetMatrix4Def(), TEXT("Grid Transform")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SetGridDimensionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.bWriteFunction = true;
		Sig.bSupportsGPU = true;
		Sig.bSupportsCPU = true;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT(" Velocity Grid")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Grid Dimension")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Function Status")));

		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, BuildVelocityField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, SampleVelocityField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, ComputeGridSize);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, UpdateGridTransform);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, SetGridDimension);

void UNiagaraDataInterfaceVelocityGrid::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIVelocityGridLocal;

	if (BindingInfo.Name == BuildVelocityFieldName)
	{
		check(BindingInfo.GetNumInputs() == 28 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, BuildVelocityField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SampleVelocityFieldName)
	{
		check(BindingInfo.GetNumInputs() == 9 && BindingInfo.GetNumOutputs() == 20);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, SampleVelocityField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == ComputeGridSizeName)
	{
		check(BindingInfo.GetNumInputs() == 7 && BindingInfo.GetNumOutputs() == 4);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, ComputeGridSize)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == UpdateGridTransformName)
	{
		check(BindingInfo.GetNumInputs() == 17 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, UpdateGridTransform)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SetGridDimensionName)
	{
		check(BindingInfo.GetNumInputs() == 4 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfaceVelocityGrid, SetGridDimension)::Bind(this, OutFunc);
	}
}

void UNiagaraDataInterfaceVelocityGrid::BuildVelocityField(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfaceVelocityGrid::SampleVelocityField(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu 
}

void UNiagaraDataInterfaceVelocityGrid::ComputeGridSize(FVectorVMExternalFunctionContext& Context)
{
	// @todo : implement function for cpu
}

void UNiagaraDataInterfaceVelocityGrid::SetGridDimension(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIVelocityGridData> InstData(Context);
	VectorVM::FExternalFuncInputHandler<float> GridDimensionX(Context);
	VectorVM::FExternalFuncInputHandler<float> GridDimensionY(Context);
	VectorVM::FExternalFuncInputHandler<float> GridDimensionZ(Context);

	VectorVM::FExternalFuncRegisterHandler<bool> OutFunctionStatus(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		FIntVector GridDimension;
		GridDimension.X = *GridDimensionX.GetDestAndAdvance();
		GridDimension.Y = *GridDimensionY.GetDestAndAdvance();
		GridDimension.Z = *GridDimensionZ.GetDestAndAdvance();

		InstData->GridSize = GridDimension;
		InstData->NeedResize = true;

		*OutFunctionStatus.GetDestAndAdvance() = true;
	}
}

void UNiagaraDataInterfaceVelocityGrid::UpdateGridTransform(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIVelocityGridData> InstData(Context);

	VectorVM::FExternalFuncInputHandler<float> Out00(Context);
	VectorVM::FExternalFuncInputHandler<float> Out01(Context);
	VectorVM::FExternalFuncInputHandler<float> Out02(Context);
	VectorVM::FExternalFuncInputHandler<float> Out03(Context);

	VectorVM::FExternalFuncInputHandler<float> Out10(Context);
	VectorVM::FExternalFuncInputHandler<float> Out11(Context);
	VectorVM::FExternalFuncInputHandler<float> Out12(Context);
	VectorVM::FExternalFuncInputHandler<float> Out13(Context);

	VectorVM::FExternalFuncInputHandler<float> Out20(Context);
	VectorVM::FExternalFuncInputHandler<float> Out21(Context);
	VectorVM::FExternalFuncInputHandler<float> Out22(Context);
	VectorVM::FExternalFuncInputHandler<float> Out23(Context);

	VectorVM::FExternalFuncInputHandler<float> Out30(Context);
	VectorVM::FExternalFuncInputHandler<float> Out31(Context);
	VectorVM::FExternalFuncInputHandler<float> Out32(Context);
	VectorVM::FExternalFuncInputHandler<float> Out33(Context);

	VectorVM::FExternalFuncRegisterHandler<bool> OutTransformStatus(Context);

	for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
	{
		FMatrix Transform;
		Transform.M[0][0] = *Out00.GetDestAndAdvance();
		Transform.M[0][1] = *Out01.GetDestAndAdvance();
		Transform.M[0][2] = *Out02.GetDestAndAdvance();
		Transform.M[0][3] = *Out03.GetDestAndAdvance();

		Transform.M[1][0] = *Out10.GetDestAndAdvance();
		Transform.M[1][1] = *Out11.GetDestAndAdvance();
		Transform.M[1][2] = *Out12.GetDestAndAdvance();
		Transform.M[1][3] = *Out13.GetDestAndAdvance();

		Transform.M[2][0] = *Out20.GetDestAndAdvance();
		Transform.M[2][1] = *Out21.GetDestAndAdvance();
		Transform.M[2][2] = *Out22.GetDestAndAdvance();
		Transform.M[2][3] = *Out23.GetDestAndAdvance();

		Transform.M[3][0] = *Out30.GetDestAndAdvance();
		Transform.M[3][1] = *Out31.GetDestAndAdvance();
		Transform.M[3][2] = *Out32.GetDestAndAdvance();
		Transform.M[3][3] = *Out33.GetDestAndAdvance();

		InstData->WorldTransform = Transform;
		InstData->WorldInverse = Transform.Inverse();

		*OutTransformStatus.GetDestAndAdvance() = true;
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceVelocityGrid::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIVelocityGridLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		BuildVelocityFieldName,
		SampleVelocityFieldName,
		ComputeGridSizeName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

void UNiagaraDataInterfaceVelocityGrid::GetCommonHLSL(FString& OutHLSL)
{
	OutHLSL.Appendf(TEXT("#include \"%s\"\n"), NDIVelocityGridLocal::CommonShaderFile);
}

void UNiagaraDataInterfaceVelocityGrid::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	Super::GetParameterDefinitionHLSL(ParamInfo, OutHLSL);

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};

	FString TemplateFile;
	LoadShaderSourceFile(NDIVelocityGridLocal::TemplateShaderFile, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceVelocityGrid::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIVelocityGridLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceVelocityGrid::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
	FNDIVelocityGridProxy& DIProxy = Context.GetProxy<FNDIVelocityGridProxy>();
	FNDIVelocityGridData* ProxyData = DIProxy.SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());

	NDIVelocityGridLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIVelocityGridLocal::FShaderParameters>();

	if (ProxyData != nullptr && ProxyData->CurrentGridBuffer != nullptr && ProxyData->DestinationGridBuffer != nullptr
		&& ProxyData->CurrentGridBuffer->IsInitialized() && ProxyData->DestinationGridBuffer->IsInitialized())
	{
		ShaderParameters->GridSize			= ProxyData->GridSize;
		ShaderParameters->WorldTransform	= FMatrix44f(ProxyData->WorldTransform);
		ShaderParameters->WorldInverse		= FMatrix44f(ProxyData->WorldInverse);

		ShaderParameters->GridCurrentBuffer = ProxyData->CurrentGridBuffer->GridDataBuffer.GetOrCreateSRV(GraphBuilder);
		ShaderParameters->GridDestinationBuffer = ProxyData->DestinationGridBuffer->GridDataBuffer.GetOrCreateUAV(GraphBuilder);
	}
	else
	{
		ShaderParameters->GridSize			= FIntVector::ZeroValue;
		ShaderParameters->WorldTransform	= FMatrix44f::Identity;
		ShaderParameters->WorldInverse		= FMatrix44f::Identity;

		ShaderParameters->GridCurrentBuffer = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(GraphBuilder, EPixelFormat::PF_R32_SINT);
		ShaderParameters->GridDestinationBuffer	= Context.GetComputeDispatchInterface().GetEmptyBufferUAV(GraphBuilder, EPixelFormat::PF_R32_SINT);
	}
}

void UNiagaraDataInterfaceVelocityGrid::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDIVelocityGridData* GameThreadData = static_cast<FNDIVelocityGridData*>(PerInstanceData);
	FNDIVelocityGridData* RenderThreadData = static_cast<FNDIVelocityGridData*>(DataForRenderThread);

	RenderThreadData->WorldTransform = GameThreadData->WorldTransform;
	RenderThreadData->WorldInverse = GameThreadData->WorldInverse;
	RenderThreadData->CurrentGridBuffer = GameThreadData->CurrentGridBuffer;
	RenderThreadData->DestinationGridBuffer = GameThreadData->DestinationGridBuffer;
	RenderThreadData->GridSize = GameThreadData->GridSize;
}

//------------------------------------------------------------------------------------------------------------

void FNDIVelocityGridProxy::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNDIVelocityGridData* SourceData = static_cast<FNDIVelocityGridData*>(PerInstanceData);
	FNDIVelocityGridData* TargetData = &(SystemInstancesToProxyData.FindOrAdd(Instance));

	ensure(TargetData);
	if (TargetData)
	{
		TargetData->WorldTransform = SourceData->WorldTransform;
		TargetData->WorldInverse = SourceData->WorldInverse;
		TargetData->GridSize = SourceData->GridSize;
		TargetData->DestinationGridBuffer = SourceData->DestinationGridBuffer;
		TargetData->CurrentGridBuffer = SourceData->CurrentGridBuffer;
	}
	else
	{
		UE_LOGF(LogVelocityGrid, Log, "ConsumePerInstanceDataFromGameThread() ... could not find %ls", *FNiagaraUtilities::SystemInstanceIDToString(Instance));
	}
	SourceData->~FNDIVelocityGridData();
}

//------------------------------------------------------------------------------------------------------------

#define NIAGARA_HAIR_STRANDS_THREAD_COUNT_VELOCITY  4

class FClearVelocityGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearVelocityGridCS)
	SHADER_USE_PARAMETER_STRUCT(FClearVelocityGridCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_HAIR_STRANDS_THREAD_COUNT_VELOCITY);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector,					GridSize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	GridDestinationBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_SHADER_TYPE(, FClearVelocityGridCS, TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraClearVelocityGrid.usf"), TEXT("MainCS"), SF_Compute);

inline void ClearTexture(FRDGBuilder& GraphBuilder, FNDIVelocityGridBuffer* DestinationGridBuffer, const FIntVector& InGridSize)
{
	if (DestinationGridBuffer->GridDataBuffer.IsValid() == false)
	{
		return;
	}

	const FIntVector GridSize((InGridSize.X + 1) * DestinationGridBuffer->NumAttributes, InGridSize.Y + 1, InGridSize.Z + 1);
	const uint32 ThreadGroupSize = NIAGARA_HAIR_STRANDS_THREAD_COUNT_VELOCITY;
	const FIntVector ThreadGroupCount(
		FMath::DivideAndRoundUp((uint32)GridSize.X, ThreadGroupSize),
		FMath::DivideAndRoundUp((uint32)GridSize.Y, ThreadGroupSize),
		FMath::DivideAndRoundUp((uint32)GridSize.Z, ThreadGroupSize)
	);

	TShaderMapRef<FClearVelocityGridCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	FClearVelocityGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearVelocityGridCS::FParameters>();
	PassParameters->GridSize = GridSize;
	PassParameters->GridDestinationBuffer = DestinationGridBuffer->GridDataBuffer.GetOrCreateUAV(GraphBuilder);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("VelocityGrid::ClearTexture"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		ThreadGroupCount
	);
}

//------------------------------------------------------------------------------------------------------------

class FCopyVelocityGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyVelocityGridCS)
	SHADER_USE_PARAMETER_STRUCT(FCopyVelocityGridCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_COUNT"), NIAGARA_HAIR_STRANDS_THREAD_COUNT_VELOCITY);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector,					GridSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	GridCurrentBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>,	GridDestinationBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_SHADER_TYPE(, FCopyVelocityGridCS, TEXT("/Plugin/Runtime/HairStrands/Private/NiagaraCopyVelocityGrid.usf"), TEXT("MainCS"), SF_Compute);

inline void CopyTexture(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FNDIVelocityGridBuffer* CurrentGridBuffer, FNDIVelocityGridBuffer* DestinationGridBuffer, const FIntVector& InGridSize)
{
	if (DestinationGridBuffer->GridDataBuffer.IsValid() == false || CurrentGridBuffer->GridDataBuffer.IsValid() == false)
	{
		return;
	}

	TShaderMapRef<FCopyVelocityGridCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));

	const FIntVector GridSize((InGridSize.X + 1) * DestinationGridBuffer->NumAttributes, InGridSize.Y + 1, InGridSize.Z + 1);
	const uint32 ThreadGroupSize = NIAGARA_HAIR_STRANDS_THREAD_COUNT_VELOCITY;
	const FIntVector ThreadGroupCount(
		FMath::DivideAndRoundUp((uint32)GridSize.X, ThreadGroupSize),
		FMath::DivideAndRoundUp((uint32)GridSize.Y, ThreadGroupSize),
		FMath::DivideAndRoundUp((uint32)GridSize.Z, ThreadGroupSize)
	);

	FCopyVelocityGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyVelocityGridCS::FParameters>();
	PassParameters->GridSize = GridSize;
	PassParameters->GridCurrentBuffer = CurrentGridBuffer->GridDataBuffer.GetOrCreateSRV(GraphBuilder);
	PassParameters->GridDestinationBuffer = DestinationGridBuffer->GridDataBuffer.GetOrCreateUAV(GraphBuilder);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("VelocityGrid::CopyTexture"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		ThreadGroupCount
	);
}

void FNDIVelocityGridProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	FNDIVelocityGridData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	if (ProxyData != nullptr)
	{
		if (Context.GetSimStageData().bFirstStage)
		{
			ClearTexture(Context.GetGraphBuilder(), ProxyData->DestinationGridBuffer, ProxyData->GridSize);
		}
	}
}

void FNDIVelocityGridProxy::PostStage(const FNDIGpuComputePostStageContext& Context)
{
	FNDIVelocityGridData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	if (ProxyData != nullptr)
	{
		//ProxyData->Swap();
		CopyTexture(Context.GetGraphBuilder(), Context.GetComputeDispatchInterface().GetFeatureLevel(), ProxyData->DestinationGridBuffer, ProxyData->CurrentGridBuffer, ProxyData->GridSize);
		//FRHICopyTextureInfo CopyInfo;
		//RHICmdList.CopyTexture(ProxyData->DestinationGridBuffer->GridDataBuffer.Buffer,
		//	ProxyData->CurrentGridBuffer->GridDataBuffer.Buffer, CopyInfo);
	}
}

void FNDIVelocityGridProxy::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	if (Context.IsFinalPostSimulate())
	{
		if (FNDIVelocityGridData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID()))
		{
			if (ProxyData->CurrentGridBuffer->GridDataBuffer.IsValid())
			{
				ProxyData->CurrentGridBuffer->GridDataBuffer.EndGraphUsage();
			}
			if (ProxyData->DestinationGridBuffer->GridDataBuffer.IsValid())
			{
				ProxyData->DestinationGridBuffer->GridDataBuffer.EndGraphUsage();
			}
		}
	}
}

void FNDIVelocityGridProxy::ResetData(const FNDIGpuComputeResetContext& Context)
{
	FNDIVelocityGridData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());

	if (ProxyData != nullptr && ProxyData->DestinationGridBuffer != nullptr && ProxyData->CurrentGridBuffer != nullptr)
	{
		ClearTexture(Context.GetGraphBuilder(), ProxyData->DestinationGridBuffer, ProxyData->GridSize);
		ClearTexture(Context.GetGraphBuilder(), ProxyData->CurrentGridBuffer, ProxyData->GridSize);
	}
}

void FNDIVelocityGridProxy::GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context)
{
	if  ( const FNDIVelocityGridData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID()) )
	{
		Context.SetDirect(FIntVector(ProxyData->GridSize.X + 1, ProxyData->GridSize.Y + 1, ProxyData->GridSize.Z + 1));
	}
}

#undef LOCTEXT_NAMESPACE

