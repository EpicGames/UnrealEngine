// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "GroomActor.generated.h"

#define UE_API HAIRSTRANDSCORE_API

class UBillboardComponent;

/** An actor that renders a simulated hair */
UCLASS(MinimalAPI, HideCategories = (Input, Replication, Mobility), showcategories = ("Input|MouseInput", "Input|TouchInput"))
class AGroomActor : public AActor
{
	GENERATED_UCLASS_BODY()

	/** Strand hair component that performs simulation and rendering */
	UPROPERTY(Category = StrandHair, VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<class UGroomComponent> GroomComponent;

	//~ Begin AActor Interface
#if WITH_EDITOR
	UE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif
	//~ End AActor Interface

#if WITH_EDITORONLY_DATA
protected:
	/** Billboard used to see the scene in the editor */
	UPROPERTY()
		TObjectPtr<UBillboardComponent> SpriteComponent;
#endif

public:
	/** Returns GroomComponent subobject **/
	class UGroomComponent* GetGroomComponent() const { return GroomComponent; }
};

#undef UE_API
