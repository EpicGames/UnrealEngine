// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "PSOPrecache.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "HairDescription.h"
#include "HairStrandsDatas.h"
#include "RenderResource.h"
#include "GroomResources.h"
#include "GroomSettings.h"
#include "GroomAssetCards.h"
#include "GroomAssetMeshes.h"
#include "GroomAssetInterpolation.h"
#include "GroomAssetPhysics.h"
#include "GroomAssetRendering.h"
#include "GroomAssetDataflow.h"
#include "Curves/CurveFloat.h"
#include "HairStrandsInterface.h"
#include "Engine/SkeletalMesh.h"
#include "Async/RecursiveMutex.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "UObject/PerPlatformProperties.h"
#include "UObject/StrongObjectPtr.h"

#include "GroomAsset.generated.h"

#define UE_API HAIRSTRANDSCORE_API

class UGroomBindingAsset;
class UAssetUserData;
class UMaterialInterface;
class UNiagaraSystem;
class FGroomAssetRenderData;
struct FHairStrandsRestResource;
struct FHairStrandsInterpolationResource;
struct FHairStrandsRaytracingResource;
namespace Nanite { struct FMaterialAudit; }

enum class EHairGroupInfoFlags : uint8
{
	HasTrimmedPoint = 1<<0,
	HasTrimmedCurve = 1<<1,
	HasInvalidPoint = 1<<2
};

USTRUCT(BlueprintType)
struct FHairGroupLODInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Point Count"))
	int32 NumPoints = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Curve Count"))
	int32 NumCurves = 0;
};

USTRUCT(BlueprintType)
struct FHairGroupInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Info")
	int32 GroupIndex = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info")
	int32 GroupID = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info")
	FName GroupName = NAME_None;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Curve Count"))
	int32 NumCurves = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Guide Count"))
	int32 NumGuides = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Curve Vertex Count"))
	int32 NumCurveVertices = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Guide Vertex Count"))
	int32 NumGuideVertices = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Length of the longest hair strands"))
	float MaxCurveLength = 0;

	UPROPERTY()
	uint32 Flags = 0;

	UPROPERTY()
	TArray<FHairGroupLODInfo> LODInfos;
};

USTRUCT(BlueprintType)
struct FHairGroupsMaterial
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Material")
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY(EditAnywhere, Category = "Material")
	FName SlotName = NAME_None;
};


struct FHairGroupResources
{
	struct FGuides
	{
		bool IsValid() const { return RestResource != nullptr; }

		/* Return the memory size for GPU resources */
		uint32 GetResourcesSize() const
		{
			uint32 Total = 0;
			if (RestResource) Total += RestResource->GetResourcesSize();
			return Total;
		}

		FHairStrandsRestResource* RestResource = nullptr;

	} Guides;

	struct FStrands
	{
		bool IsValid() const { return RestResource != nullptr; }

		/* Return the memory size for GPU resources */
		uint32 GetResourcesSize() const
		{
			uint32 Total = 0;
			if (RestResource) 			Total += RestResource->GetResourcesSize();
			if (InterpolationResource) 	Total += InterpolationResource->GetResourcesSize();
			if (ClusterResource) 		Total += ClusterResource->GetResourcesSize();
			#if RHI_RAYTRACING
			if (RaytracingResource) 	Total += RaytracingResource->GetResourcesSize();
			#endif
			return Total;
		}

		FHairStrandsRestResource*			RestResource = nullptr;
		FHairStrandsInterpolationResource*	InterpolationResource = nullptr;
		FHairStrandsClusterResource* 		ClusterResource = nullptr;
		#if RHI_RAYTRACING
		FHairStrandsRaytracingResource*		RaytracingResource = nullptr;
		#endif

		bool bIsCookedOut = false;
	} Strands;

	struct FCards
	{
		/* Return the memory size for GPU resources */
		uint32 GetResourcesSize() const
		{
			uint32 Total = 0;
			for (const FLOD& LOD : LODs)
			{
				Total += LOD.GetResourcesSize();
			}
			return Total;
		}

		struct FLOD
		{
			/* Return the memory size for GPU resources */
			uint32 GetResourcesSize() const
			{
				uint32 Total = 0;
				if (RestResource) 				Total += RestResource->GetResourcesSize();
				if (InterpolationResource) 		Total += InterpolationResource->GetResourcesSize();
				if (GuideRestResource) 			Total += GuideRestResource->GetResourcesSize();
				if (GuideInterpolationResource) Total += GuideInterpolationResource->GetResourcesSize();
				#if RHI_RAYTRACING
				if (RaytracingResource) 		Total += RaytracingResource->GetResourcesSize();
				#endif
				return Total;
			}

			bool IsValid() const { return RestResource != nullptr; }

			FHairCardsRestResource*				RestResource = nullptr;
			FHairCardsInterpolationResource*	InterpolationResource = nullptr;
			FHairStrandsRestResource*			GuideRestResource = nullptr;
			FHairStrandsInterpolationResource*	GuideInterpolationResource = nullptr;
			#if RHI_RAYTRACING
			FHairStrandsRaytracingResource*		RaytracingResource = nullptr;
			#endif

			bool bIsCookedOut = false;
		};
		TArray<FLOD> LODs;
	} Cards;

	struct FMeshes
	{	
		/* Return the memory size for GPU resources */
		uint32 GetResourcesSize() const
		{
			uint32 Total = 0;
			for (const FLOD& LOD : LODs)
			{
				Total += LOD.GetResourcesSize();
			}
			return Total;
		}

		struct FLOD
		{
			/* Return the memory size for GPU resources */
			uint32 GetResourcesSize() const
			{
				uint32 Total = 0;
				if (RestResource) 		Total += RestResource->GetResourcesSize();
				#if RHI_RAYTRACING
				if (RaytracingResource) Total += RaytracingResource->GetResourcesSize();
				#endif
				return Total;
			}

			bool IsValid() const { return RestResource != nullptr; }

			FHairMeshesRestResource* 		RestResource = nullptr;
			#if RHI_RAYTRACING
			FHairStrandsRaytracingResource* RaytracingResource = nullptr;
			#endif
			bool bIsCookedOut = false;
		};
		TArray<FLOD> LODs;
	} Meshes;

	struct FDebug
	{
		FHairStrandsDebugResources* Resource = nullptr;
	} Debug;
};

inline uint32 GetDataSize(const FHairStrandsBulkData& BulkData)
{
	uint32 Total = 0;
	Total += BulkData.Data.Positions.IsBulkDataLoaded() 		? BulkData.Data.Positions.GetBulkDataSize()   : 0;
	Total += BulkData.Data.CurveAttributes.IsBulkDataLoaded()	? BulkData.Data.CurveAttributes.GetBulkDataSize() : 0;
	Total += BulkData.Data.PointAttributes.IsBulkDataLoaded()	? BulkData.Data.PointAttributes.GetBulkDataSize() : 0;
	Total += BulkData.Data.Curves.IsBulkDataLoaded() 			? BulkData.Data.Curves.GetBulkDataSize(): 0;
	Total += BulkData.Data.PointToCurve.IsBulkDataLoaded()		? BulkData.Data.PointToCurve.GetBulkDataSize() : 0;
	Total += BulkData.Data.CurveMapping.IsBulkDataLoaded()		? BulkData.Data.CurveMapping.GetBulkDataSize() : 0;
	Total += BulkData.Data.PointMapping.IsBulkDataLoaded()		? BulkData.Data.PointMapping.GetBulkDataSize() : 0;
	return Total;
}

inline uint32 GetDataSize(const FHairStrandsInterpolationBulkData& InterpolationBulkData) 	
{
	uint32 Total = 0;
	Total += InterpolationBulkData.Data.CurveInterpolation.IsBulkDataLoaded()? InterpolationBulkData.Data.CurveInterpolation.GetBulkDataSize() : 0;
	Total += InterpolationBulkData.Data.PointInterpolation.IsBulkDataLoaded()? InterpolationBulkData.Data.PointInterpolation.GetBulkDataSize() : 0;
	return Total;
}

struct FHairGroupPlatformData
{
	struct FGuides
	{
		bool HasValidData() const		{ return BulkData.GetNumPoints() > 0;}
		const FBox& GetBounds() const	{ return BulkData.GetBounds(); }
		uint32 GetDataSize() const		{ return ::GetDataSize(BulkData); }
		FHairStrandsBulkData BulkData;
	} Guides;

	struct FStrands
	{
		bool HasValidData() const		{ return BulkData.GetNumPoints() > 0;}
		const FBox& GetBounds() const	{ return BulkData.GetBounds(); }

		uint32 GetDataSize() const;
		FHairStrandsBulkData				BulkData;
		FHairStrandsInterpolationBulkData	InterpolationBulkData;
		FHairStrandsClusterBulkData			ClusterBulkData;

		// Experimental (Optional)
		FRenderCurveResourceData	 		CurveResourceData;

		bool bIsCookedOut = false;
	} Strands;

	struct FCards
	{
		bool HasValidData() const 
		{ 
			for (const FLOD& LOD : LODs)
			{
				if (LOD.HasValidData())
				return true;
			}
			return false;
		}

		bool HasValidData(uint32 LODIt) const 	{ return LODIt < uint32(LODs.Num()) && LODs[LODIt].HasValidData(); }
		bool IsValid(uint32 LODIt) const 		{ return LODIt < uint32(LODs.Num()) && LODs[LODIt].IsValid(); }
		FBox GetBounds() const
		{
			for (const FLOD& LOD : LODs)
			{
				if (LOD.IsValid()) return LOD.BulkData.Header.BoundingBox;
			}
			return FBox();
		}

		uint32 GetDataSize() const
		{
			uint32 Total = 0;
			for (const FLOD& LOD : LODs)
			{
				Total += LOD.GetDataSize();
			}
			return Total;
		}

		struct FLOD
		{
			uint32 GetDataSize() const
			{
				uint32 Total = 0;
				Total += BulkData.Positions.GetAllocatedSize();
				Total += BulkData.Normals.GetAllocatedSize();
				Total += BulkData.UVs.GetAllocatedSize();
				Total += BulkData.VertexColors.GetAllocatedSize();
				Total += BulkData.Indices.GetAllocatedSize();
				Total += InterpolationBulkData.Interpolation.GetAllocatedSize();
				Total += ::GetDataSize(GuideBulkData);
				Total += ::GetDataSize(GuideInterpolationBulkData);
				return Total;
			}

			bool HasValidData() const	{ return BulkData.IsValid(); }
			bool IsValid() const 		{ return BulkData.IsValid(); }

			// Main data & Resources
			FHairCardsBulkData					BulkData;
			FHairCardsInterpolationBulkData		InterpolationBulkData;
			FHairStrandsBulkData				GuideBulkData;
			FHairStrandsInterpolationBulkData	GuideInterpolationBulkData;

			bool bIsCookedOut = false;
		};
		TArray<FLOD> LODs;
	} Cards;

	struct FMeshes
	{
		bool HasValidData() const
		{
			for (const FLOD& LOD : LODs)
			{
				if (LOD.HasValidData())
				return true;
			}
			return false;
		}
		bool HasValidData(uint32 LODIt) const 	{ return LODIt < uint32(LODs.Num()) && LODs[LODIt].HasValidData(); }
		bool IsValid(uint32 LODIt) const 		{ return LODIt < uint32(LODs.Num()) && LODs[LODIt].IsValid(); }
		FBox GetBounds() const
		{
			for (const FLOD& LOD : LODs)
			{
				if (LOD.IsValid()) return LOD.BulkData.Header.BoundingBox;
			}
			return FBox();
		}

		uint32 GetDataSize() const
		{
			uint32 Total = 0;
			for (const FLOD& LOD : LODs)
			{
				Total += LOD.GetDataSize();
			}
			return Total;
		}

		struct FLOD
		{
			uint32 GetDataSize() const
			{
				uint32 Total = 0;
				Total += BulkData.Positions.GetAllocatedSize();
				Total += BulkData.Normals.GetAllocatedSize();
				Total += BulkData.UVs.GetAllocatedSize();
				Total += BulkData.VertexColors.GetAllocatedSize();
				Total += BulkData.Indices.GetAllocatedSize();
				return Total;
			}

			bool HasValidData() const 	{ return BulkData.IsValid(); }
			bool IsValid() const 		{ return BulkData.IsValid(); }

			FHairMeshesBulkData BulkData;
			bool bIsCookedOut = false;
		};
		TArray<FLOD> LODs;
	} Meshes;

	struct FDebug
	{
		FHairStrandsDebugDatas Data;
	} Debug;
};

struct FHairDescriptionGroup
{
	FHairGroupInfo		 Info;
	FHairStrandsRawDatas Strands;
	FHairStrandsRawDatas Guides;
	uint32 GetHairAttributes() const { return Strands.GetAttributes() | Guides.GetAttributes(); }
	uint32 GetHairAttributeFlags() const { return Strands.GetAttributeFlags() | Guides.GetAttributeFlags(); }
};

struct FHairDescriptionGroups
{
	TArray<FHairDescriptionGroup> HairGroups;
	FBoxSphereBounds3f Bounds;
	bool  IsValid() const;
};

USTRUCT(BlueprintType)
struct FHairGroupInfoWithVisibility : public FHairGroupInfo
{
	GENERATED_BODY()

	/** Toggle hair group visibility. This visibility flag is not persistent to the asset, and exists only as a preview/helper mechanism */
	UPROPERTY(EditAnywhere, Category = "Info", meta = (DisplayName = "Visible"))
	bool bIsVisible = true;
};

UENUM(BlueprintType)
enum class EHairAtlasTextureType : uint8
{
	Depth,
	Tangent,
	Attribute,
	Coverage,
	AuxilaryData,
	Material
};

struct FHairVertexFactoryTypesPerMaterialData
{
	int16 MaterialIndex;
	EHairGeometryType HairGeometryType;
	FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
};

UENUM()
enum class EHairDescriptionType : uint8
{
	Source,
	Edit,
	Count
};

/**
 * Implements an asset that can be used to store hair strands
 */
UCLASS(BlueprintType, AutoExpandCategories = ("HairRendering", "HairPhysics", "HairInterpolation", "HairDataflow"), hidecategories = (Object, "Hidden"), MinimalAPI)
class UGroomAsset : public UObject, public IInterface_AssetUserData, public IDataflowContentOwner, public IDataflowInstanceInterface
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	/** Notification when anything changed */
	DECLARE_MULTICAST_DELEGATE(FOnGroomAssetChanged);
	DECLARE_MULTICAST_DELEGATE(FOnGroomAssetResourcesChanged);
	DECLARE_MULTICAST_DELEGATE(FOnGroomAsyncLoadFinished);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCreateGroomDataflow, UGroomAsset*);
#endif

private:
	UPROPERTY(EditAnywhere, Category = "HairLOD", meta = (DisplayName = "LOD Mode", ToolTip = "Define how LOD adapts curves & points for strands geometry. Auto: adapts the curve count based on screen coverage. Manual: use the discrete LOD created for each groups"))
	EGroomLODMode LODMode = EGroomLODMode::Default;

	UPROPERTY(EditAnywhere, Category = "HairLOD", meta = (DisplayName = "Global Auto LOD Bias", ToolTip = "When LOD mode is set to Auto, decrease the screen size at which curves reduction will occur.", ClampMin = "-1", ClampMax = "1", UIMin = "-1.0", UIMax = "1.0"))
	float AutoLODBias = 0;

	/** Dataflow settings used for any dataflow related operations */
	UPROPERTY(EditAnywhere, Category = "HairDataflow")
	FGroomDataflowSettings DataflowSettings;

public:
	/** Return the builder category name for layout */
	static FName GetDataflowBuilderCategoryName() { return FName("HairDataflow"); }

	/** Create the dataflow from the groom asset */
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "HairDataflow")
	void CreateGroomDataflow() 
	{ 
#if WITH_EDITOR
		OnCreateGroomDataflow.Broadcast(this); 
#endif
	}
	
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, EditFixedSize, Category = "HairInfo", meta = (DisplayName = "Group", TitleProperty = "{GroupName}"))
	TArray<FHairGroupInfoWithVisibility> HairGroupsInfo;

	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetHairGroupsRendering, BlueprintSetter = SetHairGroupsRendering, Category = "HairRendering", meta = (DisplayName = "Group", TitleProperty = "{GroupName}"))
	TArray<FHairGroupsRendering> HairGroupsRendering;

	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetHairGroupsPhysics, BlueprintSetter = SetHairGroupsPhysics, Category = "HairPhysics", meta = (DisplayName = "Group", TitleProperty = "{GroupName}"))
	TArray<FHairGroupsPhysics> HairGroupsPhysics;

	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetHairGroupsInterpolation, BlueprintSetter = SetHairGroupsInterpolation, Category = "HairInterpolation", meta = (DisplayName = "Group", TitleProperty = "{GroupName}"))
	TArray<FHairGroupsInterpolation> HairGroupsInterpolation;

	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetHairGroupsLOD, BlueprintSetter = SetHairGroupsLOD, Category = "HairLOD", meta = (DisplayName = "Group"))
	TArray<FHairGroupsLOD> HairGroupsLOD;

	/** Cards - Source description data */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetHairGroupsCards, BlueprintSetter = SetHairGroupsCards, Category = "HairCards", meta = (DisplayName = "Group"))
	TArray<FHairGroupsCardsSourceDescription> HairGroupsCards;

	/** Meshes - Source description data */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetHairGroupsMeshes, BlueprintSetter = SetHairGroupsMeshes, Category = "HairMeshes", meta = (DisplayName = "Group"))
	TArray<FHairGroupsMeshesSourceDescription> HairGroupsMeshes;

	/** Meshes - Source description data */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
		UPROPERTY(EditAnywhere, BlueprintGetter = GetHairGroupsMaterials, BlueprintSetter = SetHairGroupsMaterials, Category = "HairMaterials", meta = (DisplayName = "Group"))
	TArray<FHairGroupsMaterial> HairGroupsMaterials;

	/** Enable radial basis function interpolation to be used instead of the local skin rigid transform */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetEnableGlobalInterpolation, BlueprintSetter = SetEnableGlobalInterpolation, Category = "HairInterpolation", meta = (ToolTip = "Enable radial basis function interpolation to be used instead of the local skin rigid transform (WIP)", DisplayName = "RBF Interpolation"))
	bool EnableGlobalInterpolation = false;

	/** Enable guide-cache support. This allows to attach a guide-cache dynamically at runtime */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetEnableSimulationCache, BlueprintSetter = SetEnableSimulationCache, Category = "HairInterpolation", meta = (ToolTip = "Enable guide-cache support. This allows to attach a simulation-cache dynamically at runtime", DisplayName = "Enable Guide-Cache Support"))
	bool EnableSimulationCache = false;

	/** Type of interpolation used */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetHairInterpolationType, BlueprintSetter = SetHairInterpolationType, Category = "HairInterpolation", meta = (ToolTip = "Type of interpolation used (WIP)"))
	EGroomInterpolationType HairInterpolationType = EGroomInterpolationType::SmoothTransform;

	/** Deformed skeletal mesh that will drive the groom deformation/simulation. For creating this skeletal mesh, enable EnableDeformation within the interpolation settings*/
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetRiggedSkeletalMesh, BlueprintSetter = SetRiggedSkeletalMesh, Category = "HairInterpolation")
	TObjectPtr<USkeletalMesh> RiggedSkeletalMesh;

	/** Deformed skeletal mesh mapping from groups to sections */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY()
	TArray<int32> DeformedGroupSections;

	/** Minimum LOD to cook */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, Category = "HairLOD", meta = (DisplayName = "Minimum LOD"))
	FPerPlatformInt MinLOD;

	/** When true all LODs below MinLod will still be cooked */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "HairLOD")
	FPerPlatformBool DisableBelowMinLodStripping;

	/** The LOD bias to use after LOD stripping, regardless of MinLOD. Computed at cook time */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.")
	UPROPERTY()
	TArray<float> EffectiveLODBias;

	/** Store strands/cards/meshes data */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomAsset accessor.") 
	TArray<FHairGroupPlatformData> HairGroupsPlatformData;

public:
	bool UsesNanite() const;
	FGroomAssetRenderData* GetRenderData();
	const FGroomAssetRenderData* GetRenderData() const;

	bool HasValidNaniteData() const;
	bool HasNaniteFallbackMesh(EShaderPlatform ShaderPlatform) const;
#if WITH_EDITOR
	UE_API virtual void ClearAllCachedCookedPlatformData() override;
#endif
	UMaterialInterface* GetNaniteAuditMaterial(int32 MaterialIndex) const;
	const Nanite::FMaterialAudit* GetNaniteMaterials() const;

private:
	void SetRenderData(TUniquePtr<class FGroomAssetRenderData>&& InRenderData);
	TUniquePtr<class FGroomAssetRenderData> RenderData;

private:
	/** Store strands/cards/meshes resources */
	TArray<FHairGroupResources> HairGroupsResources;

public:

	/** Static member to retrieve the member name */
	static UE_API FName GetDataflowSettingsMemberName();
	
	/** Return the Dataflow settings associated to this groom asset if any, const version. */
	const FGroomDataflowSettings& GetDataflowSettings() const { return DataflowSettings; }

	/** Return the Dataflow settings associated to this groom asset if any. */
	FGroomDataflowSettings& GetDataflowSettings() { return DataflowSettings; }
	
	/** Set the Dataflow settings associated to this groom asset if any */
    void SetDataflowSettings(const FGroomDataflowSettings& InDataflowSettings) { DataflowSettings = InDataflowSettings; }

	static UE_API FName GetHairGroupsRenderingMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsRendering>& GetHairGroupsRendering();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsRendering(const TArray<FHairGroupsRendering>& In);
	UE_API const TArray<FHairGroupsRendering>& GetHairGroupsRendering() const;

	static UE_API FName GetHairGroupsPhysicsMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsPhysics>& GetHairGroupsPhysics();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsPhysics(const TArray<FHairGroupsPhysics>& In);
	UE_API const TArray<FHairGroupsPhysics>& GetHairGroupsPhysics() const;

	static UE_API FName GetHairGroupsInterpolationMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsInterpolation>& GetHairGroupsInterpolation();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsInterpolation(const TArray<FHairGroupsInterpolation>& In);
	UE_API const TArray<FHairGroupsInterpolation>& GetHairGroupsInterpolation() const;

	static UE_API FName GetHairGroupsLODMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsLOD>& GetHairGroupsLOD();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsLOD(const TArray<FHairGroupsLOD>& In);
	UE_API const TArray<FHairGroupsLOD>& GetHairGroupsLOD() const;

	static UE_API FName GetHairGroupsCardsMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsCardsSourceDescription>& GetHairGroupsCards();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsCards(const TArray<FHairGroupsCardsSourceDescription>& In);
	UE_API const TArray<FHairGroupsCardsSourceDescription>& GetHairGroupsCards() const;

	static UE_API FName GetHairGroupsMeshesMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsMeshesSourceDescription>& GetHairGroupsMeshes();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsMeshes(const TArray<FHairGroupsMeshesSourceDescription>& In);
	UE_API const TArray<FHairGroupsMeshesSourceDescription>& GetHairGroupsMeshes() const;

	static UE_API FName GetHairGroupsMaterialsMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<FHairGroupsMaterial>& GetHairGroupsMaterials();
	UFUNCTION(BlueprintSetter) UE_API void SetHairGroupsMaterials(const TArray<FHairGroupsMaterial>& In);
	UE_API const TArray<FHairGroupsMaterial>& GetHairGroupsMaterials() const;

	static UE_API FName GetEnableGlobalInterpolationMemberName();
	UFUNCTION(BlueprintGetter) UE_API bool GetEnableGlobalInterpolation() const;
	UFUNCTION(BlueprintSetter) UE_API void SetEnableGlobalInterpolation(bool In);

	static UE_API FName GetEnableSimulationCacheMemberName();
	UFUNCTION(BlueprintGetter) UE_API bool GetEnableSimulationCache() const;
	UFUNCTION(BlueprintSetter) UE_API void SetEnableSimulationCache(bool In);

	static UE_API FName GetHairInterpolationTypeMemberName();
	UFUNCTION(BlueprintGetter) UE_API EGroomInterpolationType GetHairInterpolationType() const;
	UFUNCTION(BlueprintSetter) UE_API void SetHairInterpolationType(EGroomInterpolationType In);

	static UE_API FName GetRiggedSkeletalMeshMemberName();
	UFUNCTION(BlueprintGetter) UE_API USkeletalMesh* GetRiggedSkeletalMesh() const;
	UFUNCTION(BlueprintSetter) UE_API void SetRiggedSkeletalMesh(USkeletalMesh* In);

	static UE_API FName GetDeformedGroupSectionsMemberName();
	UFUNCTION(BlueprintGetter) UE_API TArray<int32>& GetDeformedGroupSections();
	UFUNCTION(BlueprintSetter) UE_API void SetDeformedGroupSections(const TArray<int32>& In);
	UE_API const TArray<int32>& GetDeformedGroupSections() const;

	static UE_API FName GetMinLODMemberName();
	UE_API FPerPlatformInt GetMinLOD() const;
	UE_API void SetMinLOD(FPerPlatformInt In);

	static UE_API FName GetDisableBelowMinLodStrippingMemberName();
	UE_API FPerPlatformBool GetDisableBelowMinLodStripping() const;
	UE_API void SetDisableBelowMinLodStripping(FPerPlatformBool In);

	static UE_API FName GetEffectiveLODBiasMemberName();
	UE_API TArray<float>& GetEffectiveLODBias();
	UE_API void SetEffectiveLODBias(const TArray<float>& In);
	UE_API const TArray<float>& GetEffectiveLODBias() const;

	static UE_API FName GetHairGroupsPlatformDataMemberName();
	UE_API TArray<FHairGroupPlatformData>& GetHairGroupsPlatformData();
	UE_API const TArray<FHairGroupPlatformData>& GetHairGroupsPlatformData() const;
	UE_API void SetHairGroupsPlatformData(const TArray<FHairGroupPlatformData>& In);

	static UE_API FName GetHairGroupsInfoMemberName();
	UE_API TArray<FHairGroupInfoWithVisibility>& GetHairGroupsInfo();
	UE_API const TArray<FHairGroupInfoWithVisibility>& GetHairGroupsInfo() const;
	UE_API void SetHairGroupsInfo(const TArray<FHairGroupInfoWithVisibility>& In);

	UE_API const TArray<FHairGroupResources>& GetHairGroupsResources() const;
	UE_API TArray<FHairGroupResources>& GetHairGroupsResources();

	static UE_API FName GetLODModeMemberName();
	UE_API EGroomLODMode GetLODMode() const;

	static UE_API FName GetAutoLODBiasMemberName();
	// Return the asset Auto LOD Bias
	UE_API float GetAutoLODBias() const;

	// Return the group Auto LOD Bias, which combines both the asset's bias, and the group bias
	UE_API float GetAutoLODBias(int32 InGroupIndex) const;

public:

	//~ Begin UObject Interface.
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	//~ Begin IDataflowContentOwner interface 
	UE_API virtual TObjectPtr<UDataflowBaseContent> CreateDataflowContent() override;
	UE_API virtual void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const override;
	UE_API virtual void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) override;
	//~ End IDataflowContentOwner interface

	/** Begin IDataflowInstanceInterface Interface */
	virtual const FDataflowInstance& GetDataflowInstance() const override { return DataflowSettings; }
	virtual FDataflowInstance& GetDataflowInstance() override { return DataflowSettings; }
	/** End UAssetDefinition Interface */

#if WITH_EDITOR
	FOnGroomAssetChanged& GetOnGroomAssetChanged() { return OnGroomAssetChanged;  }
	FOnGroomAssetResourcesChanged& GetOnGroomAssetResourcesChanged() { return OnGroomAssetResourcesChanged; }
	FOnGroomAsyncLoadFinished& GetOnGroomAsyncLoadFinished() { return OnGroomAsyncLoadFinished; }
	FOnCreateGroomDataflow& GetOnCreateGroomDataflow() { return OnCreateGroomDataflow; }

	/**  Part of Uobject interface  */
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** Thumbnail Info used for Groom Assets */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	/** Asset data to be used when re-importing */
	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** Retrieve the asset tags*/
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

	/** Part of Uobject interface */
	UE_API virtual void PostInitProperties() override;

#endif // WITH_EDITORONLY_DATA

	bool IsValid() const { return bIsInitialized; }

	// Helper functions for setting options on all hair groups
	UE_API void SetStableRasterization(bool bEnable);
	UE_API void SetScatterSceneLighting(bool Enable);
	UE_API void SetHairWidth(float Width);

	/** Initialize/Update/Release resources. */
	UE_API void InitResources();
	UE_API void InitGuideResources();
	UE_API void InitStrandsResources();
	UE_API void InitCardsResources();
	UE_API void InitMeshesResources();
#if WITH_EDITOR
	UE_API void UpdateResource();
#endif
	UE_API void ReleaseResource();
	UE_API void ReleaseGuidesResource(uint32 GroupIndex);
	UE_API void ReleaseStrandsResource(uint32 GroupIndex);
	UE_API void ReleaseCardsResource(uint32 GroupIndex);
	UE_API void ReleaseMeshesResource(uint32 GroupIndex);

	UE_API void SetNumGroup(uint32 InGroupCount, bool bResetGroupData=true, bool bResetOtherData=true);
	UE_API void ClearNumGroup(uint32 InGroupCount);

	UE_API bool AreGroupsValid() const;
	UE_API int32 GetNumHairGroups() const;

	UE_API int32 GetLODCount() const;
#if WITH_EDITORONLY_DATA
	UE_API void StripLODs(const TArray<int32>& LODsToKeep, bool bRebuildResources);
#endif // WITH_EDITORONLY_DATA

	/** Debug data for derived asset generation (strands textures, ...). */
	UE_API bool HasDebugData() const;
	UE_API void CreateDebugData();

	/** Returns true if the asset has the HairDescription needed to recompute its groom data */
	UE_API bool CanRebuildFromDescription() const;

	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	UE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	UE_API EGroomGeometryType GetGeometryType(int32 GroupIndex, int32 LODIndex) const;
	UE_API EGroomBindingType GetBindingType(int32 GroupIndex, int32 LODIndex) const;
	UE_API bool IsVisible(int32 GroupIndex, int32 LODIndex) const;
	UE_API bool IsSimulationEnable(int32 GroupIndex, int32 LODIndex) const;
	UE_API bool IsSimulationEnable() const;
	UE_API bool IsDeformationEnable(int32 GroupIndex) const;
	UE_API bool IsGlobalInterpolationEnable(int32 GroupIndex, int32 LODIndex) const;
	UE_API bool NeedsInterpolationData(int32 GroupIndex) const;
	UE_API bool NeedsInterpolationData() const;
	EGroomGuideType GetGuideType(int32 GroupIndex) const;

	UE_API void UpdateHairGroupsInfo();
	UE_API bool HasGeometryType(EGroomGeometryType Type) const;
	UE_API bool HasGeometryType(uint32 GroupIndex, EGroomGeometryType Type) const;

	/** Used for PSO precaching of used materials and vertex factories */
	UE_API TArray<FHairVertexFactoryTypesPerMaterialData> CollectVertexFactoryTypesPerMaterialData(EShaderPlatform ShaderPlatform);

	/** Helper function to return the asset path name, optionally joined with the LOD index if LODIndex > -1. */
	UE_API FName GetAssetPathName(int32 LODIndex = -1);
	uint32 GetAssetHash() const { return AssetNameHash; }

//private :
#if WITH_EDITOR
	FOnGroomAssetChanged OnGroomAssetChanged;
	FOnGroomAssetResourcesChanged OnGroomAssetResourcesChanged;
	FOnGroomAsyncLoadFinished OnGroomAsyncLoadFinished;
	FOnCreateGroomDataflow OnCreateGroomDataflow;

	UE_API void MarkMaterialsHasChanged();

	UE_API void RecreateResources();
	UE_DEPRECATED(5.8, "Use ChangeShaderPlatform(EShaderPlatform) instead.")
	UE_API void ChangeFeatureLevel(ERHIFeatureLevel::Type PendingFeatureLevel);
	UE_DEPRECATED(5.8, "Use ChangeShaderPlatform(EShaderPlatform) instead.")
	UE_API void ChangePlatformLevel(ERHIFeatureLevel::Type PendingFeatureLevel);
	UE_API void ChangeShaderPlatform(EShaderPlatform InShaderPlatform);
#endif

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Hidden)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	/* Return the material slot index corresponding to the material name */
	UE_API int32 GetMaterialIndex(FName MaterialSlotName) const;
	UE_API bool IsMaterialSlotNameValid(FName MaterialSlotName) const;
	UE_API bool IsMaterialUsed(int32 MaterialIndex) const;
	UE_API TArray<FName> GetMaterialSlotNames() const;

	UE_API bool BuildCardsData();
	UE_API bool BuildMeshesData();

	enum EClassDataStripFlag : uint8
	{
		CDSF_ImportedStrands = 1,
		CDSF_MinLodData = 2,
		CDSF_StrandsStripped = 4,
		CDSF_CardsStripped = 8,
		CDSF_MeshesStripped = 16
	};

	UE_API uint8 GenerateClassStripFlags(FArchive& Ar);

	/** Enable the simulation cache and recompute the cached derived data */
	UE_API void ValidateSimulationCache();

private:
	void ApplyStripFlags(uint8 StripFlags, const class ITargetPlatform* CookTarget);

	/** Update the physics system based on the solver settings enum */
	void UpdatePhysicsSystems();

	// Functions allocating lazily/on-demand resources (guides, interpolation, RT geometry, ...)
	FHairStrandsRestResource*			AllocateGuidesResources(uint32 GroupIndex);
	FHairStrandsInterpolationResource*	AllocateInterpolationResources(uint32 GroupIndex);
#if RHI_RAYTRACING
	FHairStrandsRaytracingResource*		AllocateCardsRaytracingResources(uint32 GroupIndex, uint32 LODIndex);
	FHairStrandsRaytracingResource*		AllocateMeshesRaytracingResources(uint32 GroupIndex, uint32 LODIndex);
	FHairStrandsRaytracingResource*		AllocateStrandsRaytracingResources(uint32 GroupIndex);
	FHairStrandsRaytracingResource**	GetStrandsRaytracingResourcesPtr(uint32 GroupIndex);
#endif // RHI_RAYTRACING
	friend class UGroomComponent;

#if WITH_EDITORONLY_DATA
	bool HasImportedStrandsData() const;

	bool BuildHairGroup_Cards(uint32 GroupIndex);
	bool BuildHairGroup_Meshes(uint32 GroupIndex);

	bool HasChanged_Cards(uint32 GroupIndex, TArray<bool>& OutIsValid) const;
	bool HasChanged_Meshes(uint32 GroupIndex, TArray<bool>& OutIsValid) const;

	bool HasValidData_Cards(uint32 GroupIndex) const;
	bool HasValidData_Meshes(uint32 GroupIndex) const;
public:
	
	/** Commits a HairDescription to buffer for serialization */
	UE_API void CommitHairDescription(FHairDescription&& HairDescription, EHairDescriptionType Type);

	/** Get the current hair description */
	UE_API FHairDescription GetHairDescription() const;

	// Loads a specific hair description stored on the asset
	UE_API FHairDescription LoadHairDescription(const EHairDescriptionType DescriptionType) const;

	/** Get/Build render & guides data based on the hair description and interpolation settings */
	UE_API bool GetHairStrandsDatas(const int32 GroupIndex, FHairStrandsDatas& OutStrandsData, FHairStrandsDatas& OutGuidesData) const;
	UE_API bool GetHairCardsGuidesDatas(const int32 GroupIndex, const int32 LODIndex, FHairStrandsDatas& OutCardsGuidesData);

	/** Caches the computed (group) groom data with the given build settings from/to the Derived Data Cache, building it if needed.
	 *  This function assumes the interpolation settings are properly populated, as they will be used to build the asset.
	 */
	UE_API bool CacheDerivedDatas();
	UE_API bool CacheDerivedData(uint32 GroupIndex);
	UE_API bool CacheStrandsData(uint32 GroupIndex, FString& OutDerivedDataKey);
	UE_API bool CacheCardsData(uint32 GroupIndex, const FString& StrandsKey);
	UE_API bool CacheMeshesData(uint32 GroupIndex);

	UE_API FString GetDerivedDataKey(bool bUseCacheKey=false);
	UE_API FString GetDerivedDataKeyForCards(uint32 GroupIt, const FString& StrandsKey);
	UE_API FString GetDerivedDataKeyForStrands(uint32 GroupIndex);
	UE_API FString GetDerivedDataKeyForMeshes(uint32 GroupIndex);

	UE_API const FHairDescriptionGroups& GetHairDescriptionGroups() const;
private:

	/** Set the hair description type */
	void SetHairDescriptionType(const EHairDescriptionType DescriptionType) {HairDescriptionType = DescriptionType;};

	/** Get the hair description type */
	EHairDescriptionType GetHairDescriptionType() const {return HairDescriptionType;}

	/** Get the hair description index */
	uint8 GetHairDescriptionIndex() const {return FMath::Clamp(static_cast<uint8>(HairDescriptionType),0, 1);}
	
	bool IsFullyCached();
	TUniquePtr<FHairDescriptionBulkData> HairDescriptionBulkData[static_cast<uint8>(EHairDescriptionType::Count)];

	UPROPERTY()
	EHairDescriptionType HairDescriptionType = EHairDescriptionType::Source;

	// Transient HairDescription & HairDescriptionGroups, which are built from HairDescriptionBulkData.
	// All these data (bulk/desc/groups) needs to be in sync. I.e., when the HairDescription is updated, 
	// HairDescriptionGroups needs to also be updated
	mutable UE::FRecursiveMutex InternalLock;
	mutable TUniquePtr<FHairDescription> CachedHairDescription[static_cast<uint8>(EHairDescriptionType::Count)];
	mutable TUniquePtr<FHairDescriptionGroups> CachedHairDescriptionGroups[static_cast<uint8>(EHairDescriptionType::Count)];

	TArray<FString> StrandsDerivedDataKey;
	TArray<FString> CardsDerivedDataKey;
	TArray<FString> MeshesDerivedDataKey;

	TStrongObjectPtr<UGroomAsset> GroomAssetStrongPtr;
	bool bRetryLoadFromGameThread = false;
#endif // WITH_EDITORONLY_DATA
	bool bIsInitialized = false;
	uint32 AssetNameHash = 0;

#if WITH_EDITOR
public:
	UE_API void UpdateCachedSettings();
private:
	void SavePendingProceduralAssets();

	// Cached groom settings to know if we need to recompute interpolation data or 
	// decimation when the asset is saved
	TArray<FHairGroupsRendering>				CachedHairGroupsRendering;
	TArray<FHairGroupsPhysics>					CachedHairGroupsPhysics;
	TArray<FHairGroupsInterpolation>			CachedHairGroupsInterpolation;
	TArray<FHairGroupsLOD>						CachedHairGroupsLOD;
	TArray<FHairGroupsCardsSourceDescription>	CachedHairGroupsCards;
	TArray<FHairGroupsMeshesSourceDescription>	CachedHairGroupsMeshes;
	EShaderPlatform								CachedResourceShaderPlatform = SP_NumPlatforms;

	// Queue of procedural assets which needs to be saved
	TQueue<UStaticMesh*> AssetToSave_Meshes;
	TQueue<FHairGroupCardsTextures*> AssetToSave_Textures;
#endif // WITH_EDITOR
};

struct FGroomAssetMemoryStats
{
	struct FStats
	{
		uint32 Guides = 0;
		uint32 Strands= 0;
		uint32 Cards  = 0;
		uint32 Meshes = 0;
	};
	FStats CPU;
	FStats GPU;

	struct FStrandsDetails
	{
		uint32 Rest = 0;
		uint32 Interpolation= 0;
		uint32 Cluster  = 0;
		uint32 Raytracing = 0;
	};
	FStrandsDetails Memory;
	FStrandsDetails Curves;

	static FGroomAssetMemoryStats Get(const FHairGroupPlatformData& InData, const FHairGroupResources& In);
	void Accumulate(const FGroomAssetMemoryStats& In);
	uint32 GetTotalCPUSize() const;
	uint32 GetTotalGPUSize() const;
};

#undef UE_API
