// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "GroomAssetCards.generated.h"

#define UE_API HAIRSTRANDSCORE_API


class UMaterialInterface;
class UStaticMesh;
class UTexture2D;
enum class EHairAtlasTextureType : uint8;
class UGroomAsset;

UENUM(BlueprintType)
enum class EHairCardsSourceType : uint8
{
	Procedural  UMETA(DisplayName = "Procedural (experimental)"),
	Imported UMETA(DisplayName = "Imported"),
};

UENUM(BlueprintType)
enum class EHairCardsGuideType : uint8
{
	Generated  UMETA(DisplayName = "Generated"),
	GuideBased UMETA(DisplayName = "Guide-Based"),
};

UENUM(BlueprintType)
enum class EHairTextureLayout : uint8
{
	Layout0 UMETA(DisplayName = "Card Default"),
	Layout1 UMETA(DisplayName = "Mesh Default"),
	Layout2 UMETA(DisplayName = "Card Compact"),
	Layout3 UMETA(DisplayName = "Mesh Compact"),
};

// Returns the number of textures used for a particular layout
HAIRSTRANDSCORE_API uint32 GetHairTextureLayoutTextureCount(EHairTextureLayout In);
HAIRSTRANDSCORE_API const TCHAR* GetHairTextureLayoutTextureName(EHairTextureLayout InLayout, uint32 InIndex, bool bDetail);

USTRUCT(BlueprintType)
struct FHairGroupCardsInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Card Count"))
	int32 NumCards = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Card Vertex Count"))
	int32 NumCardVertices = 0;
};

USTRUCT(BlueprintType)
struct FHairGroupCardsTextures
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "CardsTextures")
	EHairTextureLayout Layout = EHairTextureLayout::Layout0;

	UPROPERTY(EditAnywhere, Category = "CardsTextures")
	TArray<TObjectPtr<UTexture2D>> Textures;

	UPROPERTY()
	TObjectPtr<UTexture2D> DepthTexture_DEPRECATED = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture2D> CoverageTexture_DEPRECATED = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture2D> TangentTexture_DEPRECATED = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture2D> AttributeTexture_DEPRECATED = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture2D> AuxilaryDataTexture_DEPRECATED = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture2D> MaterialTexture_DEPRECATED = nullptr;

	UE_API void SetLayout(EHairTextureLayout InLayout);
	UE_API void SetTexture(int32 SlotID, UTexture2D* Texture);

	bool bNeedToBeSaved = false;
};

/** 
 * Since hair-card generation can be controlled external to this plugin, this  
 * provides a way for those external generators a way to store their own 
 * generation data along with the groom/cards-entry.
 */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UHairCardGenerationSettings : public UObject
{
	GENERATED_BODY()
public:
	virtual void BuildDDCKey(FArchive& Ar) PURE_VIRTUAL(BuildDDCKey,);
};

USTRUCT(BlueprintType)
struct FHairGroupsCardsSourceDescription
{
	GENERATED_BODY()

	UE_API FHairGroupsCardsSourceDescription();

	/* Deprecated */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY()
	FName MaterialSlotName;

	UPROPERTY()
	EHairCardsSourceType SourceType_DEPRECATED = EHairCardsSourceType::Imported;
	
	UPROPERTY()
	TObjectPtr<UStaticMesh> ProceduralMesh_DEPRECATED = nullptr;

	UPROPERTY()
	bool bInvertUV = false;

	UPROPERTY(EditAnywhere, Category = "CardsSource")
	EHairCardsGuideType GuideType;

	UPROPERTY(EditAnywhere, Category = "CardsSource", meta = (DisplayName = "Mesh"))
	TObjectPtr<UStaticMesh> ImportedMesh = nullptr;

	UPROPERTY(EditAnywhere, Category = "CardsSource")
	FHairGroupCardsTextures Textures;

	/* Group index on which this cards geometry will be used (#hair_todo: change this to be a dropdown selection menu in FHairLODSettings instead) */
	UPROPERTY(EditAnywhere, Category = "CardsSource")
	int32 GroupIndex = 0;
	
	/* LOD on which this cards geometry will be used. -1 means not used  (#hair_todo: change this to be a dropdown selection menu in FHairLODSettings instead) */
	UPROPERTY(EditAnywhere, Category = "CardsSource")
	int32 LODIndex = -1; 

#if WITH_EDITORONLY_DATA
	/* Card generation data saved from the last procedural run. Dependent on the generator responsible for running the generation. */
	UPROPERTY(meta=(EditInline))
	TObjectPtr<UHairCardGenerationSettings> GenerationSettings = nullptr;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(VisibleAnywhere, Transient, Category = "CardsSource")
	mutable FHairGroupCardsInfo CardsInfo;

	UPROPERTY(Transient)
	FString ImportedMeshKey;

	UE_API bool operator==(const FHairGroupsCardsSourceDescription& A) const;

	UE_API UStaticMesh* GetMesh() const;
	UE_API FString GetMeshKey() const;
	UE_API bool HasMeshChanged() const;
	UE_API void UpdateMeshKey();
};

#undef UE_API
