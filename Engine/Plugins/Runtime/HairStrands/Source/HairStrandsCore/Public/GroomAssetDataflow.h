// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowContent.h"
#include "Dataflow/DataflowInstance.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GroomAssetDataflow.generated.h"

#define UE_API HAIRSTRANDSCORE_API

class UGroomBindingAsset;
class UAnimationAsset;

/** 
 * Dataflow content owning dataflow and binding assets that will be used to evaluate the graph
 */
UCLASS()
class UDataflowGroomContent : public UDataflowSkeletalContent
{
	GENERATED_BODY()

public:
	UE_API UDataflowGroomContent();
	virtual ~UDataflowGroomContent() override = default;
	
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //if WITH_EDITOR

	/** Data flow binding asset accessors */
	UE_API void SetBindingAsset(const TObjectPtr<UGroomBindingAsset>& InBindingAsset);
	const TObjectPtr<UGroomBindingAsset>& GetBindingAsset() const { return BindingAsset; }

	//~ UObject interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** Set all the preview actor exposed properties */
	UE_API virtual void SetActorProperties(TObjectPtr<AActor>& PreviewActor) const override;

protected:

	/** Data flow skeletal mesh*/
	UPROPERTY(EditAnywhere, Category = "Preview", Transient, SkipSerialization)
	TObjectPtr<UGroomBindingAsset> BindingAsset = nullptr;
};

USTRUCT()
struct FGroomDataflowSettings : public FDataflowInstance
{
	GENERATED_BODY()

	UE_API explicit FGroomDataflowSettings(UObject* InOwner = nullptr, UDataflow* InDataflowAsset = nullptr, FName InTerminalNodeName = NAME_None);

	/** Return the Dataflow asset name. */
	static UE_API FName GetDataflowAssetMemberName();

	/** Return the Dataflow terminal name. */
	static UE_API FName GetDataflowTerminalMemberName();

	/** Return the skeletal mesh associated to this groom asset if any. */
	USkeletalMesh* GetSkeletalMesh(const int32 GroupIndex) const { return !SkeletalMeshes.IsValidIndex(GroupIndex) ? nullptr : SkeletalMeshes[GroupIndex]; }

	/** Return the mesh LOD used to transfer the skinning */
	int32 GetMeshLOD(const int32 GroupIndex) const { return !MeshLODs.IsValidIndex(GroupIndex) ? INDEX_NONE : MeshLODs[GroupIndex]; }
	
	/** Return the groom asset dataflow settings rest collection . */
	const FManagedArrayCollection* GetRestCollection() const { return RestCollection.Get(); }
	
	/** Init the skeletal mesh associated to this groom asset if any. */
	void InitSkeletalMeshes(const int32 NumGroups) { SkeletalMeshes.Init(nullptr, NumGroups); MeshLODs.Init(INDEX_NONE, NumGroups); }

	/** Set a group skeletal mesh and its associated LOD to this groom asset */
	void SetSkeletalMesh(const int32 GroupIndex, USkeletalMesh* SkeletalMesh, const int32 MeshLOD)
	{
		if(SkeletalMeshes.IsValidIndex(GroupIndex) && MeshLODs.IsValidIndex(GroupIndex))
		{
			SkeletalMeshes[GroupIndex] = SkeletalMesh;
			MeshLODs[GroupIndex] = MeshLOD;
		}
	}
	
	/** Set the rest collection onto the groom asset dataflow settings */
	void SetRestCollection(FManagedArrayCollection* InRestCollection) { RestCollection = TSharedPtr<FManagedArrayCollection, ESPMode::ThreadSafe>(InRestCollection); }

	/** Custom serialization function for the ManagedArrayCollection */
	UE_API bool Serialize(FArchive& Ar);
	
#if WITH_EDITORONLY_DATA

	/** Preview scene binding asset setter */
	UE_API void SetPreviewBindingAsset(UGroomBindingAsset* BindingAsset);

	/** Preview scene binding asset setter */
	UE_API UGroomBindingAsset* GetPreviewBindingAsset() const;

	/** Preview scene animation setter */
	UE_API void SetPreviewAnimationAsset(UAnimationAsset* AnimationAsset);

	/** Preview scene animation getter */
	UE_API UAnimationAsset* GetPreviewAnimationAsset() const;

#endif

private :

    /** Hair geometry that could describe external cards and meshes. */
    UPROPERTY()
    TArray<TObjectPtr<USkeletalMesh>> SkeletalMeshes;

	/** LOD indices of the hair geometry used to transfer the skinning weights */
	UPROPERTY()
	TArray<int32> MeshLODs;

#if WITH_EDITORONLY_DATA
	
	/** Optional binding asset used in the dataflow Editor preview scene */
	UPROPERTY(AssetRegistrySearchable)
	TSoftObjectPtr<UGroomBindingAsset> PreviewBindingAsset;

	/** Optional animation asset used in the dataflow Editor preview scene*/
	UPROPERTY(AssetRegistrySearchable)
	TSoftObjectPtr<UAnimationAsset> PreviewAnimationAsset;
	
#endif

	/** Rest collection used to store all the dataflow attributes */
	TSharedPtr<FManagedArrayCollection> RestCollection;
};

template<> struct TStructOpsTypeTraits<FGroomDataflowSettings> : public TStructOpsTypeTraitsBase2<FGroomDataflowSettings>
{
	enum
	{
		WithSerializer = true,
	};
};

#undef UE_API
