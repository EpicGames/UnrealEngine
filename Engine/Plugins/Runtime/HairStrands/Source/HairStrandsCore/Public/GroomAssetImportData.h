// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EditorFramework/AssetImportData.h"
#include "GroomAssetImportData.generated.h"

UCLASS(MinimalAPI)
class UGroomAssetImportData : public UAssetImportData
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY()
	TObjectPtr<class UGroomImportOptions> ImportOptions;

	UPROPERTY()
	TObjectPtr<class UGroomCreateStrandsTexturesOptions> HairStrandsTexturesOptions;
};
