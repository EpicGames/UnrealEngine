// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PerPlatformProperties.h"
#include "GroomAssetInterpolation.generated.h"

#define UE_API HAIRSTRANDSCORE_API

UENUM()
enum class EHairInterpolationQuality : uint8
{
	Low		UMETA(DisplayName = "Low", ToolTip = "Build interpolation data based on nearst neighbor search. Low quality interpolation data, but fast to build (takes a few minutes)"),
	Medium	UMETA(DisplayName = "Medium", ToolTip = "Build interpolation data using curve shape matching search but within a limited spatial range. This is a tradeoff between Low and high quality in term of quality & build time (can takes several dozen of minutes)"),
	High	UMETA(DisplayName = "High", ToolTip = "Build interpolation data using curve shape matching search. This result in high quality interpolation data, but is relatively slow to build (can takes several dozen of minutes)"),
	Unknown	UMETA(Hidden),
};

UENUM()
enum class EHairInterpolationWeight : uint8
{
	Parametric	UMETA(DisplayName = "Parametric", ToolTip = "Build interpolation data based on curve parametric distance"),
	Root		UMETA(DisplayName = "Root", ToolTip = "Build interpolation data based on distance between guide's root and strands's root"),
	Index		UMETA(DisplayName = "Index", ToolTip = "Build interpolation data based on guide and strands vertex indices"),
	Distance	UMETA(DisplayName = "Distance", ToolTip = "Build interpolation data based on curve euclidean distance"),
	Unknown		UMETA(Hidden),
};

UENUM(BlueprintType)
enum class EGroomGeometryType : uint8
{
	Strands,
	Cards,
	Meshes
};

UENUM(BlueprintType)
enum class EGroomBindingType : uint8
{
	NoneBinding	UMETA(Hidden),
	Rigid		UMETA(DisplayName = "Rigid",    ToolTip = "When attached to a skeltal mesh, the hair follow the provided attachement name"),
	Skinning	UMETA(DisplayName = "Skinning", ToolTip = "When attached to a skeltal mesh, the hair follow the skin surface"),
};

UENUM(BlueprintType)
enum class EGroomOverrideType : uint8
{
	Auto	UMETA(DisplayName = "Auto", ToolTip = "Use the asset value"),
	Enable	UMETA(DisplayName = "Enable", ToolTip = "Override the asset value, and force enabled"),
	Disable UMETA(DisplayName = "Disable", ToolTip = "Override the asset value, and force disabled")
};

UENUM(BlueprintType)
enum class EGroomGuideType : uint8
{
	Imported	UMETA(DisplayName = "Imported Guides", ToolTip = "Use imported asset guides."),
	Generated	UMETA(DisplayName = "Generated Guides", ToolTip = "Generate guides from imported strands"),
	Rigged 		UMETA(DisplayName = "Rigged  Guides", ToolTip = "Generated rigged guides from imported strands")
};

UENUM(BlueprintType)
enum class EGroomLODMode : uint8
{
	Default		UMETA(DisplayName = "Default", ToolTip = "Hair strands curves & points adapt based on project settings LOD mode. (See 'Use Auto LOD' in project settings). "),
	Manual 		UMETA(DisplayName = "Manual", ToolTip = "Hair strands curves & points adapt based on LOD settings"),
	Auto		UMETA(DisplayName = "Auto", ToolTip = "Hair strands curves & points automatically adapt based on screen coverage. LOD settings are ignored.")
};

USTRUCT(BlueprintType)
struct FHairLODSettings
{
	GENERATED_BODY();

	/** Reduce the number of hair strands */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Reduce the number of hair strands in a uniform manner", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float CurveDecimation = 1;

	/** Reduce the number of points for each hair strands */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = " Reduce the number of points for each hair strands", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float VertexDecimation = 1;

	/** Max angular difference between adjacents vertices to remove vertices during simplification, in degrees. */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Max angular difference between adjacents vertices to remove vertices during simplification, in degrees", ClampMin = "0", ClampMax = "45", UIMin = "0", UIMax = "45.0"))
	float AngularThreshold = 1.f;

	/** Screen size at which this LOD should be enabled */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Screen size at which this LOD should be enabled", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float ScreenSize = 1;

	/** Scales the hair Strands radius. This can be used for manually compensating the reduction of curves.. */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Scales the hair Strands radius. This can be used for manually compensating the reduction of curves.", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float ThicknessScale = 1;

	/** If true (default), the hair group is visible. If false, the hair group is culled. */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "If disable, the hair strands won't be rendered"))
	bool bVisible = true;

	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Defines the type of geometry used by this LOD (Strands, Cards, or Meshes)"))
	EGroomGeometryType GeometryType = EGroomGeometryType::Strands;

	UPROPERTY(EditAnywhere, Category = "DecimationSettings", AdvancedDisplay, meta = (ToolTip = "Defines the type of attachment"))
	EGroomBindingType BindingType = EGroomBindingType::Skinning;
	
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", AdvancedDisplay, meta = (ToolTip = "Groom simulation"))
	EGroomOverrideType Simulation = EGroomOverrideType::Auto;

	UPROPERTY(EditAnywhere, Category = "DecimationSettings", AdvancedDisplay, meta = (DisplayName = "RBF Interpolation", ToolTip = "Global interpolation"))
	EGroomOverrideType GlobalInterpolation = EGroomOverrideType::Auto;

	UE_API bool operator==(const FHairLODSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairDecimationSettings
{
	GENERATED_BODY()

	UE_API FHairDecimationSettings();

	/** Reduce the number of hair strands in a uniform manner (initial decimation) */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Reduce the number of hair strands in a uniform manner", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float CurveDecimation;

	/**	Reduce the number of vertices for each hair strands in a uniform manner (initial decimation) */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Reduce the number of verties for each hair strands in a uniform manner", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float VertexDecimation;

	UE_API bool operator==(const FHairDecimationSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairInterpolationSettings
{
	GENERATED_USTRUCT_BODY()

	UE_API FHairInterpolationSettings();

	/** Flag to override the imported guides with generated guides. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterpolationSettings", meta = (ToolTip = "Type of guides:\n - Imported: use imported guides\n - Generated: generate guides from strands\n - Rigged: generated rigged guides from strands."))
	EGroomGuideType GuideType;

	/** Flag to override the imported guides with generated guides. DEPRECATED */
	UPROPERTY()
	bool bOverrideGuides_DEPRECATED;

	/** Density factor for converting hair into guide curve if no guides are provided. The value should be between 0 and 1, and can be thought as a ratio/percentage of strands used as guides.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterpolationSettings", meta = (DisplayName = "Generated guide density", ClampMin = "0", ClampMax = "1.0", UIMin = "0", UIMax = "1.0", EditCondition="GuideType == EGroomGuideType::Generated"))
	float HairToGuideDensity;

	/** Number of curves to generate in the skel mesh */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "InterpolationSettings", meta = (DisplayName = "Rigged guide num. curves", ToolTip = "Number of guides that will be generated on the groom and the skeletal mesh", EditCondition="GuideType == EGroomGuideType::Rigged"))
	int32 RiggedGuideNumCurves;

	/** Number of points per curve */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "InterpolationSettings", meta = (DisplayName = "Rigged guide num. points", ToolTip = "Number of points/bones per generated guide", EditCondition="GuideType == EGroomGuideType::Rigged"))
	int32 RiggedGuideNumPoints;

	/** Interpolation data quality. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterpolationSettings")
	EHairInterpolationQuality InterpolationQuality;

	/** Interpolation distance metric. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterpolationSettings")
	EHairInterpolationWeight InterpolationDistance;

	/** Randomize which guides affect a given hair strand. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterpolationSettings")
	bool bRandomizeGuide;

	/** Force a hair strand to be affected by a unique guide. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterpolationSettings")
	bool bUseUniqueGuide;

	UE_API bool operator==(const FHairInterpolationSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairDeformationSettings
{
	GENERATED_USTRUCT_BODY()

	UE_API FHairDeformationSettings();

	UPROPERTY()
	bool bEnableRigging_DEPRECATED;

	UPROPERTY()
	int32 NumCurves_DEPRECATED;

	UPROPERTY()
	int32 NumPoints_DEPRECATED;

	UE_API bool operator==(const FHairDeformationSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairGroupsInterpolation
{
	GENERATED_USTRUCT_BODY()

	UE_API FHairGroupsInterpolation();

	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Decimation settings"))
	FHairDecimationSettings DecimationSettings;

	UPROPERTY(EditAnywhere, Category = "InterpolationSettings", meta = (ToolTip = "Interpolation settings"))
	FHairInterpolationSettings InterpolationSettings;

	// DEPRECATED
	UPROPERTY()
	FHairDeformationSettings RiggingSettings;

#if WITH_EDITORONLY_DATA
	/** Group name to be displayed in the array list */
	UPROPERTY(VisibleDefaultsOnly, Transient, Category=AlwaysHidden, Meta=(EditCondition=False, EditConditionHides))
	FName GroupName = NAME_None;
#endif

	UE_API bool operator==(const FHairGroupsInterpolation& A) const;

	UE_API void BuildDDCKey(FArchive& Ar);
};

USTRUCT(BlueprintType)
struct FHairGroupsLOD
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (DisplayName = "Group Auto LOD Bias", ToolTip = "When LOD mode is set to Auto, decrease the screen size at which curves reduction will occur. The final bias value is computed by adding this value to the asset's Auto LOD bias value.", ClampMin = "-1", ClampMax = "1", UIMin = "-1.0", UIMax = "1.0"))
	float AutoLODBias = 0;

	/** LODs  */
	UPROPERTY(EditAnywhere, Category = "DecimationSettings", meta = (ToolTip = "Reduce the number of hair strands in a uniform manner", ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	TArray<FHairLODSettings> LODs;

	UE_API bool operator==(const FHairGroupsLOD& A) const;

	UE_API void BuildDDCKey(FArchive& Ar);
	static UE_API FHairGroupsLOD GetDefault();
};

#undef UE_API
