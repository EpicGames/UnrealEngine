// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GroomAssetCards.h"

#include "GroomAssetMeshes.generated.h"

#define UE_API HAIRSTRANDSCORE_API


class UMaterialInterface;
class UStaticMesh;


USTRUCT(BlueprintType)
struct FHairGroupsMeshesSourceDescription
{
	GENERATED_BODY()

	UE_API FHairGroupsMeshesSourceDescription();

	/* Deprecated */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY()
	FName MaterialSlotName;

	UPROPERTY(EditAnywhere, Category = "MeshSettings", meta = (ToolTip = "Mesh settings"))
	TObjectPtr<class UStaticMesh> ImportedMesh;

	UPROPERTY(EditAnywhere, Category = "MeshesSource")
	FHairGroupCardsTextures Textures;

	/* Group index on which this mesh geometry will be used (#hair_todo: change this to be a dropdown selection menu in FHairLODSettings instead) */
	UPROPERTY(EditAnywhere, Category = "MeshesSource")
	int32 GroupIndex = 0;

	/* LOD on which this mesh geometry will be used. -1 means not used  (#hair_todo: change this to be a dropdown selection menu in FHairLODSettings instead) */
	UPROPERTY(EditAnywhere, Category = "MeshesSource")
	int32 LODIndex = -1;

	UPROPERTY(Transient)
	FString ImportedMeshKey;

	UE_API bool operator==(const FHairGroupsMeshesSourceDescription& A) const;

	UE_API FString GetMeshKey() const;
	UE_API bool HasMeshChanged() const;
	UE_API void UpdateMeshKey();
};

#undef UE_API
