// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Curves/CurveFloat.h"
#include "GroomAssetPhysics.generated.h"

#define UE_API HAIRSTRANDSCORE_API

class UNiagaraSystem;

/** List of niagara solvers */
UENUM(BlueprintType)
enum class EGroomNiagaraSolvers : uint8
{
	None = 0 UMETA(Hidden),
	CosseratRods = 0x02 UMETA(DisplayName = "Groom Rods"),
	AngularSprings = 0x04 UMETA(DisplayName = "Groom Springs"),
	CustomSolver = 0x08 UMETA(DisplayName = "Custom Solver")
};

/** Size of each strands*/
UENUM(BlueprintType)
enum class EGroomStrandsSize : uint8
{
	None = 0 UMETA(Hidden),
	Size2 = 0x02 UMETA(DisplayName = "2"),
	Size4 = 0x04 UMETA(DisplayName = "4"),
	Size8 = 0x08 UMETA(DisplayName = "8"),
	Size16 = 0x10 UMETA(DisplayName = "16"),
	Size32 = 0x20 UMETA(DisplayName = "32")
};

/** List of interpolation type */
UENUM(BlueprintType)
enum class EGroomInterpolationType : uint8
{
	None = 0 UMETA(DisplayName = "No skinning"),
	RigidTransform = 0x02 UMETA(DisplayName = "Rigid Transform"),
	OffsetTransform = 0x04 UMETA(DisplayName = "Offset Transform"),
	SmoothTransform = 0x08 UMETA(DisplayName = "Smooth Transform")
};

USTRUCT(BlueprintType)
struct FHairSolverSettings
{
	GENERATED_BODY()

	UE_API FHairSolverSettings();
	
	/** Use deformer for simulation  */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (ToolTip = "Use deformer for the groom solver", DisplayName = "Enable Deformation"))
	bool bEnableDeformation;

	/** Use niagara for simulation */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (ToolTip = "Use niagara for the groom solver", DisplayName = "Enable Simulation"))
	bool EnableSimulation;

	/** Niagara solver to be used for simulation */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (ToolTip = "Niagara solver to be used for simulation"))
	EGroomNiagaraSolvers NiagaraSolver;

	/** Custom system to be used for simulation */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (EditCondition = "NiagaraSolver == EGroomNiagaraSolvers::CustomSolver", ToolTip = "Custom niagara system to be used if custom solver is picked"))
	TSoftObjectPtr<UNiagaraSystem> CustomSystem;

	/** Optimisation of the rest state configuration to compensate from the gravity */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (EditCondition = "NiagaraSolver == EGroomNiagaraSolvers::AngularSprings", ToolTip = "Optimisation of the rest state configuration to compensate from the gravity "))
	float GravityPreloading;

	/** Number of substeps to be used */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (ToolTip = "Number of sub steps to be done per frame. The actual solver calls are done at 24 fps"))
	int32 SubSteps;

	/** Number of iterations for the constraint solver  */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (ToolTip = "Number of iterations to solve the constraints with the xpbd solver"))
	int32 IterationCount;

	/** Force the Niagara solver component to be visible  */
	UPROPERTY(EditAnywhere, Category = "SolverSettings", meta = (ToolTip = "Force the Niagara solver component to be visible"))
	bool bForceVisible;

	UE_API bool operator==(const FHairSolverSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairExternalForces
{
	GENERATED_BODY()

	UE_API FHairExternalForces();

	/** Acceleration vector in cm/s2 to be used for the gravity*/
	UPROPERTY(EditAnywhere, Category = "ExternalForces", meta = (ToolTip = "Acceleration vector in cm/s2 to be used for the gravity"))
	FVector GravityVector;

	/** Coefficient between 0 and 1 to be used for the air drag */
	UPROPERTY(EditAnywhere, Category = "ExternalForces", meta = (ToolTip = "Coefficient between 0 and 1 to be used for the air drag"))
	float AirDrag;

	/** Velocity of the surrounding air in cm/s */
	UPROPERTY(EditAnywhere, Category = "ExternalForces", meta = (ToolTip = "Velocity of the surrounding air in cm/s"))
	FVector AirVelocity;

	UE_API bool operator==(const FHairExternalForces& A) const;
};

USTRUCT(BlueprintType)
struct FHairBendConstraint
{
	GENERATED_BODY()

	UE_API FHairBendConstraint();

	/** Enable the solve of the bend constraint during the xpbd loop */
	UPROPERTY(EditAnywhere, Category = "BendConstraint", meta = (ToolTip = "Enable the solve of the bend constraint during the xpbd loop"))
	bool SolveBend;

	/** Enable the projection of the bend constraint after the xpbd loop */
	UPROPERTY(EditAnywhere, Category = "BendConstraint", meta = (ToolTip = "Enable ther projection of the bend constraint after the xpbd loop"))
	bool ProjectBend;

	/** Damping for the bend constraint between 0 and 1 */
	UPROPERTY(EditAnywhere, Category = "BendConstraint", meta = (ToolTip = "Damping for the bend constraint between 0 and 1"))
	float BendDamping;

	/** Stiffness for the bend constraint in GPa */
	UPROPERTY(EditAnywhere, Category = "BendConstraint", meta = (ToolTip = "Stiffness for the bend constraint in GPa"))
	float BendStiffness;

	/** Stiffness scale along the strand */
	UPROPERTY(EditAnywhere, Category = "BendConstraint", meta = (DisplayName = "Stiffness Scale", ViewMinInput = "0.0", ViewMaxInput = "1.0", ViewMinOutput = "0.0", ViewMaxOutput = "1.0", TimeLineLength = "1.0", XAxisName = "Strand Coordinate (0,1)", YAxisName = "Bend Scale", ToolTip = "This curve determines how much the bend stiffness will be scaled along each strand. \n The X axis range is [0,1], 0 mapping the root and 1 the tip"))
	FRuntimeFloatCurve BendScale;

	UE_API bool operator==(const FHairBendConstraint& A) const;
};


USTRUCT(BlueprintType)
struct FHairStretchConstraint
{
	GENERATED_BODY()

	UE_API FHairStretchConstraint();

	/** Enable the solve of the stretch constraint during the xpbd loop */
	UPROPERTY(EditAnywhere, Category = "StretchConstraint", meta = (ToolTip = "Enable the solve of the stretch constraint during the xpbd loop"))
	bool SolveStretch;

	/** Enable the projection of the stretch constraint after the xpbd loop */
	UPROPERTY(EditAnywhere, Category = "StretchConstraint", meta = (ToolTip = "Enable ther projection of the stretch constraint after the xpbd loop"))
	bool ProjectStretch;

	/** Damping for the stretch constraint between 0 and 1 */
	UPROPERTY(EditAnywhere, Category = "StretchConstraint", meta = (ToolTip = "Damping for the stretch constraint between 0 and 1"))
	float StretchDamping;

	/** Stiffness for the stretch constraint in GPa */
	UPROPERTY(EditAnywhere, Category = "StretchConstraint", meta = (ToolTip = "Stiffness for the stretch constraint in GPa"))
	float StretchStiffness;

	/** Stretch scale along the strand  */
	UPROPERTY(EditAnywhere, Category = "StretchConstraint", meta = (DisplayName = "Stiffness Scale", ViewMinInput = "0.0", ViewMaxInput = "1.0", ViewMinOutput = "0.0", ViewMaxOutput = "1.0", TimeLineLength = "1.0", XAxisName = "Strand Coordinate (0,1)", YAxisName = "Stretch Scale", ToolTip = "This curve determines how much the stretch stiffness will be scaled along each strand. \n The X axis range is [0,1], 0 mapping the root and 1 the tip"))
	FRuntimeFloatCurve StretchScale;

	UE_API bool operator==(const FHairStretchConstraint& A) const;
};

USTRUCT(BlueprintType)
struct FHairCollisionConstraint
{
	GENERATED_BODY()

	UE_API FHairCollisionConstraint();

	/** Enable the solve of the collision constraint during the xpbd loop  */
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ToolTip = "Enable the solve of the collision constraint during the xpbd loop"))
	bool SolveCollision;

	/** Enable ther projection of the collision constraint after the xpbd loop */
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ToolTip = "Enable ther projection of the collision constraint after the xpbd loop"))
	bool ProjectCollision;

	/** Static friction used for collision against the physics asset */
	UPROPERTY(EditAnywhere, Category = "Collision Constraint", meta = (ToolTip = "Static friction used for collision against the physics asset"))
	float StaticFriction;

	/** Kinetic friction used for collision against the physics asset*/
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ToolTip = "Kinetic friction used for collision against the physics asset"))
	float KineticFriction;

	/** Viscosity parameter between 0 and 1 that will be used for self collision */
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ToolTip = "Viscosity parameter between 0 and 1 that will be used for self collision"))
	float StrandsViscosity;

	/** Dimension of the grid used to compute the viscosity force */
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ToolTip = "Dimension of the grid used to compute the viscosity force"))
	FIntVector GridDimension;

	/** Radius that will be used for the collision detection against the physics asset */
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ToolTip = "Radius that will be used for the collision detection against the physics asset"))
	float CollisionRadius;

	/** Radius scale along the strand */
	UPROPERTY(EditAnywhere, Category = "CollisionConstraint", meta = (ViewMinInput = "0.0", ViewMaxInput = "1.0", ViewMinOutput = "0.0", ViewMaxOutput = "1.0", TimeLineLength = "1.0", XAxisName = "Strand Coordinate (0,1)", YAxisName = "Collision Radius", ToolTip = "This curve determines how much the collision radius will be scaled along each strand. \n The X axis range is [0,1], 0 mapping the root and 1 the tip"))
	FRuntimeFloatCurve RadiusScale;

	UE_API bool operator==(const FHairCollisionConstraint& A) const;
};


USTRUCT(BlueprintType)
struct FHairMaterialConstraints
{
	GENERATED_BODY()

	UE_API FHairMaterialConstraints();

	/** Radius scale along the strand */
	UPROPERTY(EditAnywhere, Category = "MaterialConstraints", meta = (ToolTip = "Bend constraint settings to be applied to the hair strands"))
	FHairBendConstraint BendConstraint;

	/** Radius scale along the strand */
	UPROPERTY(EditAnywhere, Category = "MaterialConstraints", meta = (ToolTip = "Stretch constraint settings to be applied to the hair strands"))
	FHairStretchConstraint StretchConstraint;

	/** Radius scale along the strand */
	UPROPERTY(EditAnywhere, Category = "MaterialConstraints", meta = (ToolTip = "Collision constraint settings to be applied to the hair strands"))
	FHairCollisionConstraint CollisionConstraint;

	UE_API bool operator==(const FHairMaterialConstraints& A) const;
};

USTRUCT(BlueprintType)
struct FHairStrandsParameters
{
	GENERATED_BODY()

	UE_API FHairStrandsParameters();

	/** Number of particles per guide that will be used for simulation*/
	UPROPERTY(EditAnywhere, Category = "StrandsParameters", meta = (ToolTip = "Number of particles per guide that will be used for simulation"))
	EGroomStrandsSize StrandsSize;

	/** Density of the strands in g/cm3 */
	UPROPERTY(EditAnywhere, Category = "StrandsParameters", meta = (ToolTip = "Density of the strands in g/cm3"))
	float StrandsDensity;

	/** Smoothing between 0 and 1 of the incoming guides curves for better stability */
	UPROPERTY(EditAnywhere, Category = "StrandsParameters", meta = (ToolTip = "Smoothing between 0 and 1 of the incoming guides curves for better stability"))
	float StrandsSmoothing;

	/** Strands thickness in cm that will be used for mass and inertia computation */
	UPROPERTY(EditAnywhere, Category = "StrandsParameters", meta = (ToolTip = "Strands thickness in cm that will be used for mass and inertia computation"))
	float StrandsThickness;

	/** Thickness scale along the curve */
	UPROPERTY(EditAnywhere, Category = "StrandsParameters", meta = (ViewMinInput = "0.0", ViewMaxInput = "1.0", ViewMinOutput = "0.0", ViewMaxOutput = "1.0", TimeLineLength = "1.0", XAxisName = "Strand Coordinate (0,1)", YAxisName = "Strands Thickness", ToolTip = "This curve determines how much the strands thickness will be scaled along each strand. \n The X axis range is [0,1], 0 mapping the root and 1 the tip"))
	FRuntimeFloatCurve ThicknessScale;

	UE_API bool operator==(const FHairStrandsParameters& A) const;
};

USTRUCT(BlueprintType)
struct FHairGroupsPhysics
{
	GENERATED_BODY()

	UE_API FHairGroupsPhysics();

	/** Enable the simulation for the group */
	UPROPERTY(EditAnywhere, Category = "GroupsPhysics", meta = (ToolTip = "Solver Settings for the hair physics"))
	FHairSolverSettings SolverSettings;

	/** Enable the simulation for the group */
	UPROPERTY(EditAnywhere, Category = "GroupsPhysics", meta = (ToolTip = "External Forces for the hair physics"))
	FHairExternalForces ExternalForces;

	/** Enable the simulation for the group */
	UPROPERTY(EditAnywhere, Category = "GroupsPhysics", meta = (ToolTip = "Material Constraints for the hair physics"))
	FHairMaterialConstraints MaterialConstraints;

	/** Enable the simulation for the group */
	UPROPERTY(EditAnywhere, Category = "GroupsPhysics", meta = (ToolTip = "Strands Parameters for the hair physics"))
	FHairStrandsParameters StrandsParameters;

#if WITH_EDITORONLY_DATA
	/** Group name to be displayed in the array list */
	UPROPERTY(VisibleDefaultsOnly, Transient, Category=AlwaysHidden, Meta=(EditCondition=False, EditConditionHides))
	FName GroupName = NAME_None;
#endif

	UE_API bool operator==(const FHairGroupsPhysics& A) const;

	/** Build DDC key */
	void BuildDDCKey(FArchive& Ar);
};

USTRUCT(BlueprintType)
struct FHairSimulationSolver
{
	GENERATED_USTRUCT_BODY()

	/** Enable the simulation for the group */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SolverSettings", meta = (ToolTip = "Enable the simulation of the groom groups/LODs if both this boolean and the one in the groom asset are true"))
	bool bEnableSimulation = false;
};

USTRUCT(BlueprintType)
struct FHairSimulationForces
{
	GENERATED_USTRUCT_BODY()

	/** Acceleration vector in cm/s2 to be used for the gravity*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "ExternalForces", meta = (ToolTip = "Acceleration vector in cm/s2 to be used for the gravity"))
	FVector GravityVector = FVector(0.0, 0.0, -981.0);

	/** Coefficient between 0 and 1 to be used for the air drag */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "ExternalForces", meta = (ToolTip = "Coefficient between 0 and 1 to be used for the air drag"))
	float AirDrag = 0.1f;

	/** Velocity of the surrounding air in cm/s */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "ExternalForces", meta = (ToolTip = "Velocity of the surrounding air in cm/s"))
	FVector AirVelocity = FVector(0.0, 0.0, 0.0);
};

USTRUCT(BlueprintType)
struct FHairSimulationConstraints
{
	GENERATED_USTRUCT_BODY()

	/** Damping for the bend constraint between 0 and 1 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, AdvancedDisplay, Category = "BendConstraint", meta = (ToolTip = "Damping for the bend constraint between 0 and 1"))
	float BendDamping = 0.001f;

	/** Stiffness for the bend constraint in GPa */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "BendConstraint", meta = (ToolTip = "Stiffness for the bend constraint in GPa"))
	float BendStiffness = 0.01f;

	/** Damping for the stretch constraint between 0 and 1 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, AdvancedDisplay, Category = "StretchConstraint", meta = (ToolTip = "Damping for the stretch constraint between 0 and 1"))
	float StretchDamping = 0.001f;

	/** Stiffness for the stretch constraint in GPa */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "StretchConstraint", meta = (ToolTip = "Stiffness for the stretch constraint in GPa"))
	float StretchStiffness = 1.0f;

	/** Static friction used for collision against the physics asset */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, AdvancedDisplay, Category = "Collision Constraint", meta = (ToolTip = "Static friction used for collision against the physics asset"))
	float StaticFriction = 0.1f;

	/** Kinetic friction used for collision against the physics asset*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, AdvancedDisplay, Category = "CollisionConstraint", meta = (ToolTip = "Kinetic friction used for collision against the physics asset"))
	float KineticFriction = 0.1f;

	/** Viscosity parameter between 0 and 1 that will be used for self collision */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CollisionConstraint", meta = (ToolTip = "Viscosity parameter between 0 and 1 that will be used for self collision"))
	float StrandsViscosity = 1.0f;

	/** Radius that will be used for the collision detection against the physics asset */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "CollisionConstraint", meta = (ToolTip = "Radius that will be used for the collision detection against the physics asset"))
	float CollisionRadius = 0.001f;
};

USTRUCT(BlueprintType)
struct FHairSimulationSetup
{
	GENERATED_USTRUCT_BODY()

	/** Reset the simulation in time*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "Boolean to control if we want to reset trhe simulation at some point in time"))
	bool bResetSimulation = false;

	/** Boolean to make the simulation strands visible */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "Boolean to make the simulation strands visible"))
	bool bDebugSimulation = false;

	/** Strands simulation is done in local space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "Strands simulation is done in local space"))
	bool bLocalSimulation = true;
	
	/** The amount of linear velocities sent to the local groom space from the reference bone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "The amount of linear velocities sent to the local groom space from the reference bone", EditCondition = "bLocalSimulation", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float LinearVelocityScale = 1.0f;
	
	/** The amount of angular velocities sent to the local groom space from the reference bone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "The amount of angular velocities sent to the local groom space from the reference bone", EditCondition = "bLocalSimulation", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float AngularVelocityScale = 1.0f;

	/** Bone used for the simulation local space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "Bone used for the simulation local space", EditCondition = "bLocalSimulation"))
	FString LocalBone = "root";

	/** Teleport distance threshold to reset the simulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "SimulationSetup", meta = (ToolTip = "Teleport distance threshold to reset the simulation"))
	float TeleportDistance = 50.0;
};

USTRUCT(BlueprintType)
struct FHairSimulationSettings
{
	GENERATED_USTRUCT_BODY()

	/** Override the asset simulation settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "GroupsPhysics", meta = (ToolTip = "Boolean to control if we are going to override the groom asset physics settings"))
	bool bOverrideSettings = false;

	/** Solver settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "GroupsPhysics", meta = (ToolTip = "Solver Settings for the hair physics"))
	FHairSimulationSetup SimulationSetup;

	/** Solver settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "GroupsPhysics", meta = (ToolTip = "Solver Settings for the hair physics", EditCondition = "bOverrideSettings"))
	FHairSimulationSolver SolverSettings;

	/** Externalk forces */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "GroupsPhysics", meta = (ToolTip = "External Forces for the hair physics", EditCondition = "bOverrideSettings"))
	FHairSimulationForces ExternalForces;

	/** Material Constraints */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "GroupsPhysics", meta = (ToolTip = "Material Constraints for the hair physics", EditCondition = "bOverrideSettings"))
	FHairSimulationConstraints MaterialConstraints;
};

#undef UE_API
