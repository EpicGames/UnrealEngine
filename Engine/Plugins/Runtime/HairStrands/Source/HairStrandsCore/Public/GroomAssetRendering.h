// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "GroomAssetRendering.generated.h"

#define UE_API HAIRSTRANDSCORE_API


class UMaterialInterface;


USTRUCT(BlueprintType)
struct FHairGeometrySettings
{
	GENERATED_BODY()

	UE_API FHairGeometrySettings();

	/** Hair width (in centimeters) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeometrySettings", meta = (editcondition = "HairWidth_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "1.0", SliderExponent = 6))
	float HairWidth;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairWidth_Override = false;

	/** Scale the hair width at the root */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeometrySettings", AdvancedDisplay, meta = (ClampMin = "0.0001", UIMin = "0.001", UIMax = "2.0", SliderExponent = 6))
	float HairRootScale;

	/** Scale the hair width at the tip */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "GeometrySettings", AdvancedDisplay, meta = (ClampMin = "0.0001", UIMin = "0.001", UIMax = "2.0", SliderExponent = 6))
	float HairTipScale;

	UE_API bool operator==(const FHairGeometrySettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairShadowSettings
{
	GENERATED_BODY()

	UE_API FHairShadowSettings();

	/** Override the hair shadow density factor (unit less). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ShadowSettings", AdvancedDisplay, meta = (ClampMin = "0.0001", UIMin = "0.001", UIMax = "10.0", SliderExponent = 6))
	float HairShadowDensity;

	/** Scale the hair geometry radius for ray tracing effects (e.g. shadow) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ShadowSettings", AdvancedDisplay, meta = (ClampMin = "0.0001", UIMin = "0.001", UIMax = "10.0", SliderExponent = 6))
	float HairRaytracingRadiusScale;

	/** Enable hair strands geomtry for raytracing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ShadowSettings", AdvancedDisplay)
	bool bUseHairRaytracingGeometry;

	/** Enable stands voxelize for casting shadow and environment occlusion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ShadowSettings", AdvancedDisplay)
	bool bVoxelize;

	UE_API bool operator==(const FHairShadowSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairAdvancedRenderingSettings
{
	GENERATED_BODY()

	UE_API FHairAdvancedRenderingSettings();

	/** Insure the hair does not alias. When enable, group of hairs might appear thicker. Isolated hair should remain thin. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AdvancedRenderingSettings")
	bool bUseStableRasterization;

	/** Light hair with the scene color. This is used for vellus/short hair to bring light from the surrounding surface, like skin. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AdvancedRenderingSettings")
	bool bScatterSceneLighting;

	UE_API bool operator==(const FHairAdvancedRenderingSettings& A) const;
};

USTRUCT(BlueprintType)
struct FHairGroupsRendering
{
	GENERATED_BODY()

	UE_API FHairGroupsRendering();

	UPROPERTY()
	FName MaterialSlotName;

	/* Deprecated */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY(EditAnywhere, Category = "GeometrySettings", meta = (ToolTip = "Geometry settings"))
	FHairGeometrySettings GeometrySettings;

	UPROPERTY(EditAnywhere, Category = "ShadowSettings", meta = (ToolTip = "Shadow settings"))
	FHairShadowSettings ShadowSettings;

	UPROPERTY(EditAnywhere, Category = "MiscSettings", meta = (ToolTip = "Advanced rendering settings "))
	FHairAdvancedRenderingSettings AdvancedSettings;

#if WITH_EDITORONLY_DATA
	/** Group name to be displayed in the array list */
	UPROPERTY(VisibleDefaultsOnly, Transient, Category=AlwaysHidden, Meta=(EditCondition=False, EditConditionHides))
	FName GroupName = NAME_None;
#endif

	UE_API bool operator==(const FHairGroupsRendering& A) const;

};

#undef UE_API
