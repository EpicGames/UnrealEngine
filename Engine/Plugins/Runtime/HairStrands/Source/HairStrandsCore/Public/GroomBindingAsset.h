// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "HairDescription.h"
#include "HairStrandsDatas.h"
#include "RenderResource.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "GroomResources.h"
#include "HairStrandsInterface.h"
#include "Async/AsyncWork.h"
#include "Engine/SkeletalMesh.h"
#include "GroomBindingAsset.generated.h"

#define UE_API HAIRSTRANDSCORE_API


class UAssetUserData;
class UGeometryCache;
class UMaterialInterface;
class UNiagaraSystem;
class UGroomAsset;

USTRUCT(BlueprintType)
struct FGoomBindingGroupInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Curve Count"))
	int32 RenRootCount = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Curve LOD"))
	int32 RenLODCount = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Guide Count"))
	int32 SimRootCount = 0;

	UPROPERTY(VisibleAnywhere, Category = "Info", meta = (DisplayName = "Guide LOD"))
	int32 SimLODCount = 0;
};

/** Enum that describes the type of mesh to bind to */
UENUM(BlueprintType)
enum class EGroomBindingMeshType : uint8
{
	SkeletalMesh,
	GeometryCache
};

/*-----------------------------------------------------------------------------
	Async GroomBinding Compilation
-----------------------------------------------------------------------------*/

UENUM()
enum class EGroomBindingAsyncProperties : uint64
{
	None = 0,
	GroomBindingType = 1 << 0,
	Groom = 1 << 1,
	SourceSkeletalMesh = 1 << 2,
	SourceMeshRequestedLOD = 1 << 3,
	SourceMeshUsedLOD = 1 << 4,
	TargetSkeletalMesh = 1 << 5,
	TargetMeshRequestedMinLOD = 1 << 6,
	TargetMeshUsedMinLOD = 1 << 7,
	SourceGeometryCache = 1 << 8,
	TargetGeometryCache = 1 << 9,
	NumInterpolationPoints = 1 << 10,
	MatchingSection = 1 << 11,
	GroupInfos = 1 << 12,
	HairGroupResources = 1 << 13,
	HairGroupPlatformData = 1 << 14,
	TargetBindingAttribute = 1 << 15,
	All = MAX_uint64
};

ENUM_CLASS_FLAGS(EGroomBindingAsyncProperties);

enum class EGroomBindingAsyncPropertyLockType
{
	None = 0,
	ReadOnly = 1,
	WriteOnly = 2,
	ReadWrite = 3
};

ENUM_CLASS_FLAGS(EGroomBindingAsyncPropertyLockType);

// Any thread implicated in the build must have a valid scope to be granted access to protected properties without causing any stalls.
class FGroomBindingAsyncBuildScope
{
public:
	HAIRSTRANDSCORE_API FGroomBindingAsyncBuildScope(const UGroomBindingAsset* Asset);
	HAIRSTRANDSCORE_API ~FGroomBindingAsyncBuildScope();
	HAIRSTRANDSCORE_API static bool ShouldWaitOnLockedProperties(const UGroomBindingAsset* Asset);

private:
	const UGroomBindingAsset* PreviousScope = nullptr;
	// Only the thread(s) compiling this asset will have full access to protected properties without causing any stalls.
	static thread_local const UGroomBindingAsset* Asset;
};

UENUM()
enum class EGroomBindingAssetBuildResult : uint8
{
	Succeeded,
	Failed
};

DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnGroomBindingAssetBuildComplete, UGroomBindingAsset*, GroomBinding, EGroomBindingAssetBuildResult, Result);
DECLARE_DELEGATE_TwoParams(FOnGroomBindingAssetBuildCompleteNative, UGroomBindingAsset*, EGroomBindingAssetBuildResult);

struct FGroomBindingBuildContext
{
	FGroomBindingBuildContext() = default;
	// Non-copyable
	FGroomBindingBuildContext(const FGroomBindingBuildContext&) = delete;
	FGroomBindingBuildContext& operator=(const FGroomBindingBuildContext&) = delete;
	// Movable
	FGroomBindingBuildContext(FGroomBindingBuildContext&&) = default;
	FGroomBindingBuildContext& operator=(FGroomBindingBuildContext&&) = default;

	// All mesh LODs needed to build the binding are referenced here to prevent them from being
	// streamed out during the build.
	TArray<TRefCountPtr<FSkeletalMeshLODRenderData>> MeshLODReferences;

	FOnGroomBindingAssetBuildComplete DynamicCompletionDelegate;
	FOnGroomBindingAssetBuildCompleteNative NativeCompletionDelegate;

	int32 SourceMeshLOD = INDEX_NONE;
	int32 TargetMeshMinLOD = INDEX_NONE;
	bool bReloadResource = false;
};

/**
 * Worker used to perform async compilation.
 */
class FGroomBindingAsyncBuildWorker : public FNonAbandonableTask
{
public:
	UGroomBindingAsset* GroomBinding;
	TOptional<FGroomBindingBuildContext> BuildContext;

	/** Initialization constructor. */
	FGroomBindingAsyncBuildWorker(
		UGroomBindingAsset* InGroomBinding,
		FGroomBindingBuildContext&& InBuildContext)
		: GroomBinding(InGroomBinding)
		, BuildContext(MoveTemp(InBuildContext))
	{
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FGroomBindingAsyncBuildWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();
};

struct FGroomBindingAsyncBuildTask : public FAsyncTask<FGroomBindingAsyncBuildWorker>
{
	FGroomBindingAsyncBuildTask(
		UGroomBindingAsset* InGroomBinding,
		FGroomBindingBuildContext&& InBuildContext)
		: FAsyncTask<FGroomBindingAsyncBuildWorker>(InGroomBinding, MoveTemp(InBuildContext))
		, GroomBinding(InGroomBinding)
	{
	}

	const UGroomBindingAsset* GroomBinding;
};

/**
 * Implements an asset that can be used to store binding information between a groom and a skeletal mesh
 */
UCLASS(MinimalAPI, BlueprintType, hidecategories = (Object))
class UGroomBindingAsset : public UObject, public IInterface_AsyncCompilation
{
	GENERATED_BODY()

#if WITH_EDITOR
	/** Notification when anything changed */
	DECLARE_MULTICAST_DELEGATE(FOnGroomBindingAssetChanged);
#endif

private:
	/** Type of mesh to create groom binding for */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetGroomBindingType() or UGroomBindingAsset::SetGroomBindingType().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetGroomBindingType, BlueprintSetter = SetGroomBindingType, Category = "BuildSettings")
	EGroomBindingMeshType GroomBindingType = EGroomBindingMeshType::SkeletalMesh;

public:
	static UE_API FName GetGroomBindingTypeMemberName();
	UFUNCTION(BlueprintGetter) UE_API EGroomBindingMeshType GetGroomBindingType() const;
	UFUNCTION(BlueprintSetter) UE_API void SetGroomBindingType(EGroomBindingMeshType InGroomBindingType);

private:
	/** Groom to bind. */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetGroom() or UGroomBindingAsset::SetGroom().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetGroom, BlueprintSetter = SetGroom, Category = "BuildSettings", AssetRegistrySearchable)
	TObjectPtr<UGroomAsset> Groom;

public:
	static UE_API FName GetGroomMemberName();
	UFUNCTION(BlueprintGetter) UE_API UGroomAsset* GetGroom() const;
	UFUNCTION(BlueprintSetter) UE_API void SetGroom(UGroomAsset* InGroom);

private:
	/** Skeletal mesh on which the groom has been authored. This is optional, and used only if the hair
		binding is done a different mesh than the one which it has been authored */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetSourceSkeletalMesh() or UGroomBindingAsset::SetSourceSkeletalMesh().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetSourceSkeletalMesh, BlueprintSetter = SetSourceSkeletalMesh, Category = "BuildSettings")
	TObjectPtr<USkeletalMesh> SourceSkeletalMesh;

public:
	static UE_API FName GetSourceSkeletalMeshMemberName();
	UFUNCTION(BlueprintGetter) UE_API USkeletalMesh* GetSourceSkeletalMesh() const;
	UFUNCTION(BlueprintSetter) UE_API void SetSourceSkeletalMesh(USkeletalMesh* InSkeletalMesh);

private:
	UE_DEPRECATED(5.5, "Please do not access this member directly; use UGroomBindingAsset::GetSourceMeshRequestedLOD() or UGroomBindingAsset::SetSourceMeshRequestedLOD().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetSourceMeshRequestedLOD, BlueprintSetter = SetSourceMeshRequestedLOD, Category = "BuildSettings")
	int32 SourceMeshRequestedLOD;

public:
	static UE_API FName GetSourceMeshRequestedLODMemberName();
	UFUNCTION(BlueprintGetter) UE_API int32 GetSourceMeshRequestedLOD() const;
	UFUNCTION(BlueprintSetter) UE_API void SetSourceMeshRequestedLOD(int32 InSourceMeshRequestedLOD);

private:
	UE_DEPRECATED(5.5, "Please do not access this member directly; use UGroomBindingAsset::GetSourceMeshUsedLOD().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetSourceMeshUsedLOD, Category = "BuildSettings")
	int32 SourceMeshUsedLOD = INDEX_NONE;

	UFUNCTION(BlueprintSetter) UE_API void SetSourceMeshUsedLOD(int32 InSourceMeshUsedLOD);
public:
	static UE_API FName GetSourceMeshUsedLODMemberName();
	UFUNCTION(BlueprintGetter) UE_API int32 GetSourceMeshUsedLOD() const;

private:
	/** Skeletal mesh on which the groom is attached to. */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetTargetSkeletalMesh() or UGroomBindingAsset::SetTargetSkeletalMesh().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetTargetSkeletalMesh, BlueprintSetter = SetTargetSkeletalMesh, Category = "BuildSettings")
	TObjectPtr<USkeletalMesh> TargetSkeletalMesh;

public:
	static UE_API FName GetTargetSkeletalMeshMemberName();
	UFUNCTION(BlueprintGetter) UE_API USkeletalMesh* GetTargetSkeletalMesh() const;
	UFUNCTION(BlueprintSetter) UE_API void SetTargetSkeletalMesh(USkeletalMesh* InSkeletalMesh);
	
private:
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetTargetMeshRequestedMinLOD() or UGroomBindingAsset::SetTargetMeshRequestedMinLOD().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetTargetMeshRequestedMinLOD, BlueprintSetter = SetTargetMeshRequestedMinLOD, Category = "BuildSettings")
	int32 TargetMeshRequestedMinLOD;

public:
	static UE_API FName GetTargetMeshRequestedMinLODMemberName();
	UFUNCTION(BlueprintGetter) UE_API int32 GetTargetMeshRequestedMinLOD() const;
	UFUNCTION(BlueprintSetter) UE_API void SetTargetMeshRequestedMinLOD(int32 InTargetMeshRequestedMinLOD);

private:
	UE_DEPRECATED(5.5, "Please do not access this member directly; use UGroomBindingAsset::GetTargetMeshUsedMinLOD().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetTargetMeshUsedMinLOD, Category = "BuildSettings")
	int32 TargetMeshUsedMinLOD = INDEX_NONE;

	UFUNCTION(BlueprintSetter) UE_API void SetTargetMeshUsedMinLOD(int32 InTargetMeshUsedMinLOD);
public:
	static UE_API FName GetTargetMeshUsedMinLODMemberName();
	UFUNCTION(BlueprintGetter) UE_API int32 GetTargetMeshUsedMinLOD() const;

private:
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetSourceGeometryCache() or UGroomBindingAsset::SetSourceGeometryCache().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetSourceGeometryCache, BlueprintSetter = SetSourceGeometryCache, Category = "BuildSettings")
	TObjectPtr<UGeometryCache> SourceGeometryCache;

public:
	static UE_API FName GetSourceGeometryCacheMemberName();
	UFUNCTION(BlueprintGetter) UE_API UGeometryCache* GetSourceGeometryCache() const;
	UFUNCTION(BlueprintSetter) UE_API void SetSourceGeometryCache(UGeometryCache* InGeometryCache);

private:
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetTargetGeometryCache() or UGroomBindingAsset::SetTargetGeometryCache().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetTargetGeometryCache, BlueprintSetter = SetTargetGeometryCache, Category = "BuildSettings")
	TObjectPtr<UGeometryCache> TargetGeometryCache;

public:
	static UE_API FName GetTargetGeometryCacheMemberName();
	UFUNCTION(BlueprintGetter) UE_API UGeometryCache* GetTargetGeometryCache() const;
	UFUNCTION(BlueprintSetter) UE_API void SetTargetGeometryCache(UGeometryCache* InGeometryCache);

private:
	/** Number of points used for the rbf interpolation */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetNumInterpolationPoints() or UGroomBindingAsset::SetNumInterpolationPoints().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetNumInterpolationPoints, BlueprintSetter = SetNumInterpolationPoints, Category = "BuildSettings")
	int32 NumInterpolationPoints = 100;

public:
	static UE_API FName GetNumInterpolationPointsMemberName();
	UFUNCTION(BlueprintGetter) UE_API int32 GetNumInterpolationPoints() const;
	UFUNCTION(BlueprintSetter) UE_API void SetNumInterpolationPoints(int32 InNumInterpolationPoints);

private:
	/** Number of points used for the rbf interpolation */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetMatchingSection() or UGroomBindingAsset::SetMatchingSection().")
	UPROPERTY(VisibleAnywhere, BlueprintGetter = GetMatchingSection, BlueprintSetter = SetMatchingSection, Category = "BuildSettings")
	int32 MatchingSection = 0;

public:
	static UE_API FName GetMatchingSectionMemberName();
	UFUNCTION(BlueprintGetter) UE_API int32 GetMatchingSection() const;
	UFUNCTION(BlueprintSetter) UE_API void SetMatchingSection(int32 InMatchingSection);

private:
	/** Optional binding attribute name on target skeletal mesh, to filter out which triangles are valid to bind groom to*/
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetTargetBindingAttribute() or UGroomBindingAsset::SetTargetBindingAttribute().")
	UPROPERTY(EditAnywhere, BlueprintGetter = GetTargetBindingAttribute, BlueprintSetter = SetTargetBindingAttribute, Category = "BuildSettings")
	FName TargetBindingAttribute = NAME_None;

public:
	static UE_API FName GetTargetBindingAttributeMemberName();
	UFUNCTION(BlueprintGetter) UE_API FName GetTargetBindingAttribute() const;
	UFUNCTION(BlueprintSetter) UE_API void SetTargetBindingAttribute(FName InAttributeName);

private:
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetGroupInfos() or UGroomBindingAsset::SetGroupInfos().")
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintGetter = GetGroupInfos, BlueprintSetter = SetGroupInfos, Category = "HairGroups", meta = (DisplayName = "Group"))
	TArray<FGoomBindingGroupInfo> GroupInfos;

public:
	static UE_API FName GetGroupInfosMemberName();
	UFUNCTION(BlueprintGetter) UE_API const TArray<FGoomBindingGroupInfo>& GetGroupInfos() const;
	UFUNCTION(BlueprintSetter) UE_API void SetGroupInfos(const TArray<FGoomBindingGroupInfo>& InGroupInfos);
	UE_API TArray<FGoomBindingGroupInfo>& GetGroupInfos();

	/** GPU and CPU binding data for both simulation and rendering. */
	struct FHairGroupResource
	{
		FHairStrandsRestRootResource* SimRootResources = nullptr;
		FHairStrandsRestRootResource* RenRootResources = nullptr;
		TArray<FHairStrandsRestRootResource*> CardsRootResources;
	};
	typedef TArray<FHairGroupResource> FHairGroupResources;

private:
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetHairGroupResources() or UGroomBindingAsset::SetHairGroupResources().")
	FHairGroupResources HairGroupResources;

public:
	static UE_API FName GetHairGroupResourcesMemberName();
	UE_API FHairGroupResources& GetHairGroupResources();
	UE_API const FHairGroupResources& GetHairGroupResources() const;
	UE_API void SetHairGroupResources(FHairGroupResources InHairGroupResources);

	/** Binding bulk data */
	struct FHairGroupPlatformData
	{
		TArray<FHairStrandsRootBulkData>		 SimRootBulkDatas;
		TArray<FHairStrandsRootBulkData>		 RenRootBulkDatas;
		TArray<TArray<FHairStrandsRootBulkData>> CardsRootBulkDatas;

		// The minimum mesh LOD that this binding data can support
		int32 TargetMeshMinLOD = -1;
	};

private:
	/** Queue of resources which needs to be deleted. This queue is needed for keeping valid pointer on the group resources 
	    when the binding asset is recomputed */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetHairGroupPlatformData().")
	TQueue<FHairGroupResource> HairGroupResourcesToDelete;

	/** Queue of data which needs to be deleted. This queue is needed for keeping valid pointer on the group resources 
	    when the binding asset is recomputed */
	struct FHairGroupPlatformDataArray
	{
		TArray<FHairGroupPlatformData> Data;
	};
	TQueue<FHairGroupPlatformDataArray*> HairGroupPlatformDataToDelete;

	/** Platform data for each hair groups */
	UE_DEPRECATED(5.3, "Please do not access this member directly; use UGroomBindingAsset::GetHairGroupPlatformData().")
	TArray<FHairGroupPlatformData> HairGroupsPlatformData;

public:
	UE_API void AddHairGroupResourcesToDelete(FHairGroupResource& In);
	UE_API bool RemoveHairGroupResourcesToDelete(FHairGroupResource& Out);

	static UE_API FName GetHairGroupPlatformDataMemberName();
	UE_API const TArray<FHairGroupPlatformData>& GetHairGroupsPlatformData() const;
	UE_API TArray<FHairGroupPlatformData>& GetHairGroupsPlatformData();

public:
	//~ Begin UObject Interface.
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	static UE_API bool IsCompatible(const USkeletalMesh* InSkeletalMesh, const UGroomBindingAsset* InBinding, bool bIssueWarning);
	static UE_API bool IsCompatible(const UGeometryCache* InGeometryCache, const UGroomBindingAsset* InBinding, bool bIssueWarning);
	static UE_API bool IsCompatible(const UGroomAsset* InGroom, const UGroomBindingAsset* InBinding, bool bIssueWarning);
	static UE_API bool IsBindingAssetValid(const UGroomBindingAsset* InBinding, bool bIsBindingReloading, bool bIssueWarning);

	/** Returns true if the target is not null and matches the binding type */ 
	UE_API bool HasValidTarget() const;

	/** Helper function to return the asset path name, optionally joined with the LOD index if LODIndex > -1. */
	UE_API FName GetAssetPathName(int32 LODIndex = -1);
	uint32 GetAssetHash() const { return AssetNameHash; }

#if WITH_EDITOR
	FOnGroomBindingAssetChanged& GetOnGroomBindingAssetChanged() { return OnGroomBindingAssetChanged; }

	/**  Part of UObject interface  */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
#endif // WITH_EDITOR

	/** Initialize resources. */
	UE_API void InitResource();

	/** Update resources. */
	UE_API void UpdateResource();

	/** Release the hair strands resource. */
	UE_API void ReleaseResource(bool bResetLoadedSize);

	/**
	 * Stream in all of this binding's streamable resources and make them accessible from the CPU.
	 *
	 * This is only needed for advanced use cases involving editing grooms or binding data.
	 *
	 * @param bWait If true, this call will block until the resources have been streamed in
	 */
	UE_API void StreamInForCPUAccess(bool bWait);

	UE_API void Reset();

	/** Return true if the binding asset is valid, i.e., correctly built and loaded. */
	bool IsValid() const { return bIsValid;  }

private:
	/** Used as a bit-field indicating which properties are read by async compilation. */
	std::atomic<uint64> AccessedProperties;
	/** Used as a bit-field indicating which properties are written to by async compilation. */
	std::atomic<uint64> ModifiedProperties;
	/** Holds the pointer to an async task if one exists. */
	TUniquePtr<FGroomBindingAsyncBuildTask> AsyncTask;

	bool IsAsyncTaskComplete() const
	{
		return AsyncTask == nullptr || AsyncTask->IsWorkDone();
	}

	bool TryCancelAsyncTasks()
	{
		if (AsyncTask)
		{
			if (AsyncTask->IsDone() || AsyncTask->Cancel())
			{
				AsyncTask.Reset();
			}
		}

		return AsyncTask == nullptr;
	}

	UE_API void ExecuteCacheDerivedDatas(FGroomBindingBuildContext& Context);
	UE_API void FinishCacheDerivedDatas(FGroomBindingBuildContext& Context);
	UE_API int32 GetClampedSourceMeshLOD(const ITargetPlatform* TargetPlatform) const;
	UE_API int32 GetClampedTargetMeshMinLOD(const ITargetPlatform* TargetPlatform) const;

public:
	/** IInterface_AsyncCompilation begin*/
	virtual bool IsCompiling() const override
	{
		return AsyncTask != nullptr || AccessedProperties.load(std::memory_order_relaxed) != 0;
	}
	/** IInterface_AsyncCompilation end*/
#if WITH_EDITOR
	FOnGroomBindingAssetChanged OnGroomBindingAssetChanged;

	UE_API void RecreateResources();
	UE_DEPRECATED(5.8, "Use ChangeShaderPlatform(EShaderPlatform) instead.")
	UE_API void ChangeFeatureLevel(ERHIFeatureLevel::Type PendingFeatureLevel);
	UE_DEPRECATED(5.8, "Use ChangeShaderPlatform(EShaderPlatform) instead.")
	UE_API void ChangePlatformLevel(ERHIFeatureLevel::Type PendingFeatureLevel);
	UE_API void ChangeShaderPlatform(EShaderPlatform InShaderPlatform);
#endif
private:
	UE_API void WaitUntilAsyncPropertyReleased(EGroomBindingAsyncProperties AsyncProperties, EGroomBindingAsyncPropertyLockType LockType) const;
	void AcquireAsyncProperty(uint64 AsyncProperties = MAX_uint64, EGroomBindingAsyncPropertyLockType LockType = EGroomBindingAsyncPropertyLockType::ReadWrite)
	{
		if ((LockType & EGroomBindingAsyncPropertyLockType::ReadOnly) == EGroomBindingAsyncPropertyLockType::ReadOnly)
		{
			AccessedProperties |= AsyncProperties;
		}

		if ((LockType & EGroomBindingAsyncPropertyLockType::WriteOnly) == EGroomBindingAsyncPropertyLockType::WriteOnly)
		{
			ModifiedProperties |= AsyncProperties;
		}
	}

	void ReleaseAsyncProperty(uint64 AsyncProperties = MAX_uint64, EGroomBindingAsyncPropertyLockType LockType = EGroomBindingAsyncPropertyLockType::ReadWrite)
	{
		if ((LockType & EGroomBindingAsyncPropertyLockType::ReadOnly) == EGroomBindingAsyncPropertyLockType::ReadOnly)
		{
			AccessedProperties &= ~AsyncProperties;
		}

		if ((LockType & EGroomBindingAsyncPropertyLockType::WriteOnly) == EGroomBindingAsyncPropertyLockType::WriteOnly)
		{
			ModifiedProperties &= ~AsyncProperties;
		}
	}

	static UE_API void FlushRenderingCommandIfUsed(const UGroomBindingAsset* In);
public:
	/**
	 * Build/rebuild a binding asset.
	 * 
	 * Avoid calling Build again while a build is already in progress on this asset, as this will
	 * cause the Game Thread to block until the earlier build is finished.
	 * 
	 * Similarly, the Game Thread will block for a short time if a Groom Component is already using 
	 * this binding asset, so if performance is critical avoid calling Build on bindings that are 
	 * in use.
	 */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void Build(FOnGroomBindingAssetBuildComplete CompletionDelegate);
	UE_API void Build(const FOnGroomBindingAssetBuildCompleteNative& CompletionDelegate = FOnGroomBindingAssetBuildCompleteNative());
	UE_API void Build(const FOnGroomBindingAssetBuildComplete& DynamicCompletionDelegate, const FOnGroomBindingAssetBuildCompleteNative& NativeCompletionDelegate);

	// Internal use only
	UE_API void BeginCacheDerivedDatas(
		const FOnGroomBindingAssetBuildComplete& DynamicCompletionDelegate = FOnGroomBindingAssetBuildComplete(),
		const FOnGroomBindingAssetBuildCompleteNative& NativeCompletionDelegate = FOnGroomBindingAssetBuildCompleteNative());

	UE_API bool HasAnyDependenciesCompiling() const;

private:
	UE_API bool TryInitializeContextForMeshes(FGroomBindingBuildContext& OutContext) const;

public:
#if WITH_EDITORONLY_DATA
	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, AdvancedDisplay, Category = Thumbnail)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;

	UE_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	UE_API virtual void ClearAllCachedCookedPlatformData() override;

	UE_API TArray<FHairGroupPlatformData>* GetCachedCookedPlatformData(const ITargetPlatform* TargetPlatform);

	UE_API void InvalidateBinding();
	UE_API void InvalidateBinding(class USkeletalMesh*);

	struct FCachedCookedPlatformData
	{
		// Actual DDC key and platform data
		TArray<FString> GroupDerivedDataKeys;
		TArray<FHairGroupPlatformData> GroupPlatformDatas;

		// DDC key without source/target MeshLOD and requested MeshLOD value
		TArray<FString> GroupDerivedDataKeys_Query;
		int32 SourceMeshLOD = INDEX_NONE;
		int32 TargetMeshMinLOD = INDEX_NONE;
	};

	TArray<FCachedCookedPlatformData*> CachedCookedPlatformDatas;

	UE_API void RegisterGroomDelegates();
	UE_API void UnregisterGroomDelegates();
	UE_API void RegisterSkeletalMeshDelegates();
	UE_API void UnregisterSkeletalMeshDelegates();

	TArray<FString> CachedDerivedDataKey;
#endif
#if WITH_EDITOR
	EShaderPlatform CachedResourceShaderPlatform = SP_NumPlatforms;
#endif
	bool bIsValid = false;
	uint32 AssetNameHash = 0;

	friend class FGroomBindingCompilingManager;
	friend class FGroomBindingAsyncBuildWorker;
};

UCLASS(MinimalAPI, BlueprintType, hidecategories = (Object))
class UGroomBindingAssetList : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Transient, EditFixedSize, Category = "Bindings")
	TArray<TObjectPtr<UGroomBindingAsset>> Bindings;
};

struct FGroomBindingAssetMemoryStats
{
	struct FStats
	{
		uint32 Guides = 0;
		uint32 Strands= 0;
		uint32 Cards  = 0;
	};
	FStats CPU;
	FStats GPU;

	static FGroomBindingAssetMemoryStats Get(const UGroomBindingAsset::FHairGroupPlatformData& InCPU, const UGroomBindingAsset::FHairGroupResource& InGPU);
	void Accumulate(const FGroomBindingAssetMemoryStats& In);
	uint32 GetTotalCPUSize() const;
	uint32 GetTotalGPUSize() const;
};

#undef UE_API
