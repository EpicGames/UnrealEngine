// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GroomSettings.h"
#include "HairStrandsInterface.h"
#include "HairStrandsMeshProjection.h"
#include "HairStrandsDatas.h"
#include "GroomBindingAsset.h"

#define UE_API HAIRSTRANDSCORE_API

class ITargetPlatform;
class UGroomBindingAsset;

namespace UE::Groom
{
	UE_API bool IsRBFLocalSpaceEnabled();
}

struct FGroomBindingBuilder
{
	struct FInput
	{
		FInput() = default;
		FInput(UGroomBindingAsset* BindingAsset, const ITargetPlatform* TargetPlatform, int32 InSourceMeshLOD, int32 InTargetMeshMinLOD);

		EGroomBindingMeshType BindingType = EGroomBindingMeshType::SkeletalMesh;
		int32 NumInterpolationPoints = 0;
		int32 MatchingSection = 0;
		FName TargetBindingAttribute = NAME_None;

		// These must be initialized to valid LODs.
		// 
		// When binding to a geometry cache, they must be set to 0.
		int32 SourceMeshLOD = INDEX_NONE;
		int32 TargetMeshMinLOD = INDEX_NONE;

		bool bHasValidTarget = false;
		UGroomAsset* GroomAsset = nullptr;
		USkeletalMesh* SourceSkeletalMesh = nullptr;
		USkeletalMesh* TargetSkeletalMesh = nullptr;
		UGeometryCache* SourceGeometryCache = nullptr;
		UGeometryCache* TargetGeometryCache = nullptr;

		// This is needed to work around a known deadlock issue
		bool bForceUseRunningPlatform = false;
	};

	static UE_API FString GetVersion();

	// Build binding asset data
	UE_DEPRECATED(5.4, "Please do not access this funciton; but rather call BindingAsset->CacheDerivedDatas()")
	static UE_API bool BuildBinding(UGroomBindingAsset* BindingAsset, bool bInitResource);

	// Build binding asset data
	UE_DEPRECATED(5.4, "Please do not access this funciton; but rather call BindingAsset->CacheDerivedDatas()")
	static UE_API bool BuildBinding(UGroomBindingAsset* BindingAsset, uint32 InGroupIndex);

	// Build binding asset data
	//
	// The caller must ensure that the referenced mesh LODs stay loaded until this function returns.
	static UE_API bool BuildBinding(
		const FInput& In,
		uint32 InGroupIndex,
		const ITargetPlatform* TargetPlatform,
		UGroomBindingAsset::FHairGroupPlatformData& Out);

	// Extract root data from bulk data
	static UE_API void GetRootData(FHairStrandsRootData& Out, const FHairStrandsRootBulkData& In);
};

namespace GroomBinding_RBFWeighting
{
	struct FPointsSampler
	{
		UE_API FPointsSampler(TArray<bool>& ValidPoints, const FVector3f* PointPositions, const int32 NumSamples);

		/** Build the sample position from the sample indices */
		UE_API void BuildPositions(const FVector3f* PointPositions);

		/** Compute the furthest point */
		UE_API void FurthestPoint(const int32 NumPoints, const FVector3f* PointPositions, const uint32 SampleIndex, TArray<bool>& ValidPoints, TArray<float>& PointsDistance);

		/** Compute the starting point */
		UE_API int32 StartingPoint(const TArray<bool>& ValidPoints, int32& NumPoints) const;

		/** List of sampled points */
		TArray<uint32> SampleIndices;

		/** List of sampled positions */
		TArray<FVector3f> SamplePositions;
	};


	struct FWeightsBuilder
	{
		UE_API FWeightsBuilder(const uint32 NumRows, const uint32 NumColumns,
			const FVector3f* SourcePositions, const FVector3f* TargetPositions, const bool bLocalSpace = false);

		/** Compute the weights by inverting the matrix*/
		UE_API void ComputeWeights(const uint32 NumRows, const uint32 NumColumns);

		/** Entries in the dense structure */
		TArray<float> MatrixEntries;

		/** Entries of the matrix inverse */
		TArray<float> InverseEntries;
	};
}

#undef UE_API
