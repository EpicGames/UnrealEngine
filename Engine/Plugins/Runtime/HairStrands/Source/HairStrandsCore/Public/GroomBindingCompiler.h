// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Tickable.h"
#include "Stats/Stats.h"
#include "IAssetCompilingManager.h"
#include "AsyncCompilationHelpers.h"

class UGroomBindingAsset;
class UPrimitiveComponent;
class FQueuedThreadPool;
struct FAssetCompileContext;
enum class EQueuedWorkPriority : uint8;

class FGroomBindingCompilingManager : public IAssetCompilingManager
{
public:
	HAIRSTRANDSCORE_API static FGroomBindingCompilingManager& Get();

	/**
	 * Returns true if the feature is currently activated.
	 */
	HAIRSTRANDSCORE_API bool IsAsyncCompilationEnabled() const;

	/** 
	 * Returns the number of outstanding compilations.
	 */
	HAIRSTRANDSCORE_API int32 GetNumRemainingJobs() const;

	/** 
	 * Adds groom binding assets compiled asynchronously so they are monitored. 
	 */
	HAIRSTRANDSCORE_API void AddGroomBindings(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets);

	/**
	 * Register groom binding assets to compile once their dependencies are finished compiling.
	 */
	HAIRSTRANDSCORE_API void AddGroomBindingsWithPendingDependencies(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets);

	/** 
	 * Blocks until completion of the requested groom binding assets.
	 */
	HAIRSTRANDSCORE_API void FinishCompilation(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets);

	/** 
	 * Blocks until completion of all async groom binding asset compilation.
	 */
	HAIRSTRANDSCORE_API void FinishAllCompilation() override;

	/**
	 * Returns if asynchronous compilation is allowed for this groom binding asset.
	 */
	HAIRSTRANDSCORE_API bool IsAsyncCompilationAllowed(UGroomBindingAsset* InGroomBindingAsset) const;

	/**
	 * Returns the priority at which the given groom binding asset should be scheduled.
	 */
	HAIRSTRANDSCORE_API EQueuedWorkPriority GetBasePriority(UGroomBindingAsset* InGroomBindingAsset) const;

	/**
	 * Returns the threadpool where groom binding asset compilation should be scheduled.
	 */
	HAIRSTRANDSCORE_API FQueuedThreadPool* GetThreadPool() const;

	/**
	 * Cancel any pending work and blocks until it is safe to shut down.
	 */
	HAIRSTRANDSCORE_API void Shutdown() override;

private:
	FGroomBindingCompilingManager();

	FName GetAssetTypeName() const override;
	FTextFormat GetAssetNameFormat() const override;
	TArrayView<FName> GetDependentTypeNames() const override;
	int32 GetNumRemainingAssets() const override;
	void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;

	void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;

	friend class FAssetCompilingManager;
	
	bool bHasShutdown = false;
	TSet<UGroomBindingAsset*> GroomBindingWithPendingDependencies;
	TSet<UGroomBindingAsset*> RegisteredGroomBindingAssets;
	TMultiMap<class USkeletalMesh*, UGroomBindingAsset*> RegisteredSkeletalMeshes;
	TMultiMap<class UGroomAsset*, UGroomBindingAsset*> RegisteredGroomAssets;
	FAsyncCompilationNotification Notification;
	void FinishCompilationsForGame();
	void Reschedule();
	void ProcessGroomBindingAssets(bool bLimitExecutionTime, int32 MinBatchSize = 1);
	void UpdateCompilationNotification();
	
	void AttachDependencies(UGroomBindingAsset* GroomBindingAsset);
	void DetachDependencies(UGroomBindingAsset* GroomBindingAsset);
	void SchedulePendingCompilations();
	void PostCompilation(UGroomBindingAsset* GroomBindingAsset);
	void PostCompilation(TArrayView<UGroomBindingAsset* const> InGroomBindingAssets);
	void OnPostReachabilityAnalysis();
	FDelegateHandle PostReachabilityAnalysisHandle;
};
