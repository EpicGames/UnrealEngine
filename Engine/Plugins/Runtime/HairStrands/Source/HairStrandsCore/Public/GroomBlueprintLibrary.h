// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "GroomBlueprintLibrary.generated.h"

#define UE_API HAIRSTRANDSCORE_API

// Forward Declare
class UGeometryCache;
class UGroomAsset;
class USkeletalMesh;
class UGroomBindingAsset;

UCLASS(MinimalAPI, meta = (ScriptName = "GroomLibrary"))
class UGroomBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Create a new groom binding asset within the contents space of the project.
	 * @param InDesiredPackagePath The package path to use for the groom binding
	 * @param InGroomAsset Groom asset for binding
	 * @param InSkeletalMesh Skeletal mesh on which the groom should be bound to
	 * @param InNumInterpolationPoints Number of point used for RBF constraint (if used)
	 * @param InSourceSkeletalMeshForTransfer Skeletal mesh on which the groom was authored. This should be used only if the skeletal mesh on which the groom is attached to, does not match the rest pose of the groom
	 */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	static UE_API UGroomBindingAsset* CreateNewGroomBindingAssetWithPath(
		const FString& InDesiredPackagePath,
		UGroomAsset* InGroomAsset,
		USkeletalMesh* InSkeletalMesh,
		int32 InNumInterpolationPoints = 100,
		USkeletalMesh* InSourceSkeletalMeshForTransfer = nullptr,
		int32 InMatchingSection = 0);

	/**
	 * Create a new groom binding asset within the contents space of the project. The asset name will be auto generated based on the groom asset name and the skeletal asset name
	 * @param InGroomAsset Groom asset for binding
	 * @param InSkeletalMesh Skeletal mesh on which the groom should be bound to
	 * @param InNumInterpolationPoints (Optional) Number of point used for RBF constraint
	 * @param InSourceSkeletalMeshForTransfer  (Optional) Skeletal mesh on which the groom was authored. This should be used only if the skeletal mesh on which the groom is attached to, does not match the rest pose of the groom
	 */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	static UE_API UGroomBindingAsset* CreateNewGroomBindingAsset(
		UGroomAsset* InGroomAsset,
		USkeletalMesh* InSkeletalMesh,
		int32 InNumInterpolationPoints = 100,
		USkeletalMesh* InSourceSkeletalMeshForTransfer = nullptr,
		int32 InMatchingSection = 0);

	/**
	 * Create a new groom binding asset within the contents space of the project.
	 * @param DesiredPackagePath The package path to use for the groom binding
	 * @param GroomAsset Groom asset for binding
	 * @param SkeletalMesh Skeletal mesh on which the groom should be bound to
	 * @param NumInterpolationPoints Number of point used for RBF constraint (if used)
	 * @param SourceSkeletalMeshForTransfer Skeletal mesh on which the groom was authored. This should be used only if the skeletal mesh on which the groom is attached to, does not match the rest pose of the groom
	 */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	static UE_API UGroomBindingAsset* CreateNewGeometryCacheGroomBindingAssetWithPath(
		const FString& DesiredPackagePath,
		UGroomAsset* GroomAsset,
		UGeometryCache* GeometryCache,
		int32 NumInterpolationPoints = 100,
		UGeometryCache* SourceGeometryCacheForTransfer = nullptr,
		int32 MatchingSection = 0);

	/**
	 * Create a new groom binding asset within the contents space of the project. The asset name will be auto generated based on the groom asset name and the skeletal asset name
	 * @param GroomAsset Groom asset for binding
	 * @param SkeletalMesh Skeletal mesh on which the groom should be bound to
	 * @param NumInterpolationPoints (Optional) Number of point used for RBF constraint
	 * @param SourceSkeletalMeshForTransfer  (Optional) Skeletal mesh on which the groom was authored. This should be used only if the skeletal mesh on which the groom is attached to, does not match the rest pose of the groom
	 */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	static UE_API UGroomBindingAsset* CreateNewGeometryCacheGroomBindingAsset(
		UGroomAsset* GroomAsset,
		UGeometryCache* GeometryCache,
		int32 NumInterpolationPoints = 100,
		UGeometryCache* SourceGeometryCacheForTransfer = nullptr,
		int32 MatchingSection = 0);

	/**
	 * Check for strands support in the world of a given Actor Component
	 * @param InWorld The world to check
	 * @return true if strands are going to be rendered for the world's scene and false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Groom", meta = (WorldContext="WorldContextObject"))
	static UE_API bool IsHairStrandsSupportedInWorld(const UObject* WorldContextObject);
};

#undef UE_API
