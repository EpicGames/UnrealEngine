// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GroomSettings.h"
#include "Math/IntVector.h"

#define UE_API HAIRSTRANDSCORE_API

struct FHairStrandsDatas;
struct FHairStrandsRawDatas;
struct FHairStrandsClusterData;
struct FHairStrandsClusterBulkData;
struct FHairGroupsLOD;
struct FHairGroupData;
struct FHairDescriptionGroups;
struct FHairGroupsInterpolation;
struct FHairStrandsInterpolationDatas;
struct FHairInterpolationSettings;
struct FHairStrandsBulkData;
struct FHairStrandsInterpolationBulkData;
struct FHairDescriptionGroup;
struct FHairGroupInfo;
class FHairDescription;
class UGroomAsset;
class UGroomComponent;
struct FRenderCurveResourceData;

struct FHairStrandsVoxelData
{
	static const uint8 InvalidGroupIndex = 0xFF;
	struct FData
	{
		FVector3f BaseColor;
		float Roughness;
		uint8 GroupIndex;
	};

	FVector3f MinBound;
	FVector3f MaxBound;
	FIntVector Resolution;
	TArray<FData> Datas;

	// Return the (closest) FData corresponding to position P
	FData GetData(const FVector3f& P) const;
	bool IsValid() const { return Datas.Num() > 0; }
};

// Data flow overview
// ==================
// HairDescription -> HairDescriptionGroups (HairStrandsRawData) -> HairStrandsData -> HairStrandsBulkData*
//																 -> HairStrandsInterpolationData ->HairStrandsInterpolationBulkData*
//																 -> HairStrandsClusterData*
//
// * Data used at runtime. Other type of data are intermediate data used only during building/within the editor
struct FGroomBuilder
{
	static UE_API FString GetVersion();

	// 1. Build hair *group* description based on the hair description. This builds HairStrandsRawData.
	static UE_API bool BuildHairDescriptionGroups(
		const FHairDescription& HairDescription, 
		FHairDescriptionGroups& Out,
		const bool bAllowAddEndControlPoint = true);

	// 2.a Build FHairStrandsDatas from HairDescriptionGroups(HairStrandsRawData) and DecimationSettings (Strands / Guides)
	static UE_API void BuildData(
		const FHairDescriptionGroup& InHairDescriptionGroup,
		const FHairGroupsInterpolation& InSettings,
		FHairGroupInfo& OutGroupInfo,
		FHairStrandsDatas& OutStrands,
		FHairStrandsDatas& OutGuides,
		const bool bAllowCurveReordering = true,
		const bool bApplyDecimation = true,
		const bool bBuildSourceMapping = false);

	static UE_API void BuildData(
		const FHairDescriptionGroup& InHairDescriptionGroup,
		const FHairGroupsInterpolation& InSettings,
		FHairStrandsDatas& OutStrands,
		FHairStrandsDatas& OutGuides,
		const bool bAllowCurveReordering = true,
		const bool bApplyDecimation = true,
		const bool bBuildSourceMapping = false);

	// 2.b Build FHairStrandsDatas from a FHairStrandsRawDatas (Strands / Guides)
	UE_DEPRECATED(5.6, "Please, no longer use this build function, as it does not contains certain build steps (decimation/shuffling)")
	static UE_API void BuildData(const FHairStrandsRawDatas& In, FHairStrandsDatas& Out);

	// 3. Build bulk data from a FHairStrandsDatas (Strands / Guides)
	static UE_API void BuildBulkData(
		const FHairGroupInfo& InInfo,
		const FHairStrandsDatas& InData,
		FHairStrandsBulkData& OutBulkData,
		bool bAllowCompression);

	// 4. Build interplation data based on the hairStrands data
	static UE_API void BuildInterplationData(
		const FHairGroupInfo& InInfo,
		const FHairStrandsDatas& InRenData,
		const FHairStrandsDatas& InSimData,
		const FHairInterpolationSettings& InInterpolationSettings,
		FHairStrandsInterpolationDatas& OutInterpolationData);

	// 5. Build interplation bulk data
	static UE_API void BuildInterplationBulkData(
		const FHairStrandsDatas& InSimData,
		const FHairStrandsInterpolationDatas& InInterpolationData,
		FHairStrandsInterpolationBulkData& OutInterpolationData);

	// 6. Build cluster data
	static UE_API void BuildClusterBulkData(
		const FHairStrandsDatas& InRenData,
		const float InGroomAssetRadius,
		const FHairGroupsLOD& InSettings,
		FHairStrandsClusterBulkData& OutClusterData);

	// Optional: Voxelize hair group index
	static UE_API void VoxelizeGroupIndex(
		const FHairDescriptionGroups& In,
		FHairStrandsVoxelData& Out);

	static UE_API void BuildRenderCurveResourceBulkData(
		const FHairStrandsDatas& In, 
		FRenderCurveResourceData& Out);
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStrandsPositionOutput
{
	typedef TArray<FVector3f> FStrand;
	typedef TArray<FStrand>   FGroup;

	TArray<FGroup> Groups;
	const UGroomComponent* Component = nullptr;

	int32 Status = -1;

	bool IsValid() const { return Status == 0; }
};

HAIRSTRANDSCORE_API bool RequestStrandsPosition(const UGroomComponent* Component, TSharedPtr<FStrandsPositionOutput> Output, const bool bReadGuides = false);

#undef UE_API
