// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GroomAsset.h"
#include "GroomCacheData.h"
#include "HairDescription.h"
#include "Chaos/ChaosCache.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Serialization/BulkData.h"
#include "GroomCache.generated.h"

#define UE_API HAIRSTRANDSCORE_API

struct FGroomCacheChunk;

/**
 * Implements an asset that is used to store an animated groom
 */
UCLASS(BlueprintType, MinimalAPI)
class UGroomCache : public UObject, public IInterface_AssetUserData, public IChaosCacheData
{
	GENERATED_BODY()

public:

	//~ Begin UObject Interface.
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	//~ End UObject Interface.

	UE_API void Initialize(EGroomCacheType Type);
	UE_API int32 GetStartFrame() const;
	UE_API int32 GetEndFrame() const;

	//~ Begin IChaosCacheData interface
	UE_API virtual float GetDuration() const override;
	//~ End IChaosCacheData interface

	/** Get the frame number at the specified time within the animation range which might not start at 0 */
	UE_API int32 GetFrameNumberAtTime(const float Time, bool bLooping) const;

	/** Get the (floored) frame index at the specified time with the index 0 being the start of the animation */
	UE_API int32 GetFrameIndexAtTime(const float Time, bool bLooping) const;

	/** Get the frame indices and interpolation factor between them that correspond to the specified time */
	UE_API void GetFrameIndicesAtTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32 &OutFrameIndex, int32 &OutNextFrameIndex, float &InterpolationFactor);

	/** Get the frame indices that correspond to the specified time range */
	UE_API void GetFrameIndicesForTimeRange(float StartTime, float EndTime, bool Looping, TArray<int32>& OutFrameIndices);

	UE_API bool GetGroomDataAtTime(float Time, bool bLooping, FGroomCacheAnimationData& AnimData);
	UE_API bool GetGroomDataAtFrameIndex(int32 FrameIndex, FGroomCacheAnimationData& AnimData);

	UE_API void SetGroomAnimationInfo(const FGroomAnimationInfo& AnimInfo);
	const FGroomAnimationInfo& GetGroomAnimationInfo() const { return GroomCacheInfo.AnimationInfo; }

	UE_API EGroomCacheType GetType() const;

	TArray<FGroomCacheChunk>& GetChunks() { return Chunks; }

	TOptional<FPackageFileVersion> ArchiveVersion;

#if WITH_EDITORONLY_DATA
	/** Import options used for this GroomCache */
	UPROPERTY(Category = ImportSettings, VisibleAnywhere, Instanced)
	TObjectPtr<class UAssetImportData> AssetImportData;	
#endif

public:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = Hidden)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	UE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

protected:
	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	FGroomCacheInfo GroomCacheInfo;

	TArray<FGroomCacheChunk> Chunks;

	friend class FGroomCacheProcessor;
};

/**
 * The smallest unit of streamed GroomCache data
 * The BulkData member is loaded on-demand so that loading the GroomCache itself is relatively lightweight
 */
struct FGroomCacheChunk
{
	/** Size of the chunk of data in bytes */
	int32 DataSize = 0;

	/** Frame index of the frame stored in this block */
	int32 FrameIndex = 0;

	/** Bulk data if stored in the package. */
	FByteBulkData BulkData;

	void Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex);
};

/** Proxy that processes the HairGroupData into GroomCacheChunks that contain the groom animation data */
class FGroomCacheProcessor
{
public:
	UE_API FGroomCacheProcessor(EGroomCacheType InType, EGroomCacheAttributes InAttributes);

	UE_API void AddGroomSample(TArray<FGroomCacheInputData>&& GroomData);
	UE_API void TransferChunks(UGroomCache* GroomCache);
	EGroomCacheType GetType() const { return Type; }

private:
	TArray<FGroomCacheChunk> Chunks;
	EGroomCacheAttributes Attributes;
	EGroomCacheType Type;
};

namespace UE::Groom
{
	/** Build (create and fill) a groom cache from a processor */
	HAIRSTRANDSCORE_API void BuildGroomCache(FGroomCacheProcessor& Processor, const FGroomAnimationInfo& AnimInfo, UGroomCache* GroomCache);

	/** Build the groom groups data */
	HAIRSTRANDSCORE_API bool BuildGroupsData(const FHairDescription& HairDescription, const TArray<FHairGroupPlatformData>& PlatformData,
		TArray<FHairGroupInfoWithVisibility>& VisibilityData, const TArray<FHairGroupsInterpolation>& InterpolationData, TArray<FGroomCacheInputData>& GroupsData);
}

#undef UE_API