// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "HairStrandsDatas.h"
#include "GroomCacheData.generated.h"

#define UE_API HAIRSTRANDSCORE_API

class FRDGPooledBuffer;

/** Attributes in groom that can be animated */
UENUM()
enum class EGroomCacheAttributes : uint8
{
	None = 0,
	Position = 1,
	Width = 1 << 1,
	Color = 1 << 2,

	// For display names
	PositionWidth = (Position | Width) UMETA(DisplayName = "Position & Width"),
	PositionColor = (Position | Color) UMETA(DisplayName = "Position & Color"),
	WidthColor = (Width | Color) UMETA(DisplayName = "Width & Color"),
	PositionWidthColor = (Position | Width | Color) UMETA(DisplayName = "Position, Width, Color")
};

ENUM_CLASS_FLAGS(EGroomCacheAttributes)

/** Relevant information about a groom animation */
USTRUCT()
struct FGroomAnimationInfo
{
	GENERATED_BODY()

	UE_API FGroomAnimationInfo();

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	uint32 NumFrames;

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	float SecondsPerFrame;

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	float Duration;

	UPROPERTY()
	float StartTime;

	UPROPERTY()
	float EndTime;

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	int32 StartFrame;

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	int32 EndFrame;

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	EGroomCacheAttributes Attributes;

	UE_API bool IsValid() const;
};

/**
 * Types of GroomCache
 * Strands: animated render strands (including animatable hair attributes)
 * Guides: animated guides that require in-engine simulation (position only)
 */
UENUM()
enum class EGroomCacheType : uint8
{
	None,
	Strands,
	Guides
};

UENUM(BlueprintType)
enum class EGroomBasisType : uint8
{
	NoBasis,
	BezierBasis,
	BsplineBasis,
	CatmullromBasis,
	HermiteBasis,
	PowerBasis,
};

UENUM(BlueprintType)
enum class EGroomCurveType : uint8
{
	Cubic,
	Linear,
	VariableOrder,
};

/** Information about the GroomCache itself */
USTRUCT()
struct FGroomCacheInfo
{
	GENERATED_BODY()

	/** The serialization version of the GroomCache data */
	static int32 GetCurrentVersion();

	UPROPERTY()
	int32 Version = GetCurrentVersion();

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	EGroomCacheType Type = EGroomCacheType::None;

	UPROPERTY(VisibleAnywhere, Category = GroomCache)
	FGroomAnimationInfo AnimationInfo;
};

/** Animatable vertex data that maps to FHairStrandsPoints */
struct FGroomCacheVertexData
{
	FGroomCacheVertexData() = default;
	FGroomCacheVertexData(struct FHairStrandsPoints&& PointsData);

	TArray<FVector3f> PointsPosition;
	TArray<float> PointsRadius;
	TArray<float> PointsCoordU;
	TArray<FLinearColor> PointsBaseColor;

	// GPU resources. Lazily allocated
	TRefCountPtr<FRDGPooledBuffer> PositionBuffer;
	TRefCountPtr<FRDGPooledBuffer> RadiusBuffer;

	void Serialize(FArchive& Ar, int32 Version, EGroomCacheAttributes InAttributes);
};

/** Animatable strand data that maps to FHairStrandsCurves*/
struct FGroomCacheStrandData
{
	FGroomCacheStrandData() = default;
	FGroomCacheStrandData(struct FHairStrandsCurves&& CurvesData, float InMaxLength, float InMaxRadius);

	TArray<float> CurvesLength;
	float MaxLength = 0.0f;
	float MaxRadius = 0.0f;

	void Serialize(FArchive& Ar, int32 Version, EGroomCacheAttributes InAttributes);
};

/** Animatable group data that maps to FHairStrandsDatas */
struct FGroomCacheGroupData
{
	FGroomCacheGroupData() = default;
	FGroomCacheGroupData(struct FHairStrandsDatas&& GroupData);

	FGroomCacheVertexData VertexData;
	FGroomCacheStrandData StrandData;

	FBox BoundingBox;

	void Serialize(FArchive& Ar, int32 Version, EGroomCacheAttributes InAttributes);
};

/** Built groom data input used for a groom cache animation frame */
struct FGroomCacheInputData
{
	FHairStrandsDatas Strands;
	FHairStrandsDatas Guides;
};

/** Groom animation data for a frame*/
struct FGroomCacheAnimationData
{
	FGroomCacheAnimationData() = default;
	FGroomCacheAnimationData(TArray<struct FGroomCacheInputData>&& HairGroupData, int32 Version, EGroomCacheType Type, EGroomCacheAttributes Attributes);

	TArray<FGroomCacheGroupData> GroupsData;
	EGroomCacheAttributes Attributes;
	int32 Version;

	void Serialize(FArchive& Ar);
};

/** Interface to access GroomCache buffers for playback */
class IGroomCacheBuffers
{
public:
	virtual ~IGroomCacheBuffers() = default;

	virtual const FGroomCacheAnimationData& GetCurrentFrameBuffer() = 0;
	virtual const FGroomCacheAnimationData& GetNextFrameBuffer() = 0;

	virtual int32 GetCurrentFrameIndex() const = 0;
	virtual int32 GetNextFrameIndex() const = 0;
	virtual float GetInterpolationFactor() const = 0;

	FCriticalSection* GetCriticalSection() { return &CriticalSection; }

private:
	FCriticalSection CriticalSection;
};

#undef UE_API
