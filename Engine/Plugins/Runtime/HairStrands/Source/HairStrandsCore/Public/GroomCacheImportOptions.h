// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorFramework/AssetImportData.h"
#include "GroomSettings.h"
#include "UObject/SoftObjectPath.h"
#include "GroomCacheImportOptions.generated.h"

UENUM(BlueprintType)
enum class EGroomCacheImportType : uint8
{
	None = 0x00 UMETA(Hidden),
	Strands = 0x01,
	Guides = 0x02,
	All = Strands | Guides
};

ENUM_CLASS_FLAGS(EGroomCacheImportType);

USTRUCT(BlueprintType)
struct FGroomCacheImportSettings
{
	GENERATED_BODY()

	/** Import the animated groom that was detected in this file */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GroomCache)
	bool bImportGroomCache = true;

	/** Groom Cache types to import */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GroomCache)
	EGroomCacheImportType ImportType = EGroomCacheImportType::All;

	/** Starting index to start sampling the animation from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling)
	int32 FrameStart = 0;

	/** Ending index to stop sampling the animation at */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling)
	int32 FrameEnd = 0;

	/** Skip empty (pre-roll) frames and start importing at the frame which actually contains data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sampling, meta = (DisplayName = "Skip Empty Frames at Start of Groom Animation"))
	bool bSkipEmptyFrames = false;

	/** Import or re-import the groom asset from this file */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GroomCache)
	bool bImportGroomAsset = true;

	/** The groom asset the groom cache will be built from (must be compatible) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GroomCache, meta = (MetaClass = "/Script/HairStrandsCore.GroomAsset"))
	FSoftObjectPath GroomAsset;

	/** Set to true to override the groom conversion settings. Otherwise, use the settings from the groom import options */
	UPROPERTY(BlueprintReadWrite, Category = GroomCache)
	bool bOverrideConversionSettings = false;

	/** Conversion settings to apply to the groom cache import when override is enabled */
	UPROPERTY(BlueprintReadWrite, Category = GroomCache)
	FGroomConversionSettings ConversionSettings;
};

UCLASS(MinimalAPI, BlueprintType)
class UGroomCacheImportOptions : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = GroomCache)
	FGroomCacheImportSettings ImportSettings;
};

/** The asset import data to store the import settings within the GroomCache asset */
UCLASS(MinimalAPI)
class UGroomCacheImportData : public UAssetImportData
{
	GENERATED_BODY()

public:

	UPROPERTY()
	FGroomCacheImportSettings Settings;
};
