// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentStreaming.h"

class UGroomCache;
class UGroomComponent;

struct IGroomCacheStreamingManager : public IStreamingManager
{
	virtual ~IGroomCacheStreamingManager() = default;

	HAIRSTRANDSCORE_API static struct IGroomCacheStreamingManager& Get();
	HAIRSTRANDSCORE_API static void Register();
	HAIRSTRANDSCORE_API static void Unregister();

	/** Registers a new component to the streaming manager. */
	virtual void RegisterComponent(UGroomComponent* GroomComponent) = 0;

	/** Removes the component from the streaming manager. */
	virtual void UnregisterComponent(UGroomComponent* GroomComponent) = 0;

	/** 
	 * Prefetch data for the component. Data is automatically prefetched when initially registering the component
	 * but exposing this may be useful in other cases (eg. when seeking, etc.)
	 */
	virtual void PrefetchData(UGroomComponent* GroomComponent) = 0;

	/**
	 * Returns a pointer to the cached GroomCacheAnimationData.
	 *
	 * @param GroomCache	The GroomCache for which to retrieve the data
	 * @param ChunkIndex	Index of the chunk we want
	 * @return The cached data or null if it's not loaded yet
	 */
	virtual const struct FGroomCacheAnimationData* MapAnimationData(const UGroomCache* GroomCache, uint32 ChunkIndex) = 0;

	/**
	 * Releases pointer to the cachedGroomCacheAnimationData.
	 * Must be called for every successful call to MapAnimationData.
	 *
	 * @param GroomCache	The GroomCache for which the data was retrieved
	 * @param ChunkIndex	Index of the chunk to release
	 */
	virtual void UnmapAnimationData(const UGroomCache* GroomCache, uint32 ChunkIndex) = 0;

	/** Cleans up the resources managed by the streaming manager */
	virtual void Shutdown() = 0;
};

