// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Components/MeshComponent.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "RHIDefinitions.h"
#include "GroomDesc.h"
#include "LODSyncInterface.h"
#include "GroomInstance.h"
#include "NiagaraDataInterfacePhysicsAsset.h"

#include "GroomComponent.generated.h"

#define UE_API HAIRSTRANDSCORE_API

class UGroomCache;
class UMeshDeformer;
class UMeshDeformerInstance;
class UMeshDeformerInstanceSettings;
class UGroomComponent;
class UMeshComponent;

UCLASS(HideCategories = (Object, Physics, Activation, Mobility, "Components|Activation"), editinlinenew, meta = (BlueprintSpawnableComponent), ClassGroup = Rendering, MinimalAPI)
class UGroomComponent : public UMeshComponent, public ILODSyncInterface, public INiagaraPhysicsAssetDICollectorInterface
{
	GENERATED_UCLASS_BODY()

public:

	/** Groom asset . */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, interp, Category = "Groom")
	TObjectPtr<UGroomAsset> GroomAsset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, interp, Category = "GroomCache")
	TObjectPtr<UGroomCache> GroomCache;

	/** Niagara components that will be attached to the system*/
	UPROPERTY(Transient)
	TArray<TObjectPtr<class UNiagaraComponent>> NiagaraComponents;

	// Kept for debugging mesh transfer
	UPROPERTY()
	TObjectPtr<class USkeletalMesh> SourceSkeletalMesh;

	/** Optional binding asset for binding a groom onto a skeletal mesh. If the binding asset is not provided the projection is done at runtime, which implies a large GPU cost at startup time. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, interp, Category = "Groom")
	TObjectPtr<class UGroomBindingAsset> BindingAsset;

	/** Physics asset to be used for hair simulation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Simulation")
	TObjectPtr<class UPhysicsAsset> PhysicsAsset;

	/** List of collision components to be used */
	TArray<TWeakObjectPtr<class USkeletalMeshComponent>> CollisionComponents;
	
	/** Groom's simulation settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, interp, Category = "Simulation")
	FHairSimulationSettings SimulationSettings;

	/** If set the MeshDeformer will be applied on groom instance for deformation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Deformer", Meta = (ToolTip = "Mesh deformer that will be applied on groom instance for deformation. Enable the groom asset deformation flags on groups to be able to set it."))
	TObjectPtr<UMeshDeformer> MeshDeformer;

	/** Object containing state for the bound MeshDeformer. */
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Deformer")
	TObjectPtr<UMeshDeformerInstance> MeshDeformerInstance;

	/** Object containing instance settings for the bound MeshDeformer. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Instanced, Category = "Deformer", meta = (DisplayName = "Deformer Settings", EditCondition = "MeshDeformerInstanceSettings!=nullptr", ShowOnlyInnerProperties))
	TObjectPtr<UMeshDeformerInstanceSettings> MeshDeformerInstanceSettings;

	/* Reference of the default/debug materials for each geometric representation */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Strands_DebugMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Strands_DefaultMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Cards_DefaultMaterial;
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Meshes_DefaultMaterial;

	UPROPERTY()
	TObjectPtr<class UNiagaraSystem> AngularSpringsSystem;

	UPROPERTY()
	TObjectPtr<class UNiagaraSystem> CosseratRodsSystem;

	/** Optional socket name, where the groom component should be attached at, when parented with a skeletal mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, interp, Category = "Groom")
	FString AttachmentName;

	/** Boolean to check when the simulation should be reset */
	bool bResetSimulation;

	/** Boolean to check when the simulation should be initialized */
	bool bInitSimulation;

	/** Tracks whether the component was hidden last tick, so we can detect visibility transitions and pause/resume the Niagara sim accordingly. */
	bool bWasHiddenLastTick = false;

	/** Previous bone matrix to compare the difference and decide to reset or not the simulation */
	FMatrix	PrevBoneMatrix;

	/** Update Niagara components */
	UE_API void UpdateHairSimulation();

	/** Release Niagara components */
	UE_API void ReleaseHairSimulation();
	UE_API void ReleaseHairSimulation(const int32 GroupIndex);

	/** Create per Group/LOD the Niagara component */
	UE_API void CreateHairSimulation(const int32 GroupIndex, const int32 LODIndex);

	/** Enable/Disable hair simulation while transitioning from one LOD to another one */
	UE_API void SwitchSimulationLOD(const int32 PreviousLOD, const int32 CurrentLOD, const EHairLODSelectionType InLODSelectionType);

	/** Check if the simulation is enabled or not */
	UE_API bool IsSimulationEnable(int32 GroupIndex, int32 LODIndex) const;

	/** Check if the deformation is enabled or not (from the rigged guides or from the solver deformer) */
	UE_API bool IsDeformationEnable(int32 GroupIndex) const;

	/** Update Group Description */
	UE_API void UpdateHairGroupsDesc();
	UE_API void UpdateHairGroupsDescAndInvalidateRenderState(bool bInvalidate=true);

	/** Update simulated groups */
	UE_API void UpdateSimulatedGroups();

	//~ Begin UObject Interface.
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	UE_API uint32 GetResourcesSize() const; 
	//~ End UObject Interface.

	//~ Begin UActorComponent Interface.
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void FinishDestroy() override;
	UE_API virtual void OnAttachmentChanged() override;
	UE_API virtual void DetachFromComponent(const FDetachmentTransformRules& DetachmentRules) override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	UE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	UE_API virtual void SendRenderDynamicData_Concurrent() override;
	UE_API virtual void DestroyRenderState_Concurrent() override;
	UE_API virtual bool RequiresGameThreadEndOfFrameRecreate() const override;
	//~ End UActorComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual void PostLoad() override;
	//~ End UMeshComponent Interface.

	/** Return the guide hairs rest resources*/
	UE_API FHairStrandsRestResource* GetGuideStrandsRestResource(uint32 GroupIndex);

	/** Return the guide hairs deformed resources*/
	UE_API FHairStrandsDeformedResource* GetGuideStrandsDeformedResource(uint32 GroupIndex);

	/** Return the guide hairs root resources*/
	UE_API FHairStrandsRestRootResource* GetGuideStrandsRestRootResource(uint32 GroupIndex);
	UE_API FHairStrandsDeformedRootResource* GetGuideStrandsDeformedRootResource(uint32 GroupIndex);

#if WITH_EDITOR
	UE_API virtual void CheckForErrors() override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API void ValidateMaterials(bool bMapCheck) const;
	UE_API void Invalidate();
	UE_API void InvalidateAndRecreate();

	UE_API virtual void PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel) override;
	UE_DEPRECATED(5.8, "Use HandlePlatformPreviewChanged(EShaderPlatform) instead.")
	UE_API void HandlePlatformPreviewChanged(ERHIFeatureLevel::Type InFeatureLevel);
	UE_DEPRECATED(5.8, "Use HandlePlatformPreviewChanged(EShaderPlatform) instead.")
	UE_API void HandleFeatureLevelChanged(ERHIFeatureLevel::Type InFeatureLevel);
	UE_API void HandlePlatformPreviewChanged(EShaderPlatform InShaderPlatform);
#endif

	UE_API void PostCompilation();
	
	//~ Begin IInterface_AsyncCompilation Interface.
	UE_API virtual bool IsCompiling() const override;
	//~ End IInterface_AsyncCompilation Interface.

	/* Accessor function for changing Groom asset from blueprint/sequencer */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void SetGroomAsset(UGroomAsset* Asset);

	/* Accessor function for changing Groom binding asset from blueprint/sequencer */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void SetBindingAsset(UGroomBindingAsset* InBinding);

	/* Accessor function for changing Groom physics asset from blueprint/sequencer */
	UFUNCTION(BlueprintCallable, Category = "Simulation")
	UE_API void SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset);

	/* Change the MeshDeformer that is used for this Component. */
	UFUNCTION(BlueprintCallable, Category = "Components|SkinnedMesh")
	UE_API void SetMeshDeformer(UMeshDeformer* InMeshDeformer);

	/* Add a skeletal mesh to the collision components */
	UFUNCTION(BlueprintCallable, Category = "Simulation")
	UE_API void AddCollisionComponent(USkeletalMeshComponent* SkeletalMeshComponent);
	
	/* Reset the collision components */
	UFUNCTION(BlueprintCallable, Category = "Simulation")
	UE_API void ResetCollisionComponents();

	/* Accessor function for changing the enable simulation flag from blueprint/sequencer */
	UFUNCTION(BlueprintCallable, Category = "Simulation")
	UE_API void SetEnableSimulation(bool bInEnableSimulation);

	/* Reset the simulation, if enabled */
	UFUNCTION(BlueprintCallable, Category = "Simulation")
	UE_API void ResetSimulation();
	
	/* Given the group index return the matching niagara component */
	UFUNCTION(BlueprintCallable, Category = "Simulation")
	UNiagaraComponent* GetNiagaraComponent(const int32 GroupIndex) {return NiagaraComponents.IsValidIndex(GroupIndex) ? NiagaraComponents[GroupIndex] : nullptr;}

	/* Accessor function for changing hair length scale from blueprint/sequencer */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void SetHairLengthScale(float Scale);

	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void SetHairLengthScaleEnable(bool bEnable);

	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API bool GetIsHairLengthScaleEnabled();

	UE_API void SetStableRasterization(bool bEnable);
	UE_API void SetGroomAsset(UGroomAsset* Asset, UGroomBindingAsset* InBinding, const bool bUpdateSimulation = true);
	UE_API void SetHairRootScale(float Scale);
	UE_API void SetHairWidth(float HairWidth);
	UE_API void SetScatterSceneLighting(bool Enable);
	UE_API void SetBinding(UGroomBindingAsset* InBinding);
	UE_API void SetUseCards(bool InbUseCards);
	void SetValidation(bool bEnable) { bValidationEnable = bEnable; }

	///~ Begin ILODSyncInterface Interface.
	UE_API virtual int32 GetDesiredSyncLOD() const override;
	UE_API virtual int32 GetBestAvailableLOD() const override;
	UE_API virtual void SetForceStreamedLOD(int32 LODIndex) override;
	UE_API virtual void SetForceRenderedLOD(int32 LODIndex) override;
	UE_API virtual int32 GetNumSyncLODs() const override;
	UE_API virtual int32 GetForceStreamedLOD() const override;
	UE_API virtual int32 GetForceRenderedLOD() const override;
	//~ End ILODSyncInterface

	UE_API int32 GetNumLODs() const;
	UE_API int32 GetForcedLOD() const;
	UE_API void  SetForcedLOD(int32 LODIndex);

	/** Groom's groups info. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom")
	TArray<FHairGroupDesc> GroomGroupsDesc;

	/** Force the groom to use cards/meshes geometry instead of strands. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom")
	bool bUseCards = false;

	/** Hair group instance access */
	uint32 GetGroupCount() const { return HairGroupInstances.Num();  }
	FHairGroupInstance* GetGroupInstance(int32 Index) { return Index >= 0 && Index < HairGroupInstances.Num() ? HairGroupInstances[Index] : nullptr; }
	const FHairGroupInstance* GetGroupInstance(int32 Index) const { return Index >= 0 && Index < HairGroupInstances.Num() ? HairGroupInstances[Index] : nullptr; }
	bool ContainsGroupInstance(FHairGroupInstance* Instance) const { return HairGroupInstances.Contains(Instance); }

	//~ Begin UPrimitiveComponent Interface
	UE_API EHairGeometryType GetMaterialGeometryType(int32 ElementIndex) const;
	UE_API UMaterialInterface* GetMaterial(int32 ElementIndex, EHairGeometryType GeometryType, bool bUseDefaultIfIncompatible) const;
	UE_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	UE_API virtual int32 GetMaterialIndex(FName MaterialSlotName) const override;
	UE_API virtual TArray<FName> GetMaterialSlotNames() const override;
	UE_API virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const override;
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UPrimitiveComponent Interface

#if WITH_EDITORONLY_DATA
	// Set the component in preview mode, forcing the loading of certain data
	void SetPreviewMode(bool bValue) { bPreviewMode = bValue; };
#endif

	/** GroomCache */
	UGroomCache* GetGroomCache() const { return GroomCache; }

	/* Accessor function for changing GroomCache asset from blueprint/sequencer */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void SetGroomCache(UGroomCache* InGroomCache);

	UE_API float GetGroomCacheDuration() const;

	UE_API void SetManualTick(bool bInManualTick);
	UE_API bool GetManualTick() const;
	UE_API void TickAtThisTime(const float Time, bool bInIsRunning, bool bInBackwards, bool bInIsLooping);

	UE_API void ResetAnimationTime();
	UE_API float GetAnimationTime() const;
	bool IsLooping() const { return bLooping; }

	/** Set the groom solver onto the component */
	void SetGroomSolver(UMeshComponent* InGroomSolver) {GroomSolver = InGroomSolver;}

	/** Get the groom solver from the component */
	UMeshComponent* GetGroomSolver() const {return GroomSolver.Get();}

	/** Build the local simulation transform that could be used in strands simulation */
	UE_API void BuildSimulationTransform(FTransform& SimulationTransform) const;

	//~ Begin INiagaraPhysicsAssetDICollectorInterface Interface
	UE_API virtual UPhysicsAsset* BuildAndCollect(FTransform& BoneTransform, TArray<TWeakObjectPtr<USkeletalMeshComponent>>& SourceComponents, TArray<TWeakObjectPtr<UPhysicsAsset>>& PhysicsAssets) const override;
	//~ End INiagaraPhysicsAssetDICollectorInterface Interface

private:
	void InitIfDependenciesReady(const bool bUpdateSimulation = true);
	void UpdateGroomCache(float Time);

	UPROPERTY(EditAnywhere, Category = GroomCache)
	bool bRunning;

	UPROPERTY(EditAnywhere, Category = GroomCache)
	bool bLooping;

	UPROPERTY(EditAnywhere, Category = GroomCache)
	bool bManualTick;

	UPROPERTY(VisibleAnywhere, transient, Category = GroomCache)
	float ElapsedTime;

	TSharedPtr<class IGroomCacheBuffers, ESPMode::ThreadSafe> GroomCacheBuffers;

private:
	TArray<TRefCountPtr<FHairGroupInstance>> HairGroupInstances;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UGroomAsset> GroomAssetBeingLoaded;

	UPROPERTY(Transient)
	TObjectPtr<UGroomBindingAsset> BindingAssetBeingLoaded;
#endif

private:
	void CheckHairStrandsUsage(int32 ElementIndex) const;
	void DeleteDeferredHairGroupInstances();
	void* InitializedResources;

	UPROPERTY(Transient)
	TObjectPtr<UMeshComponent> RegisteredMeshComponent;

	UPROPERTY(Transient)
	TObjectPtr<UMeshComponent> DeformedMeshComponent;

	bool bIsGroomAssetCallbackRegistered;
	bool bIsGroomBindingAssetCallbackRegistered;
	bool bValidationEnable = true;
	bool bPreviewMode = false;

	// LOD selection
	EHairLODSelectionType LODSelectionType = EHairLODSelectionType::Immediate;
	float LODPredictedIndex = -1.f;
	float LODForcedIndex = -1.f;

	void InitResources(bool bIsBindingReloading=false);
	void ReleaseResources();

	friend class UGroomBindingAsset;
	friend class FGroomComponentRecreateRenderStateContext;
	friend class FHairStrandsSceneProxy;
	friend class FHairCardsSceneProxy;

	/** Groom solver pointer */
	TSoftObjectPtr<UMeshComponent> GroomSolver = nullptr;
};

#if WITH_EDITORONLY_DATA
/** Used to recreate render context for all GroomComponents that use a given GroomAsset */
class FGroomComponentRecreateRenderStateContext
{
public:
	UE_API FGroomComponentRecreateRenderStateContext(UGroomAsset* GroomAsset);
	UE_API ~FGroomComponentRecreateRenderStateContext();

private:
	TArray<UGroomComponent*> GroomComponents;
};

/** Return the debug color of an hair group */
HAIRSTRANDSCORE_API const FLinearColor GetHairGroupDebugColor(int32 GroupIt);

#endif

struct FGroomComponentMemoryStats
{
	uint32 Guides = 0;
	uint32 Strands= 0;
	uint32 Cards  = 0;
	uint32 Meshes = 0;

	static FGroomComponentMemoryStats Get(const FHairGroupInstance* In);
	void Accumulate(const FGroomComponentMemoryStats& In);
	uint32 GetTotalSize() const;
};

#undef UE_API 
