// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomBindingAsset.h"
#include "GroomCreateBindingOptions.generated.h"

UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings, autoCollapseCategories = (Conversion), hideCategories = (Hidden))
class UGroomCreateBindingOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Groom asset */
	UPROPERTY()
	TWeakObjectPtr<UGroomAsset> GroomAsset;

	/** Type of mesh to create groom binding for */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	EGroomBindingMeshType GroomBindingType;

	/** Skeletal mesh on which the groom has been authored. This is optional, and used only if the hair
		binding is done a different mesh than the one which it has been authored, i.e., only if the curves 
		roots and the surface geometry don't aligned and need to be wrapped/transformed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Conversion)
	TObjectPtr<USkeletalMesh> SourceSkeletalMesh;

	/** Skeletal mesh on which the groom is attached to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	TObjectPtr<USkeletalMesh> TargetSkeletalMesh;

	/** GeometryCache on which the groom has been authored */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Conversion)
	TObjectPtr<UGeometryCache> SourceGeometryCache;

	/** GeometryCache on which the groom is attached to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = BuildSettings)
	TObjectPtr<UGeometryCache> TargetGeometryCache;

	/** Number of points used for the rbf interpolation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties, ClampMin = "0", ClampMax = "100"), Category = HairInterpolation)
	int32 NumInterpolationPoints;

	/** Section to pick to transfer the position */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta = (ShowOnlyInnerProperties), Category = Conversion)
	int32 MatchingSection;

	/** Optional mesh attribute used to filter triangles to bind groom to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = BuildSettings)
	FName TargetBindingAttribute = NAME_None;
};
