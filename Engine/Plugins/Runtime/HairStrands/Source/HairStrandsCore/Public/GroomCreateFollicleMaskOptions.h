// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomSettings.h"
#include "UObject/Object.h"
#include "GroomAsset.h"
#include "GroomCreateFollicleMaskOptions.generated.h"

/** List of channel */
UENUM(BlueprintType)
enum class EFollicleMaskChannel : uint8
{
	R = 0 UMETA(DisplatName = "Red"),
	G = 1 UMETA(DisplatName = "Green"),
	B = 2 UMETA(DisplatName = "Blue"),
	A = 3 UMETA(DisplatName = "Alpha")
};


USTRUCT(BlueprintType)
struct FFollicleMaskOptions
{
	GENERATED_BODY()

	/** Groom asset */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Groom)
	TObjectPtr<UGroomAsset> Groom = nullptr;

	/** Texture channel in which the groom's roots mask will be writtent to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Groom)
	EFollicleMaskChannel Channel = EFollicleMaskChannel::R;
};

UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings, HideCategories = ("Hidden"))
class UGroomCreateFollicleMaskOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Follicle mask texture resolution. The resolution will be rounded to the closest power of two. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	int32 Resolution = 4096;

	/** Size of the root in the follicle mask (in pixels) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	int32 RootRadius = 8;

	/** Grooms which will be use to create the follicle texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	TArray<FFollicleMaskOptions> Grooms;
};
