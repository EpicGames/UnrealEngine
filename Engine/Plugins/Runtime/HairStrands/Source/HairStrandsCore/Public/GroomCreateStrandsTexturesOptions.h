// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomSettings.h"
#include "UObject/Object.h"
#include "GroomAsset.h"
#include "GroomCreateStrandsTexturesOptions.generated.h"
	

/** Size of each strands*/
UENUM(BlueprintType)
enum class EStrandsTexturesTraceType : uint8
{
	TraceInside			= 0 UMETA(DisplatName = "Trace inside"),
	TraceOuside			= 1 UMETA(DisplatName = "Trace outside"),
	TraceBidirectional  = 2 UMETA(DisplatName = "Trace both sides")
};

/** Size of each strands*/
UENUM(BlueprintType)
enum class EStrandsTexturesMeshType : uint8
{
	Static = 0 UMETA(DisplatName = "Use static mesh"),
	Skeletal = 1 UMETA(DisplatName = "Use skeletal mesh")
};

UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings, HideCategories = ("Hidden"))
class UGroomCreateStrandsTexturesOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Resolution of the output texture maps (tangent, coverage, ...) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	EHairTextureLayout Layout = EHairTextureLayout::Layout1;

	/** Resolution of the output texture maps (tangent, coverage, ...) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	int32 Resolution = 2048;

	/** Direction in which the tracing will be done: either from the mesh's surface to the outside, or from the mesh's surface to the inside. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	EStrandsTexturesTraceType TraceType = EStrandsTexturesTraceType::TraceBidirectional;

	/** Distance from the mesh surface until hair are projected onto the mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options, meta = (ClampMin = "0.0001", UIMin = "0.001", UIMax = "10.0"))
	float TraceDistance = 3.f;

	/** Select which mesh should be used for tracing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	EStrandsTexturesMeshType MeshType = EStrandsTexturesMeshType::Static;

	/** Mesh on which the groom strands will be projected on. If non empty and if the skeletal mesh entry is empty, the static mesh will be used for generating the textures. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	TObjectPtr<UStaticMesh> StaticMesh;

	/** Mesh on which the groom strands will be projected on. If non empty, the skeletal mesh will be used for generating the textures.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	/** LOD of the mesh, on which the texture projection is done */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	int32 LODIndex = 0;

	/** Section of the mesh, on which the texture projection is done */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	int32 SectionIndex = 0;

	/** UV channel to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	int32 UVChannelIndex = 0;

	/** Groom index which should be baked into the textures. When the array is empty, all groups will be included (Default). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Options)
	TArray<int32> GroupIndex;

	/** Number pixels expanded by the post-process dilation (0..64). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties, ClampMin = 0, ClampMax = 32, UIMin = 0, UIMax = 32), Category = Options)
	int32 Dilation = 8;

	/** Generated texure. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties, ClampMin = 0, ClampMax = 32, UIMin = 0, UIMax = 32), Category = Options)
	TArray<TObjectPtr<UTexture2D>> GeneratedTextures;
};
