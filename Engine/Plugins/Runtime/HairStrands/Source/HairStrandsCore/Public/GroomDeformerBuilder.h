// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UGroomAsset;
class USkeletalMesh;

struct FGroomDeformerBuilder
{
	// Create skeletal mesh bones+physics asset from groom asset guides
	static HAIRSTRANDSCORE_API USkeletalMesh* CreateSkeletalMesh(UGroomAsset* GroomAsset);
};
