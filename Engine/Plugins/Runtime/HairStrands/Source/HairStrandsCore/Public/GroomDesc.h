// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "GroomDesc.generated.h"

// Note: If a new field is added to this struct, think to update GroomComponentDestailsCustomization.cpp to handle override flags

USTRUCT(BlueprintType)
struct FHairGroupDesc
{
	GENERATED_USTRUCT_BODY()

	/** Length of the longest hair strands */
	UPROPERTY()
	float HairLength = 0;

	/** Hair width (in centimeters) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom", meta = (editcondition = "HairWidth_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "1.0", SliderExponent = 6))
	float HairWidth = 0.1f;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairWidth_Override = false;

	/** Scale the hair width at the root */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom", AdvancedDisplay, meta = (editcondition = "HairRootScale_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "2.0", SliderExponent = 6))
	float HairRootScale = 1.0f;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairRootScale_Override = false;

	/** Scale the hair width at the tip */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom", AdvancedDisplay, meta = (editcondition = "HairTipScale_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "2.0", SliderExponent = 6))
	float HairTipScale = 1.0f;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairTipScale_Override = false;

	/** Override the hair shadow density factor (unit less). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom", AdvancedDisplay, meta = (editcondition = "HairShadowDensity_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "10.0", SliderExponent = 6))
	float HairShadowDensity = 1.0f;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairShadowDensity_Override = false;

	/** Scale the hair geometry radius for ray tracing effects (e.g. shadow) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom", AdvancedDisplay, meta = (editcondition = "HairRaytracingRadiusScale_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "10.0", SliderExponent = 6))
	float HairRaytracingRadiusScale = 1.0f;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairRaytracingRadiusScale_Override = false;

	/** Enable hair strands geomtry for raytracing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Groom", AdvancedDisplay, meta = (editcondition = "bUseHairRaytracingGeometry_Override", ClampMin = "0.0001", UIMin = "0.001", UIMax = "10.0", SliderExponent = 6))
	bool bUseHairRaytracingGeometry = false;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool bUseHairRaytracingGeometry_Override = false;

	/** Bias the selected LOD. A value >0 will progressively select lower detailed lods. Used when r.HairStrands.Cluster.Culling = 1. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, interp, Category = "Performance", AdvancedDisplay, meta = (ClampMin = "-7.0", ClampMax = "7.0", UIMin = "-7.0", UIMax = "7.0", SliderExponent = 1))
	float LODBias = 0;

	/** Insure the hair does not alias. When enable, group of hairs might appear thicker. Isolated hair should remain thin. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Groom", meta = (editcondition = "bUseStableRasterization_Override"))
	bool bUseStableRasterization = false;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool bUseStableRasterization_Override = false;

	/** Light hair with the scene color. This is used for vellus/short hair to bring light from the surrounding surface, like skin. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Groom", meta = (editcondition = "bScatterSceneLighting_Override"))
	bool bScatterSceneLighting = false;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool bScatterSceneLighting_Override = false;

	UPROPERTY()
	bool bSupportVoxelization = true;
	UPROPERTY()
	bool bSupportVoxelization_Override = 0;

	/** When enabled, Length Scale allow to scale the length of the hair. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, interp, Category = "Groom", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", SliderExponent = 1, editcondition = "HairLengthScale_Override"))
	float HairLengthScale = 1.f;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (PinHiddenByDefault, InlineEditConditionToggle))
	bool HairLengthScale_Override = false;
};
typedef FHairGroupDesc FHairGroupInstanceModifer;