// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"

class UGroomAsset;

///////////////////////////////////////////////////////////////////////////////////////////////////
// Strands
struct FEditableHairStrandControlPoint
{
	FVector3f Position;
	float Radius;
	float U;
	FLinearColor BaseColor;
	float Roughness;
	float AO;

	uint8 bHasColor : 1 = false;
	uint8 bHasRoughness : 1 = false;
	uint8 bHasAO : 1 = false;
};

struct FEditableHairStrand
{
	TArray<FEditableHairStrandControlPoint> ControlPoints;

	uint32 StrandID;
	uint32 ClumpID;
	FVector2f RootUV;

	// Pre-computed Simulation
	uint32 GuideIDs[3];
	float GuideWeights[3];

	uint8 bHasStrandID : 1 = false;
	uint8 bHasClumpID : 1 = false;
	uint8 bHasClosestGuide : 1 = false;
	uint8 bHasRootUV : 1 = false;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Guides
struct FEditableHairGuideControlPoint
{
	FVector3f Position;
	float U;
};

struct FEditableHairGuide
{
	TArray<FEditableHairGuideControlPoint> ControlPoints;

	uint32 GuideID;
	FVector2f RootUV;

	uint8 bHasGuideID : 1 = false;
	uint8 bHasRootUV : 1 = false;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Group & Groom
struct FEditableGroomGroup
{
	uint32 GroupIndex;
	uint32 GroupID;
	FName GroupName;
	TArray<FEditableHairStrand> Strands;
	TArray<FEditableHairGuide> Guides;
};

struct FEditableGroom
{
	TArray<FEditableGroomGroup> Groups;
};

enum EEditableGroomOperations
{
	ControlPoints_Added    = 0x1,
	ControlPoints_Modified = 0x2,
	ControlPoints_Deleted  = 0x4,

	Strands_Added          = 0x8, 
	Strands_Modified       = 0x10,
	Strands_Deleted        = 0x20,

	Group_Added            = 0x40,
	Group_Deleted          = 0x80
};

// Convert a groom asset into an editable groom asset
HAIRSTRANDSCORE_API void ConvertFromGroomAsset(UGroomAsset* In, FEditableGroom* Out, const bool bAllowCurveReordering = true, const bool bApplyDecimation = true, const bool bAllowAddEndControlPoint = true);

// Convert an editable groom asset into a groom asset
// The 'operations' flag indicates what type of modifications have been done onto the editable groom. 
// This helps to drive the update mechanism.
HAIRSTRANDSCORE_API void ConvertToGroomAsset(UGroomAsset* Out, const FEditableGroom* In, uint32 Operations);