// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomSettings.h"
#include "UObject/Object.h"
#include "GroomAssetInterpolation.h"
#include "GroomImportOptions.generated.h"

#define UE_API HAIRSTRANDSCORE_API

struct FHairDescriptionGroups;

UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings, HideCategories = ("Hidden"))
class UGroomImportOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Conversion)
	FGroomConversionSettings ConversionSettings;

	/* Interpolation settings per group */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Hidden)
	TArray<FHairGroupsInterpolation> InterpolationSettings;
};

USTRUCT(BlueprintType)
struct FGroomHairGroupPreview
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Preview)
	int32 GroupIndex = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Preview)
	FName GroupName;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Preview)
	int32 GroupID = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Preview)
	int32 CurveCount = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Preview)
	int32 GuideCount = 0;

	UPROPERTY()
	uint32 Attributes = 0;

	UPROPERTY()
	uint32 AttributeFlags = 0;

	UPROPERTY()
	uint32 Flags = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Preview)
	FHairGroupsInterpolation InterpolationSettings;
};

UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings, HideCategories = ("Hidden"))
class UGroomHairGroupsPreview : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = Preview)
	TArray<FGroomHairGroupPreview> Groups;
};

UCLASS(MinimalAPI, BlueprintType, config = EditorPerProjectUserSettings, HideCategories = ("Hidden"))
class UGroomHairGroupsMapping : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = GroupMapping)
	TArray<FName> OldGroupNames;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = GroupMapping)
	TArray<FName> NewGroupNames;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = GroupMapping)
	TArray<int32> OldToNewGroupIndexMapping;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, meta = (ShowOnlyInnerProperties), Category = GroupMapping)
	TArray<int32> NewToOldGroupIndexMapping;

	FName GetNewGroupName(int32 In) { return NewGroupNames.IsValidIndex(In) ? NewGroupNames[In] : FName(); }
	FName GetOldGroupName(int32 In) { return OldGroupNames.IsValidIndex(In) ? OldGroupNames[In] : FName(); }

	UE_API const TArray<TSharedPtr<FString>>& GetOldGroupNames();

	UE_API bool HasValidMapping() const;
	bool IsValid() const { return OldToNewGroupIndexMapping.Num() > 0; }
	UE_API void SetIndex(int32 NewIndex, int32 OldIndex);

	TArray<TSharedPtr<FString>> CachedOldGroupNames;

	// Initialize UGroomHairGroupsMapping by mapping OldGroup onto NewGroup
	UE_API void Map(const FHairDescriptionGroups& OldGroup, const FHairDescriptionGroups& NewGroup);

	// Recreate a mapping from SrcGroups to DstGroups
	static UE_API void RemapHairDescriptionGroups(const FHairDescriptionGroups& SrcGroups, const FHairDescriptionGroups& DstGroups, TArray<int32>& Out);
};

UE_API UGroomImportOptions* CreateGroomImportOptions(const FHairDescriptionGroups& GroupsDescription, const TArray<FHairGroupsInterpolation>& BuildSettings);

#undef UE_API
