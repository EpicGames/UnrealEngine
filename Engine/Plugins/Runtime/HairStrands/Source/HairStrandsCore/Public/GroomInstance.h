// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomDesc.h"
#include "HairStrandsInterface.h"
#include "HairStrandsDefinitions.h"
#include "PrimitiveSceneInfo.h"

#define UE_API HAIRSTRANDSCORE_API

class UMeshComponent;
class UGroomComponent;
class FHairCardsVertexFactory;
enum class EGroomCacheType : uint8;
enum class EGroomViewMode : uint8;

// @hair_todo: pack card ID + card UV in 32Bits alpha channel's of the position buffer:
//  * 10/10 bits for UV -> max 1024/1024 rect resolution
//  * 12 bits for cards count -> 4000 cards for a hair group
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FHairCardsVertexFactoryUniformShaderParameters, HAIRSTRANDSCORE_API)
	SHADER_PARAMETER(uint32, Flags)
	SHADER_PARAMETER(uint32, LayoutIndex)
	SHADER_PARAMETER(uint32, TextureCount)
	SHADER_PARAMETER(uint32, AttributeTextureIndex)
	SHADER_PARAMETER(uint32, AttributeChannelIndex)
	SHADER_PARAMETER(uint32, MaxVertexCount)
	SHADER_PARAMETER(uint32, NumUVs)
	SHADER_PARAMETER(float, CoverageBias)
	SHADER_PARAMETER_SRV(Buffer<float4>, PositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, PreviousPositionBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, NormalsBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, UVsBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, MaterialsBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, VertexColorsBuffer)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, Texture0Texture) 
	SHADER_PARAMETER_SAMPLER(SamplerState,      Texture0Sampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, Texture1Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState,      Texture1Sampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, Texture2Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState,      Texture2Sampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, Texture3Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState,      Texture3Sampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, Texture4Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState,      Texture4Sampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, Texture5Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState,      Texture5Sampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FHairCardsVertexFactoryUniformShaderParameters> FHairCardsUniformBuffer;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FHairStrandsVertexFactoryUniformShaderParameters, HAIRSTRANDSCORE_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCommonParameters, Common)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceResourceRawParameters, Resources)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstancePrevResourceRawParameters, PrevResources)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCullingRawParameters, Culling)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FHairStrandsVertexFactoryUniformShaderParameters> FHairStrandsUniformBuffer;

enum class EHairLODSelectionType
{
	Immediate,	// Done on the rendering thread, prior to render
	Predicted,	// Predicted based on rendering->game thread feedback
	Forced		// Forced LOD value
};

enum class EHairViewRayTracingMask
{
	None		= 0x0,
	RayTracing  = 0x1, // Visible for raytracing effects (RT shadow, RT refleciton, Lumen, ...)
	PathTracing = 0x2, // Visible for pathtracing rendering
};
ENUM_CLASS_FLAGS(EHairViewRayTracingMask);

// Represent/Describe data & resources of a hair group belonging to a groom
struct FHairGroupInstance : public FHairStrandsInstance
{
	UE_API virtual ~FHairGroupInstance();

	//////////////////////////////////////////////////////////////////////////////////////////
	// Simulation
	struct FGuides
	{
		const FHairStrandsBulkData& GetData() const { return RestResource->BulkData; }

		bool IsValid() const { return RestResource != nullptr; }
		bool NeedsGuides() const { return IsValid() && (bIsSimulationEnable || bIsDeformationEnable || bHasGlobalInterpolation || bIsSimulationCacheEnable); }
		bool HasValidRootData() const { return RestRootResource != nullptr && DeformedRootResource != nullptr; }

		// Resources - Strands rest position data for sim & render strands
		// Resources - Strands deformed position data for sim & render strands
		FHairStrandsRestResource* RestResource = nullptr;
		FHairStrandsDeformedResource* DeformedResource = nullptr;

		// Resources - Rest root data, for deforming strands attached to a skinned mesh surface
		// Resources - Deformed root data, for deforming strands attached to a skinned mesh surface
		FHairStrandsRestRootResource* RestRootResource = nullptr;
		FHairStrandsDeformedRootResource* DeformedRootResource = nullptr;

		bool bIsSimulationEnable = false;
		bool bIsDeformationEnable = false;
		bool bHasGlobalInterpolation = false;
		bool bIsSimulationCacheEnable = false;
	} Guides;

	//////////////////////////////////////////////////////////////////////////////////////////
	// Strands
	struct FStrands
	{
		const FHairStrandsBulkData& GetData() const { return RestResource->BulkData; }

		bool IsValid() const { return RestResource != nullptr; }
		bool HasValidRootData() const { return RestRootResource != nullptr && DeformedRootResource != nullptr; }

		// Resources - Strands rest position data for sim & render strands
		// Resources - Strands deformed position data for sim & render strands
		FHairStrandsRestResource* RestResource = nullptr;
		FHairStrandsDeformedResource* DeformedResource = nullptr;

		// Data - Interpolation data (weights/Id/...) for transfering sim strands (i.e. guide) motion to render strands
		// Resources - Strands deformed position data for sim & render strands
		FHairStrandsInterpolationResource* InterpolationResource = nullptr;

		// Resources - Strands cluster data for culling/voxelization purpose
		FHairStrandsClusterResource* ClusterResource = nullptr;
		FHairStrandsCullingResource* CullingResource = nullptr;

		// Resources - Rest root data, for deforming strands attached to a skinned mesh surface
		// Resources - Deformed root data, for deforming strands attached to a skinned mesh surface
		FHairStrandsRestRootResource* RestRootResource = nullptr;
		FHairStrandsDeformedRootResource* DeformedRootResource = nullptr;

		// Resources - Raytracing data when enabling (expensive) raytracing method
		#if RHI_RAYTRACING
		FHairStrandsRaytracingResource* RenRaytracingResource = nullptr;
		FHairStrandsRaytracingResource** RenRaytracingResourceShared = nullptr;
		bool RenRaytracingResourceOwned = false;
		EHairViewRayTracingMask ViewRayTracingMask = EHairViewRayTracingMask::None;
		float CachedHairScaledRadius = 0;
		float CachedHairRootScale = 0;
		float CachedHairTipScale = 0;
		uint32 CachedProceduralSplits = 0;
		#endif

		FRDGExternalBuffer DebugCurveAttributeBuffer;
		FHairGroupInstanceModifer Modifier;

		FHairStrandsUniformBuffer UniformBuffer;
		FHairStrandsVertexFactory*                   VertexFactory = nullptr;
		FLinearSweptSpheresHairStrandsVertexFactory* LinearSweptSphereVertexFactory = nullptr;

		EHairInterpolationType HairInterpolationType = EHairInterpolationType::NoneSkinning;		
		bool bCullingEnable = false; // Indicates if culling is enabled for this hair strands data.
	} Strands;

	//////////////////////////////////////////////////////////////////////////////////////////
	// Cards
	struct FCards
	{
		bool IsValid() const { for (const FLOD& LOD: LODs) { if (LOD.IsValid()) { return true; } } return false; }
		bool IsValid(int32 LODIndex) const { return LODIndex >= 0 && LODIndex < LODs.Num() && LODs[LODIndex].RestResource != nullptr; }
		struct FLOD
		{
			bool IsValid() const { return RestResource != nullptr; }
			const FHairCardsBulkData& GetData() const { return RestResource->BulkData; }

			// Resources
			FHairCardsRestResource* RestResource = nullptr;
			FHairCardsDeformedResource* DeformedResource = nullptr;
			FHairCardsInterpolationResource* InterpolationResource = nullptr;
			#if RHI_RAYTRACING
			FHairStrandsRaytracingResource* RaytracingResource = nullptr;
			bool RaytracingResourceOwned = false;
			#endif

			struct FGuides
			{
				bool IsValid() const { return RestResource != nullptr; }
				bool HasValidRootData() const { return RestRootResource != nullptr && DeformedRootResource != nullptr; }
				const FHairStrandsBulkData& GetData() const { return RestResource->BulkData; }

				// Resources - Strands rest position data for sim & render strands
				// Resources - Strands deformed position data for sim & render strands
				FHairStrandsRestResource* RestResource = nullptr;
				FHairStrandsDeformedResource* DeformedResource = nullptr;
	
				// Resources - Rest root data, for deforming strands attached to a skinned mesh surface
				// Resources - Deformed root data, for deforming strands attached to a skinned mesh surface
				FHairStrandsRestRootResource* RestRootResource = nullptr;
				FHairStrandsDeformedRootResource* DeformedRootResource = nullptr;
	
				// Resources - Strands deformed position data for sim & render strands
				FHairStrandsInterpolationResource* InterpolationResource = nullptr;

				EHairInterpolationType HairInterpolationType = EHairInterpolationType::NoneSkinning;
			} Guides;

			FHairCardsUniformBuffer UniformBuffer;
			FHairCardsVertexFactory* VertexFactory = nullptr;
			FHairCardsVertexFactory* GetVertexFactory() const
			{
				return VertexFactory;
			}
			void InitVertexFactory();

		};
		TArray<FLOD> LODs;
	} Cards;

	//////////////////////////////////////////////////////////////////////////////////////////
	// Meshes
	struct FMeshes
	{
		bool IsValid() const { for (const FLOD& LOD : LODs) { if (LOD.IsValid()) { return true; } } return false; }
		bool IsValid(int32 LODIndex) const { return LODIndex >= 0 && LODIndex < LODs.Num() && LODs[LODIndex].RestResource != nullptr; }
		struct FLOD
		{
			bool IsValid() const { return RestResource != nullptr; }
			const FHairMeshesBulkData& GetData() const { return RestResource->BulkData; }

			// Resources
			FHairMeshesRestResource* RestResource = nullptr;
			FHairMeshesDeformedResource* DeformedResource = nullptr;
			#if RHI_RAYTRACING
			FHairStrandsRaytracingResource* RaytracingResource = nullptr;
			bool RaytracingResourceOwned = false;
			#endif

			FHairCardsUniformBuffer UniformBuffer;
			FHairCardsVertexFactory* VertexFactory = nullptr;
			FHairCardsVertexFactory* GetVertexFactory() const 
			{
				return VertexFactory;
			}
			void InitVertexFactory();
		};
		TArray<FLOD> LODs;
	} Meshes;
	
	//////////////////////////////////////////////////////////////////////////////////////////
	// Debug
	struct FDebug
	{
		// Data
		uint32					GroupIndex = ~0;
		uint32					GroupCount = 0;
		FString					GroomAssetName;
		uint32					LastFrameIndex = ~0;
		uint32					GroomAssetHash = 0;
		uint32					GroomBindingAssetHash = 0;

		float					LODPredictedIndex = -1.f;	// Computed on the rendering-thread, readback on the game-thread during tick()
		float					LODForcedIndex = -1.f;		// Set by the game-thread, read on the rendering-thread to force a particular LOD
		EHairLODSelectionType	LODSelectionTypeForDebug = EHairLODSelectionType::Immediate; // For debug only, not used

		int32					MeshLODIndex = ~0;
		EGroomBindingMeshType	GroomBindingType;
		EGroomCacheType			GroomCacheType;
		FPrimitiveSceneProxy*	Proxy = nullptr;
		FString					MeshComponentName;
		FPrimitiveComponentId	MeshComponentId;
		FPersistentPrimitiveIndex CachedMeshPersistentPrimitiveIndex;
		const UMeshComponent*	MeshComponentForDebug = nullptr;
		const UGroomComponent*	GroomComponentForDebug = nullptr; // For debug only, shouldn't be deferred on the rendering thread
		FTransform				RigidRestLocalToWorld = FTransform::Identity;
		FTransform				RigidCurrentLocalToWorld = FTransform::Identity;
		FTransform				SkinningRestLocalToWorld = FTransform::Identity;
		FTransform				SkinningCurrentLocalToWorld = FTransform::Identity;
		FTransform				RigidPreviousLocalToWorld = FTransform::Identity;
		FTransform				SkinningPreviousLocalToWorld = FTransform::Identity;

		TSharedPtr<class IGroomCacheBuffers, ESPMode::ThreadSafe> GroomCacheBuffers;

		// Resources
		FHairStrandsDebugResources* HairDebugResource = nullptr;
	} Debug;

	FTransform				LocalToWorld = FTransform::Identity;
	FHairGroupPublicData*	HairGroupPublicData = nullptr;
	EHairGeometryType		GeometryType = EHairGeometryType::NoneGeometry;
	EHairBindingType		BindingType = EHairBindingType::NoneBinding;
	bool					bForceCards = false;
	bool					bUpdatePositionOffset = false;
	bool 					bSupportStreaming = true;
	
	// Deformed component to extract the bone buffer 
	UMeshComponent* DeformedComponent = nullptr;
	
	// Section of the deformed component to be used 
	int32 DeformedSection = INDEX_NONE;
	
	bool IsValid() const 
	{
		return Meshes.IsValid() || Cards.IsValid() || Strands.IsValid();
	}

	virtual const FBoxSphereBounds& GetBounds() const override { return Debug.Proxy->GetBounds(); }
	virtual const FBoxSphereBounds& GetLocalBounds() const { return Debug.Proxy->GetLocalBounds(); }
	virtual const FHairGroupPublicData* GetHairData() const { return HairGroupPublicData; }
	virtual const EHairGeometryType GetHairGeometry() const { return GeometryType; }

	/** Get the rest local to world transform according to the internal binding type */
	inline const FTransform& GetRestLocalToWorld() const
	{
		return (BindingType == EHairBindingType::Skinning) ? Debug.SkinningRestLocalToWorld: 
															 Debug.RigidRestLocalToWorld;
	}

	/** Get the current local to world transform according to the internal binding type */
	inline const FTransform& GetCurrentLocalToWorld() const
	{
		return (BindingType == EHairBindingType::Skinning) ? Debug.SkinningCurrentLocalToWorld: 
															 Debug.RigidCurrentLocalToWorld;
	}

	/** Get the previous local to world transform according to the internal binding type */
	inline const FTransform& GetPreviousLocalToWorld() const
	{
		return (BindingType == EHairBindingType::Skinning) ? Debug.SkinningPreviousLocalToWorld :
															 Debug.RigidPreviousLocalToWorld;
	}

	UE_API FHairStrandsVertexFactoryUniformShaderParameters GetHairStandsUniformShaderParameters(EGroomViewMode ViewMode) const;
};

#undef UE_API
