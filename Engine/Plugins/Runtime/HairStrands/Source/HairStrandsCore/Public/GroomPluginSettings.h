// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "GroomPluginSettings.generated.h"

#define UE_API HAIRSTRANDSCORE_API

/** Settings for the groom plug-in */
UCLASS(MinimalAPI, config=Engine)
class UGroomPluginSettings : public UObject
{
	GENERATED_BODY()

public:
	 
	UE_API UGroomPluginSettings();

	/** The amount of groom cache animation (in seconds) to pre-load ahead of time by the streaming manager */
	UPROPERTY(config, EditAnywhere, Category=GroomCache, meta = (ClampMin = "0.05", ClampMax = "60.0", DisplayName = "Look-Ahead Buffer (in seconds)"))
	float GroomCacheLookAheadBuffer;
};

#undef UE_API
