// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API HAIRSTRANDSCORE_API

class UGroomAsset;
class UGroomBindingAsset;
struct FTextureSource;

struct FGroomRBFDeformer
{
	// Return a new GroomAsset with the RBF deformation from the BindingAsset baked into it
	//
	// If bShouldBuildStaticMeshes is true, this function will ensure the static meshes owned by 
	// the groom are built after being modified. In this case, GetRBFDeformedGroomAsset can only be
	// called from the Game Thread.
	//
	// If bShouldBuildStaticMeshes is false, GetRBFDeformedGroomAsset may be called from a 
	// background thread, but the caller must:
	//
	//	1.	Call ConditionalPostLoad and PreEditChange on each ImportedMesh returned by both 
	//		GetHairGroupsCards and GetHairGroupsMeshes *before* calling GetRBFDeformedGroomAsset.
	//
	//	2.	Call UStaticMesh::BatchBuild and MarkPackageDirty on the same ImportedMeshes *after* 
	//		calling GetRBFDeformedGroomAsset to begin async building of the meshes.
	//		
	//		To block until the async build is finished (if necessary), call 
	//		FStaticMeshCompilingManager::Get().FinishAllCompilation().
	UE_API void GetRBFDeformedGroomAsset(
		const UGroomAsset* InGroomAsset, 
		const UGroomBindingAsset* BindingAsset, 
		FTextureSource* MaskTextureSource, 
		const float MaskScale, 
		UGroomAsset* OutGroomAsset, 
		const ITargetPlatform* TargetPlatform = nullptr,
		bool bShouldBuildStaticMeshes = true);

	static UE_API uint32 GetEntryCount(uint32 InSampleCount);
	static UE_API uint32 GetWeightCount(uint32 InSampleCount);
};

#undef UE_API
