// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "HairDescription.h"
#include "HairStrandsDatas.h"
#include "HairCardsDatas.h"
#include "RenderResource.h"
#include "RayTracingGeometry.h"
#include "HairStrandsInterface.h"
#include "HairStrandsMeshProjection.h"

namespace UE::DerivedData { class FRequestOwner; }

DECLARE_RENDER_COMMAND_PIPE(Groom, HAIRSTRANDSCORE_API);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers

inline uint32 GetBufferTotalNumBytes(const FRDGExternalBuffer& In) 
{
	return In.Buffer ? In.Buffer->GetSize() : 0;
}

enum class EHairStrandsResourcesType : uint8
{
	Guides,		// Guides used for simulation
	Strands,	// Rendering strands 
	Cards		// Guides used for deforming the cards geometry (which is different from the simulation guides)
};

enum class EHairStrandsAllocationType : uint8
{
	Immediate,	// Resources are allocated immediately
	Deferred	// Resources allocation is deferred to first usage
};

/* Hair resource name - Allow more precise resource name for debug purpose. Enabled with HAIR_RESOURCE_DEBUG_NAME */
#define HAIR_RESOURCE_DEBUG_NAME 0
struct FHairResourceName
{
	FHairResourceName() = default;
#if HAIR_RESOURCE_DEBUG_NAME
	FHairResourceName(const FName& In) : AssetName(In) {}
	FHairResourceName(const FName& In, int32 InGroupIndex) : AssetName(In), GroupIndex(InGroupIndex) {}
	FHairResourceName(const FName& In, int32 InGroupIndex, int32 InLODIndex) : AssetName(In), GroupIndex(InGroupIndex), InLODIndex(InLODIndex) {}

	FName AssetName;
	int32 GroupIndex = -1;
	int32 LODIndex = -1;
	TArray<FString> Names;
#else
	FHairResourceName(const FName& In) {}
	FHairResourceName(const FName& In, int32 InGroupIndex) {}
	FHairResourceName(const FName& In, int32 InGroupIndex, int32 InLODIndex) {}
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Loading Type/Status

enum class EHairResourceLoadingType : uint8
{
	Async,
	Sync
};

struct EHairResourceStatus
{
	enum class EStatus : uint8
	{
		None = 0,
		Loading = 1,
		Valid = 2
	};
	void AddAvailableCurve(uint32 In) { AvailableCurveCount = FMath::Min(AvailableCurveCount, In); }
	bool HasStatus(EStatus In) const { return !!(static_cast<uint8>(Status) & static_cast<uint8>(In)); }
	EStatus Status = EStatus::None;
	uint32 AvailableCurveCount = 0;
};

inline EHairResourceStatus  operator& (EHairResourceStatus In,  EHairResourceStatus::EStatus InStatus) { EHairResourceStatus Out = In; Out.Status = static_cast<EHairResourceStatus::EStatus>(static_cast<uint8>(Out.Status) & static_cast<uint8>(InStatus)); return Out; }
inline EHairResourceStatus  operator| (EHairResourceStatus In,  EHairResourceStatus::EStatus InStatus) { EHairResourceStatus Out = In; Out.Status = static_cast<EHairResourceStatus::EStatus>(static_cast<uint8>(Out.Status) | static_cast<uint8>(InStatus)); return Out; }
inline EHairResourceStatus& operator|=(EHairResourceStatus&Out, EHairResourceStatus::EStatus InStatus) { return Out=Out|InStatus; }

inline bool operator! (EHairResourceStatus A) { return static_cast<uint8>(A.Status) != 0 && A.AvailableCurveCount > 0; }

EHairResourceLoadingType GetHairResourceLoadingType(EHairGeometryType InGeometryType, int32 InLODIndex);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Common resources

/* Hair resouces which whom allocation can be deferred */
struct FHairCommonResource : public FRenderResource
{
	/** Build the hair strands resource */
	FHairCommonResource(EHairStrandsAllocationType AllocationType, const FHairResourceName& InResourceName, const FName& InOwnerName, bool bUseRenderGraph=true);

	/* Init/Release buffers (FRenderResource) */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	/* Init/Release buffers (FHairCommonResource) */
	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType);
	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status);
	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, uint32 InRequestedCurveCount, uint32 InRequestedPointCount, bool bAllowDeallocation=true);

	void StreamInData();

	virtual void InternalAllocate() {}
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) {}
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount) { InternalAllocate(GraphBuilder); }
	virtual void InternalRelease() {}
	virtual bool InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount);
	virtual void InternalResetLoadedSize() { }
	virtual FHairStrandsBulkCommon* InternalGetBulkData() { return nullptr; }

	bool bUseRenderGraph = true;
	bool bIsInitialized = false;
	EHairStrandsAllocationType AllocationType = EHairStrandsAllocationType::Deferred;

	/* Store (precis) debug name */
	FHairResourceName ResourceName;
	FName OwnerName;

	/* Handle streaming request */
	FHairStreamingRequest StreamingRequest;
	uint32 MaxAvailableCurveCount = HAIR_MAX_NUM_CURVE_PER_GROUP;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Strands resources

/* Render buffers that will be used for rendering */
struct FHairStrandsRestResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairStrandsRestResource(FHairStrandsBulkData& InBulkData, EHairStrandsResourcesType CurveType, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) override;
	virtual void InternalRelease() override;
	virtual bool InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount) override;
	virtual void InternalResetLoadedSize() override;
	virtual FHairStrandsBulkCommon* InternalGetBulkData() override { return &BulkData; }

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsResource"); }

	FRDGExternalBuffer GetTangentBuffer(class FRDGBuilder& GraphBuilder, class FGlobalShaderMap* ShaderMap, uint32 PointCount, uint32 CurveCount);
	
	FVector GetPositionOffset() const { return BulkData.GetPositionOffset(); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(PositionBuffer);
		Total += GetBufferTotalNumBytes(PositionOffsetBuffer);
		Total += GetBufferTotalNumBytes(TangentBuffer);
		Total += GetBufferTotalNumBytes(PointAttributeBuffer);
		Total += GetBufferTotalNumBytes(CurveAttributeBuffer);
		Total += GetBufferTotalNumBytes(PointToCurveBuffer);
		Total += GetBufferTotalNumBytes(CurveBuffer);
		Total += GetBufferTotalNumBytes(CurveMappingBuffer);
		Total += GetBufferTotalNumBytes(PointMappingBuffer);
		return Total;
	}

	/* Strand hair rest position buffer */
	FRDGExternalBuffer  PositionBuffer;

	/* Strand hair rest offset position buffer */
	FRDGExternalBuffer  PositionOffsetBuffer;

	/* Strand hair tangent buffer (non-allocated unless used for static geometry) */
	FRDGExternalBuffer TangentBuffer;

	/* Strand hair per-point attribute buffer */
	FRDGExternalBuffer PointAttributeBuffer;

	/* Strand hair per-curve attribute buffer */
	FRDGExternalBuffer CurveAttributeBuffer;
	
	/* Strand hair per-curve source mapping buffer */
	FRDGExternalBuffer CurveMappingBuffer;

	/* Strand hair per-point source mapping buffer */
	FRDGExternalBuffer PointMappingBuffer;

	/* Strand hair vertex to curve index mapping */
	FRDGExternalBuffer PointToCurveBuffer;

	/* Strand hair curves buffer (contains curves' points offset and count) */
	FRDGExternalBuffer CurveBuffer;

	/* Reference to the hair strands render data */
	FHairStrandsBulkData& BulkData;

	/* Type of curves */
	const EHairStrandsResourcesType CurveType;

	/* Curve: Point offset & count */
	TArray<FPackedHairCurve> CurveData;

	inline uint32 GetPointCount() const { return BulkData.GetNumPoints(); }
	inline uint32 GetCurveCount() const { return BulkData.GetNumCurves(); }

	inline uint32 GetSourcePointCount() const { return BulkData.GetNumSourcePoints(); }
	inline uint32 GetSourceCurveCount() const { return BulkData.GetNumSourceCurves(); }

	uint32 CachedTangentPointCount = 0;

	/** List of external buffers */
	TMap<FString,FRDGExternalBuffer> ExternalBuffers;
};

struct FHairStrandsDeformedResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairStrandsDeformedResource(FHairStrandsBulkData& BulkData, EHairStrandsResourcesType CurveType, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsDeformedResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(DeformedPositionBuffer[0]);
		Total += GetBufferTotalNumBytes(DeformedPositionBuffer[1]);
		Total += GetBufferTotalNumBytes(TangentBuffer);
		Total += GetBufferTotalNumBytes(DeformerBuffer);
		return Total;
	}

	/* Strand hair deformed position buffer (previous and current) */
	FRDGExternalBuffer DeformedPositionBuffer[2];

	/* Strand hair deformed position buffer (previous and current) */
	FRDGExternalBuffer DeformedOffsetBuffer[2];

	/* Strand hair tangent buffer */
	FRDGExternalBuffer TangentBuffer;

	/* Strand hair deformer buffer. This buffer is optionally created & filled in by a mesh deformer */
	FRDGExternalBuffer DeformerBuffer;
	FRDGExternalBuffer DeformerPointAttributeBuffer;
	FRDGExternalBuffer DeformerCurveAttributeBuffer;

	/* Position offset as the deformed positions are expressed in relative coordinate (16bits) */
	FVector PositionOffset[2] = {FVector::ZeroVector, FVector::ZeroVector};

	/* Reference to the hair strands render data */
	FHairStrandsBulkData& BulkData;

	/* Whether the GPU data should be initialized with the asset data or not */
	uint32 CurrentIndex = 0;

	/* Track the view which has update the formed position. This is used when rendering the same 
		instance accross several editor viewport (not views of the same viewport), to prevent 
		incorrect motion vector, as editor viewport are refresh not on the same tick, but with a 
		throttling mechanism which make the updates not coherent. */
	uint32 UniqueViewIDs[2] = { 0, 0 };

	/* Type of curves */
	const EHairStrandsResourcesType CurveType;

	enum EFrameType
	{
		Previous,
		Current
	};

	// Helper accessors
	inline uint32 GetIndex(EFrameType T) const					{ return T == EFrameType::Current ? CurrentIndex : 1u - CurrentIndex; }
	inline FRDGExternalBuffer& GetBuffer(EFrameType T)			{ return DeformedPositionBuffer[GetIndex(T)];  }
	inline FVector& GetPositionOffset(EFrameType T)				{ return PositionOffset[GetIndex(T)]; }
	inline const FRDGExternalBuffer& GetPositionOffsetBuffer(EFrameType T) const { return DeformedOffsetBuffer[GetIndex(T)]; }
	inline FRDGExternalBuffer& GetPositionOffsetBuffer(EFrameType T) { return DeformedOffsetBuffer[GetIndex(T)]; }
	inline const FVector& GetPositionOffset(EFrameType T) const { return PositionOffset[GetIndex(T)]; }
	inline void SetPositionOffset(EFrameType T, const FVector& Offset)  { PositionOffset[GetIndex(T)] = Offset; }
	inline void SwapBuffer()									{ CurrentIndex = 1u - CurrentIndex; }
	inline uint32& GetUniqueViewID(EFrameType T)				{ return UniqueViewIDs[GetIndex(T)]; }
	inline uint32 GetUniqueViewID(EFrameType T) const			{ return UniqueViewIDs[GetIndex(T)]; }
	//bool NeedsToUpdateTangent();

	// Return deformer buffers
	HAIRSTRANDSCORE_API FRDGExternalBuffer& GetDeformerBuffer(FRDGBuilder& GraphBuilder);
	HAIRSTRANDSCORE_API FRDGExternalBuffer& GetDeformerCurveAttributeBuffer(FRDGBuilder& GraphBuilder);
	HAIRSTRANDSCORE_API FRDGExternalBuffer& GetDeformerPointAttributeBuffer(FRDGBuilder& GraphBuilder);

	/** List of external buffers */
	TMap<FString,FRDGExternalBuffer> ExternalBuffers;
};


struct FHairStrandsInterpolationResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairStrandsInterpolationResource(FHairStrandsInterpolationBulkData& InBulkData, const FHairResourceName& ResourceName, const FName& OwnerName);
	
	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) override;
	virtual void InternalRelease() override;
	virtual bool InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount) override;
	virtual void InternalResetLoadedSize() override;
	virtual FHairStrandsBulkCommon* InternalGetBulkData() override { return &BulkData; }

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsInterpolationResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(CurveInterpolationBuffer);
		Total += GetBufferTotalNumBytes(PointInterpolationBuffer);
		return Total;
	}

	bool UseSingleGuide() const { return (BulkData.Header.Flags & FHairStrandsInterpolationBulkData::DataFlags_HasSingleGuideData) != 0; }

	FRDGExternalBuffer CurveInterpolationBuffer;
	FRDGExternalBuffer PointInterpolationBuffer;

	/* Reference to the hair strands interpolation render data */
	FHairStrandsInterpolationBulkData& BulkData;
};


struct FHairStrandsClusterResource : public FHairCommonResource
{
	FHairStrandsClusterResource(FHairStrandsClusterBulkData& Data, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) override;
	virtual void InternalRelease() override;
	virtual bool InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount) override;
	virtual void InternalResetLoadedSize() override;
	virtual FHairStrandsBulkCommon* InternalGetBulkData() override { return &BulkData; }

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsClusterResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(ClusterInfoBuffer);
		Total += GetBufferTotalNumBytes(CurveToClusterIdBuffer);
		Total += GetBufferTotalNumBytes(PointLODBuffer);
		return Total;
	}

	/* Cluster info buffer */
	FRDGExternalBuffer ClusterInfoBuffer;

	/* CurveId => ClusterId to know which AABB to contribute to*/
	FRDGExternalBuffer CurveToClusterIdBuffer;

	/* Min. LOD a which a Point becomes visible */
	FRDGExternalBuffer PointLODBuffer;

	FHairStrandsClusterBulkData& BulkData;
};

struct FHairStrandsCullingResource : public FHairCommonResource
{
	FHairStrandsCullingResource(uint32 InPointCount, uint32 InCurveCount, uint32 InClusterCount, const FHairResourceName& InResourceName, const FName& InOwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsCullingResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(Resources.DrawIndirectBuffer);
		Total += GetBufferTotalNumBytes(Resources.DrawIndirectRasterComputeBuffer);
		Total += GetBufferTotalNumBytes(Resources.CulledCurveBuffer);
		Total += GetBufferTotalNumBytes(Resources.CulledVertexIdBuffer);
		return Total;
	}

	uint32 ClusterCount = 0;
	uint32 MaxPointCount = 0;
	uint32 MaxCurveCount = 0;
	FHairGroupPublicData::FCulling Resources;
};

/* Render buffers for root deformation for dynamic meshes */
struct FHairStrandsLODRestRootResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairStrandsLODRestRootResource(TArray<FHairStrandsRootBulkData>& InBulkDataArray, int32 InBulkDataIndex, EHairStrandsResourcesType CurveType, uint32 InMeshLODIndex, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount) override;
	virtual void InternalRelease() override;
	virtual bool InternalGetOrRequestData(uint32 InRequestedCurveCount, uint32 InRequestedPointCount) override;
	virtual void InternalResetLoadedSize() override;
	virtual FHairStrandsBulkCommon* InternalGetBulkData() override { return &GetBulkData(); }

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsLODRestRootResource"); }

	/* Populate GPU LOD data from RootData (this function doesn't initialize resources) */
	void Reset();

	// Accessors
	uint32 GetRootCount()const { return GetBulkData().GetRootCount(); }

	/* Accessor to deference bulk data*/
	inline bool IsBulkDataValid() const { return BulkDataArray.IsValidIndex(BulkDataIndex); }

	inline const FHairStrandsRootBulkData& GetBulkData() const
	{
		check(BulkDataArray.IsValidIndex(BulkDataIndex));
		return BulkDataArray[BulkDataIndex];
	}

	inline FHairStrandsRootBulkData& GetBulkData()
	{
		check(BulkDataArray.IsValidIndex(BulkDataIndex));
		return BulkDataArray[BulkDataIndex];
	}

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const 
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(UniqueTriangleIndexBuffer);
		Total += GetBufferTotalNumBytes(RootToUniqueTriangleIndexBuffer);
		Total += GetBufferTotalNumBytes(RootBarycentricBuffer);
		Total += GetBufferTotalNumBytes(RestUniqueTrianglePositionBuffer);
		Total += GetBufferTotalNumBytes(MeshInterpolationWeightsBuffer);
		Total += GetBufferTotalNumBytes(MeshSampleIndicesAndSectionsBuffer);
		Total += GetBufferTotalNumBytes(RestSamplePositionsBuffer);
		return Total;
	}

	enum class EStatus { Invalid, Initialized, Completed };

	const bool IsDataValid() const { return GetBulkData().Header.RootCount > 0 && GetBulkData().Header.LODIndex >= 0; }
	const bool IsValid() const { return Status == EStatus::Completed; }
	EStatus Status = EStatus::Invalid;
	uint32 AvailableCurveCount = 0;

	/* Triangle on which a root is attached */
	/* When the projection is done with source to target mesh transfer, the projection indices does not match.
	In this case we need to separate index computation. The barycentric coords remain the same however. */
	FRDGExternalBuffer UniqueTriangleIndexBuffer;
	/* Strands hair root to unique triangle index */
	FRDGExternalBuffer RootToUniqueTriangleIndexBuffer;
	FRDGExternalBuffer RootBarycentricBuffer;

	/* Strand hair roots translation and rotation in rest position relative to the bound triangle. Positions are relative to the rest root center */
	FRDGExternalBuffer RestUniqueTrianglePositionBuffer;

	/* Strand hair mesh interpolation matrix and sample indices */
	uint32 SampleCount = 0;
	FRDGExternalBuffer MeshInterpolationWeightsBuffer;
	FRDGExternalBuffer MeshSampleIndicesAndSectionsBuffer;
	FRDGExternalBuffer RestSamplePositionsBuffer;

	/* Store CPU data for root info & root binding */
	TArray<FHairStrandsRootBulkData>& BulkDataArray;
	int32 BulkDataIndex = -1;

	/* Type of curves */
	const EHairStrandsResourcesType CurveType;

	/* Mesh LOD index for this root resource. Used for sanity check */
	const uint32 MeshLODIndex = 0;
};

/* Render buffers for root deformation for dynamic meshes */
struct FHairStrandsRestRootResource
{
	/** Build the hair strands resource */
	FHairStrandsRestRootResource(TArray<FHairStrandsRootBulkData>& BulkData, EHairStrandsResourcesType CurveType, const FHairResourceName& ResourceName, const FName& OwnerName);

	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, int32 InMeshLODIndex);
	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, uint32 InRequestedCurveCount, uint32 InRequestedPointCount, bool bAllowDeallocation/*=true*/, int32 InMeshLODIndex);
	void StreamInData(int32 MeshLODIndex);

	/* Forward call to each mesh LOD version */
	void BeginInitResource();
	void BeginUpdateResourceRHI();
	void ReleaseResource();
	void InternalResetLoadedSize();

	// Accessors
	bool IsDataValid(int32 InMeshLODIndex) const { return LODs.IsValidIndex(InMeshLODIndex) && LODs[InMeshLODIndex]->IsDataValid(); }
	bool IsValid(int32 InMeshLODIndex) const { return LODs.IsValidIndex(InMeshLODIndex) && LODs[InMeshLODIndex]->IsValid(); }
	FHairStrandsLODRestRootResource* GetLOD(int32 InMeshLODIndex) { check(LODs.IsValidIndex(InMeshLODIndex)); return LODs[InMeshLODIndex]; }
	const FHairStrandsLODRestRootResource* GetLOD(int32 InMeshLODIndex) const { check(LODs.IsValidIndex(InMeshLODIndex)); return LODs[InMeshLODIndex]; }

	uint32 GetLODCount() const { return LODs.Num(); }
	uint32 GetRootCount() const
	{ 
		// Find the first valid root count and return that
		for (const FHairStrandsLODRestRootResource* LOD : LODs)
		{
			if (LOD)
			{
				const uint32 RootCount = LOD->GetRootCount();
				if (RootCount > 0)
				{
					return RootCount;
				}
			}
		}

		return 0u;
	}

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const 
	{
		uint32 Total = 0;
		for (const FHairStrandsLODRestRootResource* LOD : LODs)
		{
			Total += LOD->GetResourcesSize();
		}
		return Total;
	}

	/* Store the hair projection information for each mesh LOD */
	TArray<FHairStrandsLODRestRootResource*> LODs;

	/* Type of curves */
	const EHairStrandsResourcesType CurveType;
};

/* Render buffers for root deformation for dynamic meshes */
struct FHairStrandsLODDeformedRootResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairStrandsLODDeformedRootResource (FHairStrandsRestRootResource* InRestResources, EHairStrandsResourcesType CurveType, const uint32 MeshLODIndex, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsLODDeformedRootResource "); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(DeformedUniqueTrianglePositionBuffer[0]);
		Total += GetBufferTotalNumBytes(DeformedSamplePositionsBuffer[0]);
		Total += GetBufferTotalNumBytes(MeshSampleWeightsBuffer[0]);

		// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
		if (IsHairStrandContinuousDecimationReorderingEnabled())
		{
			Total += GetBufferTotalNumBytes(DeformedUniqueTrianglePositionBuffer[1]);
			Total += GetBufferTotalNumBytes(DeformedSamplePositionsBuffer[1]);
			Total += GetBufferTotalNumBytes(MeshSampleWeightsBuffer[1]);
		}
		return Total;
	}
	
	enum class EStatus { Invalid, Initialized, Completed };

	// A LOD is considered valid as long as its resources are initialized. 
	// Its state will become completed once its triangle position will be 
	// update, but in order to be update its status needs to be valid.
	bool IsDataValid() const { return RootCount > 0 && MeshLODIndex >= 0; }
	bool IsValid() const { return Status == EStatus::Initialized || Status == EStatus::Completed; }
	EStatus Status = EStatus::Invalid;
	uint32 AvailableCurveCount = 0;

	/* Strand hair roots translation and rotation in triangle-deformed position relative to the bound triangle. Positions are relative the deformed root center*/
	FRDGExternalBuffer DeformedUniqueTrianglePositionBuffer[2];

	/* Strand hair mesh interpolation matrix and sample indices */
	uint32 SampleCount = 0;
	FRDGExternalBuffer DeformedSamplePositionsBuffer[2];
	FRDGExternalBuffer MeshSampleWeightsBuffer[2];

	/* Whether the GPU data should be initialized with the asset data or not */
	uint32 CurrentIndex = 0;

	enum EFrameType
	{
		Previous,
		Current
	};

	// Double buffering is disabled by default unless the read-only cvar r.HairStrands.ContinuousDecimationReordering is set
	inline uint32 GetIndex(EFrameType T) const { return T == EFrameType::Current ? CurrentIndex : 1u - CurrentIndex; }
	inline const FRDGExternalBuffer& GetDeformedUniqueTrianglePositionBuffer(EFrameType T) const { return IsHairStrandContinuousDecimationReorderingEnabled() ? DeformedUniqueTrianglePositionBuffer[GetIndex(T)] : DeformedUniqueTrianglePositionBuffer[0]; }
	inline const FRDGExternalBuffer& GetDeformedSamplePositionsBuffer(EFrameType T) const { return IsHairStrandContinuousDecimationReorderingEnabled() ? DeformedSamplePositionsBuffer[GetIndex(T)] : DeformedSamplePositionsBuffer[0]; }
	inline const FRDGExternalBuffer& GetMeshSampleWeightsBuffer(EFrameType T) const { return IsHairStrandContinuousDecimationReorderingEnabled() ? MeshSampleWeightsBuffer[GetIndex(T)] : MeshSampleWeightsBuffer[0]; }
	inline void SwapBuffer() { CurrentIndex = 1u - CurrentIndex; }


	/* Store the hair projection information for each mesh LOD */
	uint32 RootCount = 0;

	/* Mesh index corresponding for this root data. Used for sanity check. */
	const int32 MeshLODIndex = -1;

	/* Type of curves */
	const EHairStrandsResourcesType CurveType;
};

/* Render buffers for root deformation for dynamic meshes */
struct FHairStrandsDeformedRootResource
{
	/** Build the hair strands resource */
	FHairStrandsDeformedRootResource(FHairStrandsRestRootResource* InRestResources, EHairStrandsResourcesType CurveType, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, int32 InMeshLODIndex);
	void Allocate(FRDGBuilder& GraphBuilder, EHairResourceLoadingType LoadingType, EHairResourceStatus& Status, uint32 InRequestedCurveCount, uint32 InRequestedPointCount, bool bAllowDeallocation, int32 InMeshLODIndex);
	void ReleaseResource();
	//void InternalRelease();
	void SwapBuffer();

	/* Indirect if the current root resources are valid and up to date */
	bool IsDataValid(int32 InMeshLODIndex) const { return LODs.IsValidIndex(InMeshLODIndex) && LODs[InMeshLODIndex]->IsDataValid(); }
	bool IsValid(int32 InMeshLODIndex) const { return LODs.IsValidIndex(InMeshLODIndex) && LODs[InMeshLODIndex]->IsValid(); }
	FHairStrandsLODDeformedRootResource* GetLOD(int32 InMeshLODIndex) { check(LODs.IsValidIndex(InMeshLODIndex)); return LODs[InMeshLODIndex]; }
	const FHairStrandsLODDeformedRootResource* GetLOD(int32 InMeshLODIndex) const { check(LODs.IsValidIndex(InMeshLODIndex)); return LODs[InMeshLODIndex]; }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		for (const FHairStrandsLODDeformedRootResource* LOD : LODs)
		{
			Total += LOD->GetResourcesSize();
		}
		return Total;
	}

	/* Store the hair projection information for each mesh LOD */
	TArray<FHairStrandsLODDeformedRootResource*> LODs;

	/* Type of curves */
	const EHairStrandsResourcesType CurveType;
};

#if RHI_RAYTRACING
struct FHairStrandsRaytracingResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairStrandsRaytracingResource(const FHairStrandsBulkData& InData, const FHairResourceName& ResourceName, const FName& OwnerName);
	FHairStrandsRaytracingResource(const FHairCardsBulkData& InData, const FHairResourceName& ResourceName, const FName& OwnerName);
	FHairStrandsRaytracingResource(const FHairMeshesBulkData& InData, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/Release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder, uint32 InCurveCount, uint32 InPointCount) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairStrandsRaytracingResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(PositionBuffer);
		Total += GetBufferTotalNumBytes(IndexBuffer);
		return Total;
	}

	FRDGExternalBuffer PositionBuffer;
	FRDGExternalBuffer IndexBuffer;
	FRayTracingGeometry RayTracingGeometry;
	uint32 MaxVertexCount = 0;
	uint32 MaxIndexCount = 0;
	bool bOwnBuffers = false;
	bool bIsRTGeometryInitialized = false;
	EHairStrandsRayTracingPrimitiveType RayTracingPrimitiveType = EHairStrandsRayTracingPrimitiveType::Triangle;
};
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Cards

class FHairCardsVertexBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override {}
};

class FHairCardIndexBuffer : public FIndexBuffer
{
public:
	const TArray<FHairCardsIndexFormat::Type>& Indices;
	FHairCardIndexBuffer(const TArray<FHairCardsIndexFormat::Type>& InIndices, const FName& InOwnerName);
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/* Render buffers that will be used for rendering */
struct FHairCardsRestResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairCardsRestResource(const FHairCardsBulkData& InBulkData, const FHairResourceName& ResourceName, const FName& OwnerName);
	
	virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseResource() override;

	/* Init/release buffers */
	virtual void InternalAllocate() override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairCardsResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += BulkData.Positions.GetAllocatedSize();
		Total += BulkData.Normals.GetAllocatedSize();
		Total += BulkData.Indices.GetAllocatedSize();
		Total += BulkData.UVs.GetAllocatedSize();
		Total += BulkData.Materials.GetAllocatedSize();
		return Total;
	}

	uint32 GetVertexCount() const { return BulkData.GetNumVertices();  }
	uint32 GetPrimitiveCount() const { return BulkData.GetNumTriangles(); }

	/* Strand hair rest position buffer */
	FHairCardsVertexBuffer RestPositionBuffer;
	FHairCardIndexBuffer RestIndexBuffer;
	bool bInvertUV = false;

	FHairCardsVertexBuffer NormalsBuffer;
	FHairCardsVertexBuffer UVsBuffer;
	FHairCardsVertexBuffer MaterialsBuffer;
	FHairCardsVertexBuffer VertexColorsBuffer;

	TArray<FSamplerStateRHIRef> Samplers;
	TArray<FTextureReferenceRHIRef> Textures;
	uint32 LayoutIndex = 0;

	/* Reference to the hair strands render data */
	const FHairCardsBulkData& BulkData;
};

struct FHairCardsDeformedResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairCardsDeformedResource(const FHairCardsBulkData& BulkData, bool bInitializeData, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairCardsDeformedResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(DeformedPositionBuffer[0]);
		Total += GetBufferTotalNumBytes(DeformedPositionBuffer[1]);
		return Total;
	}

	/* Strand hair deformed position buffer (previous and current) */
	FRDGExternalBuffer DeformedPositionBuffer[2];
	FRDGExternalBuffer DeformedNormalBuffer;

	/* Reference to the hair strands render data */
	const FHairCardsBulkData& BulkData;

	/* Whether the GPU data should be initialized with the asset data or not */
	const bool bInitializedData = false;

	enum EFrameType
	{
		Previous,
		Current
	};

	// Helper accessors
	inline uint32 GetIndex(EFrameType T)				{ return T == EFrameType::Current ? 0u : 1u; }
	inline FRDGExternalBuffer& GetBuffer(EFrameType T)	{ return DeformedPositionBuffer[GetIndex(T)];  }
};

struct FHairCardsInterpolationResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairCardsInterpolationResource(FHairCardsInterpolationBulkData& InBulkData, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairCardsInterplationResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(InterpolationBuffer);
		return Total;
	}

	FRDGExternalBuffer InterpolationBuffer;

	/* Reference to the hair strands interpolation render data */
	FHairCardsInterpolationBulkData& BulkData;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Meshes

/* Render buffers that will be used for rendering */
struct FHairMeshesRestResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairMeshesRestResource(const FHairMeshesBulkData& BulkData, const FHairResourceName& ResourceName, const FName& OwnerName);

	virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseResource() override;

	/* Init/release buffers */
	virtual void InternalAllocate() override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairMeshesRestResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += BulkData.Positions.GetAllocatedSize();
		Total += BulkData.Normals.GetAllocatedSize();
		Total += BulkData.Indices.GetAllocatedSize();
		Total += BulkData.UVs.GetAllocatedSize();
		Total += BulkData.VertexColors.GetAllocatedSize();
		return Total;
	}

	uint32 GetVertexCount() const { return BulkData.GetNumVertices(); }
	uint32 GetPrimitiveCount() const { return BulkData.GetNumTriangles(); }

	/* Strand hair rest position buffer */
	FHairCardsVertexBuffer RestPositionBuffer;
	FHairCardIndexBuffer IndexBuffer;	
	bool bInvertUV = false;

	FHairCardsVertexBuffer NormalsBuffer;
	FHairCardsVertexBuffer UVsBuffer;
	FHairCardsVertexBuffer VertexColorsBuffer;

	TArray<FSamplerStateRHIRef> Samplers;
	TArray<FTextureReferenceRHIRef>	Textures;
	uint32 LayoutIndex = 0;

	/* Reference to the hair strands render data */
	const FHairMeshesBulkData& BulkData;
};


/* Render buffers that will be used for rendering */
struct FHairMeshesDeformedResource : public FHairCommonResource
{
	/** Build the hair strands resource */
	FHairMeshesDeformedResource(const FHairMeshesBulkData& InBulkData, bool bInInitializedData, const FHairResourceName& ResourceName, const FName& OwnerName);

	/* Init/release buffers */
	virtual void InternalAllocate(FRDGBuilder& GraphBuilder) override;
	virtual void InternalRelease() override;

	/* Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FHairMeshesDeformedResource"); }

	/* Return the memory size for GPU resources */
	uint32 GetResourcesSize() const
	{
		uint32 Total = 0;
		Total += GetBufferTotalNumBytes(DeformedPositionBuffer[0]);
		Total += GetBufferTotalNumBytes(DeformedPositionBuffer[1]);
		return Total;
	}

	/* Strand hair deformed position buffer (previous and current) */
	FRDGExternalBuffer DeformedPositionBuffer[2];

	/* Reference to the hair strands render data */
	const FHairMeshesBulkData& BulkData;

	/* Whether the GPU data should be initialized with the asset data or not */
	const bool bInitializedData = false;

	/* Whether the GPU data should be initialized with the asset data or not */
	uint32 CurrentIndex = 0;

	enum EFrameType
	{
		Previous,
		Current
	};

	// Helper accessors
	inline uint32 GetIndex(EFrameType T) { return T == EFrameType::Current ? CurrentIndex : 1u - CurrentIndex; }
	inline FRDGExternalBuffer& GetBuffer(EFrameType T) { return DeformedPositionBuffer[GetIndex(T)]; }
	inline void SwapBuffer() { CurrentIndex = 1u - CurrentIndex; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Debug data (used for debug visalization but also for texture generation)
void CreateHairStrandsDebugDatas(const FHairStrandsDatas& InData, FHairStrandsDebugDatas& Out);
void CreateHairStrandsDebugResources(class FRDGBuilder& GraphBuilder, const FHairStrandsDebugDatas* In, FHairStrandsDebugResources* Out);
