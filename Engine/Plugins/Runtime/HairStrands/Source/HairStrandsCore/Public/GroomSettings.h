// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "GroomSettings.generated.h"

USTRUCT(BlueprintType)
struct FGroomConversionSettings
{
	GENERATED_USTRUCT_BODY()

		FGroomConversionSettings()
		: Rotation(FVector::ZeroVector)
		, Scale(FVector(1.0f, 1.0f, 1.0f))
	{}

	/** Rotation in Euler angles in degrees to fix up or front axes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	FVector Rotation;

	/** Scale value to convert file unit into centimeters */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Conversion)
	FVector Scale;
};

UENUM()
enum class EGroomInterpolationQuality : uint8
{
	Low		UMETA(DisplayName = "Low",		ToolTip = "Build interpolation data based on nearst neighbor search. Low quality interpolation data, but fast to build (takes a few minutes)"),
	Medium	UMETA(DisplayName = "Medium",	ToolTip = "Build interpolation data using curve shape matching search but within a limited spatial range. This is a tradeoff between Low and high quality in term of quality & build time (can takes several dozen of minutes)"),
	High	UMETA(DisplayName = "High",		ToolTip = "Build interpolation data using curve shape matching search. This result in high quality interpolation data, but is relatively slow to build (can takes several dozen of minutes)"),
	Unknown	UMETA(Hidden),
};

UENUM()
enum class EGroomInterpolationWeight : uint8
{
	Parametric	UMETA(DisplayName = "Parametric", ToolTip = "Build interpolation data based on curve parametric distance"),
	Root		UMETA(DisplayName = "Root", ToolTip = "Build interpolation data based on distance between guide's root and strands's root"),
	Index		UMETA(DisplayName = "Index", ToolTip = "Build interpolation data based on guide and strands vertex indices"),
	Unknown		UMETA(Hidden),
};

USTRUCT(BlueprintType)
struct FGroomBuildSettings
{
	GENERATED_USTRUCT_BODY()

	FGroomBuildSettings()
		: bOverrideGuides(false)
		, HairToGuideDensity(0.1f)
		, InterpolationQuality(EGroomInterpolationQuality::High)
		, InterpolationDistance(EGroomInterpolationWeight::Parametric)
		, bRandomizeGuide(false)
		, bUseUniqueGuide(false)
	{}

	/** Flag to override the imported guides with generated guides. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BuildSettings", meta = (ToolTip = "If checked, override imported guides with generated ones."))
	bool bOverrideGuides;

	/** Density factor for converting hair into guide curve if no guides are provided. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BuildSettings", meta = (ClampMin = "0", UIMin = "0", UIMax = "1.0"))
	float HairToGuideDensity;

	/** Interpolation data quality. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BuildSettings")
	EGroomInterpolationQuality InterpolationQuality;

	/** Interpolation distance metric. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BuildSettings")
	EGroomInterpolationWeight InterpolationDistance;

	/** Randomize which guides affect a given hair strand. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BuildSettings")
	bool bRandomizeGuide;

	/** Force a hair strand to be affected by a unique guide. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "BuildSettings")
	bool bUseUniqueGuide;

	friend FArchive& operator<<(FArchive& Ar, FGroomBuildSettings& GroupInfo);
};
