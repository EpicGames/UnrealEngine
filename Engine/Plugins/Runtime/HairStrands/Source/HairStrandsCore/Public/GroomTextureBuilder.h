// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GroomSettings.h"
#include "HairStrandsInterface.h"
#include "GroomAssetCards.h"

#define UE_API HAIRSTRANDSCORE_API

class UTexture2D;
class UGroomAsset;
class USkeletalMesh;
class UStaticMesh;
class FRHIBuffer;
class FRHIShaderResourceView;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Follicle texture generation

struct FFollicleInfo
{
	enum EChannel { R = 0, G = 1, B = 2, A = 3 };

	UGroomAsset*GroomAsset = nullptr;
	EChannel	Channel = R;
	uint32		KernelSizeInPixels = 0;
	bool		bGPUOnly = false; // Indirect of the texture should be saved on CPU, or if it will only be used directly on GPU
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair strands texture generation on meshes surface

struct FStrandsTexturesInfo
{
	EHairTextureLayout Layout = EHairTextureLayout::Layout1;
	UGroomAsset* GroomAsset = nullptr;
	USkeletalMesh* SkeletalMesh = nullptr;
	UStaticMesh* StaticMesh = nullptr;
	uint32 Resolution = 2048;
	uint32 LODIndex = 0;
	uint32 SectionIndex = 0;
	uint32 UVChannelIndex = 0;
	float MaxTracingDistance = 1;
	int32 TracingDirection = 1;
	uint32 Dilation = 0;
	TArray<int32> GroupIndices;
};

struct FStrandsTexturesOutput
{
	TArray<UTexture2D*> Textures;
	bool IsValid() const { return Textures.Num() > 0; }
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FGroomTextureBuilder
{
	// Follicle texture
#if WITH_EDITOR
	static UE_API UTexture2D* CreateGroomFollicleMaskTexture(const UGroomAsset* GroomAsset, uint32 Resolution);
#endif
	static UE_API void AllocateFollicleTextureResources(UTexture2D* OuTexture);
	static UE_API void AllocateFollicleTextureResources(UTexture2D* OuTexture, const FIntPoint& Resolution, uint32 MipCount);
	static UE_API void BuildFollicleTexture(const TArray<FFollicleInfo>& InInfos, UTexture2D* OutFollicleTexture, bool bUseGPU);

	// Strands textures
#if WITH_EDITOR
	static UE_API FStrandsTexturesOutput CreateGroomStrandsTexturesTexture(const UGroomAsset* GroomAsset, uint32 Resolution, EHairTextureLayout InLayout);
	static UE_API void BuildStrandsTextures(const FStrandsTexturesInfo& InInfo, const FStrandsTexturesOutput& Output);
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
