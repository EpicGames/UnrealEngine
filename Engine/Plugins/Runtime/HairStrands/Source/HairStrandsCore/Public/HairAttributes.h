// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace HairAttribute
{
	namespace Vertex
	{
		extern HAIRSTRANDSCORE_API const FName Color;			// FVector
		extern HAIRSTRANDSCORE_API const FName Roughness;		// float
		extern HAIRSTRANDSCORE_API const FName AO;				// float
		extern HAIRSTRANDSCORE_API const FName Position;		// FVector
		extern HAIRSTRANDSCORE_API const FName Width;			// float
	}

	namespace Strand
	{
		extern HAIRSTRANDSCORE_API const FName Color;			// FVector
		extern HAIRSTRANDSCORE_API const FName Roughness;		// float
		extern HAIRSTRANDSCORE_API const FName GroupID;			// int
		extern HAIRSTRANDSCORE_API const FName Guide;			// int
		extern HAIRSTRANDSCORE_API const FName ID;				// int
		extern HAIRSTRANDSCORE_API const FName ClumpID;			// int
		extern HAIRSTRANDSCORE_API const FName RootUV;			// FVector2f
		extern HAIRSTRANDSCORE_API const FName VertexCount;		// int
		extern HAIRSTRANDSCORE_API const FName Width;			// float
		extern HAIRSTRANDSCORE_API const FName ClosestGuides;	// FVector
		extern HAIRSTRANDSCORE_API const FName GuideWeights;	// FVector
		extern HAIRSTRANDSCORE_API const FName BasisType;		// FName (EGroomBasisType)
		extern HAIRSTRANDSCORE_API const FName CurveType;		// FName (EGroomCurveType)
		extern HAIRSTRANDSCORE_API const FName Knots;			// float[]
		extern HAIRSTRANDSCORE_API const FName GroupName;		// FName
		extern HAIRSTRANDSCORE_API const FName GroupCardsName;	// FName
	}

	namespace Groom
	{
		extern HAIRSTRANDSCORE_API const FName Color;			// FVector
		extern HAIRSTRANDSCORE_API const FName Roughness;		// float
		extern HAIRSTRANDSCORE_API const FName MajorVersion;	// int
		extern HAIRSTRANDSCORE_API const FName MinorVersion;	// int
		extern HAIRSTRANDSCORE_API const FName Tool;			// FName
		extern HAIRSTRANDSCORE_API const FName Properties;		// FName
		extern HAIRSTRANDSCORE_API const FName Width;			// float
	}
}

enum class EHairAttribute : uint8
{
	RootUV,
	ClumpID,
	StrandID,
	PrecomputedGuideWeights,
	Color,
	Roughness,
	AO,
	Width,
	Count
};
HAIRSTRANDSCORE_API bool HasHairAttribute(uint32 In, EHairAttribute InAttribute);
HAIRSTRANDSCORE_API void SetHairAttribute(uint32& Out, EHairAttribute InAttribute);

enum class EHairAttributeFlags : uint8
{
	HasRootUDIM,
	HasMultipleClumpIDs
};
HAIRSTRANDSCORE_API bool HasHairAttributeFlags(uint32 In, EHairAttributeFlags InFlag);
HAIRSTRANDSCORE_API void SetHairAttributeFlags(uint32& Out, EHairAttributeFlags InFlag);
