// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/MathFwd.h"


struct FHairCardsBulkData;
struct FHairCardsDatas;
struct FHairCardsInterpolationBulkData;
struct FHairCardsRestResource;
struct FHairGroupCardsTextures;
struct FHairMeshesBulkData;
struct FHairStrandsDatas;
struct FHairStrandsVoxelData;

class FString;
class UStaticMesh;

#if WITH_EDITOR
namespace FHairCardsBuilder
{
	bool ImportGeometry(
		const UStaticMesh* StaticMesh,
		const FHairStrandsDatas& InGuidesData,
		const FHairStrandsDatas& InStrandsData,
		const FHairStrandsVoxelData& InStrandsVoxelData,
		const bool bGenerateGuidesFromCardGeometry,
		FHairCardsBulkData& OutBulk,
		FHairStrandsDatas& OutCardGuides,
		FHairCardsInterpolationBulkData& OutInterpolationBulkData);

	HAIRSTRANDSCORE_API bool ExtractCardsData(
		const UStaticMesh* StaticMesh, 
		const FHairStrandsDatas& InStrandsData,
		FHairCardsDatas& Out);

	FString GetVersion();
}

namespace FHairMeshesBuilder
{
	void BuildGeometry(
		const FBox& InBox,
		FHairMeshesBulkData& OutBulk);

	void ImportGeometry(
		const UStaticMesh* StaticMesh,
		FHairMeshesBulkData& OutBulk);

	FString GetVersion();
}
#endif // WITH_EDITOR
