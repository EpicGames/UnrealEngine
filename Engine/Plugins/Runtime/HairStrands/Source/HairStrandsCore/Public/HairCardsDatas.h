// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "RHIDefinitions.h"
#include "PackedNormal.h"
#include "HairStrandsVertexFactory.h"

#define UE_API HAIRSTRANDSCORE_API

////////////////////////////////////////////////////////////////////////////////////////////////
// Formats

struct FHairCardsPositionFormat
{
	typedef FVector4f Type;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float4;
	static const EPixelFormat Format = PF_A32B32G32R32F;
};

struct FHairCardsUVFormat
{
	// Store atlas UV and (approximated) root UV
	typedef FVector4f Type;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float4;
	static const EPixelFormat Format = PF_A32B32G32R32F; // TODO
};

struct FHairCardsMaterialFormat
{
	typedef uint32 Type;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UByte4N; 
	static const EPixelFormat Format = PF_R8G8B8A8;
};

struct FHairCardsNormalFormat
{
	typedef FPackedNormal Type;

	// TangentX & tangentZ are packed into 2 * PF_R8G8B8A8_SNORM
	static const uint32 ComponentCount = 2;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_PackedNormal;
	static const EPixelFormat Format = PF_R8G8B8A8_SNORM;
};

struct FHairCardsIndexFormat
{
	typedef uint32 Type;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairCardsInterpolationVertex
{
	uint32 VertexIndex : 24;
	uint32 VertexLerp  : 8;
};

FArchive& operator<<(FArchive& Ar, FHairCardsInterpolationVertex& Vertex);

struct FHairCardsInterpolationFormat
{
	typedef FHairCardsInterpolationVertex Type;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

////////////////////////////////////////////////////////////////////////////////////////////////
// Hair Cards Data/Bulk

// Data structure holding cards geometry information
struct FHairCardsGeometry
{
	struct FUVChannel
	{
		TArray<FVector2f> UV;
		static constexpr uint32 GetCardUVIndex() { return 1u; }
	};

	// Geometry
	TArray<FUVChannel>UVs;
	TArray<FVector3f>Normals;
	TArray<FVector3f>Tangents;
	TArray<FVector3f>Positions;
	TArray<uint32>   Indices;
	TArray<float>    CoordU;  // Transient data storing [0..1] parametric value along main axis. This is used for generating guides & interpolation data
	TArray<FVector2f>LocalUVs;// Transient data storing [0..1, 0..1] local UV of the cards. U is aligned with the card's main direction, and V is aligned perpendicularity
	TArray<FLinearColor> VertexColors;

	// Index offset and index count of each cards geometry
	TArray<uint32> IndexOffsets;
	TArray<uint32> IndexCounts;

	FBox3f BoundingBox = FBox3f(EForceInit::ForceInit);

	void Reset()
	{
		UVs.Reset();
		Normals.Reset();
		Tangents.Reset();
		Positions.Reset();
		Indices.Reset();
		CoordU.Reset();
		LocalUVs.Reset();
		VertexColors.Reset();

		IndexOffsets.Reset();
		IndexCounts.Reset();

		BoundingBox.Init();
	}

	void SetNum(uint32 Count)
	{
		// Geometry
		UVs.Empty();
		Normals.Empty();
		Tangents.Empty();
		Positions.Empty();
		Indices.Empty();
		CoordU.Reset();
		VertexColors.Empty();

		// Cards' indices offset & count
		IndexOffsets.SetNum(Count);
		IndexCounts.SetNum(Count);
	}

	uint32 GetNumTriangles() const
	{
		return Indices.Num() / 3;
	}

	uint32 GetNumVertices() const
	{
		return Positions.Num();
	}

	uint32 GetNumCards() const
	{
		return IndexOffsets.Num();
	}
};

struct FHairCardsDatas
{
	bool IsValid() const { return Cards.Positions.Num() > 0; }

	FHairCardsGeometry Cards;
};

struct FHairCardsBulkData
{
	bool IsValid() const { return Positions.Num() > 0; }
	void Reset()
	{
		DepthTexture = nullptr;
		TangentTexture = nullptr;
		CoverageTexture = nullptr;
		AttributeTexture = nullptr;

		Positions.Empty();
		Normals.Empty();
		UVs.Empty();
		Materials.Empty();
		VertexColors.Empty();
		Indices.Empty();
		Header = FHeader();
	}

	uint32 GetNumTriangles() const { return Indices.Num() / 3; }
	uint32 GetNumVertices() const { return Positions.Num(); }
	const FBox& GetBounds() const { return Header.BoundingBox; }
	uint32 GetNumUVs() const { return Header.NumUVs; }

	UTexture2D* DepthTexture = nullptr;
	UTexture2D* TangentTexture = nullptr;
	UTexture2D* CoverageTexture = nullptr;
	UTexture2D* AttributeTexture = nullptr;

	TArray<FHairCardsPositionFormat::Type> Positions;
	TArray<FHairCardsNormalFormat::Type> Normals;
	TArray<FHairCardsUVFormat::Type> UVs;
	TArray<FHairCardsMaterialFormat::Type> Materials;
	TArray<FHairCardsMaterialFormat::Type> VertexColors;
	TArray<FHairCardsIndexFormat::Type> Indices;

	struct FHeader
	{
		FBox BoundingBox = FBox(EForceInit::ForceInit);
		uint32 NumUVs = 0;
		bool bVertexColor = false;
	} Header;

	void Serialize(FArchive& Ar);
};

////////////////////////////////////////////////////////////////////////////////////////////////
// Hair Card interpolation Data/Bulk

/** Hair cards points interpolation attributes */
struct FHairCardsInterpolationDatas
{
	/** Set the number of interpolated points */
	UE_API void SetNum(const uint32 NumPoints);

	/** Reset the interpolated points to 0 */
	UE_API void Reset();

	/** Get the number of interpolated points */
	uint32 Num() const { return PointsSimCurvesVertexIndex.Num(); }

	/** Simulation curve indices */
	TArray<int32> PointsSimCurvesIndex;

	/** Closest vertex indices on simulation curve */
	TArray<int32> PointsSimCurvesVertexIndex;

	/** Lerp value between the closest vertex indices and the next one */
	TArray<float> PointsSimCurvesVertexLerp;
};

struct FHairCardsInterpolationBulkData
{
	TArray<FHairCardsInterpolationFormat::Type> Interpolation;

	UE_API void Serialize(FArchive& Ar);
};

////////////////////////////////////////////////////////////////////////////////////////////////
// Hair Meshes Data/Bulk

struct FHairMeshes
{
	struct FUVChannel
	{
		TArray<FVector2f> UV;
	};

	// Geometry
	TArray<FUVChannel>UVs;
	TArray<FColor>    VertexColors;
	TArray<FVector3f> Normals;
	TArray<FVector3f> Tangents;
	TArray<FVector3f> Positions;
	TArray<uint32>    Indices;

	FBox3f BoundingBox = FBox3f(EForceInit::ForceInit);

	void SetNum(uint32 Count)
	{
		// Geometry
		UVs.Empty();
		VertexColors.Empty();
		Normals.Empty();
		Tangents.Empty();
		Positions.Empty();
		Indices.Empty();

		BoundingBox.Init();
	}

	uint32 GetNumTriangles() const { return Indices.Num() / 3; }
	uint32 GetNumVertices() const { return Positions.Num(); }
};

struct FHairMeshesDatas
{
	bool IsValid() const { return Meshes.Positions.Num() > 0; }

	FHairMeshes Meshes;
};

struct FHairMeshesBulkData
{
	bool IsValid() const { return Positions.Num() > 0; }

	uint32 GetNumTriangles() const { return Indices.Num() / 3; }
	uint32 GetNumVertices() const { return Positions.Num(); }
	const FBox& GetBounds() const { return Header.BoundingBox; }
	uint32 GetNumUVs() const { return Header.NumUVs; }

	TArray<FHairCardsPositionFormat::Type> Positions;
	TArray<FHairCardsNormalFormat::Type> Normals;
	TArray<FHairCardsUVFormat::Type> UVs;
	TArray<FHairCardsMaterialFormat::Type> VertexColors;
	TArray<FHairCardsIndexFormat::Type> Indices;

	struct FHeader
	{
		FBox BoundingBox = FBox(EForceInit::ForceInit);
		uint32 NumUVs = 0;
		bool bVertexColor = false;
	} Header;

	void Serialize(FArchive& Ar);
};

#undef UE_API
