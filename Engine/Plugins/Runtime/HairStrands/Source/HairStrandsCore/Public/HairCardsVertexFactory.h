// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Components.h"
#include "VertexFactory.h"
#include "RenderGraphResources.h"
#include "HairCardsDatas.h"
#include "HairStrandsInterface.h"
#include "PrimitiveSceneProxy.h"
#include "MeshBatch.h"

#define UE_API HAIRSTRANDSCORE_API

class FMaterial;
class FSceneView;

// Wrapper to reinterepet FRDGPooledBuffer as a FVertexBuffer
class FRDGWrapperVertexBuffer : public FVertexBuffer
{
public:
	FRDGWrapperVertexBuffer() {}
	FRDGWrapperVertexBuffer(FRDGExternalBuffer& In): ExternalBuffer(In) { check(ExternalBuffer.Buffer); }
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		check(ExternalBuffer.Buffer && ExternalBuffer.Buffer->GetRHI());
		VertexBufferRHI = ExternalBuffer.Buffer->GetRHI();
	}

	virtual void ReleaseRHI() override
	{
		VertexBufferRHI = nullptr;
	}

	FRDGExternalBuffer ExternalBuffer;
};

/**
 * A vertex factory which simply transforms explicit vertex attributes from local to world space.
 */
class FHairCardsVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FHairCardsVertexFactory);
public:
	struct FDataType
	{
		FHairGroupInstance* Instance = nullptr;
		uint32 LODIndex = 0;
		EHairGeometryType GeometryType = EHairGeometryType::NoneGeometry;
	};

	UE_API FHairCardsVertexFactory(FHairGroupInstance* Instance, uint32 LODIndex, EHairGeometryType GeometryType, EShaderPlatform InShaderPlatform, ERHIFeatureLevel::Type InFeatureLevel, const char* InDebugName);
	
	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static UE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);
	static UE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static UE_API void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);
	static UE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	// Return the primitive id supported by the VF
	UE_API EPrimitiveIdMode GetPrimitiveIdMode(ERHIFeatureLevel::Type In) const;

	/**
	 * An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	 */
	UE_API void SetData(const FDataType& InData);

	/**
	* Copy the data from another vertex factory
	* @param Other - factory to copy from
	*/
	UE_API void Copy(const FHairCardsVertexFactory& Other);

	UE_API void InitResources(FRHICommandListBase& RHICmdList);
	UE_API virtual void ReleaseResource() override;

	// FRenderResource interface.
	UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	UE_API virtual void ReleaseRHI() override;
	const FDataType& GetData() const { return Data; }
	FDataType Data;
protected:

	bool bIsInitialized = false;

	FRDGWrapperVertexBuffer DeformedPositionVertexBuffer[2];
	FRDGWrapperVertexBuffer DeformedNormalVertexBuffer;

	struct FDebugName
	{
		FDebugName(const char* InDebugName)
#if !UE_BUILD_SHIPPING
			: DebugName(InDebugName)
#endif
		{}
	private:
#if !UE_BUILD_SHIPPING
		const char* DebugName;
#endif
	} DebugName;
};

#undef UE_API
