// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HairAttributes.h"

#include "MeshAttributeArray.h" // for TMeshAttributesRef
#include "MeshDescription.h" // for TVertexAttributesRef
#include "MeshTypes.h" // for FElementID, FVertexID
#include "Serialization/EditorBulkData.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Serialization/BulkData.h"
#endif

struct FStrandID : public FElementID
{
	FStrandID()
	{
	}

	explicit FStrandID(const FElementID InitElementID)
		: FElementID(InitElementID.GetValue())
	{
	}

	explicit FStrandID(const int32 InitIDValue)
		: FElementID(InitIDValue)
	{
	}

	inline friend uint32 GetTypeHash(const FStrandID& Other)
	{
		return GetTypeHash(Other.IDValue);
	}

	HAIRSTRANDSCORE_API static const FStrandID Invalid;
};

struct FGroomID : public FElementID
{
	FGroomID()
	{
	}

	explicit FGroomID(const FElementID InitElementID)
		: FElementID(InitElementID.GetValue())
	{
	}

	explicit FGroomID(const int32 InitIDValue)
		: FElementID(InitIDValue)
	{
	}

	inline friend uint32 GetTypeHash(const FGroomID& Other)
	{
		return GetTypeHash(Other.IDValue);
	}

	HAIRSTRANDSCORE_API static const FGroomID Invalid;
};

class FHairDescription
{
public:
	HAIRSTRANDSCORE_API FHairDescription();

	~FHairDescription() = default;
	FHairDescription(const FHairDescription&) = default;
	FHairDescription(FHairDescription&&) = default;
	FHairDescription& operator=(const FHairDescription&) = default;
	FHairDescription& operator=(FHairDescription&&) = default;

	HAIRSTRANDSCORE_API void InitializeVertices(int32 InNumVertices);
	HAIRSTRANDSCORE_API void InitializeStrands(int32 InNumStrands);

	HAIRSTRANDSCORE_API FVertexID AddVertex();
	HAIRSTRANDSCORE_API FStrandID AddStrand();

	HAIRSTRANDSCORE_API void Reset();
	HAIRSTRANDSCORE_API bool IsValid() const;

	TAttributesSet<FVertexID>& VertexAttributes() { return VertexAttributesSet; }
	const TAttributesSet<FVertexID>& VertexAttributes() const { return VertexAttributesSet; }

	TAttributesSet<FStrandID>& StrandAttributes() { return StrandAttributesSet; }
	const TAttributesSet<FStrandID>& StrandAttributes() const { return StrandAttributesSet; }

	TAttributesSet<FGroomID>& GroomAttributes() { return GroomAttributesSet; }
	const TAttributesSet<FGroomID>& GroomAttributes() const { return GroomAttributesSet; }

	int32 GetNumVertices() const { return NumVertices; }
	int32 GetNumStrands() const { return NumStrands; }

	HAIRSTRANDSCORE_API void Serialize(FArchive& Ar);

	// #ueent_todo: Expose "remove" operations to allow editing HairDescription
	HAIRSTRANDSCORE_API bool HasAttribute(EHairAttribute InAttribute) const;

	UE_DEPRECATED(5.3, "HasRootUV accessor is deprecated. Use HasAttribute instead")
	HAIRSTRANDSCORE_API bool HasRootUV() const;
	UE_DEPRECATED(5.3, "HasClumpID accessor is deprecated. Use HasAttribute instead")
	HAIRSTRANDSCORE_API bool HasClumpID() const;
	UE_DEPRECATED(5.3, "HasGuideWeights accessor is deprecated. Use HasAttribute instead")
	HAIRSTRANDSCORE_API bool HasGuideWeights() const;
	UE_DEPRECATED(5.3, "HasColorAttributes accessor is deprecated. Use HasAttribute instead")
	HAIRSTRANDSCORE_API bool HasColorAttributes() const;
	UE_DEPRECATED(5.3, "HasRoughnessAttributes accessor is deprecated. Use HasAttribute instead")
	HAIRSTRANDSCORE_API bool HasRoughnessAttributes() const;
	UE_DEPRECATED(5.3, "HasAOAttributes accessor is deprecated. Use HasAttribute instead")
	HAIRSTRANDSCORE_API bool HasAOAttributes() const;

private:
	TAttributesSet<FVertexID> VertexAttributesSet;
	TAttributesSet<FStrandID> StrandAttributesSet;
	TAttributesSet<FGroomID> GroomAttributesSet;

	int32 NumVertices;
	int32 NumStrands;
};

struct FHairDescriptionVersion
{
	FCustomVersionContainer CustomVersions;
	FPackageFileVersion UEVersion;
	int32 LicenseeVersion = 0;
	bool bIsValid = false;
	bool IsValid() const { return bIsValid; }
	void CopyVersionsFromArchive(const FArchive& Ar);
	void CopyVersionsToArchive(FArchive& Ar) const;
};
FArchive& operator<<(FArchive& Ar, FHairDescriptionVersion& Version);

/**
 * Bulk data storage for FHairDescription
 */
struct FHairDescriptionBulkData
{
public:
	FHairDescriptionBulkData() {}

#if WITH_EDITORONLY_DATA
	HAIRSTRANDSCORE_API void Serialize(FArchive& Ar, UObject* Owner);

	/** Stores a new hair description in the bulk data */
	HAIRSTRANDSCORE_API void SaveHairDescription(FHairDescription& HairDescription);

	/** Loads the hair description from the bulk data */
	HAIRSTRANDSCORE_API void LoadHairDescription(FHairDescription& HairDescription);

	/** Empties the bulk data */
	HAIRSTRANDSCORE_API void Empty();

	/** Returns true if there is nothing in the bulk data */
	bool IsEmpty() const { return !BulkData.HasPayloadData(); }

	/** Returns unique ID string for this bulk data */
	HAIRSTRANDSCORE_API FString GetIdString() const;

private:
	/** Internally store bulk data as bytes */
	UE::Serialization::FEditorBulkData BulkData;

	/** UE/Custom/Licensee version to propagate to archive when serializing the bulk data */
	FHairDescriptionVersion BulkDataVersion;

	/** Whether the bulk data has been written via SaveHairDescription */
	bool bBulkDataUpdated = false;
#endif
};

template <typename AttributeType> using TStrandAttributesRef = TMeshAttributesRef<FStrandID, AttributeType>;
template <typename AttributeType> using TGroomAttributesRef = TMeshAttributesRef<FGroomID, AttributeType>;

template <typename AttributeType> using TStrandAttributesConstRef = TMeshAttributesConstRef<FStrandID, AttributeType>;
template <typename AttributeType> using TGroomAttributesConstRef = TMeshAttributesConstRef<FGroomID, AttributeType>;

template <typename AttributeType>
void SetHairVertexAttribute(FHairDescription& HairDescription, FVertexID VertexID, FName AttributeName, AttributeType AttributeValue)
{
	TVertexAttributesRef<AttributeType> VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<AttributeType>(AttributeName);
	if (!VertexAttributeRef.IsValid())
	{
		HairDescription.VertexAttributes().RegisterAttribute<AttributeType>(AttributeName);
		VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<AttributeType>(AttributeName);
	}
	VertexAttributeRef[VertexID] = AttributeValue;
}

template <typename AttributeType>
void SetHairStrandAttribute(FHairDescription& HairDescription, FStrandID StrandID, FName AttributeName, AttributeType AttributeValue)
{
	TStrandAttributesRef<AttributeType> StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<AttributeType>(AttributeName);
	if (!StrandAttributeRef.IsValid())
	{
		HairDescription.StrandAttributes().RegisterAttribute<AttributeType>(AttributeName);
		StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<AttributeType>(AttributeName);
	}
	StrandAttributeRef[StrandID] = AttributeValue;
}

template <typename AttributeType>
void SetGroomAttribute(FHairDescription& HairDescription, FGroomID GroomID, FName AttributeName, AttributeType AttributeValue)
{
	TGroomAttributesRef<AttributeType> GroomAttributeRef = HairDescription.GroomAttributes().GetAttributesRef<AttributeType>(AttributeName);
	if (!GroomAttributeRef.IsValid())
	{
		HairDescription.GroomAttributes().RegisterAttribute<AttributeType>(AttributeName);
		GroomAttributeRef = HairDescription.GroomAttributes().GetAttributesRef<AttributeType>(AttributeName);
	}
	GroomAttributeRef[GroomID] = AttributeValue;
}
