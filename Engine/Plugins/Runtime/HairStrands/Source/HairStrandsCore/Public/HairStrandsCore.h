// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GroomBindingAsset.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UGeometryCache;
class UGroomAsset;
class USkeletalMesh;
class UStaticMesh;
class UTexture2D;

//typedef UTexture2D* (TCreateTextureHelper*)(FName Package, const FIntPoint& Resolution);
typedef void (*TTextureAllocation)(UTexture2D* Out, const FIntPoint& Resolution, uint32 MipCount);

struct FHairAssetHelper
{
	/* Generate a unique asset & package name */
	typedef void (*TCreateFilename)(const FString& InAssetName, const FString& Suffix, FString& OutPackageName, FString& OutAssetName);

	/* Register a texture asset */
	typedef void (*TRegisterAsset)(UObject* Out);

	/* Save an object within a package */
	typedef void (*TSaveAsset)(UObject* Object);

	TCreateFilename CreateFilename;
	TRegisterAsset RegisterAsset;
	TSaveAsset SaveAsset;
};

/** Implements the HairStrands module  */
class FHairStrandsCore : public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override { return true; }

	HAIRSTRANDSCORE_API static void RegisterAssetHelper(const FHairAssetHelper& Helper);
	
#if WITH_EDITOR
	static UTexture2D* CreateTexture(const FString& PackgeName, const FIntPoint& Resolution, const FString& Suffix, TTextureAllocation TextureAllocation);
	static void ResizeTexture(UTexture2D* InTexture, const FIntPoint& Resolution, TTextureAllocation TextureAllocation);
	static UStaticMesh* CreateStaticMesh(const FString& InPackageName, const FString& Suffix);

	// Create binding asset from groom asset and skeletal asset. These functions are only valid when build with the editor. They will return null asset otherwise
	UE_DEPRECATED(4.27, "CreateGroomBindingAsset with SkeletalMesh is deprecated. Use the version with binding type and UObject instead.")
	HAIRSTRANDSCORE_API static UGroomBindingAsset* CreateGroomBindingAsset(UGroomAsset* GroomAsset, USkeletalMesh* SourceSkelMesh, USkeletalMesh* TargetSkelMesh, const int32 NumInterpolationPoints, const int32 MatchingSection);
	UE_DEPRECATED(4.27, "CreateGroomBindingAsset with SkeletalMesh is deprecated. Use the version with binding type and UObject instead.")
	HAIRSTRANDSCORE_API static UGroomBindingAsset* CreateGroomBindingAsset(const FString& InPackageName, UObject* InParent, UGroomAsset* GroomAsset, USkeletalMesh* SourceSkelMesh, USkeletalMesh* TargetSkelMesh, const int32 NumInterpolationPoints, const int32 MatchingSection);

	// Create binding asset from groom asset and assets of specified groom binding type
	HAIRSTRANDSCORE_API static UGroomBindingAsset* CreateGroomBindingAsset(EGroomBindingMeshType BindingType, UGroomAsset* GroomAsset, UObject* Source, UObject* Target, const int32 NumInterpolationPoints, const int32 MatchingSection, const FName TargetMeshAttribute = NAME_None);
	HAIRSTRANDSCORE_API static UGroomBindingAsset* CreateGroomBindingAsset(EGroomBindingMeshType BindingType, const FString& InPackageName, UObject* InParent, UGroomAsset* GroomAsset, UObject* Source, UObject* Target, const int32 NumInterpolationPoints, const int32 MatchingSection, const FName TargetMeshAttribute = NAME_None);

	HAIRSTRANDSCORE_API static void SaveAsset(UObject* Object);

	HAIRSTRANDSCORE_API static FHairAssetHelper& AssetHelper();
	
#endif
};
