// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "RHIDefinitions.h"
#include "PackedNormal.h"
#include "RenderGraphResources.h"
#include "Serialization/BulkData.h"
#include "HairStrandsDefinitions.h"
#include "IO/IoDispatcher.h"
#include "Memory/SharedBuffer.h"

#define UE_API HAIRSTRANDSCORE_API

DECLARE_LOG_CATEGORY_EXTERN(LogHairStrands, Log, All);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Forward declarations

struct FHairStrandsBulkCommon;
struct FHairBulkContainer;
struct FHairStrandsDatas;

float GetHairStrandsMaxLength(const FHairStrandsDatas& In);
float GetHairStrandsMaxRadius(const FHairStrandsDatas& In);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Formats

enum class EHairAttribute : uint8;
namespace UE::DerivedData
{
	struct FCacheGetChunkRequest;
	struct FCachePutValueRequest;
	class FRequestOwner;
}

struct FPackedHairVertex
{
	typedef uint64 BulkType;

	FFloat16 X, Y, Z;
	uint8 UCoord;
	uint8 Radius : 6;
	uint8 Type : 2;
};

struct FTranscodedHairPositions
{
	typedef FUintVector4 BulkType;

	struct FPosition
	{
		uint32 X    : 10;
		uint32 Y    : 10;
		uint32 Z    : 10;
		uint32 Type : 2;
	};

	struct FAttributePacking0
	{
		uint32 Radius0 : 6;
		uint32 Radius1 : 6;
		uint32 Radius2 : 6;

		uint32 UCoord0 : 6;
		uint32 UCoord2 : 6;
		uint32 Interp  : 2;
	};

	struct FAttributePacking1
	{
		uint32 Radius0 : 6;
		uint32 Radius1 : 6;
		uint32 Radius2 : 6;

		uint32 UCoord0 : 7;
		uint32 UCoord1 : 7;
	};

	FPosition CP0;
	FPosition CP1;
	FPosition CP2;
	union
	{
		FAttributePacking0 Attribute0;
		FAttributePacking1 Attribute1;
	};
};

struct FPackedHairAttribute0Vertex
{
	typedef uint16 BulkType;

	uint8 NormalizedLength;
	uint8 Seed;
};

struct FPackedHairCurve
{
	typedef uint32 BulkType;

	uint32 PointOffset : 24;
	uint32 PointCount  : 8;
};

struct FVector4_16
{
	FFloat16 X;
	FFloat16 Y;
	FFloat16 Z;
	FFloat16 W;
}; 
FArchive& operator<<(FArchive& Ar, FVector4_16& Vertex);

struct FHairStrandsTranscodedPositionFormat
{
	typedef FTranscodedHairPositions Type;
	typedef FTranscodedHairPositions::BulkType BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UShort4;
	static const EPixelFormat Format = PF_Unknown;
};

struct FHairStrandsPositionFormat
{
	typedef FPackedHairVertex Type;
	typedef FPackedHairVertex::BulkType BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UShort4;
	static const EPixelFormat Format = PF_Unknown;
};

struct FHairStrandsPositionOffsetFormat
{
	typedef FVector4f Type;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float4;
	static const EPixelFormat Format = PF_A32B32G32R32F;
};

struct FHairStrandsAttributeFormat
{
	typedef uint32 Type;
	typedef uint32 BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsPointToCurveFormat
{
	typedef uint32 Type;
	typedef uint32 BulkType;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsTangentFormat
{
	typedef FPackedNormal Type;
	typedef uint32 BulkType;
	// TangentX & tangentZ are packed into 2 * PF_R8G8B8A8_SNORM
	static const uint32 ComponentCount = 2;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float4;
	static const EPixelFormat Format = PF_R8G8B8A8_SNORM;
};

struct FHairStrandsInterpolationFormat
{
	typedef uint32 Type;
	typedef uint32 BulkType;

	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsRootIndexFormat
{
	typedef uint32 Type;
	typedef uint32 BulkType;

	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsCurveFormat
{
	typedef FPackedHairCurve Type;
	typedef FPackedHairCurve::BulkType BulkType;

	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsRaytracingFormat
{
	typedef FVector4f Type;

	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float4;
	static const EPixelFormat Format = PF_A32B32G32R32F;
};

/** Hair strands index format */
struct FHairStrandsIndexFormat
{
	using Type = uint32;
	using BulkType = uint32;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

/** Hair strands RBF sample index format */
struct FHairStrandsRBFSampleIndexFormat
{
	using Type = uint32;
	using BulkType = uint32;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

/** Hair strands weights format */
struct FHairStrandsWeightFormat
{
	using Type = float;
	using BulkType = float;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float1;
	static const EPixelFormat Format = PF_R32_FLOAT;
};

/** 
 * Skinned mesh triangle vertex position format
 */
struct FHairStrandsMeshTrianglePositionFormat
{
	using Type = FVector4f;
	using BulkType = FVector4f;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_Float4;
	static const EPixelFormat Format = PF_A32B32G32R32F;
};

// Encode Section ID and triangle Index from the source skel. mesh
struct FHairStrandsUniqueTriangleIndexFormat
{
	using Type = uint32;
	using BulkType = uint32;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsRootToUniqueTriangleIndexFormat
{
	using Type = uint32;
	using BulkType = uint32;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsRootBarycentricFormat
{
	using Type = uint32;
	using BulkType = uint32;
	static const uint32 ComponentCount = 1;
	static const uint32 SizeInByte = sizeof(Type);
	static const EVertexElementType VertexElementType = VET_UInt;
	static const EPixelFormat Format = PF_R32_UINT;
};

struct FHairStrandsRootUtils
{
	static uint32	 PackTriangleIndex(uint32 TriangleIndex, uint32 SectionIndex);
	static void		 UnpackTriangleIndex(uint32 Encoded, uint32& OutTriangleIndex, uint32& OutSectionIndex);
	static uint32	 PackBarycentrics(const FVector2f& B);
	static FVector2f UnpackBarycentrics(uint32 B);
	static uint32	 PackUVs(const FVector2f& UV);
	static float	 PackUVsToFloat(const FVector2f& UV);
};

/*  Structure describing the LOD settings (Screen size, vertex info, ...) for each clusters.
	The packed version of this structure corresponds to the GPU data layout (HairStrandsClusterCommon.ush)
	This uses by the GPU LOD selection. */
struct FHairClusterInfo
{
	static const uint32 MaxLOD = 8;

	struct Packed
	{
		uint32 Screen0;
		uint32 Screen1;
		uint32 Radius0;
		uint32 Radius1;
	};
	typedef FUintVector4 BulkType;

	FHairClusterInfo()
	{
		for (uint32 LODIt = 0; LODIt < MaxLOD; ++LODIt)
		{
			ScreenSize[LODIt] = 0;
			RadiusScale[LODIt] = 1.f;
			bIsVisible[LODIt] = true;
		}
	}

	uint32 LODCount = 0;
	TStaticArray<float,MaxLOD> ScreenSize;
	TStaticArray<float,MaxLOD> RadiusScale;
	TStaticArray<bool, MaxLOD> bIsVisible;
};

struct FHairLODInfo
{
	uint32 CurveCount = 0;
	uint32 PointCount = 0;
	float RadiusScale = 1;
	float ScreenSize = 1;
	bool bIsVisible = true;
};

struct FHairClusterInfoFormat
{
	typedef FHairClusterInfo::Packed Type;
	typedef FHairClusterInfo::Packed BulkType;
	static const uint32 SizeInByte = sizeof(Type);
};

struct FHairClusterIndexFormat
{
	typedef uint32 Type;
	typedef uint32 BulkType;
	static const uint32 SizeInByte = sizeof(Type);
	static const EPixelFormat Format = PF_R32_UINT;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Streaming 

// Streaming request are used for reading hair strands data from DDC or IO
// The request are translated later to FHairStrandsBulkCommon::FQuery for appropriate reading.
// FHairStrandsBulkCommon::FQuery abstract DDC/IO/Read/Write for bulk data
//
// A query is processed as follow:
//  FHairStreamingRequest -> FChunk -> FQuery
//   ____________________________________________________________
//  |                    FHairCommonResource                     |
//  |_______________________            _________________________|
//  | FHairStreamingRequest |          |  FHairStrandsBulkCommon |
//	|                       |          |                         |
//	|       FChunk  --------|--FQuery--|--> FHairBulkContainer   |
//	|       FChunk  --------|--FQuery--|--> FHairBulkContainer   |
//	|       FChunk  --------|--FQuery--|--> FHairBulkContainer   |
//  |_______________________|          |_________________________|

struct FHairStreamingRequest
{
	// Hold request at the container level (transient to the request)
	struct FChunk
	{
		enum EStatus { None, Pending, Completed, Failed, Unloading };
		FSharedBuffer Data_DDC;
		FIoBuffer Data_IO;
		uint32 Offset = 0;				// Offset to the requested data
		uint32 Size = 0;				// Size of the requested data
		uint32 TotalSize = 0; 			// Size of the total data (existing + requested)
		EStatus Status = EStatus::None;	// Status of the current request
		FHairBulkContainer* Container = nullptr;

		const uint8* GetData() const;
		void Release();
	};

	void Request(uint32 InRequestedCurveCount, uint32 InRequestedPointCount, FHairStrandsBulkCommon& In,
		bool bWait=false, bool bFillBulkData=false, bool bWarmCache=false, const FName& InOwnerName = NAME_None,
		bool* bWaitResult = nullptr);
	bool IsNone() const;
	bool IsCompleted();
	bool IsUnloading() const;

#if WITH_EDITORONLY_DATA
	bool WarmCache(uint32 InRequestedCurveCount, uint32 InRequestedPointCount, FHairStrandsBulkCommon& In);
#endif

#if !WITH_EDITORONLY_DATA
	// IO
	FBulkDataBatchRequest IORequest;
#else
	// DDC
	FString PathName;
	TUniquePtr<UE::DerivedData::FRequestOwner> DDCRequestOwner;
#endif
	TArray<FChunk> Chunks;
	uint32 CurveCount = 0;
	uint32 PointCount = 0;
	int32 LODIndex = 0;

	// When enabled, data can be loaded from an offset. Otherwisee, start from the beginning of the resource
	// This is used when cooking data to force the loading of the entire resource (i.e., bSupportOffsetLoad=false)
	bool bSupportOffsetLoad = true; 
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Common hair data container and hair data

struct FHairBulkContainer
{
	// Streaming
	uint32 LoadedSize = 0;
	FByteBulkData Data;
	FHairStreamingRequest::FChunk* ChunkRequest = nullptr;

	// Forward BulkData functions
	bool IsBulkDataLoaded() const 		{ return Data.IsBulkDataLoaded(); }
	int64 GetBulkDataSize() const 		{ return Data.GetBulkDataSize(); }
	FString GetDebugName() const 		{ return Data.GetDebugName(); }
	void SetBulkDataFlags(uint32 Flags) { Data.SetBulkDataFlags(Flags); }
	void RemoveBulkData() 				{ Data.RemoveBulkData();  }
	void Serialize(FArchive& Ar, UObject* Owner, int32 ChunkIndex, bool bAttemptFileMapping) { Data.Serialize(Ar, Owner, ChunkIndex, bAttemptFileMapping); }

	// Returns true if all loadable data is loaded, i.e. either the data is loaded or there's no data to load
	bool IsBulkDataEmptyOrLoaded()const { return Data.GetBulkDataSize() == 0 || Data.IsBulkDataLoaded(); }
};

struct FHairStrandsBulkCommon
{
	virtual ~FHairStrandsBulkCommon() = default;
	UE_API void Serialize(FArchive& Ar, UObject* Owner);
	virtual void SerializeHeader(FArchive& Ar, UObject* Owner) = 0;
	UE_API void SerializeData(FArchive& Ar, UObject* Owner);

	UE_API void Write_DDC(UObject* Owner, TArray<UE::DerivedData::FCachePutValueRequest>& Out);
	UE_API void Read_DDC(FHairStreamingRequest* In, TArray<UE::DerivedData::FCacheGetChunkRequest>& Out);
	UE_API void Write_IO(UObject* Owner, FArchive& Out);
	UE_API void Read_IO(FHairStreamingRequest* In, FBulkDataBatchRequest& Out);
	UE_API void Unload(FHairStreamingRequest* In);

	struct FQuery
	{
		void Add(FHairBulkContainer& In, const TCHAR* InSuffix, uint32& InOffset, uint32 InSize=0);
		uint32 GetCurveCount() const { check(StreamingRequest); return StreamingRequest->CurveCount; }
		uint32 GetPointCount() const { check(StreamingRequest); return StreamingRequest->PointCount; }
		enum EQueryType { None, ReadDDC, WriteDDC, ReadIO, ReadWriteIO /* i.e. regular Serialize() */, UnloadData};
		EQueryType Type = None;
		FHairStreamingRequest* StreamingRequest = nullptr;
		FBulkDataBatchRequest::FBatchBuilder* 			OutReadIO = nullptr;
		FArchive*										OutWriteIO = nullptr;
	#if WITH_EDITORONLY_DATA
		TArray<UE::DerivedData::FCacheGetChunkRequest>*	OutReadDDC = nullptr;
		TArray<UE::DerivedData::FCachePutValueRequest>*	OutWriteDDC = nullptr;
		FString* DerivedDataKey = nullptr;
	#endif
		UObject* Owner = nullptr; 
	};

	virtual uint32 GetResourceCount() const = 0;
	virtual void GetResources(FQuery& Out) = 0;
	virtual void GetResourceVersion(FArchive& Ar) const {}
	virtual void ResetLoadedSize() = 0;

#if WITH_EDITORONLY_DATA
	// Transient Name/DDCkey for streaming
	FString DerivedDataKey;
#endif
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Strands Data/Bulk

enum class EHairStrandsFlags : uint8
{
	HasTrimmedPoint		= 1<<0,
	HasTrimmedCurve		= 1<<1,
	HasInvalidPoint		= 1<<2,
	HasExtraEndingPoint	= 1<<3,
	HasShuffledCurve	= 1<<4,
	HasDecimatedPoint	= 1<<5,
	HasDecimatedCurve	= 1<<6,
};

/** Hair strands points attribute */
struct FHairStrandsPoints
{
	/** Set the number of points */
	UE_API void SetNum(const uint32 NumPoints, uint32 InAttributes, bool bHasMapping = false);

	/** Reset the points to 0 */
	UE_API void Reset();

	/** Get the number of points */
	uint32 Num() const { return PointsPosition.Num();  }

	UE_API bool HasAttribute(EHairAttribute In) const;

	/** Points position in local space */
	TArray<FVector3f> PointsPosition;

	/** Normalized radius relative to the max one */
	TArray<float> PointsRadius; // [0..1]

	/** Normalized length */
	TArray<float> PointsCoordU; // [0..1]

	/** Material per-vertex 'baked' base color (optional) */
	TArray<FLinearColor> PointsBaseColor; // [0..1]

	/** Material per-vertex 'baked' roughness (optional) */
	TArray<float> PointsRoughness; // [0..1]

	/** Material per-vertex 'baked' AO (optional) */
	TArray<float> PointsAO; // [0..1]

	/** Points source mapping */
	TArray<uint32> PointsMapping;

	/** Source point count */
	uint32 SourcePointCount = 0;
};

/** Hair strands Curves attribute */
struct FHairStrandsCurves
{
	/** Set the number of Curves */
	UE_API void SetNum(const uint32 NumPoints, uint32 InAttributes, bool bHasMapping = false);

	/** Reset the curves to 0 */
	UE_API void Reset();

	/** Get the number of Curves */
	uint32 Num() const { return CurvesCount.Num(); }

	UE_API bool HasPrecomputedWeights() const;

	UE_API bool HasAttribute(EHairAttribute In) const;

	/** Number of points per rod */
	TArray<uint16> CurvesCount;

	/** An offset represent the rod start in the point list */
	TArray<uint32> CurvesOffset;

	/** Normalized length relative to the max one */
	TArray<float> CurvesLength; // [0..1]

	/** Roots UV. Support UDIM coordinate up to 256x256 (optional) */
	TArray<FVector2f> CurvesRootUV; // [0..256]

	/** Strand ID associated with each curve (optional) */
	TArray<int> StrandIDs;

	/** Clump ID associated with each curve (optional) */	
	TArray<FIntVector> ClumpIDs;

	/** Mapping of imported Groom ID to index */
	TMap<int, int> GroomIDToIndex;

	/** Custom guide IDs (indexed with StrandID) (optional) */
	TArray<FIntVector> CurvesClosestGuideIDs;

	/** Custom guid weights (indexed with StrandID) (optional) */
	TArray<FVector3f> CurvesClosestGuideWeights;

	/** Flags for attributes */
	uint32 AttributeFlags = 0;
	
	/** Curves source mapping */
	TArray<uint32> CurvesMapping;

	/** Source curve count */
	uint32 SourceCurveCount = 0;
};

/** Hair strands datas that are stored on CPU */
struct FHairStrandsCommonDatas
{
	struct FHash
	{
		uint64 PositionHash = 0;
		uint64 CurveCountHash = 0;
		uint64 CurveOffsetHash = 0;

		bool operator==(const FHash& In) const
		{
			return 
				In.PositionHash == PositionHash &&
				In.CurveCountHash == CurveCountHash &&
				In.CurveOffsetHash == CurveOffsetHash;
		}
	};

	/* Get the total number of points */
	uint32 GetNumPoints() const { return StrandsPoints.Num(); }

	/* Get the total number of Curves */
	uint32 GetNumCurves() const { return StrandsCurves.Num(); }

	UE_API uint32 GetAttributes() const;
	UE_API uint32 GetAttributeFlags() const;
	bool HasFlags(EHairStrandsFlags In) const { return !!(Flags & uint32(In)); }

	UE_API void Reset();

	bool IsValid() const { return StrandsCurves.Num() > 0 && StrandsPoints.Num() > 0; }

	FHash GetHash() const;

	/** List of all the strands points */
	FHairStrandsPoints StrandsPoints;

	/** List of all the strands curves */
	FHairStrandsCurves StrandsCurves;

	/** General flags (shuffled curve, extra ending point, ...) */
	uint32 Flags = 0;
};

/** Hair strands raw datas that are stored on CPU */
/** Unlike FHairStrandsDatas, this is raw datas, i.e. not processed by any built step */
struct FHairStrandsRawDatas : public FHairStrandsCommonDatas
{
	UE_API void Reset();
};

/** Built hair strands data (decimated/shuffled/...) */
struct FHairStrandsDatas : public FHairStrandsCommonDatas
{
	UE_API void Reset();

	/* Copy a point or a curve from In to Out data */
	static UE_API void CopyCurve(const FHairStrandsDatas& In, FHairStrandsDatas& Out, uint32 InAttributes, uint32 InIndex, uint32 OutIndex);
	static UE_API void CopyPoint(const FHairStrandsDatas& In, FHairStrandsDatas& Out, uint32 InAttributes, uint32 InIndex, uint32 OutIndex);
	static UE_API void CopyPointLerp(const FHairStrandsDatas& In, FHairStrandsDatas& Out, uint32 InAttributes, uint32 InIndex0, uint32 InIndex1, float InAlpha, uint32 OutIndex);

	/** The Standard Hair Density */
	float HairDensity = 1;

	/* Strands bounding box */
	FBox3f BoundingBox = FBox3f(EForceInit::ForceInit);
};

struct FHairStrandsBulkData : FHairStrandsBulkCommon
{
	enum EDataFlags
	{
		DataFlags_HasData = 0x1u,				// Contains valid data. Otherwise: Position, Attributes, ... are all empty
		DataFlags_HasPointAttribute = 0x2,		// Contains point attribute data.
		DataFlags_HasTranscodedPosition = 0x4u,	// Contains transcoded position
		DataFlags_HasTrimmedCurve = 0x8u,		// Source data had more curves per group than the limit and some curves were trimmed
		DataFlags_HasTrimmedPoint = 0x10u,		// Source data had more points per curve than the limit and some points were trimmed
		DataFlags_HasPointMapping = 0x20u,		// Contains platform to source points mapping data
		DataFlags_HasCurveMapping = 0x40u,		// Contains platform to source curves mapping data
		DataFlags_HasShuffledCurves = 0x80u,	// Curve data has been shuffled
		DataFlags_HasExtraEndingPoint = 0x100u	// Curve contains an extra ending control point
	};

	static uint32 ConvertDataFlagsToHairStrandsFlags(uint32 In);
	static uint32 ConvertDataFlagsFromHairStrandsFlags(uint32 In);

	UE_API virtual void SerializeHeader(FArchive& Ar, UObject* Owner) override;
	UE_API virtual uint32 GetResourceCount() const override;
	UE_API virtual void GetResources(FQuery& Out) override;
	UE_API virtual void GetResourceVersion(FArchive& Ar) const override;

	bool IsValid() const { return Header.CurveCount > 0 && Header.PointCount > 0; }
	UE_API void Reset();
	UE_API virtual void ResetLoadedSize() override;
	UE_API float GetCoverageScale(float InCurvePercentage /*[0..1]*/) const;
#if !WITH_EDITORONLY_DATA
	UE_API bool IsCookedBulkDataLoadedForCPUAccess() const;
#endif

	uint32 GetNumCurves() const { return Header.CurveCount;  };
	uint32 GetNumPoints() const { return Header.PointCount; };
	uint32 GetNumSourceCurves() const { return Header.SourceCurveCount; };
	uint32 GetNumSourcePoints() const { return Header.SourcePointCount; };
	float  GetMaxLength() const	{ return Header.MaxLength; };
	float  GetMaxRadius() const { return Header.MaxRadius; }
	FVector GetPositionOffset() const { return Header.BoundingBox.GetCenter(); }
	const FBox& GetBounds() const { return Header.BoundingBox; }
	UE_API uint32 GetSize() const;
	bool HasFlags(EDataFlags In) const { return !!(Header.Flags & In); }

	uint32 GetCurveAttributeSizeInBytes(uint32 InCurveCount=HAIR_MAX_NUM_CURVE_PER_GROUP) const			{ return InCurveCount > 0 ? FMath::DivideAndRoundUp(FMath::Min(Header.CurveCount, InCurveCount), Header.Strides.CurveAttributeChunkElementCount) * Header.Strides.CurveAttributeChunkStride : 0; }
	uint32 GetPointAttributeSizeInBytes(uint32 InPointCount=HAIR_MAX_NUM_POINT_PER_GROUP) const			{ return InPointCount > 0 ? FMath::DivideAndRoundUp(FMath::Min(Header.PointCount, InPointCount), Header.Strides.PointAttributeChunkElementCount) * Header.Strides.PointAttributeChunkStride : 0; }
	uint32 GetPointToCurveSizeInBytes(uint32 InPointCount=HAIR_MAX_NUM_POINT_PER_GROUP) const			{ return InPointCount > 0 ? FMath::DivideAndRoundUp(FMath::Min(Header.PointCount, InPointCount), Header.Strides.PointToCurveChunkElementCount)   * Header.Strides.PointToCurveChunkStride   : 0; }
	uint32 GetTranscodedPositionSizeInBytes(uint32 InPointCount = HAIR_MAX_NUM_POINT_PER_GROUP) const 	{ return InPointCount > 0 ? FMath::DivideAndRoundUp(FMath::Min(Header.PointCount, InPointCount), Header.Strides.TranscodedPositionChunkElementCount) * Header.Strides.TranscodedPositionChunkStride : 0; }

	struct FHeader
	{
		uint32 CurveCount = 0;
		uint32 PointCount = 0;
		uint32 MinPointPerCurve = 0;
		uint32 MaxPointPerCurve = 0;
		uint32 AvgPointPerCurve = 0;
		float  MaxLength = 0;
		float  MaxRadius = 0;
		FBox   BoundingBox = FBox(EForceInit::ForceInit);
		uint32 Flags = 0;
		uint32 CurveAttributeOffsets[HAIR_CURVE_ATTRIBUTE_COUNT] = {0};
		uint32 PointAttributeOffsets[HAIR_POINT_ATTRIBUTE_COUNT] = {0};
	
		/** Imported attribute info */
		uint32 ImportedAttributes = 0;
		uint32 ImportedAttributeFlags = 0;

		/** Imported (Source) point/curve info */
		uint32 SourceCurveCount = 0;
		uint32 SourcePointCount = 0;

		// Map 'curve' count to 'point' count (used for CLOD)
		TArray<uint32> CurveToPointCount;

		// Coverage scale
		TArray<float> CoverageScales;

		FHairStrandsCommonDatas::FHash Hash;

		// Data transcoding parameters
		struct FTranscoding
		{
			FVector3f PositionOffset = FVector3f::OneVector;
			FVector3f PositionScale = FVector3f::OneVector;
		} Transcoding;

		// Data strides
		struct FStrides
		{
			uint32 PositionStride = 0;
			uint32 CurveStride = 0;
			uint32 PointToCurveChunkStride = 0;
			uint32 CurveAttributeChunkStride = 0;
			uint32 PointAttributeChunkStride = 0;
			uint32 TranscodedPositionChunkStride = 0;
			uint32 CurveMappingStride = 0;
			uint32 PointMappingStride = 0;

			// Number of element per chunk block
			uint32 PointToCurveChunkElementCount = 0;
			uint32 CurveAttributeChunkElementCount = 0;
			uint32 PointAttributeChunkElementCount = 0;
			uint32 TranscodedPositionChunkElementCount = 0;
		} Strides;
	} Header;

	struct FData
	{
		FHairBulkContainer Positions;			// Size = PointCount
		FHairBulkContainer TranscodedPositions;	// Size = PointCount / TranscodedPositionChunkElementCount (=3)
		FHairBulkContainer CurveAttributes;		// Size = y*CurveCount (depends on the per-curve stored attributes)
		FHairBulkContainer PointAttributes;		// Size = x*PointCount (depends on the per-point stored attributes)
		FHairBulkContainer PointToCurve; 		// Size = PointCount
		FHairBulkContainer Curves;				// Size = CurveCount
		FHairBulkContainer CurveMapping;		// Size = CurveCount
		FHairBulkContainer PointMapping;		// Size = PointCount
	} Data;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Interpolation Data/Bulk

/** Hair strands points interpolation attributes */
struct FHairStrandsInterpolationDatas
{
	/** Set the number of interpolated points */
	UE_API void SetNum(const uint32 InCurveCount, const uint32 InPointCount);

	/** Reset the interpolated points to 0 */
	UE_API void Reset();

	/** Get the number of interpolated points */
	uint32 GetPointCount() const { return PointSimIndices.Num(); }
	uint32 GetCurveCount() const { return CurveSimIndices.Num(); }

	bool IsValid() const { return CurveSimIndices.Num() > 0; }

	/** Simulation curve indices, ordered by closest influence (per-curve data)*/
	TArray<FIntVector2> CurveSimIndices;

	/** Weight of simulation curve, ordered by closest influence (per-curve data)*/
	TArray<FVector2f> CurveSimWeights;

	/** (Global) Index of the root point of the curve*/
	TArray<FIntVector2> CurveSimRootPointIndex;

	/** Closest vertex (local, 0..255) indices on simulation curve, ordered by closest influence (per-point data)*/
	TArray<FIntVector2> PointSimIndices;

	/** Lerp value between the closest vertex indices and the next one, ordered by closest influence (per-point data)*/
	TArray<FVector2f> PointSimLerps;

	/** True, if interpolation data are built using a single guide */
	bool bUseUniqueGuide = false;
};

struct FHairStrandsInterpolationBulkData : FHairStrandsBulkCommon
{
	enum EDataFlags
	{
		DataFlags_HasData = 1,
		DataFlags_HasSingleGuideData = 2,
	};

	UE_API void Reset();
	UE_API virtual void ResetLoadedSize() override;
	UE_API virtual void SerializeHeader(FArchive& Ar, UObject* Owner) override;

	UE_API virtual uint32 GetResourceCount() const override;
	UE_API virtual void GetResources(FQuery& Out) override;
	uint32 GetPointCount() const { return Header.PointCount; };
	UE_API uint32 GetSize() const;
#if !WITH_EDITORONLY_DATA
	UE_API bool IsCookedBulkDataLoadedForCPUAccess() const;
#endif

	struct FHeader
	{
		uint32 Flags = 0;
		uint32 PointCount = 0;
		uint32 CurveCount = 0;

		struct FStrides
		{
			uint32 CurveInterpolationStride = 0;
			uint32 PointInterpolationStride = 0;
		} Strides;
	} Header;

	struct FData
	{
		FHairBulkContainer CurveInterpolation;	// FHairStrandsInterpolationFormat  - Per-rendering-curve interpolation data (closest guides, weight factors, ...). Data for a 1 or 2 guide(s))
		FHairBulkContainer PointInterpolation;	// FHairStrandsInterpolationFormat  - Per-rendering-point interpolation data (local point index, lerp factors,...). Data for a 1 or 2 guide(s))
	} Data;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Cluster Data/Bulk

struct FHairStrandsClusterData
{
	UE_API void Reset();
	bool IsValid() const { return ClusterCount > 0 && PointCount > 0; }

	/* LOD info for the various clusters for LOD management on GPU */
	TArray<FHairClusterInfo>	ClusterInfos;
	TArray<uint32>				CurveToClusterIds;
	TArray<FHairLODInfo>		LODInfos;
	TArray<uint32>				PointLODs;

	uint32 ClusterCount = 0;		// Number of clusters
	float  ClusterScale = 0;		// Cluster scale factor
	uint32 PointCount = 0;			// Number of points
	uint32 CurveCount = 0;			// Number of curves
};

struct FHairStrandsClusterBulkData : FHairStrandsBulkCommon
{
	UE_API void Reset();
	UE_API virtual void ResetLoadedSize() override;

	UE_API virtual void SerializeHeader(FArchive& Ar, UObject* Owner) override;
	UE_API virtual uint32 GetResourceCount() const override;
	UE_API virtual void GetResources(FQuery& Out) override;
	UE_API uint32 GetCurveCount(float InLODIndex) const;
	UE_API uint32 GetSize() const;
#if !WITH_EDITORONLY_DATA
	UE_API bool IsCookedBulkDataLoadedForCPUAccess() const;
#endif

	bool IsValid() const { return Header.ClusterCount > 0 && Header.PointCount > 0; }

	struct FHeader
	{
		/* Curve count and Point count per LOD */
		TArray<FHairLODInfo> LODInfos;
	
		uint32 ClusterCount = 0;
		float  ClusterScale = 0;
		uint32 PointCount = 0;
		uint32 CurveCount = 0;
		FVector4f ClusterInfoParameters = FVector4f::Zero(); // xy:Scale/Offset for ScreenSize, zw:Scale/Offset for Radius

		struct FStrides
		{
			uint32 PackedClusterInfoStride = 0;
			uint32 CurveToClusterIdStride = 0;
			uint32 PointLODStride = 0;
		} Strides;
	} Header;

	struct FData
	{
		/* LOD info for the various clusters for LOD management on GPU */
		FHairBulkContainer	PackedClusterInfos;		// Size - ClusterCount
		FHairBulkContainer	CurveToClusterIds;		// Size - CurveCount
		FHairBulkContainer	PointLODs;				// Size - PointCount / HAIR_POINT_LOD_COUNT_PER_UINT
	} Data;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Root Data/Bulk

/* Source data for building root bulk data */
struct FHairStrandsRootData
{
	void Reset();
	bool HasProjectionData() const;
	bool IsValid() const { return RootCount > 0; }

	/* Triangle on which a root is attached */
	/* When the projection is done with source to target mesh transfer, the projection indices does not match.
		In this case we need to separate index computation. The barycentric coords remain the same however. */
	TArray<FHairStrandsRootToUniqueTriangleIndexFormat::Type> RootToUniqueTriangleIndexBuffer;
	TArray<FHairStrandsRootBarycentricFormat::Type> RootBarycentricBuffer;

	/* Strand hair roots translation and rotation in rest position relative to the bound triangle. Positions are relative to the rest root center */
	TArray<FHairStrandsUniqueTriangleIndexFormat::Type> UniqueTriangleIndexBuffer;
	TArray<FHairStrandsMeshTrianglePositionFormat::Type> RestUniqueTrianglePositionBuffer;

	/* Optional: Unique triangle positions before transfer. */
	/* This data is available if triangles transfer from source to target was needed. */
	TArray<FHairStrandsMeshTrianglePositionFormat::Type> RestUniqueTrianglePositionBuffer_TargetNonTransfered;

	/* Number of samples used for the mesh interpolation */
	uint32 SampleCount = 0;

	/* Store the hair interpolation weights | Size = SamplesCount * SamplesCount */
	TArray<FHairStrandsWeightFormat::Type> MeshInterpolationWeightsBuffer;

	/* Store the samples vertex indices */
	TArray<uint32> MeshSampleIndicesBuffer;

	/* Store the samples rest positions */
	TArray<FHairStrandsMeshTrianglePositionFormat::Type> RestSamplePositionsBuffer;

	/* Store the samples rest section */
	TArray<uint32> MeshSampleSectionsBuffer;

	/* Store the mesh section indices which are relevant for this root LOD data */
	TArray<uint32> UniqueSectionIds;

	/* Mesh LOD index for which the root data are computed */
	int32 LODIndex = -1;

	/* Number of render section of the target mesh */
	uint32 MeshSectionCount = 0;

	/* Number of roots */
	uint32 RootCount = 0;

	/* Number of control points */
	uint32 PointCount = 0;

	/** General flags (shuffled curve, extra ending point, ...) */
	uint32 Flags = 0;

	/** Source data's hash for validation */
	FHairStrandsDatas::FHash Hash;
};

/* Bulk data for root resources (GPU resources are stored into FHairStrandsRootResources) */
struct FHairStrandsRootBulkData : FHairStrandsBulkCommon
{
	virtual void SerializeHeader(FArchive& Ar, UObject* Owner) override;
	virtual uint32 GetResourceCount() const override;
	virtual void GetResources(FQuery& Out) override;
	uint32 GetSize() const;

	void Reset();
	virtual void ResetLoadedSize() override;
	bool IsValid() const { return Header.RootCount > 0; }
	const TArray<uint32>& GetValidSectionIndices() const;
	uint32 GetRootCount()const { return Header.RootCount; }
	uint32 GetDataSize() const;
#if !WITH_EDITORONLY_DATA
	bool IsCookedBulkDataLoadedForCPUAccess() const;
#endif
	bool HasFlags(FHairStrandsBulkData::EDataFlags In) const { return !!(Header.Flags & In); }

	struct FHeader
	{		
		int32  LODIndex = -1;
		uint32 RootCount = 0;					// Number of roots
		uint32 PointCount = 0;					// Number of control points
		uint32 SampleCount = 0; 				// Number of samples used for the mesh interpolation
		uint32 MeshSectionCount = 0;			// Number of section of the target mesh
		uint32 UniqueTriangleCount = 0;			
		uint32 Flags = 0;
		FHairStrandsDatas::FHash Hash;
		TArray<uint32> UniqueSectionIndices; 	// Store the mesh section indices which are relevant for this root LOD data

		struct FStrides
		{
			uint32 RootToUniqueTriangleIndexBufferStride = 0;
			uint32 RootBarycentricBufferStride = 0;
			uint32 UniqueTriangleIndexBufferStride = 0;
			uint32 RestUniqueTrianglePositionBufferStride = 0;

			uint32 MeshInterpolationWeightsBufferStride = 0;
			uint32 MeshSampleIndicesAndSectionsBufferStride = 0;
			uint32 RestSamplePositionsBufferStride = 0;
		} Strides;
	} Header;

	struct FData
	{
		// Binding
		FHairBulkContainer RootToUniqueTriangleIndexBuffer; 	// Map each root onto the unique triangle Id (per-root)
		FHairBulkContainer RootBarycentricBuffer; 				// Root's barycentric (per-root)
		FHairBulkContainer UniqueTriangleIndexBuffer; 			// Unique triangles list from skeleton mesh section IDs and triangle IDs (per-unique-triangle)
		FHairBulkContainer RestUniqueTrianglePositionBuffer;	// Rest triangle positions (per-unique-triangle)

		// RBF
		FHairBulkContainer MeshInterpolationWeightsBuffer; 		// Store the hair interpolation weights | Size = SamplesCount * SamplesCount (per-sample
		FHairBulkContainer MeshSampleIndicesAndSectionsBuffer;	// Store the samples vertex indices (per-sample)
		FHairBulkContainer RestSamplePositionsBuffer; 			// Store the samples rest positions (per-sample)
	} Data;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair debug data

/** Hair strands debug data */
struct  FHairStrandsDebugDatas
{
	bool IsValid() const { return VoxelData.Num() > 0;  }

	static const uint32 InvalidIndex = ~0u;
	struct FOffsetAndCount
	{
		uint32 Offset = 0u;
		uint32 Count = 0u;
	};

	struct FVoxel
	{
		uint32 Index0 = InvalidIndex;
		uint32 Index1 = InvalidIndex;
	};

	struct FDesc
	{
		FVector3f VoxelMinBound = FVector3f::ZeroVector;
		FVector3f VoxelMaxBound = FVector3f::ZeroVector;
		FIntVector VoxelResolution = FIntVector::ZeroValue;
		float VoxelSize = 0;
		uint32 MaxSegmentPerVoxel = 0;
	};

	FDesc VoxelDescription;
	TArray<FOffsetAndCount> VoxelOffsetAndCount;
	TArray<FVoxel> VoxelData;
};

struct FHairStrandsDebugResources
{
	FHairStrandsDebugDatas::FDesc VoxelDescription;
	TRefCountPtr<FRDGPooledBuffer> VoxelOffsetAndCount;
	TRefCountPtr<FRDGPooledBuffer> VoxelData;
};

#undef UE_API
