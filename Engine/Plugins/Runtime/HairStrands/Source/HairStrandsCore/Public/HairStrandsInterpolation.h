// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Math/MathFwd.h"
#include "RenderGraphFwd.h"

#include "HairStrandsInterface.h"
#include "GroomResources.h"

class FScene;
class FGlobalShaderMap;
class FRDGBuilder;
class FSkeletalMeshRenderData;
class UGroomBindingAsset;
class FRHIShaderResourceView;

struct FHairGroupInstance;
struct FHairStrandClusterData;
struct FRWBuffer;
struct FShaderPrintData;
struct FHairStrandsRestRootResource;
struct FHairStrandsDeformedRootResource;
struct FRDGImportedBuffer;
struct FGroomCacheVertexData;

enum class EGroomViewMode : uint8;

// Reset the interpolation data. This needs to be called prior to ComputeHairStrandsInterpolation 
// and prior to the actual hair simulation in order to insure that:
//  1) when hair simulation is enabled, the first frame is correct
//  2) when hair simulation is enabled/disabled (i.e., toggle/change) 
//     we reset to deform buffer to rest state)
void AddDeformSimHairStrandsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 InstanceRegisteredIndex,
	const uint32 MeshLODIndex,
	const uint32 VertexCount,
	FHairStrandsRestRootResource* SimRestRootResources,
	FHairStrandsDeformedRootResource* SimDeformedRootResources,
	FRDGBufferSRVRef SimRestPosePositionBuffer,
	FRDGBufferSRVRef SimPointToCurveBuffer,
	FRDGImportedBuffer& OutSimDeformedPositionBuffer,
	const FVector& SimRestOffset,
	FRDGBufferSRVRef SimDeformedOffsetBuffer,
	const bool bHasGlobalInterpolation,
	FRHIShaderResourceView* BoneBufferSRV);

void ComputeHairStrandsInterpolation(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 ViewUniqueID,
	const uint32 ViewRayTracingMask,
	const EGroomViewMode ViewMode,
	const FVector& TranslatedWorldOffset,
	const FShaderPrintData* ShaderPrintData,
	FHairGroupInstance* Instance,
	int32 LODIndex,
	FHairStrandClusterData* ClusterData);

struct FGroomCacheResources
{
	FRDGBufferSRVRef PositionBuffer = nullptr;
	FRDGBufferSRVRef RadiusBuffer = nullptr;
	bool bHasRadiusData = false;
};
FGroomCacheResources CreateGroomCacheBuffer(FRDGBuilder& GraphBuilder, FGroomCacheVertexData& InVertexData);

void AddGroomCacheUpdatePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 InstanceRegisteredIndex,
	uint32 PointCount,
	uint32 CurveCount,
	float InterpolationFactor,
	float InMaxHairRadius,
	bool bAllowAddedControlPoint,
	FGroomCacheResources CacheResources0,
	FGroomCacheResources CacheResources1,
	FRDGBufferSRVRef InRestPositionBuffer,
	FRDGBufferSRVRef InRestCurveBuffer,
	FRDGBufferSRVRef InRestPointToCurveBuffer,
	FRDGBufferSRVRef InDeformedOffsetBuffer,
	FRDGBufferUAVRef OutBuffer);

HAIRSTRANDSCORE_API void ComputeInterpolationWeights(UGroomBindingAsset* BindingAsset, FSkeletalMeshRenderData* TargetRenderData, TArray<FRWBuffer>& TransferedPositions);

HAIRSTRANDSCORE_API FHairGroupPublicData::FVertexFactoryInput ComputeHairStrandsVertexInputData(const FHairGroupInstance* Instance, EGroomViewMode ViewMode);

struct FRDGHairStrandsCullingData
{
	bool bCullingResultAvailable = false;
	FRDGImportedBuffer HairStrandsVF_CullingIndirectBuffer;
	FRDGImportedBuffer HairStrandsVF_CullingIndexBuffer;
};

FRDGHairStrandsCullingData ImportCullingData(FRDGBuilder& GraphBuilder, FHairGroupPublicData* In);

enum class EHairAABBUpdateType
{
	UpdateClusterAABB,
	UpdateGroupAABB
};

void AddClearAABBPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 AABBCount,
	FRDGBufferUAVRef& OutAABBUAV);

void AddHairStrandsInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	EShaderPlatform InPlatform,
	const FShaderPrintData* ShaderPrintData,
	const FHairGroupInstance* Instance,
	const uint32 VertexCount,
	const uint32 CurveCount,
	const FHairStrandsRestResource* RenRestResource,
	const FHairStrandsRestResource* SimRestResource,
	const bool bSimDataValid,
	const uint32 MaxPointPerCurve,
	const int32 MeshLODIndex,
	const float HairLengthScale,
	const EHairInterpolationType HairInterpolationType,
	const EHairGeometryType InstanceGeometryType,
	const FRDGHairStrandsCullingData& CullingData,
	const FVector& InRenHairWorldOffset,
	const FVector& InSimHairWorldOffset,
	const FRDGBufferSRVRef& OutRenHairPositionOffsetBuffer,
	const FRDGBufferSRVRef& OutSimHairPositionOffsetBuffer,
	const FHairStrandsRestRootResource* RenRestRootResources,
	const FHairStrandsRestRootResource* SimRestRootResources,
	const FHairStrandsDeformedRootResource* RenDeformedRootResources,
	const FHairStrandsDeformedRootResource* SimDeformedRootResources,
	const FRDGBufferSRVRef& RenRestPosePositionBuffer,
	const FRDGBufferSRVRef& RenCurveBuffer,
	const bool bUseSingleGuide,
	const FRDGBufferSRVRef& CurveInterpolationBuffer,
	const FRDGBufferSRVRef& PoinInterpolationBuffer,
	const FRDGBufferSRVRef& SimRestPosePositionBuffer,
	const FRDGBufferSRVRef& SimDeformedPositionBuffer,

	const FRDGBufferSRVRef& RenDeformerPositionBuffer,
	FRDGBufferUAVRef& OutRenPositionBuffer,
	const FHairStrandsLODDeformedRootResource::EFrameType DeformedFrame);


void AddHairTangentPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	uint32 PointCount,
	FHairGroupPublicData* HairGroupPublicData,
	FRDGBufferSRVRef PositionBuffer,
	FRDGBufferUAVRef OutTangentBuffer);


enum class EHairPatchAttribute : uint8
{
	None,
	GuideInflucence,
	ClusterInfluence
};

void AddPatchAttributePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 CurveCount,
	const EHairPatchAttribute Mode,
	const bool bSimulation,
	const bool bUseSingleGuide,
	const FHairStrandsBulkData& RenBulkData,
	const FRDGBufferRef& RenAttributeBuffer,
	const FRDGBufferSRVRef& RenCurveBuffer,
	const FRDGBufferSRVRef& RenCurveToClusterIdBuffer,
	const FRDGBufferSRVRef& CurveInterpolationBuffer,
	const FRDGBufferSRVRef& PointInterpolationBuffer,
	FRDGImportedBuffer& OutRenAttributeBuffer);

void AddTransferPositionPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const uint32 PointOffset,
	const uint32 PointCount,
	const uint32 TotalPointCount,
	FRDGBufferSRVRef InBuffer,
	FRDGBufferUAVRef OutBuffer);

#if RHI_RAYTRACING
void AddGenerateRaytracingGeometryPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FShaderPrintData* ShaderPrintData,
	uint32 InstanceRegisteredIndex,
	uint32 ActivePointCount,
	uint32 TotalPointCount,
	EHairStrandsRayTracingPrimitiveType RayTracingPrimitiveType,
	int ProceduralSplits,
	float HairRadius,
	float RootScale,
	float TipScale,
	const FRDGBufferSRVRef& HairWorldOffsetBuffer,
	FRDGHairStrandsCullingData& CullingData,
	const FRDGBufferSRVRef& PositionBuffer,
	const FRDGBufferSRVRef& TangentBuffer,
	const FRDGBufferUAVRef& OutPositionBuffer,
	const FRDGBufferUAVRef& OutIndexBuffer);

void AddBuildStrandsAccelerationStructurePass(
	FRDGBuilder& GraphBuilder,
	FHairGroupInstance* Instance,
	uint32 ProceduralSplits,
	bool bNeedUpdate,
	FRDGBufferRef Raytracing_PositionBuffer,
	FRDGBufferRef Raytracing_IndexBuffer);

void AddBuildHairCardAccelerationStructurePass(
	FRDGBuilder& GraphBuilder,
	FHairGroupInstance* Instance,
	int32 HairLODIndex,
	bool bNeedUpdate);

void AddBuildHairMeshAccelerationStructurePass(
	FRDGBuilder& GraphBuilder,
	FHairGroupInstance* Instance,
	int32 HairLODIndex,
	bool bNeedUpdate);
#endif

void AddHairCardsDeformationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const ERHIFeatureLevel::Type FeatureLevel,
	const FShaderPrintData* ShaderPrintData,
	FHairGroupInstance* Instance,
	const int32 HairLODIndex,
	const int32 MeshLODIndex,
	ERHIPipeline CardPositionExternalAccessPipeline);
