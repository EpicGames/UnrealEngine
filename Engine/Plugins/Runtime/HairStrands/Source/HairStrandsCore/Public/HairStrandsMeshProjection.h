// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "HairStrandsInterface.h"
#include "RenderGraphResources.h"
#include "Shader.h"


class FSkeletalMeshRenderData;
class FSkeletalMeshLODRenderData;
struct FHairStrandsRestRootResource;
struct FHairStrandsDeformedRootResource;
struct FHairMeshesRestResource;
struct FHairMeshesDeformedResource;
struct FHairCardsRestResource;
struct FHairCardsDeformedResource;
struct FHairStrandsDeformedResource;

// Return the max number of section/triangle a skeletal mesh can have. After this count, binding will be disabled
uint32 GetHairStrandsMaxSectionCount();
uint32 GetHairStrandsMaxTriangleCount();

/* Update the triangles information on which hair stands have been projected */
void AddHairStrandUpdateMeshTrianglesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	const FCachedGeometry& MeshLODData,
	FHairStrandsRestRootResource* RestResources,
	FHairStrandsDeformedRootResource* DeformedResources);

/* Init the samples information to be used for interpolation*/
void AddHairStrandInitMeshSamplesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	const FCachedGeometry& MeshLODData,
	FHairStrandsRestRootResource* RestResources,
	FHairStrandsDeformedRootResource* DeformedResources);

/* Update the samples information to be used for interpolation*/
void AddHairStrandUpdateMeshSamplesPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	const FCachedGeometry& MeshLODData,
	FHairStrandsRestRootResource* RestResources,
	FHairStrandsDeformedRootResource* DeformedResources);

void GenerateFolliculeMask(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const EPixelFormat Format,
	const FIntPoint Resolution,
	const uint32 MipCount,
	const uint32 KernelSizeInPixels,
	const uint32 Channel,
	const int32 LODIndex,
	FHairStrandsRestRootResource* RestResources,
	FRDGTextureRef& OutTexture);

void GenerateFolliculeMask(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const EPixelFormat Format,
	const FIntPoint Resolution,
	const uint32 MipCount,
	const uint32 KernelSizeInPixels,
	const uint32 Channel,
	const TArray<FRDGBufferRef>& RootUVBuffers,
	FRDGTextureRef& OutTexture);

void AddComputeMipsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FRDGTextureRef& OutTexture);

struct FSkinUpdateSection
{
	uint32 SectionIndex = 0;
	uint32 NumVertexToProcess = 0;
	uint32 SectionVertexBaseIndex = 0;
	FRHIShaderResourceView* BoneBuffer = nullptr;
	FRHIShaderResourceView* BonePrevBuffer = nullptr;
	uint32 BoneTransformOffset = 0;
	uint32 PrevBoneTransformOffset = 0;
};

void AddSkinUpdatePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FSkeletalMeshLODRenderData& RenderData,
	const TArray<FSkinUpdateSection>& Sections,
	FRDGBufferRef OutDeformedPositionBuffer,
	FRDGBufferRef OutPrevDeformedPositionBuffer,
	FRDGBufferRef OutTangentBuffer);

void AddHairMeshesRBFInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	FHairMeshesRestResource* RestResources,
	FHairMeshesDeformedResource* DeformedResources,
	FHairStrandsRestRootResource* RestRootResources,
	FHairStrandsDeformedRootResource* DeformedRootResources);

void AddHairCardsRBFInterpolationPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 MeshLODIndex,
	FHairCardsRestResource* RestResources,
	FHairCardsDeformedResource* DeformedResources,
	FHairStrandsRestRootResource* RestRootResources,
	FHairStrandsDeformedRootResource* DeformedRootResources);

enum class EHairPositionUpdateType : uint8
{
	Guides  = 0,
	Strands = 1,
	Cards   = 2,
};

void AddHairStrandUpdatePositionOffsetPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	EHairPositionUpdateType UpdateType,
	const int32 InstanceRegisteredIndex,
	const int32 HairLODIndex,
	const int32 MeshLODIndex,
	FHairStrandsDeformedRootResource* DeformedRootResources,
	FHairStrandsDeformedResource* DeformedResources);