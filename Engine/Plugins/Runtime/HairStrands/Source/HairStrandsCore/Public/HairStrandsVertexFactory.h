// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Components.h"
#include "VertexFactory.h"
#include "RenderGraphResources.h"
#include "HairStrandsDatas.h"
#include "HairStrandsInterface.h"
#include "PrimitiveSceneProxy.h"
#include "MeshBatch.h"

#define UE_API HAIRSTRANDSCORE_API

class FMaterial;
class FSceneView;
struct FHairGroupInstance;

/**
 * A vertex factory which simply transforms explicit vertex attributes from local to world space.
 * 
 * This is templated to allow different behavior for Linear Swept Spheres, which do not require an intersection shader.
 */
template<bool bSupportLinearSweptSpheres>
class THairStrandsVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(THairStrandsVertexFactory);
public:
	
	struct FDataType
	{
		FHairGroupInstance* Instance = nullptr;
	};

	THairStrandsVertexFactory(FHairGroupInstance* Instance, ERHIFeatureLevel::Type InFeatureLevel, const char* InDebugName)
		: FVertexFactory(InFeatureLevel)
		, DebugName(InDebugName)
	{		
		Data.Instance = Instance;
	}

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	static UE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);
	static UE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static UE_API void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);
	static UE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	// Return the primitive id supported by the VF
	UE_API EPrimitiveIdMode GetPrimitiveIdMode(ERHIFeatureLevel::Type In) const;

	/**
	 * An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	 */
	UE_API void SetData(const FDataType& InData);

	/**
	* Copy the data from another vertex factory
	* @param Other - factory to copy from
	*/
	UE_API void Copy(const THairStrandsVertexFactory& Other);

	// FRenderResource interface.
	UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	UE_API virtual void ReleaseRHI() override;
	UE_API void InitResources(FRHICommandListBase& RHICmdList);

	const FDataType& GetData() const { return Data; }
	FDataType Data;
protected:
	bool bIsInitialized = false;

	struct FDebugName
	{
		FDebugName(const char* InDebugName)
#if !UE_BUILD_SHIPPING
			: DebugName(InDebugName)
#endif
		{}
	private:
#if !UE_BUILD_SHIPPING
		const char* DebugName;
#endif
	} DebugName;
};

using FLinearSweptSpheresHairStrandsVertexFactory = THairStrandsVertexFactory<true>;
using FHairStrandsVertexFactory = THairStrandsVertexFactory<false>;

#undef UE_API
