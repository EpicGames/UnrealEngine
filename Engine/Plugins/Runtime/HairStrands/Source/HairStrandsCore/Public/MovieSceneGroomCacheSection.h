// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "MovieSceneGroomCacheSection.generated.h"

#define UE_API HAIRSTRANDSCORE_API

USTRUCT()
 struct FMovieSceneGroomCacheParams
{
	GENERATED_BODY()

	UE_API FMovieSceneGroomCacheParams();

	/** Gets the animation sequence length, not modified by play rate */
	 UE_API float GetSequenceLength() const;
	/** The animation this section plays */
	UPROPERTY(EditAnywhere, Category = GroomCache)
	TObjectPtr<class UGroomCache> GroomCache;

	/** The offset for the first loop of the animation clip */
	UPROPERTY(EditAnywhere, Category = GroomCache)
	FFrameNumber FirstLoopStartFrameOffset;

	/** The offset into the beginning of the animation clip */
	UPROPERTY(EditAnywhere, Category = GroomCache)
	FFrameNumber StartFrameOffset;

	/** The offset into the end of the animation clip */
	UPROPERTY(EditAnywhere, Category = GroomCache)
	FFrameNumber EndFrameOffset;

	/** The playback rate of the animation clip */
	UPROPERTY(EditAnywhere, Category = GroomCache)
	float PlayRate;

	/** Reverse the playback of the animation clip */
	UPROPERTY(EditAnywhere, Category = "Animation")
	uint32 bReverse : 1;
};

/**
 * Movie scene section that control GroomCache playback
 */
UCLASS(MinimalAPI)
class UMovieSceneGroomCacheSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Animation", meta = (ShowOnlyInnerProperties))
	FMovieSceneGroomCacheParams Params;

	/** Get Frame Time as Animation Time*/
	virtual float MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const;

protected:
	//~ UMovieSceneSection interface
	virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const override;
	virtual TOptional<FFrameTime> GetOffsetTime() const override;

private:

#if WITH_EDITOR

	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
public:
	float PreviousPlayRate;

#endif
};

#undef UE_API
