// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraCommon.h"
#include "NiagaraRenderGraphUtils.h"
#include "NiagaraSimCacheCustomStorageInterface.h"
#include "VectorVM.h"
#include "GroomAsset.h"
#include "GroomActor.h"
#include "NiagaraDataInterfaceHairStrands.generated.h"

#define UE_API HAIRSTRANDSCORE_API

static const int32 MaxDelay = 2;
static const int32 NumScales = 4;
static const int32 StretchOffset = 0;
static const int32 BendOffset = 1;
static const int32 RadiusOffset = 2;
static const int32 ThicknessOffset = 3;

struct FNDIHairStrandsData;
struct FNDIHairStrandsInfo;

bool UsesNanite(const UGroomComponent* In);
FHairGroupInstance* GetHairGroupInstance(UGroomComponent* In, int32 InGroupIndex);

/** Render buffers that will be used in hlsl functions */
struct FNDIHairStrandsBuffer : public FRenderResource
{
	/** Set the asset that will be used to affect the buffer */
	void Initialize(
		const FNDIHairStrandsInfo& In,
		const TStaticArray<float, 32 * NumScales>& InParamsScale);

	/** Transfer CPU datas to GPU */
	void Transfer(FRDGBuilder& GraphBuilder, const TStaticArray<float, 32 * NumScales>& InParamsScale);

	/** Init the buffer */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Release the buffer */
	virtual void ReleaseRHI() override;

	/** Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FNDIHairStrandsBuffer"); }

	/** Bounding Box Buffer*/
	FNiagaraPooledRWBuffer BoundingBoxBuffer;

	/** Params scale buffer */
	FNiagaraPooledRWBuffer ParamsScaleBuffer;

	/** Scales along the strand */
	TStaticArray<float, 32 * NumScales> ParamsScale;

	/** Bounding box offsets */
	FIntVector4 BoundingBoxOffsets;

	/** Valid geometry type for hair (strands, cards, mesh)*/
	bool bValidGeometryType = false;

	/** Mesh LOD that is being used for the root resources */
	int32 CurrentMeshLOD = INDEX_NONE;

	/** True if the internal resources (BoundingBoxBuffer/ParamsScaleBuffer) needs to be built */
	bool bNeedResouces = false;

	/** Boolean to trigger the reset */
	bool bShouldReset = false;
	
	/** Counter to reset the simulation once triggered */
	int32 ResetCount = 0;

	// For debug only
	//FRHIGPUBufferReadback* ReadbackBuffer = nullptr;
};

/** Data stored per strand base instance*/
struct FNDIHairStrandsData
{
	FNDIHairStrandsData()
	{
		ResetDatas();
	}
	/** Initialize the buffers */
	bool Init(class UNiagaraDataInterfaceHairStrands* Interface, FNiagaraSystemInstance* SystemInstance);

	/** Release the buffers */
	void Release();

	/** Update the buffers */
	void Update(
		UNiagaraDataInterfaceHairStrands* Interface, 
		const FNDIHairStrandsInfo& InData,
		const float DeltaSeconds);

	inline void ResetDatas()
	{
		WorldTransform.SetIdentity();
		BoneTransform.SetIdentity();
		BoneLinearVelocity = FVector3f::Zero();
		BoneAngularVelocity = FVector3f::Zero();
			
		BoneLinearAcceleration = FVector3f::Zero();
		BoneAngularAcceleration = FVector3f::Zero();
			
		PreviousBoneTransform.SetIdentity();
		PreviousBoneLinearVelocity = FVector3f::Zero();
		PreviousBoneAngularVelocity = FVector3f::Zero();
		
		GlobalInterpolation = false;
		bSkinningTransfer = false;
		HairGroupInstSource = nullptr;
		HairGroupInstance = nullptr;
		HairGroupIndex = -1;

		ForceReset = true;

		NumStrands = 0;
		StrandsSize = 0;

		SubSteps = 5;
		IterationCount = 20;

		GravityVector = FVector(0.0, 0.0, -981.0);
		GravityPreloading = 0.0;
		AirDrag = 0.1f;
		AirVelocity = FVector(0, 0, 0);

		SolveBend = true;
		ProjectBend = false;
		BendDamping = 0.01f;
		BendStiffness = 0.01f;

		SolveStretch = true;
		ProjectStretch = false;
		StretchDamping = 0.01f;
		StretchStiffness = 1.0f;

		SolveCollision = true;
		ProjectCollision = true;
		KineticFriction = 0.1f;
		StaticFriction = 0.1f;
		StrandsViscosity = 1.0f;
		GridDimension = FIntVector(30,30,30);
		CollisionRadius = 1.0f;

		StrandsDensity = 1.0f;
		StrandsSmoothing = 0.1f;
		StrandsThickness = 0.01f;

		TickingGroup = NiagaraFirstTickGroup;

		for (int32 i = 0; i < 32 * NumScales; ++i)
		{
			ParamsScale[i] = 1.0;
		}
		SkeletalMeshes = 0;
		LocalSimulation = false;
		TimeDilation = 1.0f;
		bUsesNanite = false;
	}

	inline void CopyDatas(const FNDIHairStrandsData* OtherDatas)
	{
		if (OtherDatas != nullptr)
		{
			HairStrandsBuffer = OtherDatas->HairStrandsBuffer;

			WorldTransform = OtherDatas->WorldTransform;
			BoneTransform = OtherDatas->BoneTransform;
			BoneLinearVelocity = OtherDatas->BoneLinearVelocity;
			BoneAngularVelocity = OtherDatas->BoneAngularVelocity;
			
			BoneLinearAcceleration = OtherDatas->BoneLinearAcceleration;
			BoneAngularAcceleration = OtherDatas->BoneAngularAcceleration;
			
			PreviousBoneTransform = OtherDatas->PreviousBoneTransform;
			PreviousBoneLinearVelocity = OtherDatas->PreviousBoneLinearVelocity;
			PreviousBoneAngularVelocity = OtherDatas->PreviousBoneAngularVelocity;

			GlobalInterpolation = OtherDatas->GlobalInterpolation;
			bSkinningTransfer = OtherDatas->bSkinningTransfer;
			BindingType = OtherDatas->BindingType;
			HairGroupInstSource = OtherDatas->HairGroupInstSource;
			HairGroupInstance = nullptr;
			HairGroupIndex = OtherDatas->HairGroupIndex;

			bUsesNanite = false;
			if (HairGroupInstSource != nullptr)
			{
				bUsesNanite = UsesNanite(HairGroupInstSource.Get());
				if (!bUsesNanite)
				{
					HairGroupInstance = GetHairGroupInstance(HairGroupInstSource.Get(), HairGroupIndex);
				}

			}

			ForceReset = OtherDatas->ForceReset;
			
			NumStrands = OtherDatas->NumStrands;
			StrandsSize = OtherDatas->StrandsSize;

			SubSteps = OtherDatas->SubSteps;
			IterationCount = OtherDatas->IterationCount;

			GravityVector = OtherDatas->GravityVector;
			GravityPreloading = OtherDatas->GravityPreloading;
			AirDrag = OtherDatas->AirDrag;
			AirVelocity = OtherDatas->AirVelocity;

			SolveBend = OtherDatas->SolveBend;
			ProjectBend = OtherDatas->ProjectBend;
			BendDamping = OtherDatas->BendDamping;
			BendStiffness = OtherDatas->BendStiffness;

			SolveStretch = OtherDatas->SolveStretch;
			ProjectStretch = OtherDatas->ProjectStretch;
			StretchDamping = OtherDatas->StretchDamping;
			StretchStiffness = OtherDatas->StretchStiffness;

			SolveCollision = OtherDatas->SolveCollision;
			ProjectCollision = OtherDatas->ProjectCollision;
			StaticFriction = OtherDatas->StaticFriction;
			KineticFriction = OtherDatas->KineticFriction;
			StrandsViscosity = OtherDatas->StrandsViscosity;
			GridDimension = OtherDatas->GridDimension;
			CollisionRadius = OtherDatas->CollisionRadius;

			StrandsDensity = OtherDatas->StrandsDensity;
			StrandsSmoothing = OtherDatas->StrandsSmoothing;
			StrandsThickness = OtherDatas->StrandsThickness;

			ParamsScale = OtherDatas->ParamsScale;

			SkeletalMeshes = OtherDatas->SkeletalMeshes;

			TickingGroup = OtherDatas->TickingGroup;
			LocalSimulation = OtherDatas->LocalSimulation;
			TimeDilation = OtherDatas->TimeDilation;
		}
	}

	/** Cached World transform. */
	FTransform WorldTransform;

	/** Bone transform that will be used for local strands simulation */
	FTransform BoneTransform;
	
	/** Bone transform that will be used for local strands simulation */
	FTransform PreviousBoneTransform;

	/** Bone Linear Velocity */
	FVector3f BoneLinearVelocity;

	/** Bone Previous Linear Velocity */
	FVector3f PreviousBoneLinearVelocity;

	/** Bone Angular Velocity */
	FVector3f BoneAngularVelocity;

	/** Bone Previous Angular Velocity */
	FVector3f PreviousBoneAngularVelocity;

	/** Bone Linear Acceleration */
	FVector3f BoneLinearAcceleration;

	/** Bone Angular Acceleration */
	FVector3f BoneAngularAcceleration;

	/** Global Interpolation */
	bool GlobalInterpolation;
	
	/** Skinning transfer from a source to a target skelmesh */
    bool bSkinningTransfer;

	/** Number of strands*/
	int32 NumStrands;

	/** Strand size */
	int32 StrandsSize;

	/** Force reset simulation */
	bool ForceReset;

	/** Strands Gpu buffer */
	FNDIHairStrandsBuffer* HairStrandsBuffer = nullptr;

	/** Hair group index */
	int32 HairGroupIndex = -1;

	/** Hair group instance */
	FHairGroupInstance* HairGroupInstance = nullptr;

	/** Source component of the hair group instance */
	TWeakObjectPtr<class UGroomComponent> HairGroupInstSource;

	/** Binding type between the groom asset and the attached skeletal mesh */
	EHairBindingType BindingType;
	
	/** Number of substeps to be used */
	int32 SubSteps;

	/** Number of iterations for the constraint solver  */
	int32 IterationCount;

	/** Acceleration vector in cm/s2 to be used for the gravity*/
	FVector GravityVector;
	
	/** Optimisation of the rest state configuration to compensate from the gravity */
	float GravityPreloading;

	/** Coefficient between 0 and 1 to be used for the air drag */
	float AirDrag;

	/** Velocity of the surrounding air in cm/s  */
	FVector AirVelocity;

	/** Velocity of the surrounding air in cm/s */
	bool SolveBend;

	/** Enable the solve of the bend constraint during the xpbd loop */
	bool ProjectBend;

	/** Damping for the bend constraint between 0 and 1 */
	float BendDamping;

	/** Stiffness for the bend constraint in GPa */
	float BendStiffness;

	/** Enable the solve of the stretch constraint during the xpbd loop */
	bool SolveStretch;

	/** Enable the projection of the stretch constraint after the xpbd loop */
	bool ProjectStretch;

	/** Damping for the stretch constraint between 0 and 1 */
	float StretchDamping;

	/** Stiffness for the stretch constraint in GPa */
	float StretchStiffness;

	/** Enable the solve of the collision constraint during the xpbd loop  */
	bool SolveCollision;

	/** Enable ther projection of the collision constraint after the xpbd loop */
	bool ProjectCollision;

	/** Static friction used for collision against the physics asset */
	float StaticFriction;

	/** Kinetic friction used for collision against the physics asset*/
	float KineticFriction;

	/** Radius that will be used for the collision detection against the physics asset */
	float StrandsViscosity;

	/** Grid Dimension used to compute the viscosity forces */
	FIntVector GridDimension;

	/** Radius scale along the strand */
	float CollisionRadius;

	/** Density of the strands in g/cm3 */
	float StrandsDensity;

	/** Smoothing between 0 and 1 of the incoming guides curves for better stability */
	float StrandsSmoothing;

	/** Strands thickness in cm that will be used for mass and inertia computation */
	float StrandsThickness;

	/** Scales along the strand */
	TStaticArray<float, 32 * NumScales> ParamsScale;

	/** List of all the skel meshes in the hierarchy*/
	uint32 SkeletalMeshes;

	/** The instance ticking group */
	ETickingGroup TickingGroup;

	/** Check if the simulation is running in local coordinate */
	bool LocalSimulation;

	/** Get the world time dilation */
	float TimeDilation;

	/** Check if instance use Nanite geometry */
	bool bUsesNanite;
};

/** Data Interface for the strand base */
UCLASS(MinimalAPI, EditInlineNew, Category = "Strands", meta = (DisplayName = "Hair Strands"))
class UNiagaraDataInterfaceHairStrands : public UNiagaraDataInterface, public INiagaraSimCacheCustomStorageInterface
{
	GENERATED_UCLASS_BODY()

public:
	/** Hair Strands Asset used to sample from when not overridden by a source actor from the scene. Also useful for previewing in the editor. */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UGroomAsset> DefaultSource;

	/** The source actor from which to sample */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<AActor> SourceActor;

	/** The source component from which to sample */
	TWeakObjectPtr<class UGroomComponent> SourceComponent;

	/** UObject Interface */
	UE_API virtual void PostInitProperties() override;

	/** Begin UNiagaraDataInterface Interface */
	UE_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
	UE_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	UE_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDIHairStrandsData); }
	UE_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	UE_API virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;

	/** GPU simulation  functionality */
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	UE_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	UE_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	UE_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif
	UE_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	UE_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	UE_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
	/** End UNiagaraDataInterface Interface */

	/** Begin INiagaraSimCacheCustomStorageInterface Interface */
	UE_API virtual void SimCachePostReadFrame(void* OptionalPerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual TArray<FNiagaraVariableBase> GetSimCacheRendererAttributes(const UObject* UsageContext) const override;
	/** End INiagaraSimCacheCustomStorageInterface Interface */

	/** Update the source component */
	UE_API void ExtractSourceComponent(FNiagaraSystemInstance* SystemInstance);

	/** Check if the component is Valid */
	UE_API bool IsComponentValid() const;

	/** Extract datas and resources */
	UE_API void ExtractDatasAndResources(
		FNiagaraSystemInstance* SystemInstance, 
		FNDIHairStrandsInfo& Out);

	/** Get the number of strands */
	UE_API void GetNumStrands(FVectorVMExternalFunctionContext& Context);

	/** Get the groom asset datas  */
	UE_API void GetStrandSize(FVectorVMExternalFunctionContext& Context);

	UE_API void GetSubSteps(FVectorVMExternalFunctionContext& Context);

	UE_API void GetIterationCount(FVectorVMExternalFunctionContext& Context);

	UE_API void GetGravityVector(FVectorVMExternalFunctionContext& Context);

	UE_API void GetGravityPreloading(FVectorVMExternalFunctionContext& Context);

	UE_API void GetAirDrag(FVectorVMExternalFunctionContext& Context);

	UE_API void GetAirVelocity(FVectorVMExternalFunctionContext& Context);

	UE_API void GetSolveBend(FVectorVMExternalFunctionContext& Context);

	UE_API void GetProjectBend(FVectorVMExternalFunctionContext& Context);

	UE_API void GetBendDamping(FVectorVMExternalFunctionContext& Context);

	UE_API void GetBendStiffness(FVectorVMExternalFunctionContext& Context);

	UE_API void GetBendScale(FVectorVMExternalFunctionContext& Context);

	UE_API void GetSolveStretch(FVectorVMExternalFunctionContext& Context);

	UE_API void GetProjectStretch(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStretchDamping(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStretchStiffness(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStretchScale(FVectorVMExternalFunctionContext& Context);

	UE_API void GetSolveCollision(FVectorVMExternalFunctionContext& Context);

	UE_API void GetProjectCollision(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStaticFriction(FVectorVMExternalFunctionContext& Context);

	UE_API void GetKineticFriction(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStrandsViscosity(FVectorVMExternalFunctionContext& Context);

	UE_API void GetGridDimension(FVectorVMExternalFunctionContext& Context);

	UE_API void GetCollisionRadius(FVectorVMExternalFunctionContext& Context);

	UE_API void GetRadiusScale(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStrandsSmoothing(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStrandsDensity(FVectorVMExternalFunctionContext& Context);

	UE_API void GetTimeDilation(FVectorVMExternalFunctionContext& Context);

	UE_API void GetStrandsThickness(FVectorVMExternalFunctionContext& Context);

	UE_API void GetThicknessScale(FVectorVMExternalFunctionContext& Context);

	/** Get the world transform */
	UE_API void GetWorldTransform(FVectorVMExternalFunctionContext& Context);

	/** Get the world inverse */
	UE_API void GetWorldInverse(FVectorVMExternalFunctionContext& Context);

	/** Get the strand vertex position in world space*/
	UE_API void GetPointPosition(FVectorVMExternalFunctionContext& Context);

	/** Get the strand node position in world space*/
	UE_API void ComputeNodePosition(FVectorVMExternalFunctionContext& Context);

	/** Get the strand node orientation in world space*/
	UE_API void ComputeNodeOrientation(FVectorVMExternalFunctionContext& Context);

	/** Get the strand node mass */
	UE_API void ComputeNodeMass(FVectorVMExternalFunctionContext& Context);

	/** Get the strand node inertia */
	UE_API void ComputeNodeInertia(FVectorVMExternalFunctionContext& Context);

	/** Compute the edge length (diff between 2 nodes positions)*/
	UE_API void ComputeEdgeLength(FVectorVMExternalFunctionContext& Context);

	/** Compute the edge orientation (diff between 2 nodes orientations) */
	UE_API void ComputeEdgeRotation(FVectorVMExternalFunctionContext& Context);

	/** Compute the rest local position */
	UE_API void ComputeRestPosition(FVectorVMExternalFunctionContext& Context);

	/** Compute the rest local orientation */
	UE_API void ComputeRestOrientation(FVectorVMExternalFunctionContext& Context);

	/** Update the root node orientation based on the current transform */
	UE_API void AttachNodePosition(FVectorVMExternalFunctionContext& Context);

	/** Update the root node position based on the current transform */
	UE_API void AttachNodeOrientation(FVectorVMExternalFunctionContext& Context);

	/** Report the node displacement onto the points position*/
	UE_API void UpdatePointPosition(FVectorVMExternalFunctionContext& Context);

	/** Reset the point position to be the rest one */
	UE_API void ResetPointPosition(FVectorVMExternalFunctionContext& Context);

	/** Add external force to the linear velocity and advect node position */
	UE_API void AdvectNodePosition(FVectorVMExternalFunctionContext& Context);

	/** Add external torque to the angular velocity and advect node orientation*/
	UE_API void AdvectNodeOrientation(FVectorVMExternalFunctionContext& Context);

	/** Update the node linear velocity based on the node position difference */
	UE_API void UpdateLinearVelocity(FVectorVMExternalFunctionContext& Context);

	/** Update the node angular velocity based on the node orientation difference */
	UE_API void UpdateAngularVelocity(FVectorVMExternalFunctionContext& Context);

	/** Get the bounding box center */
	UE_API void GetBoundingBox(FVectorVMExternalFunctionContext& Context);

	/** Reset the bounding box extent */
	UE_API void ResetBoundingBox(FVectorVMExternalFunctionContext& Context);

	/** Build the groom bounding box */
	UE_API void BuildBoundingBox(FVectorVMExternalFunctionContext& Context);

	/** Setup the distance spring material */
	UE_API void SetupDistanceSpringMaterial(FVectorVMExternalFunctionContext& Context);

	/** Solve the distance spring material */
	UE_API void SolveDistanceSpringMaterial(FVectorVMExternalFunctionContext& Context);

	/** Project the distance spring material */
	UE_API void ProjectDistanceSpringMaterial(FVectorVMExternalFunctionContext& Context);

	/** Setup the angular spring material */
	UE_API void SetupAngularSpringMaterial(FVectorVMExternalFunctionContext& Context);

	/** Solve the angular spring material */
	UE_API void SolveAngularSpringMaterial(FVectorVMExternalFunctionContext& Context);

	/** Project the angular spring material */
	UE_API void ProjectAngularSpringMaterial(FVectorVMExternalFunctionContext& Context);

	/** Setup the stretch rod material */
	UE_API void SetupStretchRodMaterial(FVectorVMExternalFunctionContext& Context);

	/** Solve the stretch rod material */
	UE_API void SolveStretchRodMaterial(FVectorVMExternalFunctionContext& Context);

	/** Project the stretch rod material */
	UE_API void ProjectStretchRodMaterial(FVectorVMExternalFunctionContext& Context);

	/** Setup the bend rod material */
	UE_API void SetupBendRodMaterial(FVectorVMExternalFunctionContext& Context);

	/** Solve the bend rod material */
	UE_API void SolveBendRodMaterial(FVectorVMExternalFunctionContext& Context);

	/** Project the bend rod material */
	UE_API void ProjectBendRodMaterial(FVectorVMExternalFunctionContext& Context);

	/** Solve the static collision constraint */
	UE_API void SolveHardCollisionConstraint(FVectorVMExternalFunctionContext& Context);

	/** Project the static collision constraint */
	UE_API void ProjectHardCollisionConstraint(FVectorVMExternalFunctionContext& Context);

	/** Solve the soft collision constraint */
	UE_API void SolveSoftCollisionConstraint(FVectorVMExternalFunctionContext& Context);

	/** Project the soft collision constraint */
	UE_API void ProjectSoftCollisionConstraint(FVectorVMExternalFunctionContext& Context);

	/** Setup the soft collision constraint */
	UE_API void SetupSoftCollisionConstraint(FVectorVMExternalFunctionContext& Context);

	/** Compute the rest direction*/
	UE_API void ComputeEdgeDirection(FVectorVMExternalFunctionContext& Context);

	/** Update the strands material frame */
	UE_API void UpdateMaterialFrame(FVectorVMExternalFunctionContext& Context);

	/** Compute the strands material frame */
	UE_API void ComputeMaterialFrame(FVectorVMExternalFunctionContext& Context);

	/** Compute the air drag force */
	UE_API void ComputeAirDragForce(FVectorVMExternalFunctionContext& Context);

	/** Get the rest position and orientation relative to the transform or to the skin cache */
	UE_API void ComputeLocalState(FVectorVMExternalFunctionContext& Context);

	/** Attach the node position and orientation to the transform or to the skin cache */
	UE_API void AttachNodeState(FVectorVMExternalFunctionContext& Context);

	/** Update the node position and orientation based on rbf transfer */
	UE_API void UpdateNodeState(FVectorVMExternalFunctionContext& Context);

	/** Check if we need or not a simulation reset*/
	UE_API void NeedSimulationReset(FVectorVMExternalFunctionContext& Context);

	/** Check if we have a global interpolation */
	UE_API void HasGlobalInterpolation(FVectorVMExternalFunctionContext& Context);

	/** Check if we need a rest pose update */
	UE_API void NeedRestUpdate(FVectorVMExternalFunctionContext& Context);

	/** Eval the skinned position given a rest position*/
	UE_API void EvalSkinnedPosition(FVectorVMExternalFunctionContext& Context);

	/** Init the samples along the strands that will be used to transfer informations to the grid */
	UE_API void InitGridSamples(FVectorVMExternalFunctionContext& Context);

	/** Get the sample state given an index */
	UE_API void GetSampleState(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	/** Copy one niagara DI to this */
	UE_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
};

/** Proxy to send data to gpu */
struct FNDIHairStrandsProxy : public FNiagaraDataInterfaceProxy
{
	/** Get the size of the data that will be passed to render*/
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIHairStrandsData); }

	/** Get the data that will be passed to render*/
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;

	/** Initialize the Proxy data strands buffer */
	void InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Destroy the proxy data if necessary */
	void DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Launch all pre stage functions */
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;

	/** MGPU buffer copy after simulation*/
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	/** List of proxy data for each system instances*/
	TMap<FNiagaraSystemInstanceID, FNDIHairStrandsData> SystemInstancesToProxyData;
};

#undef UE_API
