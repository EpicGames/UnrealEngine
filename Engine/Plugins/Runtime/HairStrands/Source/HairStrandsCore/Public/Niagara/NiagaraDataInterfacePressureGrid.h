// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceVelocityGrid.h"
#include "NiagaraDataInterfacePressureGrid.generated.h"

#define UE_API HAIRSTRANDSCORE_API

/** Data Interface for the strand base */
UCLASS(MinimalAPI, EditInlineNew, Category = "Grid", meta = (DisplayName = "Pressure Grid"))
class UNiagaraDataInterfacePressureGrid : public UNiagaraDataInterfaceVelocityGrid
{
	GENERATED_UCLASS_BODY()

public:
	/** UNiagaraDataInterface Interface */
	UE_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool HasPreSimulateTick() const override { return true; }

	/** GPU simulation  functionality */
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	UE_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	UE_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	UE_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
#endif

	/** Build the velocity field */
	UE_API void BuildDistanceField(FVectorVMExternalFunctionContext& Context);

	/** Project the velocity field to be divergence free */
	UE_API void SolveGridPressure(FVectorVMExternalFunctionContext& Context);

	/** Scale Cell Fields */
	UE_API void ScaleCellFields(FVectorVMExternalFunctionContext& Context);

	/** Set the solid boundary */
	UE_API void SetSolidBoundary(FVectorVMExternalFunctionContext& Context);

	/** Compute the solid weights */
	UE_API void ComputeBoundaryWeights(FVectorVMExternalFunctionContext& Context);

	/** Get Node Position */
	UE_API void GetNodePosition(FVectorVMExternalFunctionContext& Context);

	/** Get Density Field */
	UE_API void GetDensityField(FVectorVMExternalFunctionContext& Context);

	/** Build the Density Field */
	UE_API void BuildDensityField(FVectorVMExternalFunctionContext& Context);

	/** Update the deformation gradient */
	UE_API void UpdateDeformationGradient(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
};

/** Proxy to send data to gpu */
struct FNDIPressureGridProxy : public FNDIVelocityGridProxy
{
	/** Launch all pre stage functions */
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;
};

#undef UE_API
