// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraRenderGraphUtils.h"
#include "NiagaraCommon.h"
#include "VectorVM.h"
#include "NiagaraDataInterfaceVelocityGrid.generated.h"

#define UE_API HAIRSTRANDSCORE_API


/** Render buffers that will be used in hlsl functions */
struct FNDIVelocityGridBuffer : public FRenderResource
{
	/** Set the grid size */
	void Initialize(const FIntVector GridSize, const int32 NumAttributes);

	/** Init the buffer */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Release the buffer */
	virtual void ReleaseRHI() override;

	/** Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FNDIVelocityGridBuffer"); }

	/** Grid data texture */
	FNiagaraPooledRWBuffer GridDataBuffer;

	/** Grid size that will be used for the collision*/
	FIntVector GridSize;

	/** Num attributes in the buffer*/
	int32 NumAttributes;
};

/** Data stored per strand base instance*/
struct FNDIVelocityGridData
{
	/** Swap the current and the destination data */
	void Swap();

	/** Initialize the buffers */
	bool Init(const FIntVector& InGridSize, const int32 InNumAttributes, FNiagaraSystemInstance* SystemInstance);

	/** Release the buffers */
	void Release();

	/** Resize the buffers */
	void Resize();

	/** Grid Size */
	FIntVector GridSize;

	/** Num Attributes */
	int32 NumAttributes;

	/** Need a resize */
	bool NeedResize;

	/** World Transform */
	FMatrix WorldTransform;

	/** Inverse world transform */
	FMatrix WorldInverse;

	/** Pointer to the current buffer */
	FNDIVelocityGridBuffer* CurrentGridBuffer;

	/** Pointer to the destination buffer */
	FNDIVelocityGridBuffer* DestinationGridBuffer;
};

/** Data Interface for the strand base */
UCLASS(MinimalAPI, EditInlineNew, Category = "Grid", meta = (DisplayName = "Velocity Grid"))
class UNiagaraDataInterfaceVelocityGrid : public UNiagaraDataInterfaceRWBase
{
	GENERATED_UCLASS_BODY()

public:
	/** Grid size along the X axis. */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	FIntVector GridSize;

	/** Num Attributes */
	int32 NumAttributes;

	/** UObject Interface */
	UE_API virtual void PostInitProperties() override;

	/** UNiagaraDataInterface Interface */
	UE_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return Target == ENiagaraSimTarget::GPUComputeSim; }
	UE_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	UE_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDIVelocityGridData); }
	UE_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;

	/** GPU simulation  functionality */
#if WITH_EDITORONLY_DATA
	UE_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	UE_API virtual void GetCommonHLSL(FString& OutHLSL) override;
	UE_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	UE_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	UE_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	UE_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	UE_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	/** Build the velocity field */
	UE_API void BuildVelocityField(FVectorVMExternalFunctionContext& Context);

	/** Sample the grid */
	UE_API void SampleVelocityField(FVectorVMExternalFunctionContext& Context);

	/** Compute the grid Size (Origin and length) */
	UE_API void ComputeGridSize(FVectorVMExternalFunctionContext& Context);

	/** Update the grid transform */
	UE_API void UpdateGridTransform(FVectorVMExternalFunctionContext& Context);

	/** Set the grid dimension */
	UE_API void SetGridDimension(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif
	/** Copy one niagara DI to this */
	UE_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
};

/** Proxy to send data to gpu */
struct FNDIVelocityGridProxy : public FNiagaraDataInterfaceProxyRW
{
	/** Get the size of the data that will be passed to render*/
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIVelocityGridData); }

	/** Get the data that will be passed to render*/
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;

	/** Launch all pre stage functions */
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;

	/** Launch all post stage functions */
	virtual void PostStage(const FNDIGpuComputePostStageContext& Context) override;

	/** Called at the end of each simulate tick. */
	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	/** Reset the buffers  */
	virtual void ResetData(const FNDIGpuComputeResetContext& Context) override;

	// Get the dispatch arguments for this instance
	virtual void GetDispatchArgs(const FNDIGpuComputeDispatchArgsGenContext& Context) override;

	/** List of proxy data for each system instances*/
	TMap<FNiagaraSystemInstanceID, FNDIVelocityGridData> SystemInstancesToProxyData;
};

#undef UE_API
