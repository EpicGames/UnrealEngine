// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairStrandsDataflow: ModuleRules
	{
		public HairStrandsDataflow(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEditor",
					"DataflowNodes",
					"DataflowEnginePlugin",
					"HairStrandsCore",
					"Engine",
					"GeometryAlgorithms",
					"GeometryCore",
					"MeshConversion",
					"MeshConversionEngineTypes",
					"MeshDescription",
					"DynamicMesh",
					"SkeletalMeshDescription",
					"SkeletalMeshUtilitiesCommon",
					"StaticMeshDescription",
					"Slate",
					"SlateCore",
					"UnrealEd"
				}
			);
		}
	}
}