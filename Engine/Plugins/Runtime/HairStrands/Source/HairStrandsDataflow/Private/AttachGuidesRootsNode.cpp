// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttachGuidesRootsNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttachGuidesRootsNode)

void FAttachGuidesRootsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&KinematicWeightsKey))
	{
		FCollectionAttributeKey Key;
		Key.Group =  FGeometryCollection::VerticesGroup.ToString();
		Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::KinematicWeightAttributeName.ToString();
		
		SetValue(Context, Key, &KinematicWeightsKey);
	}
}

FAttachCurveRootsDataflowNode::FAttachCurveRootsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CurveSelection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&KinematicWeightsKey);
}

void FAttachCurveRootsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GeometryCollection);

		if(CurvesFacade.IsValid())
		{
			GeometryCollection::Facades::FVertexBoneWeightsFacade SkinningFacade(GeometryCollection, false);
			SkinningFacade.DefineSchema();
			
			const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());
			
			const FDataflowCurveSelection& DataflowSelection = GetValue<FDataflowCurveSelection>(Context, &CurveSelection);
			
			ParallelFor(CurvesFacade.GetNumCurves(), [&SkinningFacade, &CurvesFacade, DataflowSelection, bValidSelection](int32 CurveIndex)
			{
				int32 PointIndex = (CurveIndex == 0) ? 0 : CurvesFacade.GetCurvePointOffsets()[CurveIndex-1];
				if(!bValidSelection || (bValidSelection && DataflowSelection.IsSelected(CurveIndex)))
				{
					SkinningFacade.ModifyKinematicWeight(2*PointIndex,1.0f);
					SkinningFacade.ModifyKinematicWeight(2*PointIndex+1, 1.0f);
				}
					
				PointIndex += 1;
				if(!bValidSelection || (bValidSelection && DataflowSelection.IsSelected(CurveIndex)))
				{
					SkinningFacade.ModifyKinematicWeight(2*PointIndex, 1.0f);
					SkinningFacade.ModifyKinematicWeight(2*PointIndex+1, 1.0f);
				}
			}, EParallelForFlags::None);
		}
		
		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&KinematicWeightsKey))
	{
		FCollectionAttributeKey Key;
		Key.Group =  FGeometryCollection::VerticesGroup.ToString();
		Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::KinematicWeightAttributeName.ToString();
		
		SetValue(Context, Key, &KinematicWeightsKey);
	}
}

void FBuildCurveWeightsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GeometryCollection);

		if(CurvesFacade.IsValid())
		{
			const FCollectionAttributeKey WeightKey = GetValue<FCollectionAttributeKey>(Context, &WeightsAttribute);
			const FDataflowCurveSelection& DataflowSelection = GetValue<FDataflowCurveSelection>(Context, &CurveSelection);
			
			if(TManagedArray<float>* WeightsValue = GeometryCollection.FindAttribute<float>(FName(WeightKey.Attribute), FName(WeightKey.Group)))
			{
				const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());;
				
				ParallelFor(CurvesFacade.GetNumPoints(), [this, &WeightsValue, &CurvesFacade, &DataflowSelection, bValidSelection](int32 PointIndex)
				{
					const int32 CurveIndex = CurvesFacade.GetPointCurveIndices()[PointIndex];
					if(!bValidSelection || (bValidSelection && DataflowSelection.IsSelected(CurveIndex)))
					{
						const int32 NextPoint = CurvesFacade.GetCurvePointOffsets()[CurveIndex];
						const int32 PrevPoint = (CurveIndex == 0) ? 0 : CurvesFacade.GetCurvePointOffsets()[CurveIndex-1];
						
						const float PointCoord = static_cast<float>(PointIndex - PrevPoint) / (NextPoint - 1 - PrevPoint);

						(*WeightsValue)[2 * PointIndex] = CurveWeights.GetRichCurveConst()->Eval(PointCoord);
						(*WeightsValue)[2 * PointIndex + 1] = (*WeightsValue)[2 * PointIndex];
					}
				}, EParallelForFlags::None);
			}
		}
		
		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
}


