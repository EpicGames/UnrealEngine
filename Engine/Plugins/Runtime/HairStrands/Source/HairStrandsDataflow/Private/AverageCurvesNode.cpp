// Copyright Epic Games, Inc. All Rights Reserved.

#include "AverageCurvesNode.h"

#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AverageCurvesNode)

#define LOCTEXT_NAMESPACE "AverageCurvesNode"

FDataflowAverageCurves::FDataflowAverageCurves(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CurveSelection);
	RegisterInputConnection(&NumSamples);
	RegisterOutputConnection(&Collection);
}

void FDataflowAverageCurves::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection OutCollection;

		const int32 InNumSamples = GetValue(Context, &NumSamples);
		if (InNumSamples < 2)
		{
			Context.Error(LOCTEXT("InvalidCollection", "Collection does not contain curves, result collection will be empty"), this, Out);
			SetValue(Context, MoveTemp(OutCollection), &Collection);
			return;
		}

		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowCurveSelection& InCurveSelection = GetValue(Context, &CurveSelection);
		if (!InCurveSelection.IsValidForCollection(InCollection))
		{
			Context.Error(LOCTEXT("InvalidCollection", "Collection does not contain curves, result collection will be empty"), this, Out);
			SetValue(Context, MoveTemp(OutCollection), &Collection);
			return;
		}

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(InCollection);
		if (!CurvesFacade.IsValid())
		{
			Context.Error(LOCTEXT("InvalidCollection", "Collection does not contain curves, result collection will be empty"), this, Out);
			SetValue(Context, MoveTemp(OutCollection), &Collection);
			return;
		}

		TArray<FVector3f> AveragedCurve;
		AveragedCurve.Init(FVector3f::ZeroVector, InNumSamples);
		int32 NumProcessedCurves = 0;

		TArray<FVector3f> ResampledCurve;
		const int32 NumCurves = CurvesFacade.GetNumCurves();
		for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			if (InCurveSelection.IsSelected(CurveIndex))
			{
				CurvesFacade.GetCurveResampledPositions(CurveIndex, InNumSamples, ResampledCurve);
				const int32 MaxIndex = FMath::Min(ResampledCurve.Num(), InNumSamples);
				for (int32 Index = 0; Index < MaxIndex; ++Index)
				{
					AveragedCurve[Index] += ResampledCurve[Index];
				}
				++NumProcessedCurves;
			}
		}
		
		if (NumProcessedCurves == 0)
		{
			Context.Error(LOCTEXT("NoCurveSelected", "Selection seems empty, result collection will be empty"), this, Out);
			SetValue(Context, MoveTemp(OutCollection), &Collection);
			return;
		}

		const float InvNumProcessedCurves = 1.0f / (float)NumProcessedCurves;
		for (int32 Index = 0; Index < AveragedCurve.Num(); ++Index)
		{
			AveragedCurve[Index] *= InvNumProcessedCurves;
		}

		// now add the resample curve to the collection
		const TArray<int32> CurvePointOffsets = { AveragedCurve.Num() };
		const TArray<int32> GeometryCurveOffsets = { 1 };
		const TArray<FString> GeometryGroupNames = { TEXT("AverageCurve") };
		const TArray<float> GeometryCurveThickness = { 0.5f };
		const TArray<int32> CurveSourceIndices = { 0 };

		GeometryCollection::Facades::FCollectionCurveGeometryFacade OutCurvesFacade(OutCollection);
		OutCurvesFacade.InitCurvesCollection(AveragedCurve, CurvePointOffsets, GeometryCurveOffsets, GeometryGroupNames, GeometryCurveThickness, CurveSourceIndices);

		SetValue(Context, MoveTemp(OutCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE