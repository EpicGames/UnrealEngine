// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowTerminalNode.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "AverageCurvesNode.generated.h"

/** 
* Take an collection curve selection and generate a single curve collection that is the average of the input curves
*/
USTRUCT(meta = (Experimental, DataflowGroom))
struct FDataflowAverageCurves : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowAverageCurves, "AverageCurves", "Groom", "Curves Guide Groom")

public:
	FDataflowAverageCurves(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection source of the curves to be average from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Curve selection to use */
	UPROPERTY(meta = (DataflowInput))
	FDataflowCurveSelection CurveSelection;

	/** Number of samples to use for averaged result curve */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DataflowInput, ClampMin = 2, UIMin = 2))
	int32 NumSamples = 8;
};
