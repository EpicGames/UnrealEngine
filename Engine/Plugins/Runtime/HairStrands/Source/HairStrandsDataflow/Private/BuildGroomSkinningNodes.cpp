// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildGroomSkinningNodes.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshLODRenderDataToDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "DynamicMesh/MeshTransforms.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "Operations/TransferBoneWeights.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuildGroomSkinningNodes)

namespace UE::Groom::Private
{
	static bool SkeletalMeshToDynamicMesh(const USkeletalMesh* SkeletalMesh, const int32 LodIndex, UE::Geometry::FDynamicMesh3& ToDynamicMesh)
	{
		if (SkeletalMesh->HasMeshDescription(LodIndex))
		{
			const FMeshDescription* SourceMesh = SkeletalMesh->GetMeshDescription(LodIndex);
			if (!SourceMesh)
			{
				return false;
			}

			FMeshDescriptionToDynamicMesh Converter;
			Converter.Convert(SourceMesh, ToDynamicMesh);
		}
		else
		{ 
			const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
			if (!RenderData)
			{
				return false;
			}

			if (!RenderData->LODRenderData.IsValidIndex(LodIndex))
			{
				return false;
			}

			const FSkeletalMeshLODRenderData* SkeletalMeshLODRenderData = &(RenderData->LODRenderData[LodIndex]);

			UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::ConversionOptions ConversionOptions;
			ConversionOptions.bWantUVs = false;
			ConversionOptions.bWantVertexColors = false;
			ConversionOptions.bWantMaterialIDs = false;
			ConversionOptions.bWantSkinWeights = true;

			UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::Convert(SkeletalMeshLODRenderData, SkeletalMesh->GetRefSkeleton(), ConversionOptions, ToDynamicMesh);
		}

		return true;
	}
		
	static void BuildGeometrySkinWeights(GeometryCollection::Facades::FCollectionMeshFacade& MeshFacade, GeometryCollection::Facades::FVertexBoneWeightsFacade& SkinningFacade,
		const FDataflowVertexSelection& VertexSelection, const TObjectPtr<USkeletalMesh> SkeletalMesh, const int32 LODIndex, const FTransform& RelativeTransform)
	{
		if(MeshFacade.IsValid() && SkeletalMesh && SkeletalMesh->IsValidLODIndex(LODIndex))
		{
			const bool bValidSelection = VertexSelection.IsValidForCollection(SkinningFacade.GetManagedArrayCollection());

			TMap<FName, FBoneIndexType> TargetBoneToIndex;
			TargetBoneToIndex.Reserve(SkeletalMesh->GetRefSkeleton().GetRawBoneNum());
			for (int32 BoneIdx = 0; BoneIdx < SkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++BoneIdx)
			{
				TargetBoneToIndex.Add(SkeletalMesh->GetRefSkeleton().GetRawRefBoneInfo()[BoneIdx].Name, BoneIdx);
			}
			UE::Geometry::FDynamicMesh3 DynamicMesh;
			if (UE::Groom::Private::SkeletalMeshToDynamicMesh(SkeletalMesh, LODIndex, DynamicMesh))
			{
				MeshTransforms::ApplyTransform(DynamicMesh, RelativeTransform, true);

				UE::Geometry::FTransferBoneWeights TransferBoneWeights(&DynamicMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
				TransferBoneWeights.bUseParallel = true;
				TransferBoneWeights.MaxNumInfluences = 4;
				TransferBoneWeights.TransferMethod = UE::Geometry::FTransferBoneWeights::ETransferBoneWeightsMethod::ClosestPointOnSurface;

				if (TransferBoneWeights.Validate() == UE::Geometry::EOperationValidationResult::Ok)
				{
					for(int32 GeometryIndex = 0; GeometryIndex < SkinningFacade.NumGeometry(); ++GeometryIndex)
					{
						SkinningFacade.ModifyGeometryBinding(GeometryIndex, SkeletalMesh, LODIndex);
						
						const TArrayView<const FVector3f> VertexPositions = MeshFacade.GetVertexPositions(GeometryIndex);
						const TArray<int32> VertexIndices = MeshFacade.GetVertexIndices(GeometryIndex);
						
						const int32 NumVertices = VertexPositions.Num();
						ParallelFor(NumVertices, [&TransferBoneWeights, &TargetBoneToIndex, &VertexPositions, &VertexIndices, &SkinningFacade,
							&VertexSelection, bValidSelection](const int32 LocalIndex)
						{
							const int32 VertexIndex = VertexIndices[LocalIndex];

							if(!bValidSelection || (bValidSelection && (VertexSelection.IsSelected(VertexIndex))))
							{
								TArray<int32> BoneIndices;
								TArray<float> BoneWeights;
									
								TransferBoneWeights.TransferWeightsToPoint(BoneIndices, BoneWeights,VertexPositions[LocalIndex], &TargetBoneToIndex);
								SkinningFacade.ModifyBoneWeight(VertexIndex, BoneIndices, BoneWeights);
							}
						}, TransferBoneWeights.bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
					}
				}
			}
		}
	}

	FCollectionAttributeKey GetBoneIndicesKey()
	{
		FCollectionAttributeKey Key;
		Key.Group = FGeometryCollection::VerticesGroup.ToString();
		Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::BoneIndicesAttributeName.ToString();
	
		return Key;
	}

	FCollectionAttributeKey GetBoneWeightsKey()
	{
		FCollectionAttributeKey Key;
		Key.Group = FGeometryCollection::VerticesGroup.ToString();
		Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::BoneWeightsAttributeName.ToString();
		return Key;
	}
}

FTransferSkinWeightsGroomNode::FTransferSkinWeightsGroomNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&BoneIndicesKey);
	RegisterOutputConnection(&BoneWeightsKey);
	RegisterOutputConnection(&SkeletalMesh, &SkeletalMesh);
}

void FTransferSkinWeightsGroomNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneIndicesKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneIndicesKey(), &BoneIndicesKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneWeightsKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneWeightsKey(), &BoneWeightsKey);
	}
	else if (Out->IsA<TObjectPtr<USkeletalMesh>>(&SkeletalMesh))
	{
		SafeForwardInput(Context, &SkeletalMesh, &SkeletalMesh);
	}
}

FTransferGeometrySkinWeightsDataflowNode::FTransferGeometrySkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&VertexSelection);
	RegisterInputConnection(&SkeletalMesh);
	RegisterInputConnection(&LODIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RelativeTransform).SetCanHidePin(true).SetPinIsHidden(true);
	
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&BoneIndicesKey);
	RegisterOutputConnection(&BoneWeightsKey);
}

void FTransferGeometrySkinWeightsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue(Context, &Collection);
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(GeometryCollection);

		if(MeshFacade.IsValid())
		{
			GeometryCollection::Facades::FVertexBoneWeightsFacade SkinningFacade(GeometryCollection, false);
			SkinningFacade.DefineSchema();
			
			UE::Groom::Private::BuildGeometrySkinWeights(MeshFacade, SkinningFacade, GetValue<FDataflowVertexSelection>(Context, &VertexSelection),
				GetValue<TObjectPtr<USkeletalMesh>>(Context, &SkeletalMesh), GetValue<int32>(Context, &LODIndex),
				GetValue<FTransform>(Context, &RelativeTransform));
		}

		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneIndicesKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneIndicesKey(), &BoneIndicesKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneWeightsKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneWeightsKey(), &BoneWeightsKey);
	}
}





