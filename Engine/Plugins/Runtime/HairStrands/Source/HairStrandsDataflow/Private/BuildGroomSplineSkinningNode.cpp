// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildGroomSplineSkinningNode.h"
#include "BuildGroomSkinningNodes.h"
#include "Algo/MaxElement.h"
#include "Algo/Count.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuildGroomSplineSkinningNode)

namespace UE::Groom::Private
{
	const FName FCollectionSplineAttributes::VertexSplineParamAttribute("SplineParam");
	const FName FCollectionSplineAttributes::VertexSplineBonesAttribute("SplineBones");

	FCollectionAttributeKey GetSplineParamKey()
	{
		FCollectionAttributeKey Key;
		Key.Group = FGeometryCollection::VerticesGroup.ToString();
		Key.Attribute = UE::Groom::Private::FCollectionSplineAttributes::VertexSplineParamAttribute.ToString();
	
		return Key;
	}

	FCollectionAttributeKey GetSplineBonesKey()
	{
		FCollectionAttributeKey Key;
		Key.Group = FGeometryCollection::VerticesGroup.ToString();
		Key.Attribute = UE::Groom::Private::FCollectionSplineAttributes::VertexSplineBonesAttribute.ToString();
		return Key;
	}
	
	struct FCatmullRomSpline
	{
		TArray<FVector> CVs;
		TArray<float> Params;
		FIntVector2 Bones;

		void ClosestParam(const FVector& Position, float& Param, float& ClosestDistance) const;
		void Init(FReferenceSkeleton& Skeleton, int32 RootIndex, int32 NumSamplesPerSegment);
	};


	void FCatmullRomSpline::Init(FReferenceSkeleton& Skeleton, int32 RootIndex, int32 NumSamplesPerSegment)
	{
		// Spline evaluation code from DrawCentripetalCatmullRomSpline (DrawDebugHelpers.cpp)
		const float Alpha = 0.5f; // Centripedal Catmull-Rom

		auto GetT = [](float T, float Alpha, const FVector& P0, const FVector& P1)
			{
				const FVector P1P0 = P1 - P0;
				const float Dot = P1P0 | P1P0;
				const float Pow = FMath::Pow(Dot, Alpha * .5f);
				return Pow + T;
			};

		TArray<int32> BoneIndices;
		BoneIndices.SetNum(0);
		BoneIndices.Add(RootIndex);
		TArray<int32> Children;
		int32 ParentIndex = RootIndex;
		while (Skeleton.GetDirectChildBones(BoneIndices.Last(), Children))
		{
			if (Children.Num() > 0)
			{
				BoneIndices.Add(Children[0]);
			}
			else
			{
				// TODO: Add support for branching splines
			}
		}

		Bones[0] = RootIndex;
		Bones[1] = BoneIndices.Last();

		TArray<FTransform> BoneTransforms;
		Skeleton.GetRawBoneAbsoluteTransforms(BoneTransforms);

		const int32 NumSegments = BoneIndices.Num() - 1;

		CVs.SetNum(NumSegments * NumSamplesPerSegment + 1);
		Params.SetNum(NumSegments * NumSamplesPerSegment + 1);

		for (int32 BoneIndex = 0; BoneIndex < NumSegments; ++BoneIndex)
		{
			const FVector P0 = BoneTransforms[BoneIndices[FMath::Max(BoneIndex - 1, 0)			]].GetTranslation();
			const FVector P1 = BoneTransforms[BoneIndices[BoneIndex								]].GetTranslation();
			const FVector P2 = BoneTransforms[BoneIndices[BoneIndex + 1							]].GetTranslation();
			const FVector P3 = BoneTransforms[BoneIndices[FMath::Min(BoneIndex + 2, NumSegments)]].GetTranslation();

			const float T0 = 0.0f;
			const float T1 = GetT(T0, Alpha, P0, P1);
			const float T2 = GetT(T1, Alpha, P1, P2);
			const float T3 = GetT(T2, Alpha, P2, P3);

			const float T1T0 = T1 - T0;
			const float T2T1 = T2 - T1;
			const float T3T2 = T3 - T2;
			const float T2T0 = T2 - T0;
			const float T3T1 = T3 - T1;

			const bool bIsNearlyZeroT1T0 = FMath::IsNearlyZero(T1T0, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT2T1 = FMath::IsNearlyZero(T2T1, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT3T2 = FMath::IsNearlyZero(T3T2, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT2T0 = FMath::IsNearlyZero(T2T0, UE_KINDA_SMALL_NUMBER);
			const bool bIsNearlyZeroT3T1 = FMath::IsNearlyZero(T3T1, UE_KINDA_SMALL_NUMBER);

			for (int SampleIndex = 0; SampleIndex < NumSamplesPerSegment; ++SampleIndex)
			{
				const float ParametricDistance = float(SampleIndex) / float(NumSamplesPerSegment);

				const float T = FMath::Lerp(T1, T2, ParametricDistance);

				const FVector A1 = bIsNearlyZeroT1T0 ? P0 : (T1 - T) / T1T0 * P0 + (T - T0) / T1T0 * P1;
				const FVector A2 = bIsNearlyZeroT2T1 ? P1 : (T2 - T) / T2T1 * P1 + (T - T1) / T2T1 * P2;
				const FVector A3 = bIsNearlyZeroT3T2 ? P2 : (T3 - T) / T3T2 * P2 + (T - T2) / T3T2 * P3;
				const FVector B1 = bIsNearlyZeroT2T0 ? A1 : (T2 - T) / T2T0 * A1 + (T - T0) / T2T0 * A2;
				const FVector B2 = bIsNearlyZeroT3T1 ? A2 : (T3 - T) / T3T1 * A2 + (T - T1) / T3T1 * A3;

				CVs[BoneIndex * NumSamplesPerSegment + SampleIndex] = bIsNearlyZeroT2T1 ? B1 : (T2 - T) / T2T1 * B1 + (T - T1) / T2T1 * B2;
				Params[BoneIndex * NumSamplesPerSegment + SampleIndex] = float(BoneIndices[BoneIndex]) + ParametricDistance;
			}
		}

		CVs.Last() = BoneTransforms[BoneIndices.Last()].GetTranslation();
		Params.Last() = BoneIndices.Last();
	}

	void FCatmullRomSpline::ClosestParam(const FVector& Position, float& Param, float& ClosestDistance) const
	{
		ClosestDistance = UE_MAX_FLT;
		Param = 0;

		for (int32 SampleIndex = 0; SampleIndex < CVs.Num(); ++SampleIndex)
		{
			double Distance = FVector::DistSquared(Position, CVs[SampleIndex]);

			if (Distance < ClosestDistance)
			{
				Param = Params[SampleIndex];
				ClosestDistance = Distance;
			}
		}
	}

	void BuildSplines(const TObjectPtr<USkeletalMesh> SkeletalMesh, const TArray<FString>& RootBones, int32 SamplesPerSegment, TArray<UE::Groom::Private::FCatmullRomSpline>& Splines)
	{
		if (SkeletalMesh)
		{
			if (RootBones.Num() > 0)
			{
				for (int32 BoneIndex = 0; BoneIndex < RootBones.Num(); ++BoneIndex)
				{
					int32 RootBoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(FName(*RootBones[BoneIndex]));
					if (RootBoneIndex != INDEX_NONE)
					{
						UE::Groom::Private::FCatmullRomSpline Spline;
						Spline.Init(SkeletalMesh->GetRefSkeleton(), RootBoneIndex, SamplesPerSegment);
						Splines.Add(Spline);
					}
				}
			}

			// Use skeleton root if no root bones specified or found
			if (Splines.IsEmpty())
			{
				UE::Groom::Private::FCatmullRomSpline Spline;
				Spline.Init(SkeletalMesh->GetRefSkeleton(), 0, SamplesPerSegment);
				Splines.Add(Spline);
			}
		}
	}
	
	static void BuildSplineSkinningWeights(FManagedArrayCollection& GeometryCollection, const GeometryCollection::Facades::FCollectionMeshFacade& MeshFacade,
		const GeometryCollection::Facades::FCollectionCurveGeometryFacade& CurveFacade, const FDataflowVertexSelection& VertexSelection, 
		const TArray<UE::Groom::Private::FCatmullRomSpline>& Splines, const FCollectionAttributeKey& SplineParamKey, const FCollectionAttributeKey& SplineBonesKey)
	{
		if(MeshFacade.IsValid() && CurveFacade.IsValid() && !SplineParamKey.Attribute.IsEmpty() && !SplineBonesKey.Attribute.IsEmpty())
		{
			const bool bValidSelection = VertexSelection.IsValidForCollection(GeometryCollection);

			const bool ExistingSplineParams = GeometryCollection.HasAttribute(FName(SplineParamKey.Attribute), FName(SplineParamKey.Group)) &&
				GeometryCollection.HasAttribute(FName(SplineBonesKey.Attribute), FName(SplineBonesKey.Group));

			TManagedArray<float>& SplineParams = GeometryCollection.AddAttribute<float>(FName(SplineParamKey.Attribute), FName(SplineParamKey.Group));
			TManagedArray<FIntVector2>& SplineBones = GeometryCollection.AddAttribute<FIntVector2>(FName(SplineBonesKey.Attribute), FName(SplineBonesKey.Group));

			if (!ExistingSplineParams)
			{
				for (int32 VertexIndex = 0; VertexIndex < SplineParams.Num(); ++VertexIndex)
				{
					SplineParams[VertexIndex] = 0;
					SplineBones[VertexIndex] = FIntVector2(INDEX_NONE, INDEX_NONE);
				}
			}

			const int32 NumCurves = CurveFacade.GetNumCurves();
			ParallelFor(NumCurves, [&Splines, &SplineParams, &SplineBones, &CurveFacade, &VertexSelection, bValidSelection](int32 CurveIndex)
			{
				TArray<int32> SplineCounts;
				SplineCounts.Init(0, Splines.Num());

				const int32 FirstPoint = CurveIndex > 0 ? CurveFacade.GetCurvePointOffsets()[CurveIndex-1] : 0;
				const int32 LastPoint = CurveFacade.GetCurvePointOffsets()[CurveIndex];

				for (int32 PointIndex = FirstPoint; PointIndex < LastPoint; ++PointIndex)
				{
					if(!bValidSelection || (bValidSelection && (VertexSelection.IsSelected(2*PointIndex) || VertexSelection.IsSelected(2*PointIndex+1))))
					{
						const FVector PointPosition = FVector(CurveFacade.GetPointRestPositions()[PointIndex]);
						int32 ClosestSplineIndex = 0;
					
						float MinDistance = UE_MAX_FLT;
						float MinParam = 0.0f; 

						for (int32 SplineIndex = 0; SplineIndex < Splines.Num(); ++SplineIndex)
						{
							float Param, Distance;
							Splines[SplineIndex].ClosestParam(PointPosition, Param, Distance);

							if (Distance < MinDistance)
							{
								MinParam = Param;
								MinDistance = Distance;
								ClosestSplineIndex = SplineIndex;

							}
						}
				
						SplineParams[PointIndex * 2] = MinParam;
						SplineParams[PointIndex * 2 + 1] = MinParam;
						SplineBones[PointIndex * 2] = Splines[ClosestSplineIndex].Bones;
						SplineBones[PointIndex * 2 + 1] = Splines[ClosestSplineIndex].Bones;

						++SplineCounts[ClosestSplineIndex];
					}
				}

				// Ensure that all CVs on a strand are bound to the same spline
				if (Algo::Count(SplineCounts, 0) < SplineCounts.Num() - 1)
				{
					// Use the most commonly bound spline
					const int32 SplineIndex = Algo::MaxElement(SplineCounts) - &SplineCounts[0];

					for (int32 PointIndex = FirstPoint; PointIndex < LastPoint; ++PointIndex)
					{
						if(!bValidSelection || (bValidSelection && (VertexSelection.IsSelected(2*PointIndex) || VertexSelection.IsSelected(2*PointIndex+1))))
						{
							if (SplineBones[PointIndex * 2] != Splines[SplineIndex].Bones)
							{
								const FVector PointPosition = FVector(CurveFacade.GetPointRestPositions()[PointIndex]);
								float MinDistance = UE_MAX_FLT;
								float MinParam = 0.0f;
								Splines[SplineIndex].ClosestParam(PointPosition, MinParam, MinDistance);
							
								SplineParams[PointIndex * 2] = MinParam;
								SplineParams[PointIndex * 2 + 1] = MinParam;
								SplineBones[PointIndex * 2] = Splines[SplineIndex].Bones;
								SplineBones[PointIndex * 2 + 1] = Splines[SplineIndex].Bones;
							}
						}
					}
				}
			});
		}
	}
	
	static void ConvertFromSplineToLinearWeights(FManagedArrayCollection& GeometryCollection, const GeometryCollection::Facades::FCollectionMeshFacade& MeshFacade,
		GeometryCollection::Facades::FVertexBoneWeightsFacade& SkinningFacade, const FDataflowVertexSelection& VertexSelection, const FCollectionAttributeKey& SplineParamKey, const FCollectionAttributeKey& SplineBonesKey)
	{
		if(MeshFacade.IsValid() && SkinningFacade.IsValid() && !SplineParamKey.Attribute.IsEmpty() && !SplineBonesKey.Attribute.IsEmpty())
		{
			const bool bValidSelection = VertexSelection.IsValidForCollection(SkinningFacade.GetManagedArrayCollection());
			
			const TManagedArray<float>* SplineParams = GeometryCollection.FindAttributeTyped<float>(FName(SplineParamKey.Attribute), FName(SplineParamKey.Group));
			const TManagedArray<FIntVector2>* SplineBones = GeometryCollection.FindAttributeTyped<FIntVector2>(FName(SplineBonesKey.Attribute), FName(SplineBonesKey.Group));

			if (SplineParams && SplineBones)
			{
				TArray<int32> BoneIndices;
				TArray<float> BoneWeights;
				
				for(int32 GeometryIndex = 0; GeometryIndex < SkinningFacade.NumGeometry(); ++GeometryIndex)
				{
					const TArray<int32> VertexIndices = MeshFacade.GetVertexIndices(GeometryIndex);
					for (const int32& VertexIndex : VertexIndices)
					{
						if ((*SplineBones)[VertexIndex][0] != INDEX_NONE && (*SplineBones)[VertexIndex][1] != INDEX_NONE)
						{
							if (!bValidSelection || (bValidSelection && (VertexSelection.IsSelected(VertexIndex))))
							{
								const float SplineParam = (*SplineParams)[VertexIndex];
								const int32 SegmentIndex = trunc(SplineParam);
								const float SegmentParam = SplineParam - SegmentIndex;

								if (SegmentIndex < (*SplineBones)[VertexIndex][1])
								{
									BoneIndices = {SegmentIndex, SegmentIndex + 1};
									BoneWeights = {1.0f - SegmentParam, SegmentParam};
								}
								else
								{
									BoneIndices = {SegmentIndex};
									BoneWeights = {1.0f};
								}

								SkinningFacade.ModifyBoneWeight(VertexIndex, BoneIndices, BoneWeights);
							}
						}
					}
				}
			}
		}
	}
	
	static void ConvertFromLinearToSplineWeights(FManagedArrayCollection& GeometryCollection, GeometryCollection::Facades::FCollectionMeshFacade& MeshFacade,
		GeometryCollection::Facades::FVertexBoneWeightsFacade& SkinningFacade, const FDataflowVertexSelection& VertexSelection,
		const FCollectionAttributeKey& SplineParamKey, const FCollectionAttributeKey& SplineBonesKey)
	{
		if(MeshFacade.IsValid() && SkinningFacade.IsValid() && !SplineParamKey.Attribute.IsEmpty() && !SplineBonesKey.Attribute.IsEmpty())
		{
			const bool bValidSelection = VertexSelection.IsValidForCollection(SkinningFacade.GetManagedArrayCollection());

			const TArray<TObjectPtr<UObject>>& SkeletalMeshs = SkinningFacade.GetSkeletalMeshes().GetConstArray();
			const TArray<int32>& GeometryLods = SkinningFacade.GetGeometryLODs().GetConstArray();
			
			const TArray<TArray<int32>>& BoneIndices = SkinningFacade.GetBoneIndices().GetConstArray();
			const TArray<TArray<float>>& BoneWeights = SkinningFacade.GetBoneWeights().GetConstArray();

			const bool ExistingSplineParams = GeometryCollection.HasAttribute(FName(SplineParamKey.Attribute), FName(SplineParamKey.Group)) &&
				GeometryCollection.HasAttribute(FName(SplineBonesKey.Attribute), FName(SplineBonesKey.Group));
			
			TManagedArray<float>& SplineParams = GeometryCollection.AddAttribute<float>(FName(SplineParamKey.Attribute), FName(SplineParamKey.Group));
			TManagedArray<FIntVector2>& SplineBones = GeometryCollection.AddAttribute<FIntVector2>(FName(SplineBonesKey.Attribute), FName(SplineBonesKey.Group));

			if (!ExistingSplineParams)
			{
				for (int32 VertexIndex = 0; VertexIndex < SplineParams.Num(); ++VertexIndex)
				{
					SplineParams[VertexIndex] = 0;
					SplineBones[VertexIndex] = FIntVector2(INDEX_NONE, INDEX_NONE);
				}
			}
		
			for(int32 GeometryIndex = 0; GeometryIndex < SkinningFacade.NumGeometry(); ++GeometryIndex)
			{
				if (SkeletalMeshs.IsValidIndex(GeometryIndex) && GeometryLods.IsValidIndex(GeometryIndex))
				{
					if (TObjectPtr<USkeletalMesh> SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshs[GeometryIndex]))
					{
						const TArray<int32> VertexIndices = MeshFacade.GetVertexIndices(GeometryIndex);
					
						// Find all used bones for this group						
						TSet<int32> UsedBoneIndices;
						for (const int32& VertexIndex : VertexIndices)
						{
							if(!bValidSelection || (bValidSelection && (VertexSelection.IsSelected(VertexIndex))))
							{
								for (int32 InfluenceIndex = 0; InfluenceIndex < BoneIndices[VertexIndex].Num(); ++InfluenceIndex)
								{
									if (BoneIndices[VertexIndex][InfluenceIndex] != INDEX_NONE)
									{
										UsedBoneIndices.Add(BoneIndices[VertexIndex][InfluenceIndex]);
									}
								}
							}
						}

						// Find root bones
						TArray<int32> RootBones;
						TMap<int32, int32> BoneSplineMap;
						
						const FReferenceSkeleton& Skeleton = SkeletalMesh->GetRefSkeleton();
						for (int32 BoneIndex : UsedBoneIndices)
						{
							int32 ParentIndex = Skeleton.GetParentIndex(BoneIndex);
							if (ParentIndex == INDEX_NONE || !UsedBoneIndices.Contains(ParentIndex))
							{
								RootBones.Add(BoneIndex);
							}
						}

						// Find end bones
						TArray<int32> EndBones;						
						TArray<int32> Children;

						for (int32 SplineIndex = 0; SplineIndex < RootBones.Num(); ++SplineIndex)
						{
							BoneSplineMap.Add(RootBones[SplineIndex], SplineIndex);
							int32 BoneIndex = RootBones[SplineIndex];

							while (Skeleton.GetDirectChildBones(BoneIndex, Children))
							{
								BoneSplineMap.Add(Children[0], SplineIndex);
								BoneIndex = Children[0];
								if (Children.Num() > 1)
								{
									// TODO: Add support for branching splines
								}
							}
							EndBones.Add(BoneIndex);
						}
						
						for (const int32& VertexIndex : VertexIndices)
						{
							if(!bValidSelection || (bValidSelection && (VertexSelection.IsSelected(VertexIndex))))
							{
								const TArray<int32>& VertexBones = BoneIndices[VertexIndex];
								const TArray<float>& VertexWeights = BoneWeights[VertexIndex];

								int32 MaxWeightIndex = INDEX_NONE;
								float MaxWeight = 0;
								for (int32 BoneIndex = 0; BoneIndex < BoneIndices[VertexIndex].Num(); ++BoneIndex)
								{
									if (VertexWeights[BoneIndex] > MaxWeight)
									{
										MaxWeight = VertexWeights[BoneIndex];
										MaxWeightIndex = BoneIndex;
									}
								}
								
								if (MaxWeight > 0)
								{
									const int32 SplineIndex = BoneSplineMap[VertexBones[MaxWeightIndex]];
									float ParentWeight = 0, ChildWeight = 0;
									
									for (int32 InfluenceIndex = 0; InfluenceIndex < VertexBones.Num(); ++InfluenceIndex)
									{
										if (VertexBones[MaxWeightIndex] > RootBones[SplineIndex] && VertexBones[InfluenceIndex] == VertexBones[MaxWeightIndex] - 1)
										{
											ParentWeight = VertexWeights[InfluenceIndex];
										}

										if (VertexBones[MaxWeightIndex] < EndBones[SplineIndex] && VertexBones[InfluenceIndex] == VertexBones[MaxWeightIndex] + 1)
										{
											ChildWeight = VertexWeights[InfluenceIndex];
										}
									}
									
									if (ParentWeight > ChildWeight)
									{
										int Segment = VertexBones[MaxWeightIndex] - 1; 
										float SegmentParam = MaxWeight / (MaxWeight + ParentWeight);
										SplineParams[VertexIndex] = Segment + SegmentParam;
									}
									else
									{
										int Segment = VertexBones[MaxWeightIndex];
										float SegmentParam = (1 - MaxWeight) / (MaxWeight + ChildWeight);
										SplineParams[VertexIndex] = Segment + SegmentParam;
									}		
									
									SplineBones[VertexIndex] = FIntVector2(RootBones[SplineIndex], EndBones[SplineIndex]);
								}
								else if (!RootBones.IsEmpty())
								{
									// Rigidly attach to first spline root if there are no weights for this vertex
									SplineBones[VertexIndex] = FIntVector2(RootBones[0], EndBones[0]);
									SplineParams[VertexIndex] = 0;
								}
							}
						}
					}
				}
			}
		}
	}
}

FBuildGroomSplineSkinWeightsNode::FBuildGroomSplineSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SkeletalMesh);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&SplineParamKey);
	RegisterOutputConnection(&SplineBoneKey);
}

void FBuildGroomSplineSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineParamKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineParamKey(), &SplineParamKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineBoneKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineBonesKey(), &SplineBoneKey);
	}
}

FConvertSplineToLinearSkinWeightsNode::FConvertSplineToLinearSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SplineParamKey);
	RegisterInputConnection(&SplineBonesKey);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&BoneIndicesKey);
	RegisterOutputConnection(&BoneWeightsKey);
}

void FConvertSplineToLinearSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneIndicesKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneIndicesKey(), &BoneIndicesKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneWeightsKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneWeightsKey(), &BoneWeightsKey);
	}
}

FConvertLinearToSplineSkinWeightsNode::FConvertLinearToSplineSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&SplineParamKey);
	RegisterOutputConnection(&SplineBoneKey);
}

void FConvertLinearToSplineSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineParamKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineParamKey(), &SplineParamKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineBoneKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineBonesKey(), &SplineBoneKey);
	}
}

FBuildSplineSkinWeightsDataflowNode::FBuildSplineSkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&VertexSelection);
	RegisterInputConnection(&SkeletalMesh);
	RegisterInputConnection(&LODIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootBones).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SamplesPerSegment).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&SplineParamKey);
	RegisterOutputConnection(&SplineBonesKey);
}

void FBuildSplineSkinWeightsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(GeometryCollection);
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(GeometryCollection);
		
		if(MeshFacade.IsValid() && CurveFacade.IsValid())
		{
			GeometryCollection::Facades::FVertexBoneWeightsFacade SkinningFacade(GeometryCollection, false);
			SkinningFacade.DefineSchema();

			const TObjectPtr<USkeletalMesh> InputSkelMesh = GetValue(Context, &SkeletalMesh);
			const int32 InputLOD = GetValue(Context, &LODIndex);

			if(InputSkelMesh && InputSkelMesh->IsValidLODIndex(InputLOD))
			{
				const FDataflowVertexSelection InputSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
				const bool bValidSelection = InputSelection.IsValidForCollection(GeometryCollection);
				
				TArray<UE::Groom::Private::FCatmullRomSpline> Splines;
				UE::Groom::Private::BuildSplines(InputSkelMesh,
					GetValue(Context, &RootBones), GetValue(Context, &SamplesPerSegment), Splines);

				UE::Groom::Private::BuildSplineSkinningWeights(GeometryCollection, MeshFacade, CurveFacade,
						InputSelection, Splines, UE::Groom::Private::GetSplineParamKey(), UE::Groom::Private::GetSplineBonesKey());
				
				for(int32 GeometryIndex = 0; GeometryIndex < SkinningFacade.NumGeometry(); ++GeometryIndex)
				{
					bool bValidGeometry = false;
					for(int32 VertexIndex :  MeshFacade.GetVertexIndices(GeometryIndex))
					{
						if(!bValidSelection || (bValidSelection && InputSelection.IsSelected(VertexIndex)))
						{
							bValidGeometry = true;
							break;
						}
					}
					if(bValidGeometry)
					{
						SkinningFacade.ModifyGeometryBinding(GeometryIndex, InputSkelMesh, InputLOD);
					}
				}
			}
		}

		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineParamKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineParamKey(), &SplineParamKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineBonesKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineBonesKey(), &SplineBonesKey);
	}
}

FSplineToLinearSkinWeightsDataflowNode::FSplineToLinearSkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&VertexSelection);
	RegisterInputConnection(&SplineParamKey).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SplineBonesKey).SetCanHidePin(true).SetPinIsHidden(true);
	
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&BoneIndicesKey);
	RegisterOutputConnection(&BoneWeightsKey);
}

void FSplineToLinearSkinWeightsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(GeometryCollection);
		
		if(MeshFacade.IsValid())
		{
			GeometryCollection::Facades::FVertexBoneWeightsFacade SkinningFacade(GeometryCollection, false);
			if(!SkinningFacade.IsValid())
			{
				SkinningFacade.DefineSchema();
			}
			UE::Groom::Private::ConvertFromSplineToLinearWeights(GeometryCollection, MeshFacade, SkinningFacade,
				GetValue<FDataflowVertexSelection>(Context, &VertexSelection), GetSplineParamKey(Context), GetSplineBonesKey(Context));
		}
		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneIndicesKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneIndicesKey(), &BoneIndicesKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&BoneWeightsKey))
	{
		SetValue(Context, UE::Groom::Private::GetBoneWeightsKey(), &BoneWeightsKey);
	}
}

FCollectionAttributeKey FSplineToLinearSkinWeightsDataflowNode::GetSplineParamKey(UE::Dataflow::FContext& Context) const
{
	FCollectionAttributeKey Key = GetValue(Context, &SplineParamKey, SplineParamKey);
	
	if (Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		return UE::Groom::Private::GetSplineParamKey();
	}

	return Key;
}

FCollectionAttributeKey FSplineToLinearSkinWeightsDataflowNode::GetSplineBonesKey(UE::Dataflow::FContext& Context) const
{
	FCollectionAttributeKey Key = GetValue(Context, &SplineBonesKey, SplineBonesKey);

	if (Key.Attribute.IsEmpty() && Key.Group.IsEmpty())
	{
		return UE::Groom::Private::GetSplineBonesKey();
	}

	return Key;
}

FLinearToSplineSkinWeightsDataflowNode::FLinearToSplineSkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&VertexSelection);
	RegisterInputConnection(&BoneIndicesKey).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&BoneWeightsKey).SetCanHidePin(true).SetPinIsHidden(true);
	
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&SplineParamKey);
	RegisterOutputConnection(&SplineBonesKey);
}

void FLinearToSplineSkinWeightsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Dataflow;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(GeometryCollection);
		GeometryCollection::Facades::FVertexBoneWeightsFacade SkinningFacade(GeometryCollection);
		
		if(MeshFacade.IsValid() && SkinningFacade.IsValid())
		{
			UE::Groom::Private::ConvertFromLinearToSplineWeights(GeometryCollection, MeshFacade, SkinningFacade,
				GetValue<FDataflowVertexSelection>(Context, &VertexSelection), UE::Groom::Private::GetSplineParamKey(), UE::Groom::Private::GetSplineBonesKey());
		}

		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineParamKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineParamKey(), &SplineParamKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&SplineBonesKey))
	{
		SetValue(Context, UE::Groom::Private::GetSplineBonesKey(), &SplineBonesKey);
	}
}



