// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildGuidesLODsNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuildGuidesLODsNode)

namespace UE::Groom::Private
{
	FORCEINLINE float ComputeGuidesMetric(const TArray<FVector3f>& PointRestPositions, const TArray<int32>& CurvePointOffsets,const uint32 GuideIndexA, const uint32 GuideIndexB, const float& GuideLengthA, const float& GuideLengthB,
										 const float ShapeWeight, const float ProximityWeight)
	{
		const uint32 PointOffsetA = (GuideIndexA > 0) ? CurvePointOffsets[GuideIndexA-1] : 0;
		const uint32 PointOffsetB = (GuideIndexB > 0) ? CurvePointOffsets[GuideIndexB-1] : 0;
		
		const uint32 NumPointsA = CurvePointOffsets[GuideIndexA] - PointOffsetA;
		const uint32 NumPointsB = CurvePointOffsets[GuideIndexB] - PointOffsetB;

		if(NumPointsA == NumPointsB)
		{
			float ProximityMetric = 0.0f;
			float ShapeMetric = 0.0f;
			for(uint32 PointIndex = 0; PointIndex < NumPointsA; ++PointIndex)
			{
				const FVector3f& GuidePositionA = PointRestPositions[PointIndex+PointOffsetA];
				const FVector3f& GuidePositionB = PointRestPositions[PointIndex+PointOffsetB];
				
				ProximityMetric += (GuidePositionB - GuidePositionA).Size();
				ShapeMetric += (GuidePositionB - PointRestPositions[PointOffsetB] - GuidePositionA + PointRestPositions[PointOffsetA]).Size();
			}

			const float MetricScale = 1.0f / (NumPointsA * 0.5f * (GuideLengthA+GuideLengthB));
			return FMath::Exp(-ShapeWeight * ShapeMetric * MetricScale) * FMath::Exp(-ProximityWeight * ProximityMetric * MetricScale);
		}
		return 0.0f;
	}
	
	FORCEINLINE void BuildInternalLODs(const int32 NumObjects, const int32 NumCurves, const TArray<int32>& GeometryCurveOffsets,
		const TArray<int32>& CurvePointOffsets, const TArray<FVector3f>& PointRestPositions, TArray<int32>& CurveParentIndices, TArray<int32>& CurveLodIndices,
			const FDataflowCurveSelection& CurveSelection, const bool bValidSelection)
	{
		CurveParentIndices.Init(INDEX_NONE, NumCurves);
		CurveLodIndices.Init(INDEX_NONE, NumCurves);
		
		uint32 CurveOffset = 0, PointOffset = 0;
		for(int32 ObjectIndex = 0; ObjectIndex < NumObjects; ++ObjectIndex)
		{ 
			TArray<float> GuidesLengths; GuidesLengths.Init(0.0f, GeometryCurveOffsets[ObjectIndex] - CurveOffset);
			for(int32 GuideIndex = CurveOffset; GuideIndex < GeometryCurveOffsets[ObjectIndex]; ++GuideIndex)
			{
				for(int32 PointIndex = PointOffset+1; PointIndex < CurvePointOffsets[GuideIndex]; ++PointIndex)
				{
					GuidesLengths[GuideIndex-CurveOffset] += (PointRestPositions[PointIndex]-PointRestPositions[PointIndex-1]).Size();
				}
				PointOffset = CurvePointOffsets[GuideIndex];
			}
			const int32 NumObjectLODs = FMath::CeilLogTwo(GuidesLengths.Num());
			ParallelFor(GuidesLengths.Num(), [CurveOffset, &GuidesLengths, &PointRestPositions, &CurvePointOffsets, &CurveParentIndices, &CurveLodIndices, &NumObjectLODs, bValidSelection, &CurveSelection](int32 CurveIndex)
			{
				const uint32 GuideIndex = CurveIndex + CurveOffset;

				if(!bValidSelection || (bValidSelection && (CurveSelection.IsSelected(GuideIndex))))
				{
					const uint32 GuideLod = NumObjectLODs - 1 - FMath::FloorLog2(CurveIndex);
					const uint32 LodOffset = (GuideLod == NumObjectLODs - 1) ? 0 : 1 << FMath::FloorLog2(CurveIndex);

					float MinMetric = FLT_MAX;
					int32 MinParent = INDEX_NONE;
					for(uint32 ParentIndex = CurveOffset; ParentIndex < CurveOffset+LodOffset; ++ParentIndex)
					{
						const float ParentMetric = 1.0f - ComputeGuidesMetric(PointRestPositions, CurvePointOffsets,
							GuideIndex, ParentIndex, GuidesLengths[CurveIndex], GuidesLengths[ParentIndex-CurveOffset],  1.0f, 1.0f);
						if (ParentMetric < MinMetric)
						{
							MinMetric = ParentMetric;
							MinParent = ParentIndex;
						}
					}
					CurveParentIndices[GuideIndex] = (MinParent != INDEX_NONE) ? GuideIndex-MinParent : INDEX_NONE;
					CurveLodIndices[GuideIndex] = GuideLod;
				}

				//UE_LOGF(LogTemp, Log, "Guide Index = %d, Num Guides = %d, Num LODs = %d, Guide LOD = %d, LOD Offset = %d, Parent Index = %d",
				//	CurveIndex, GuidesLengths.Num(), NumObjectLODs, GuideLod, LodOffset, MinParent);
			}, EParallelForFlags::None);

			CurveOffset = GeometryCurveOffsets[ObjectIndex];
		}
	}
	
	FORCEINLINE void BuildCurvesLODs(GeometryCollection::Facades::FCollectionCurveGeometryFacade& CurvesFacade, GeometryCollection::Facades::FCollectionCurveHierarchyFacade& HierarchyFacade, const FDataflowCurveSelection& CurveSelection)
	{
		const uint32 NumObjects = CurvesFacade.GetNumGeometry();
		
		TArray<int32> CurveParentIndices; CurveParentIndices.Init(INDEX_NONE, CurvesFacade.GetNumCurves());
		TArray<int32> CurveLodIndices; CurveLodIndices.Init(INDEX_NONE, CurvesFacade.GetNumCurves());

		const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());

		BuildInternalLODs(CurvesFacade.GetNumGeometry(), CurvesFacade.GetNumCurves(), CurvesFacade.GetGeometryCurveOffsets(),
			CurvesFacade.GetCurvePointOffsets(), CurvesFacade.GetPointRestPositions(), CurveParentIndices, CurveLodIndices, CurveSelection, bValidSelection);
		
		HierarchyFacade.SetCurveParentIndices(CurveParentIndices);
		HierarchyFacade.SetCurveLodIndices(CurveLodIndices);
	}

	FCollectionAttributeKey GetCurveParentsKey()
    {
    	FCollectionAttributeKey Key;
    	Key.Group =  GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup.ToString();
    	Key.Attribute = GeometryCollection::Facades::FCollectionCurveHierarchyFacade::CurveParentIndicesAttribute.ToString();
    	return Key;
    }

	FCollectionAttributeKey GetCurveLodsKey()
	{
		FCollectionAttributeKey Key;
		Key.Group =  GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup.ToString();
		Key.Attribute = GeometryCollection::Facades::FCollectionCurveHierarchyFacade::CurveLodIndicesAttribute.ToString();
		return Key;
	}
}

void FBuildGuidesLODsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&CurveParentsKey))
	{
		SetValue(Context, UE::Groom::Private::GetCurveParentsKey(), &CurveParentsKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&CurveLodsKey))
	{
		SetValue(Context, UE::Groom::Private::GetCurveLodsKey(), &CurveLodsKey);
	}
}

FBuildCurveLODsDataflowNode::FBuildCurveLODsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CurveSelection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&CurveParentsKey);
	RegisterOutputConnection(&CurveLodsKey);
}

void FBuildCurveLODsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GeometryCollection);

		if(CurvesFacade.IsValid())
		{
			GeometryCollection::Facades::FCollectionCurveHierarchyFacade HierarchyFacade(GeometryCollection);
			HierarchyFacade.DefineSchema();
			
			UE::Groom::Private::BuildCurvesLODs(CurvesFacade, HierarchyFacade, GetValue(Context, &CurveSelection));
		}
		
		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&CurveParentsKey))
	{
		SetValue(Context, UE::Groom::Private::GetCurveParentsKey(), &CurveParentsKey);
	}
	else if (Out->IsA<FCollectionAttributeKey>(&CurveLodsKey))
	{
		SetValue(Context, UE::Groom::Private::GetCurveLodsKey(), &CurveLodsKey);
	}
}

