// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateGuidesCurvesNode.h"

#include "GroomBindingBuilder.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateGuidesCurvesNode)

namespace UE::Groom::Private
{
	FORCEINLINE void GenerateGeometryCurves(const GeometryCollection::Facades::FCollectionCurveGeometryFacade& CurvesFacade, const FDataflowCurveSelection& CurveSelection, const uint32 GuidesCount, TArray<uint32>& SampleIndices)
	{
		TArray<FVector3f> RootPositions;
		RootPositions.Reserve(CurvesFacade.GetNumCurves());

		TArray<bool> ValidPoints;
		ValidPoints.Reserve(CurvesFacade.GetNumCurves());

		const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());;
			
		int32 PointOffset = 0;
		for (int32 CurveIndex = 0; CurveIndex < CurvesFacade.GetNumCurves(); ++CurveIndex)
		{
			RootPositions.Add(CurvesFacade.GetPointRestPositions()[PointOffset]);
			ValidPoints.Add( (!bValidSelection || (bValidSelection && CurveSelection.IsSelected(CurveIndex))));
			
			PointOffset = CurvesFacade.GetCurvePointOffsets()[CurveIndex];
		}

		GroomBinding_RBFWeighting::FPointsSampler PointsSampler(ValidPoints, RootPositions.GetData(), GuidesCount);
		SampleIndices = PointsSampler.SampleIndices;
	}

	FORCEINLINE void GenerateGeometryPoints(const GeometryCollection::Facades::FCollectionCurveGeometryFacade& SourceFacade, const TArray<uint32>& SampleIndices,
		const bool bMergeCurves, GeometryCollection::Facades::FCollectionCurveGeometryFacade& TargetFacade)
	{
		TArray<TArray<FVector3f>> GeometryPointPositions;
		TArray<TArray<int32>> GeometryCurvePoints;
		TArray<TArray<int32>> GeometrySourceIndices;
			
		GeometryPointPositions.SetNum(SourceFacade.GetNumGeometry());
		GeometryCurvePoints.SetNum(SourceFacade.GetNumGeometry());
		GeometrySourceIndices.SetNum(SourceFacade.GetNumGeometry());
		
		if(bMergeCurves && TargetFacade.GetNumGeometry() == SourceFacade.GetNumGeometry())
		{
			int32 PointOffset = 0;
			int32 CurveOffset = 0;
			for (int32 GeometryIndex = 0; GeometryIndex < TargetFacade.GetNumGeometry(); ++GeometryIndex)
			{
				for (int32 CurveIndex = CurveOffset; CurveIndex < TargetFacade.GetGeometryCurveOffsets()[GeometryIndex]; ++CurveIndex)
				{
					for (int32 PointIndex = PointOffset; PointIndex < TargetFacade.GetCurvePointOffsets()[CurveIndex]; ++PointIndex)
					{
						GeometryPointPositions[GeometryIndex].Add(TargetFacade.GetPointRestPositions()[PointIndex]);
					}
					GeometryCurvePoints[GeometryIndex].Add(TargetFacade.GetCurvePointOffsets()[CurveIndex]-PointOffset);
					GeometrySourceIndices[GeometryIndex].Add(TargetFacade.GetCurveSourceIndices()[CurveIndex]);
					PointOffset = TargetFacade.GetCurvePointOffsets()[CurveIndex];
				}
				CurveOffset = TargetFacade.GetGeometryCurveOffsets()[GeometryIndex];
			}
		}
			
		for(const uint32& SampleIndex : SampleIndices)
		{
			const int32 PrevPoint = (SampleIndex == 0) ? 0 : SourceFacade.GetCurvePointOffsets()[SampleIndex-1];
			const int32 NextPoint = SourceFacade.GetCurvePointOffsets()[SampleIndex];

			const int32 GeometryIndex = SourceFacade.GetCurveGeometryIndices()[SampleIndex];
			for(int32 PointIndex = PrevPoint; PointIndex < NextPoint; ++PointIndex)
			{
				GeometryPointPositions[GeometryIndex].Add(SourceFacade.GetPointRestPositions()[PointIndex]);
			}
			GeometryCurvePoints[GeometryIndex].Add(NextPoint-PrevPoint);
			GeometrySourceIndices[GeometryIndex].Add(SourceFacade.GetCurveSourceIndices()[SampleIndex]);
		}

		TArray<FVector3f> PointRestPositions;
		TArray<int32> GeometryCurveOffsets, CurvePointOffsets, CurveSourceIndices;
		TArray<FString> GeometryGroupNames;
		
		int32 CurveOffset = 0;
		int32 PointOffset = 0;
		for (int32 GeometryIndex = 0; GeometryIndex < SourceFacade.GetNumGeometry(); ++GeometryIndex)
		{
			PointRestPositions.Append(GeometryPointPositions[GeometryIndex]);
			for (int32 CurveIndex = 0; CurveIndex < GeometryCurvePoints[GeometryIndex].Num(); ++CurveIndex)
			{
				PointOffset += GeometryCurvePoints[GeometryIndex][CurveIndex];
				CurvePointOffsets.Add(PointOffset);
				CurveSourceIndices.Add(GeometrySourceIndices[GeometryIndex][CurveIndex]);
			}
			CurveOffset += GeometryCurvePoints[GeometryIndex].Num();
			GeometryCurveOffsets.Add(CurveOffset);
				
			const FString GroupName = SourceFacade.GetGeometryGroupNames()[GeometryIndex];
			GeometryGroupNames.Add(GroupName);
		}
		TargetFacade.InitCurvesCollection(PointRestPositions, CurvePointOffsets, 
		GeometryCurveOffsets, GeometryGroupNames,  SourceFacade.GetGeometryCurveThickness(), CurveSourceIndices);
	}

	
}

void FGenerateGuidesCurvesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FGenerateCurveGeometryDataflowNode::FGenerateCurveGeometryDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SourceCurves);
	RegisterInputConnection(&CurveSelection);
	RegisterInputConnection(&CurveCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}

void FGenerateCurveGeometryDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection TargetCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FManagedArrayCollection& SourceCollection = GetValue<FManagedArrayCollection>(Context, &SourceCurves);
		const int32 CountValue = FMath::Max(1, GetValue<int32>(Context, &CurveCount));
		const FDataflowCurveSelection& DataflowSelection = GetValue<FDataflowCurveSelection>(Context, &CurveSelection);
		
		GeometryCollection::Facades::FCollectionCurveGeometryFacade SourceFacade(SourceCollection);
		
		if(SourceFacade.IsValid() && CountValue > 0)
		{
			TArray<uint32> SampleIndices;
			UE::Groom::Private::GenerateGeometryCurves(SourceFacade, DataflowSelection, CountValue, SampleIndices);

			GeometryCollection::Facades::FCollectionCurveGeometryFacade TargetFacade(TargetCollection);
			UE::Groom::Private::GenerateGeometryPoints(SourceFacade, SampleIndices, bMergeCurves, TargetFacade);
		}
		
		SetValue(Context, MoveTemp(TargetCollection), &Collection);
	}
}



