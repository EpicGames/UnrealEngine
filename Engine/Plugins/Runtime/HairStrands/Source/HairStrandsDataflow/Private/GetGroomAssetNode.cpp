// Copyright Epic Games, Inc. All Rights Reserved.

#include "GetGroomAssetNode.h"
#include "GroomEdit.h"
#include "GroomInstance.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetGroomAssetNode)

namespace UE::Groom::Private
{
	FORCEINLINE void BuildCurveCollection(FManagedArrayCollection& GroomCollection, FEditableGroom& EditGroom, const FString& GroomName, const EGroomCollectionType CurvesType, const float CurveThickness)
	{
		TArray<FVector3f> PointRestPositions;
		TArray<int32> GeometryCurveOffsets;
		TArray<int32> CurvePointOffsets;
		TArray<FString> GeometryGroupNames;
		TArray<float> GeometryCurveThickness;
		TArray<int32> CurveSourceIndices;
		
		uint32 GroupIndex = 0;
		for(FEditableGroomGroup& GroomGroup : EditGroom.Groups)
		{
			if(CurvesType == EGroomCollectionType::Guides)
			{
				int32 CurveIndex = 0;
				int32 SourceType = static_cast<uint8>(EGroomCollectionType::Guides);
				for (const FEditableHairGuide& EditType : GroomGroup.Guides)
				{
					for(auto& GuidePoint : EditType.ControlPoints)
					{
						PointRestPositions.Add(GuidePoint.Position);
					}
					CurvePointOffsets.Add(PointRestPositions.Num());
					CurveSourceIndices.Add((CurveIndex << 1) | SourceType);
					++CurveIndex;
				}
			}
			else
			{
				int32 CurveIndex = 0;
				int32 SourceType = static_cast<uint8>(EGroomCollectionType::Strands);
				for (const FEditableHairStrand& EditType : GroomGroup.Strands)
				{
					for(auto& GuidePoint : EditType.ControlPoints)
					{
						PointRestPositions.Add(GuidePoint.Position);
					}
					CurvePointOffsets.Add(PointRestPositions.Num());
					CurveSourceIndices.Add((CurveIndex << 1) | SourceType);
					++CurveIndex;
				}
			}
			GeometryCurveOffsets.Add(CurvePointOffsets.Num());
			
			FString GroupName = (CurvesType == EGroomCollectionType::Guides) ? GroomName + TEXT("_Guides") : GroomName + TEXT("_Strands");
			GroupName += TEXT("_") + GroomGroup.GroupName.ToString();
			GeometryGroupNames.Add(GroupName);
			GeometryCurveThickness.Add(CurveThickness);
			++GroupIndex;
		}
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GroomCollection);
		CurvesFacade.InitCurvesCollection(PointRestPositions, CurvePointOffsets, GeometryCurveOffsets, GeometryGroupNames, GeometryCurveThickness, CurveSourceIndices);
	}
}

void FGetGroomAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GroomCollection;
		SetValue(Context, MoveTemp(GroomCollection), &Collection);
	}
}

void FGetGroomAssetDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<const UGroomAsset>>(&GroomAsset))
	{
		TObjectPtr<const UGroomAsset> LocalAsset = GroomAsset;
		if (!LocalAsset)
		{
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				LocalAsset = Cast<UGroomAsset>(EngineContext->Owner);
			}
		}
		SetValue(Context, LocalAsset, &GroomAsset);
	}
}

FGroomAssetToCollectionDataflowNode::FGroomAssetToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
		: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&GroomAsset);
	RegisterInputConnection(&CurvesThickness).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CurvesType).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);
}

void FGroomAssetToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GroomCollection;
		FEditableGroom EditGroom;
		if(TObjectPtr<const UGroomAsset> InputAsset = GetValue<TObjectPtr<const UGroomAsset>>(Context, &GroomAsset))
		{
			const float GeometryThickness = GetValue<float>(Context, &CurvesThickness);
			ConvertFromGroomAsset(const_cast<UGroomAsset*>(InputAsset.Get()), &EditGroom, false, false, false);

			UE::Groom::Private::BuildCurveCollection(GroomCollection, EditGroom, InputAsset->GetName(), CurvesType, GeometryThickness);
		}

		SetValue(Context, MoveTemp(GroomCollection), &Collection);
	}
}

