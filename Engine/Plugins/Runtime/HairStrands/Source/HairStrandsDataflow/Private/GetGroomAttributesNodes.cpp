// Copyright Epic Games, Inc. All Rights Reserved.

#include "GetGroomAttributesNodes.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetGroomAttributesNodes)

void FGetGroomAttributesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FCollectionAttributeKey>(&AttributeKey))
	{
		FCollectionAttributeKey Key;
		SetValue(Context, MoveTemp(Key), &AttributeKey);
	}
}

FGetCurveAttributesDataflowNode::FGetCurveAttributesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&AttributeKey);
}

void FGetCurveAttributesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FCollectionAttributeKey>(&AttributeKey))
	{
		FCollectionAttributeKey Key;

		const FString VertexGroup = FGeometryCollection::VerticesGroup.ToString();
		const FString CurveGroup = GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup.ToString();
		if(AttributeType == EGroomAttributeType::KinematicWeights)
		{ 
			Key.Group = VertexGroup;
			Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::KinematicWeightAttributeName.ToString();
		}
		else if (AttributeType == EGroomAttributeType::BoneIndices)
		{
			Key.Group = VertexGroup;
			Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::BoneIndicesAttributeName.ToString();
		}
		else if (AttributeType == EGroomAttributeType::BoneWeights)
		{
			Key.Group = VertexGroup;
			Key.Attribute = GeometryCollection::Facades::FVertexBoneWeightsFacade::BoneWeightsAttributeName.ToString();
		}
		else if (AttributeType == EGroomAttributeType::CurveLods)
		{
			Key.Group = CurveGroup;
			Key.Attribute = GeometryCollection::Facades::FCollectionCurveHierarchyFacade::CurveLodIndicesAttribute.ToString();
		}
		else if (AttributeType == EGroomAttributeType::CurveParents)
		{
			Key.Group = CurveGroup;
			Key.Attribute = GeometryCollection::Facades::FCollectionCurveHierarchyFacade::CurveParentIndicesAttribute.ToString();
		}
		
		SetValue(Context, MoveTemp(Key), &AttributeKey);
	}
}

