// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetTerminalNode.h"

#include "AssetCompilingManager.h"
#include "GetGroomAssetNode.h"
#include "GroomEdit.h"
#include "HLSLTypeAliases.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetTerminalNode)

namespace UE::Groom::Private
{
	static const FName AttributeKeyTypeName(TEXT("FCollectionAttributeKey"));

	static void BuildEditableGuides(const TArray<int32>& ObjectCurveOffsets,
		const TArray<int32>& CurvePointOffsets, const TArray<FVector3f>& PointRestPositions, const TArray<int32>& CurveStrandIndices,
		const FEditableGroom& SourceGroom, FEditableGroom& EditGroom)
	{
		if (SourceGroom.Groups.Num() == EditGroom.Groups.Num())
		{
			int32 ObjectIndex = 0, CurveIndex = 0;
			int32 PrevCurve = 0, PrevPoint = 0;;
			for (FEditableGroomGroup& Group : EditGroom.Groups)
			{
				const int32 NextCurve = ObjectCurveOffsets[ObjectIndex];
				Group.Guides.SetNum(NextCurve-PrevCurve);
				
				for (FEditableHairGuide& Guide : Group.Guides)
				{
					const int32 NextPoint = CurvePointOffsets[CurveIndex];
					const int32 SourceType = CurveStrandIndices[CurveIndex] & 1;
					const int32 SourceCurve = CurveStrandIndices[CurveIndex] >> 1;
						
					Guide.ControlPoints.Reset();

					if((NextPoint - PrevPoint - 1) > 0)
					{ 
						for (int32 PointIndex = PrevPoint; PointIndex < NextPoint; ++PointIndex)
						{
							Guide.ControlPoints.Add({ PointRestPositions[PointIndex],
									FMath::Clamp(static_cast<float>(PointIndex - PrevPoint) / (NextPoint - PrevPoint - 1), 0.f, 1.f) });
						}
					}

					const FEditableGroomGroup& SourceGroup = SourceGroom.Groups[ObjectIndex];

					Guide.bHasGuideID = false;
					Guide.bHasRootUV = false;
					
					if (SourceType == static_cast<uint8>(EGroomCollectionType::Guides))
					{
						if(SourceGroup.Guides.IsValidIndex(SourceCurve))
						{
							Guide.bHasGuideID = SourceGroup.Guides[SourceCurve].bHasGuideID;
							Guide.GuideID = SourceGroup.Guides[SourceCurve].GuideID;

							Guide.bHasRootUV = SourceGroup.Guides[SourceCurve].bHasRootUV;
							Guide.RootUV = SourceGroup.Guides[SourceCurve].RootUV;	
						}
					}
				
					PrevPoint = NextPoint;
					++CurveIndex;
				}
				PrevCurve = NextCurve;
				++ObjectIndex;
			}
		}
	}

	static void BuildEditableStrands(const TArray<int32>& ObjectCurveOffsets,
			const TArray<int32>& CurvePointOffsets, const TArray<FVector3f>& PointRestPositions, const TArray<int32>& CurveStrandIndices,
			const FEditableGroom& SourceGroom, FEditableGroom& EditGroom)
	{
		if (SourceGroom.Groups.Num() == EditGroom.Groups.Num())
		{
			int32 ObjectIndex = 0, CurveIndex = 0;
			int32 PrevCurve = 0, PrevPoint = 0;
			for (FEditableGroomGroup& Group : EditGroom.Groups)
			{
				const int32 NextCurve = ObjectCurveOffsets[ObjectIndex];
				Group.Strands.SetNum(NextCurve-PrevCurve);

				const FEditableGroomGroup& SourceGroup = SourceGroom.Groups[ObjectIndex];
				
				for (FEditableHairStrand& Strand : Group.Strands)
				{
					const int32 NextPoint = CurvePointOffsets[CurveIndex];
					const int32 SourceType = CurveStrandIndices[CurveIndex] & 1;
					const int32 SourceCurve = CurveStrandIndices[CurveIndex] >> 1;
						
					Strand.ControlPoints.Reset();

					if((NextPoint - PrevPoint - 1) > 0)
					{ 
						for (int32 PointIndex = PrevPoint; PointIndex < NextPoint; ++PointIndex)
						{
							FEditableHairStrandControlPoint ControlPoint;
							ControlPoint.Position = PointRestPositions[PointIndex];
							ControlPoint.U = static_cast<float>(PointIndex - PrevPoint) / (NextPoint - PrevPoint - 1);

							ControlPoint.bHasAO = false;
							ControlPoint.bHasRoughness = false;
							ControlPoint.bHasColor = false;
							
							if (SourceType == static_cast<uint8>(EGroomCollectionType::Strands))
							{
								if(SourceGroup.Strands.IsValidIndex(SourceCurve) && SourceGroup.Strands[SourceCurve].ControlPoints.Num() >= 2)
								{
									const float SourcePointCoord = (SourceGroup.Strands[SourceCurve].ControlPoints.Num()-1) * ControlPoint.U;
									const int32 SourcePointIndex = FMath::Floor(SourcePointCoord);
									const float SourceLerpValue = SourcePointCoord - SourcePointIndex;
									
									const uint32 SourcePrevIndex = FMath::Clamp(SourcePointIndex,
													0, SourceGroup.Strands[SourceCurve].ControlPoints.Num()-2);
									const uint32 SourceNextIndex = SourcePrevIndex + 1;
									
									const FEditableHairStrandControlPoint& SourcePrevPoint =
										SourceGroup.Strands[SourceCurve].ControlPoints[SourcePrevIndex];
									const FEditableHairStrandControlPoint& SourceNextPoint =
										SourceGroup.Strands[SourceCurve].ControlPoints[SourceNextIndex];

									ControlPoint.Radius = SourcePrevPoint.Radius * (1.0-SourceLerpValue) + SourceNextPoint.Radius * SourceLerpValue;
						
									ControlPoint.bHasAO = SourcePrevPoint.bHasAO && SourceNextPoint.bHasAO;
									ControlPoint.bHasRoughness = SourcePrevPoint.bHasRoughness && SourceNextPoint.bHasRoughness;
									ControlPoint.bHasColor = SourcePrevPoint.bHasColor && SourceNextPoint.bHasColor;

									ControlPoint.AO = SourcePrevPoint.AO * (1.0-SourceLerpValue) + SourceNextPoint.AO * SourceLerpValue;
									ControlPoint.Roughness = SourcePrevPoint.Roughness * (1.0-SourceLerpValue) + SourceNextPoint.Roughness * SourceLerpValue;
									ControlPoint.BaseColor = FLinearColor::LerpUsingHSV(SourcePrevPoint.BaseColor, SourceNextPoint.BaseColor, SourceLerpValue);
								}
							}
							Strand.ControlPoints.Add(ControlPoint);
						}
					}

					Strand.bHasStrandID = false;
					Strand.bHasRootUV = false;
					Strand.bHasClosestGuide = false;
					Strand.bHasClumpID = false;
					
					if (SourceType == static_cast<uint8>(EGroomCollectionType::Strands))
					{
						if(SourceGroup.Strands.IsValidIndex(SourceCurve))
						{
							Strand.bHasStrandID = SourceGroup.Strands[SourceCurve].bHasStrandID;
							Strand.StrandID = SourceGroup.Strands[SourceCurve].StrandID;

							Strand.bHasRootUV = SourceGroup.Strands[SourceCurve].bHasRootUV;
							Strand.RootUV = SourceGroup.Strands[SourceCurve].RootUV;	
							
							Strand.bHasClumpID = SourceGroup.Strands[SourceCurve].bHasClumpID;
                            Strand.ClumpID = SourceGroup.Strands[SourceCurve].ClumpID;

							Strand.bHasClosestGuide = SourceGroup.Strands[SourceCurve].bHasClosestGuide;
							for (uint32 GuideIndex = 0; GuideIndex < 3; ++GuideIndex)
							{
								Strand.GuideIDs[GuideIndex] = SourceGroup.Strands[SourceCurve].GuideIDs[GuideIndex];
								Strand.GuideWeights[GuideIndex] = SourceGroup.Strands[SourceCurve].GuideWeights[GuideIndex];
							}
						}
					}
				
					PrevPoint = NextPoint;
					++CurveIndex;
				}
				PrevCurve = NextCurve;
				++ObjectIndex;
			}
		}
	}

	FORCEINLINE void CopyCollectionAttribute(const FManagedArrayCollection* InputCollection, FManagedArrayCollection* OutputCollection,
		const FCollectionAttributeKey& AttributeToCopy, const FString& GroupPrefix)
	{
		const FName AttributeName(AttributeToCopy.Attribute);
		const FName SourceGroup(AttributeToCopy.Group);
		const FName TargetGroup( GroupPrefix + AttributeToCopy.Group);

		if (InputCollection->HasGroup(SourceGroup))
		{
			if(!OutputCollection->HasGroup(TargetGroup))
			{
				OutputCollection->AddGroup(TargetGroup);
			}
			if (InputCollection->NumElements(SourceGroup) != OutputCollection->NumElements(TargetGroup))
			{
				OutputCollection->EmptyGroup(TargetGroup);
				OutputCollection->AddElements(InputCollection->NumElements(SourceGroup), TargetGroup);
			}
			OutputCollection->CopyAttribute(*InputCollection, AttributeName, AttributeName, SourceGroup, TargetGroup);
		}
	}

	FORCEINLINE void CopyCollectionAttributes(const FManagedArrayCollection* InputCollection, FManagedArrayCollection* OutputCollection,
		const TArray<FCollectionAttributeKey>& AttributesToCopy = TArray<FCollectionAttributeKey>())
	{
		for(const FCollectionAttributeKey& AttributeToCopy : AttributesToCopy)
		{
			CopyCollectionAttribute(InputCollection, OutputCollection, AttributeToCopy, "");
		}
	}

	template<typename AttributeType>
	static void BuildVerticesAttribute(const FManagedArrayCollection& InCollection, FManagedArrayCollection* OutCollection, const int32 NumPoints, const FName& AttributeName, const FName& VerticesGroup, const FName& PointsGroup)
	{
		const TManagedArray<AttributeType>& VerticesAttribute = InCollection.GetAttribute<AttributeType>(AttributeName, VerticesGroup);
		if(OutCollection->NumElements(PointsGroup) != NumPoints)
		{
			if(OutCollection->NumElements(PointsGroup) > 0)
			{
				OutCollection->EmptyGroup(PointsGroup);
			}
			OutCollection->AddElements(NumPoints, PointsGroup);
		}
		TManagedArray<AttributeType>& PointsAttribute = OutCollection->AddAttribute<AttributeType>(AttributeName, PointsGroup);
		
		for(int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			PointsAttribute[PointIndex] = VerticesAttribute[2*PointIndex];
		}
	}

	template<typename AttributeType, typename CompressedType, int32 NumElements>
	static void BuildVerticesArray(const FManagedArrayCollection& InCollection, FManagedArrayCollection* OutCollection, const int32 NumPoints,
		const FName& AttributeName, const FName& VerticesGroup, const FName& PointsGroup, const AttributeType DefaultValue)
	{
		const TManagedArray<TArray<AttributeType>>& VerticesArray = InCollection.GetAttribute<TArray<AttributeType>>(AttributeName, VerticesGroup);
		if(OutCollection->NumElements(PointsGroup) != NumPoints)
		{
			if(OutCollection->NumElements(PointsGroup) > 0)
			{
				OutCollection->EmptyGroup(PointsGroup);
			}
			OutCollection->AddElements(NumPoints, PointsGroup);
		}
		TManagedArray<CompressedType>& PointsAttribute = OutCollection->AddAttribute<CompressedType>(AttributeName, PointsGroup);
		
		for(int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			for(int32 ElemIndex = 0; ElemIndex < NumElements; ++ElemIndex)
			{
				PointsAttribute[PointIndex][ElemIndex] = VerticesArray[2*PointIndex].IsValidIndex(ElemIndex) ? VerticesArray[2*PointIndex][ElemIndex] : DefaultValue;
			}
		}
	}

	static void TransferVerticesAttribute(const FManagedArrayCollection& InCollection, FManagedArrayCollection* OutCollection, const FCollectionAttributeKey& AttributeToCopy,
		const int32 NumPoints, const FName& PointsGroup)
	{
		const FName AttributeName(AttributeToCopy.Attribute);
		const FName VerticesGroup(AttributeToCopy.Group);
		
		if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FFloatType)
		{
			BuildVerticesAttribute<float>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FVector4fType)
		{
			BuildVerticesAttribute<FVector4f>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FVectorType)
		{
			BuildVerticesAttribute<FVector3f>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FVector2DType)
		{
			BuildVerticesAttribute<FVector2f>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FInt32Type)
		{
			BuildVerticesAttribute<int32>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FIntVector4Type)
		{
			BuildVerticesAttribute<FIntVector4>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FIntVectorType)
		{
			BuildVerticesAttribute<FIntVector3>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FIntVector2Type)
		{
			BuildVerticesAttribute<FIntVector2>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FBoolType)
		{
			BuildVerticesAttribute<bool>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FLinearColorType)
		{
			BuildVerticesAttribute<FLinearColor>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FQuatType)
		{
			BuildVerticesAttribute<FQuat4f>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FTransform3fType)
		{
			BuildVerticesAttribute<FTransform3f>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup); 
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FInt32ArrayType)
		{
			BuildVerticesArray<int32,FIntVector4,4>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup, INDEX_NONE);
		}
		else if(InCollection.GetAttributeType(AttributeName, VerticesGroup) == EManagedArrayType::FFloatArrayType)
		{
			BuildVerticesArray<float,FVector4f,4>(InCollection, OutCollection, NumPoints, AttributeName, VerticesGroup, PointsGroup, 0.0f);
		}
	}
	
	static void TransferVerticesAttributes(const FManagedArrayCollection& InCollection, FManagedArrayCollection* OutCollection,
		const int32 NumPoints, const TArray<FName>& AttributesToSkip, const FName& VerticesGroup, const FName& PointsGroup)
	{
		// Transfer vertices weight maps onto the points to be stored onto the rest collection
		const TArray<FName> AttributeNames = InCollection.AttributeNames(VerticesGroup);
		for(const FName& AttributeName : AttributeNames)
		{
			if(!AttributesToSkip.Contains(AttributeName))
			{
				FCollectionAttributeKey AttributeKey(AttributeName.ToString(),VerticesGroup.ToString());
				TransferVerticesAttribute(InCollection, OutCollection, AttributeKey, NumPoints, PointsGroup);
			}
		}
	}

	static void RegisterSkeletalMeshes(const FManagedArrayCollection& InCollection, const FName& SkelMeshAttribute, const FName& MeshLODAttribute, const FName& GeometryGroup, const int32 NumGeometry, UGroomAsset* GroomAsset)
	{
		const TManagedArray<TObjectPtr<UObject>>& ObjectSkeletalMeshes =
						InCollection.GetAttribute<TObjectPtr<UObject>>(SkelMeshAttribute, GeometryGroup);

		const TManagedArray<int32>& ObjectMeshLODs =
			InCollection.GetAttribute<int32>(MeshLODAttribute, GeometryGroup);
				
		for(int32 GroupIndex = 0; GroupIndex < NumGeometry; ++GroupIndex)
		{
			GroomAsset->GetDataflowSettings().SetSkeletalMesh(GroupIndex,
				Cast<USkeletalMesh>(ObjectSkeletalMeshes[GroupIndex]), ObjectMeshLODs[GroupIndex]);
		}
	}
	
	static void TransferCurvesAttributes(const FManagedArrayCollection& InCollection, FManagedArrayCollection* OutCollection, 
		const TArray<FCollectionAttributeKey>& ExternalAttributes, const TArray<FCollectionAttributeKey>& InternalAttributes, const FString& GroupPrefix)
	{
		const FName PointsGroup(GroupPrefix + GeometryCollection::Facades::FCollectionCurveGeometryFacade::PointsGroup.ToString());
		const int32 NumPoints = InCollection.NumElements(FGeometryCollection::VerticesGroup) / 2;
		
		auto AddAttributeKeys = [&InCollection, &OutCollection, &GroupPrefix, &PointsGroup, &NumPoints](const TArray<FCollectionAttributeKey>& AttributeKeys)
			{
				for (const FCollectionAttributeKey& AttributeKey : AttributeKeys)
				{
					if(InCollection.HasAttribute(FName(AttributeKey.Attribute), FName(AttributeKey.Group)))
					{
						if (AttributeKey.Group == GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup ||
							AttributeKey.Group == GeometryCollection::Facades::FCollectionCurveGeometryFacade::PointsGroup)
						{
							CopyCollectionAttribute(&InCollection, OutCollection, AttributeKey, GroupPrefix);
						}
						else if (AttributeKey.Group == FGeometryCollection::GeometryGroup)
						{
							CopyCollectionAttribute(&InCollection, OutCollection, AttributeKey, GroupPrefix);
						}
						else if(AttributeKey.Group == FGeometryCollection::VerticesGroup)
						{
							TransferVerticesAttribute(InCollection, OutCollection, AttributeKey, NumPoints, PointsGroup);
						}
					}
				}
			};

		AddAttributeKeys(ExternalAttributes);
		AddAttributeKeys(InternalAttributes);
	}
}

FGroomAssetTerminalDataflowNode::FGroomAssetTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	DataflowAssetWeakPtr = Cast<UDataflow>(InParam.OwningObject);

	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FGroomAssetTerminalDataflowNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
}

void FGroomAssetTerminalDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FGroomAssetTerminalDataflowNode_v2::FGroomAssetTerminalDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	DataflowAssetWeakPtr = Cast<UDataflow>(InParam.OwningObject);

	RegisterInputConnection(&GuidesCollection);
	RegisterInputConnection(&StrandsCollection);
}

void FGroomAssetTerminalDataflowNode_v2::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(Asset.Get()))
	{
		const FManagedArrayCollection& GroomStrands = GetValue<FManagedArrayCollection>(Context, &StrandsCollection);
		const FManagedArrayCollection& GroomGuides = GetValue<FManagedArrayCollection>(Context, &GuidesCollection);
		
		GeometryCollection::Facades::FCollectionCurveGeometryFacade GuidesFacade(GroomGuides);
		GeometryCollection::Facades::FCollectionCurveGeometryFacade StrandsFacade(GroomStrands);

		if(GuidesFacade.IsValid() || StrandsFacade.IsValid())
		{ 
			FManagedArrayCollection* OutCollection = new FManagedArrayCollection();

			TArray<FCollectionAttributeKey> SkinningAttributes = {
				{GeometryCollection::Facades::FVertexBoneWeightsFacade::KinematicWeightAttributeName.ToString(), FGeometryCollection::VerticesGroup.ToString()},
				{GeometryCollection::Facades::FVertexBoneWeightsFacade::BoneIndicesAttributeName.ToString(), FGeometryCollection::VerticesGroup.ToString()},
				{GeometryCollection::Facades::FVertexBoneWeightsFacade::BoneWeightsAttributeName.ToString(), FGeometryCollection::VerticesGroup.ToString()}};

			TArray<FCollectionAttributeKey> HierarchyAttributes = {
				{GeometryCollection::Facades::FCollectionCurveHierarchyFacade::CurveParentIndicesAttribute.ToString(), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup.ToString()},
				{GeometryCollection::Facades::FCollectionCurveHierarchyFacade::CurveLodIndicesAttribute.ToString(), GeometryCollection::Facades::FCollectionCurveGeometryFacade::CurvesGroup.ToString()}};

			TArray<FCollectionAttributeKey> ExternalAttributes;
			for(int32 AttributeIndex = 0; AttributeIndex < AttributeKeys.Num(); ++AttributeIndex)
			{
				ExternalAttributes.Add(GetValue<FCollectionAttributeKey>(Context, GetConnectionReference(AttributeIndex)));
			}
		
			TArray<FCollectionAttributeKey> InternalAttributes;
			//InternalAttributes.Append(SkinningAttributes);
			//InternalAttributes.Append(HierarchyAttributes);

			FEditableGroom EditGroom;
			ConvertFromGroomAsset(const_cast<UGroomAsset*>(GroomAsset), &EditGroom, false, false, false);
			FEditableGroom SourceGroom = EditGroom;
		
			if(StrandsFacade.IsValid())
			{
				UE::Groom::Private::TransferCurvesAttributes(GroomStrands, OutCollection, ExternalAttributes, InternalAttributes, FString("Strands"));

				if(StrandsFacade.GetNumGeometry() == EditGroom.Groups.Num())
				{
					// Build the editable strands
					UE::Groom::Private::BuildEditableStrands(StrandsFacade.GetGeometryCurveOffsets(), StrandsFacade.GetCurvePointOffsets(),
						StrandsFacade.GetPointRestPositions(), StrandsFacade.GetCurveSourceIndices(), SourceGroom, EditGroom);
				}
			}
			if(GuidesFacade.IsValid())
			{
				UE::Groom::Private::TransferCurvesAttributes(GroomGuides, OutCollection, ExternalAttributes, InternalAttributes, FString("Guides"));
				
				if(GuidesFacade.GetNumGeometry() == EditGroom.Groups.Num())
				{
					// Build the editable guides
					UE::Groom::Private::BuildEditableGuides(GuidesFacade.GetGeometryCurveOffsets(), GuidesFacade.GetCurvePointOffsets(),
						GuidesFacade.GetPointRestPositions(), GuidesFacade.GetCurveSourceIndices(), SourceGroom, EditGroom);
				}
			}

			// Ensure compilation dependent assets is done
			FAssetCompilingManager::Get().FinishCompilationForObjects({ GroomAsset });

			// Convert to groom asset
			ConvertToGroomAsset(const_cast<UGroomAsset*>(GroomAsset), &EditGroom, EEditableGroomOperations::ControlPoints_Modified);
	
			// To prevent future reconstruction in the BuildData we set the type to be imported
			for(FHairGroupsInterpolation& GroupInterpolation : GroomAsset->GetHairGroupsInterpolation())
			{
				GroupInterpolation.InterpolationSettings.GuideType = EGroomGuideType::Imported;
			}

			GroomAsset->GetDataflowSettings().SetRestCollection(OutCollection);
			GroomAsset->GetDataflowSettings().InitSkeletalMeshes(GuidesFacade.GetNumGeometry());

			if(GroomGuides.HasAttribute(GeometryCollection::Facades::FVertexBoneWeightsFacade::SkeletalMeshAttributeName, FGeometryCollection::GeometryGroup))
			{
				UE::Groom::Private::RegisterSkeletalMeshes(GroomGuides, GeometryCollection::Facades::FVertexBoneWeightsFacade::SkeletalMeshAttributeName,
					GeometryCollection::Facades::FVertexBoneWeightsFacade::GeometryLODAttributeName,
					FName(FGeometryCollection::GeometryGroup), GuidesFacade.GetNumGeometry(), GroomAsset);
			}
			else if(GroomStrands.HasAttribute(GeometryCollection::Facades::FVertexBoneWeightsFacade::SkeletalMeshAttributeName, FGeometryCollection::GeometryGroup))
			{
				UE::Groom::Private::RegisterSkeletalMeshes(GroomStrands, GeometryCollection::Facades::FVertexBoneWeightsFacade::SkeletalMeshAttributeName,
					GeometryCollection::Facades::FVertexBoneWeightsFacade::GeometryLODAttributeName,
					FName(FGeometryCollection::GeometryGroup), StrandsFacade.GetNumGeometry(), GroomAsset);
			}
		}
	}
}

void FGroomAssetTerminalDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{}

UE::Dataflow::TConnectionReference<FCollectionAttributeKey> FGroomAssetTerminalDataflowNode_v2::GetConnectionReference(int32 Index) const
{
	return { &AttributeKeys[Index], Index, &AttributeKeys };
}

TArray<UE::Dataflow::FPin> FGroomAssetTerminalDataflowNode_v2::AddPins()
{
	const int32 Index = AttributeKeys.AddDefaulted();
	FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));

	// Make sure we have the right number of names
	AttributeNames.SetNum(AttributeKeys.Num());

	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FGroomAssetTerminalDataflowNode_v2::GetPinsToRemove() const
{
	const int32 Index = AttributeKeys.Num() - 1;
	check(AttributeKeys.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FGroomAssetTerminalDataflowNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = AttributeKeys.Num() - 1;
	check(AttributeKeys.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	AttributeKeys.SetNum(Index);

	// Make sure we have the right number of names
	AttributeNames.SetNum(AttributeKeys.Num());

	return Super::OnPinRemoved(Pin);
}

void FGroomAssetTerminalDataflowNode_v2::PostSerialize(const FArchive& Ar)
{
	// because we add pins we need to make sure we restore them when loading
	// to make sure they can get properly reconnected

	if (Ar.IsLoading())
	{
		check(AttributeKeys.Num() >= 0);
		// register new elements from the array as inputs
		for (int32 Index = 0; Index < AttributeKeys.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			// if we have more inputs than materials then we need to unregister the inputs 
			const int32 NumAttributeInputs = (GetNumInputs() - NumOtherInputs);
			const int32 NumInputs = AttributeKeys.Num();
			if (NumAttributeInputs > NumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				AttributeKeys.SetNum(NumAttributeInputs);
				for (int32 Index = NumInputs; Index < AttributeKeys.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				AttributeKeys.SetNum(NumInputs);
			}
		}
		else
		{
			ensureAlways(AttributeKeys.Num() + NumOtherInputs == GetNumInputs());
		}
		// make sure the number of names is the same as the current number of key inputs
		AttributeNames.SetNum(AttributeKeys.Num());
		SyncInputNames();
	}
}

bool FGroomAssetTerminalDataflowNode_v2::ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FGroomAssetTerminalDataflowNode_v2, AttributeNames))
	{
		// we only changing the name of the inputs, no need to inavlidate the node
		return false;
	}
	return Super::ShouldInvalidateOnPropertyChanged(InPropertyChangedEvent);
}

void FGroomAssetTerminalDataflowNode_v2::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FGroomAssetTerminalDataflowNode_v2, AttributeNames))
	{
		SyncInputNames();
	}
}

bool FGroomAssetTerminalDataflowNode_v2::SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const
{
	if (TypeName == UE::Groom::Private::AttributeKeyTypeName && Direction == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		return true;
	}
	return false;
}

const FDataflowConnection* FGroomAssetTerminalDataflowNode_v2::OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection)
{
	if (DroppedConnection.GetType() == UE::Groom::Private::AttributeKeyTypeName && DroppedConnection.GetDirection() == UE::Dataflow::FPin::EDirection::OUTPUT)
	{
		// Make sure we have the right number of names
		AttributeNames.SetNum(AttributeKeys.Num());

		const int32 Index = AttributeKeys.AddDefaulted();
		FDataflowInput& NewInput = RegisterInputArrayConnection(GetConnectionReference(Index));

		// infer the name for the new entry from the dropped connection
		const FProperty* DroppedProperty = DroppedConnection.GetProperty();
		const FName DroppedDisplayName = DroppedProperty 
			? FName(DroppedProperty->GetDisplayNameText().ToString()) 
			: DroppedConnection.GetName();

		const FName NewName = GenerateUniqueInputName(DroppedDisplayName);
		if (!NewName.IsNone())
		{
			AttributeNames.Add(NewName);
			NewInput.SetName(NewName);
		}
		return &NewInput;
	}
	return nullptr;
}

void FGroomAssetTerminalDataflowNode_v2::SyncInputNames()
{
	bool bChanged = false;
	for (int32 KeyIndex = 0; KeyIndex < AttributeKeys.Num(); ++KeyIndex)
	{
		if (FDataflowInput* Input = FindInput(GetConnectionReference(KeyIndex)))
		{
			if (AttributeNames.IsValidIndex(KeyIndex) && Input->GetName() != AttributeNames[KeyIndex])
			{
				const FName UniqueName = GenerateUniqueInputName(AttributeNames[KeyIndex]);
				if (!UniqueName.IsNone())
				{
					Input->SetName(UniqueName);
					bChanged = true;
				}
			}
		}
	}

	// refresh the Ed Node is necessary
	if (bChanged)
	{
		if (TStrongObjectPtr<UDataflow> DataflowAsset = DataflowAssetWeakPtr.Pin())
		{
			DataflowAsset->RefreshEdNodeByGuid(GetGuid());
		}
	}
}

FName FGroomAssetTerminalDataflowNode_v2::GenerateUniqueInputName(FName BaseName) const
{
	if (BaseName.IsNone())
	{
		return BaseName;
	}
	FName NewName = BaseName;
	int32 SuffixNumber = 1;
	while (FindInput(NewName) != nullptr)
	{
		NewName.SetNumber(SuffixNumber++);
	}
	return NewName;
}

