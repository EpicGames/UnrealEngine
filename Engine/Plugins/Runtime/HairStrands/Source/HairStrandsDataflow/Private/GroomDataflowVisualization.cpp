// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomDataflowVisualization.h"
#include "GroomComponent.h"
#include "GroomVisualizationData.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationViewportClient.h"
#include "Dataflow/DataflowSimulationScene.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetDataflowSimulationVisualization"

namespace UE::Groom
{
	const FName FGroomDataflowSimulationVisualization::Name = FName("GroomDataflowSimulationVisualization");

	FName FGroomDataflowSimulationVisualization::GetName() const
	{
		return Name;
	}

	FGroomDataflowSimulationVisualization::FGroomDataflowSimulationVisualization()
	{
		VisualizationFlags.Init(false, static_cast<uint8>(EGroomViewMode::Count));
	}

	void FGroomDataflowSimulationVisualization::ExtendSimulationVisualizationMenu(const TSharedPtr<FDataflowSimulationViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder)
	{
		if (ViewportClient)
		{
			if (TSharedPtr<FDataflowEditorToolkit> Toolkit = ViewportClient->GetDataflowEditorToolkit().Pin())
			{
				if (const TSharedPtr<FDataflowSimulationScene>& SimulationScene = Toolkit->GetSimulationScene())
				{
					if (const UGroomComponent* GroomComponent = GetGroomComponent(SimulationScene.Get()))
					{
						constexpr uint8 NumFlags = static_cast<uint8>(EGroomViewMode::Count);
						auto AddSimulationVisualisation = [this, &ViewportClient , &MenuBuilder, NumFlags](const TAttribute<FText>& FlagLabel, const EGroomViewMode ViewMode)
							{
								const uint8 FlagIndex = static_cast<uint8>(ViewMode);

								// Handler for visualization entry being clicked
								const FExecuteAction ExecuteAction = FExecuteAction::CreateLambda([this, FlagIndex, ViewMode, ViewportClient]()
									{
										VisualizationFlags[FlagIndex] = !VisualizationFlags[FlagIndex];
										if (VisualizationFlags[FlagIndex])
										{
											ViewportClient->ChangeGroomVisualizationMode(GetGroomViewModeName(ViewMode));
											for (int32 OtherIndex = 0; OtherIndex < NumFlags; ++OtherIndex)
											{
												if (OtherIndex != FlagIndex)
												{
													VisualizationFlags[OtherIndex] = false;
												}
											}
										}
										else
										{
											ViewportClient->ChangeGroomVisualizationMode(GetGroomViewModeName(EGroomViewMode::None));
										}
									});

								// Check state function for visualization entries
								const FIsActionChecked IsActionChecked = FIsActionChecked::CreateLambda([this, FlagIndex]()
									{
										return VisualizationFlags[FlagIndex];
									});

								const FUIAction Action(ExecuteAction, FCanExecuteAction(), IsActionChecked);

								// Add menu entry
								MenuBuilder.AddMenuEntry(FlagLabel, FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::ToggleButton);
							};

						bool bEnableGuidesVisualisation = false;
						if(TObjectPtr<UGroomAsset> GroomAsset = GroomComponent->GroomAsset)
						{ 
							for (int32 GroupIndex = 0; GroupIndex < GroomAsset->GetNumHairGroups(); ++GroupIndex)
							{
								if (GroomAsset->IsSimulationEnable(GroupIndex, INDEX_NONE) || GroomAsset->IsDeformationEnable(GroupIndex))
								{
									bEnableGuidesVisualisation = true;
									break;
								}
							}
						}

						MenuBuilder.BeginSection(TEXT("GroomSimulation_Visualizations"), LOCTEXT("GroomSimulationVisualization", "Groom Simulation Visualization"));

						if(bEnableGuidesVisualisation)
						{ 
							AddSimulationVisualisation(LOCTEXT("DeformedGuides", "Deformed guides"), EGroomViewMode::SimHairStrands);
							AddSimulationVisualisation(LOCTEXT("GuidesInfluence", "Guides influence"), EGroomViewMode::RenderHairStrands);
						}
						AddSimulationVisualisation(LOCTEXT("StrandsGroups", "Strands groups"), EGroomViewMode::Group);
						AddSimulationVisualisation(LOCTEXT("StrandsClumps", "Strands clumps"), EGroomViewMode::ClumpID);
						AddSimulationVisualisation(LOCTEXT("StrandsClusters", "Strands clusters"), EGroomViewMode::Cluster);
						AddSimulationVisualisation(LOCTEXT("MeshProjection", "Mesh projection"), EGroomViewMode::MeshProjection);
						AddSimulationVisualisation(LOCTEXT("CardsGuides", "Cards guides"), EGroomViewMode::CardGuides);
						AddSimulationVisualisation(LOCTEXT("LODColoration", "LOD Coloration"), EGroomViewMode::LODColoration);
						
						MenuBuilder.EndSection();
					}
				}
			}
		}
	}

	const UGroomComponent* FGroomDataflowSimulationVisualization::GetGroomComponent(const FDataflowSimulationScene* SimulationScene) const
	{
		if (SimulationScene)
		{
			if (const TObjectPtr<AActor> PreviewActor = SimulationScene->GetPreviewActor())
			{
				return PreviewActor->GetComponentByClass<UGroomComponent>();
			}
		}
		return nullptr;
	}

	void FGroomDataflowSimulationVisualization::Draw(const FDataflowSimulationScene* SimulationScene, FPrimitiveDrawInterface* PDI)
	{
		
	}

	void FGroomDataflowSimulationVisualization::DrawCanvas(const FDataflowSimulationScene* SimulationScene, FCanvas* Canvas, const FSceneView* SceneView)
	{
		
	}

	FText FGroomDataflowSimulationVisualization::GetDisplayString(const FDataflowSimulationScene* SimulationScene) const
	{
		return FText();
	}

}  // End namespace UE::Groom

#undef LOCTEXT_NAMESPACE 
 