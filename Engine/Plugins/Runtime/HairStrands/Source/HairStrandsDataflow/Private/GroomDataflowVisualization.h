// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowSimulationVisualization.h"

class UGroomComponent;

namespace UE::Groom
{
	/** Dataflow simulation customisation for groom */
	class FGroomDataflowSimulationVisualization : public Dataflow::IDataflowSimulationVisualization
	{
	public:
		static const FName Name;
		virtual ~FGroomDataflowSimulationVisualization() = default;
		FGroomDataflowSimulationVisualization();

	private:

		//~ Begin IDataflowSimulationVisualization interface
		virtual FName GetName() const override;
		virtual void ExtendSimulationVisualizationMenu(const TSharedPtr<FDataflowSimulationViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder) override;
		virtual void Draw(const FDataflowSimulationScene* SimulationScene, FPrimitiveDrawInterface* PDI) override;
		virtual void DrawCanvas(const FDataflowSimulationScene* SimulationScene, FCanvas* Canvas, const FSceneView* SceneView) override;
		virtual FText GetDisplayString(const FDataflowSimulationScene* SimulationScene) const override;
		//~ End IDataflowSimulationVisualization interface

		/** Get the groom component given a simulation scene */
		const UGroomComponent* GetGroomComponent(const FDataflowSimulationScene* SimulationScene) const;

		/** Visualization flags */
		TArray<bool> VisualizationFlags;
	};

}  // End namespace UE::Chaos::ClothAsset
