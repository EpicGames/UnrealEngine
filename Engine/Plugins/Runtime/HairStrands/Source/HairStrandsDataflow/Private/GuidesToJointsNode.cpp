// Copyright Epic Games, Inc. All Rights Reserved.

#include "GuidesToJointsNode.h"

#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "StaticToSkeletalMeshConverter.h"
#include "SkeletalMeshAttributes.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "SkinnedAssetCompiler.h"
#include "Widgets/Notifications/SNotificationList.h"

#if WITH_EDITOR
#include "Editor.h"		// For GEditor
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GuidesToJointsNode)

#define LOCTEXT_NAMESPACE "DataflowGuidesToJointsNode"

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "Animation/Skeleton.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDataflowGuidesToJointsNode::FDataflowGuidesToJointsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CurveSelection);
	RegisterInputConnection(&ParentBone);
	RegisterInputConnection(&BoneBaseName);
	RegisterInputConnection(&SkeletalMeshAssetPath);
	RegisterInputConnection(&SkeletonAssetPath);
	RegisterInputConnection(&BaseSkeletalMesh);

	RegisterOutputConnection(&SkeletalMeshAsset);
	RegisterOutputConnection(&SkeletonAsset);

	BoneBaseName.Value = TEXT("Curve");
}

void FDataflowGuidesToJointsNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue, Evaluate only return an existing one or nullptr
	const FString& InAssetPath = GetValue(Context, &SkeletalMeshAssetPath);
	TObjectPtr<USkeletalMesh> OutSkeletalMesh = ::LoadObject<USkeletalMesh>(nullptr, InAssetPath);
	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);

	TObjectPtr<USkeleton> OutSkeleton = (OutSkeletalMesh)? OutSkeletalMesh->GetSkeleton(): nullptr;
	SetValue(Context, OutSkeleton, &SkeletonAsset);
}

void FDataflowGuidesToJointsNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	TObjectPtr<USkeletalMesh> NullSkeletalMeshPtr = nullptr;
	TObjectPtr<USkeleton> NullSkeletonPtr = nullptr;

	auto SetOutputsOnFailure = [this, &Context, &NullSkeletalMeshPtr, &NullSkeletonPtr]()
		{
			SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
			SetValue(Context, NullSkeletonPtr, &SkeletonAsset);
		};

	//-------------------------------------------------------------------
	// input validation section
	//-------------------------------------------------------------------

	// veryfy the collection and selection
	const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
	const FDataflowCurveSelection& InCurveSelection = GetValue(Context, &CurveSelection);

	// consider all curves when selection is not valid 
	const bool bValidSelection = InCurveSelection.IsValidForCollection(InCollection);;
	const bool bUseAllCurves = (!bValidSelection);

	GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(InCollection);
	if (!CurvesFacade.IsValid())
	{
		Context.Error(LOCTEXT("InvalidCurveFacade", "Invalid collection - do not contains curve information"), this);
		SetOutputsOnFailure();
		return;
	}

	constexpr int32 MaxNumSelectedCurves = 100;
	const int32 NumSelectedCurves = bUseAllCurves? CurvesFacade.GetNumCurves(): InCurveSelection.NumSelected();
	if (NumSelectedCurves > MaxNumSelectedCurves)
	{
		Context.Error(FText::Format(LOCTEXT("TooManyCurves", "Too many curve selected - Limit is {0}"), FText::AsNumber(MaxNumSelectedCurves)), this);
		SetOutputsOnFailure();
		return;
	}

	TObjectPtr<const USkeletalMesh> InBaseSkeletalMesh = GetValue(Context, &BaseSkeletalMesh);
	if (IsValid(InBaseSkeletalMesh))
	{
		const FString SkeletalMeshPathFromInput = GetValue(Context, &SkeletalMeshAssetPath);
		if (InBaseSkeletalMesh == Asset || InBaseSkeletalMesh->GetPathName() == SkeletalMeshPathFromInput)
		{
			Context.Error(FText::Format(LOCTEXT("BaseSkeletalMeshSameAsOuput", "Base skeletal mesh is the same as the output asset : {0}"), FText::FromString(InBaseSkeletalMesh->GetPathName())), this);
			SetOutputsOnFailure();
			return;
		}
	}

#if WITH_EDITOR

	//-------------------------------------------------------------------
	// skeleton section
	//-------------------------------------------------------------------
	FString SkeletonPathFromInput = GetValue(Context, &SkeletonAssetPath);
	if (SkeletonPathFromInput.IsEmpty())
	{
		// use the skeletal mesh name 
		SkeletonPathFromInput = GetValue(Context, &SkeletalMeshAssetPath);
		if (!SkeletonPathFromInput.IsEmpty())
		{
			FString PathPart;
			FString FilenamePart;
			FString ExtensionPart;
			FPaths::Split(SkeletonPathFromInput, PathPart, FilenamePart, ExtensionPart);
			const TCHAR* PostFix  = TEXT("_Skeleton");
			FilenamePart += PostFix;
			ExtensionPart += PostFix;
			SkeletonPathFromInput = FPaths::SetExtension(PathPart / FilenamePart, ExtensionPart);
		}
	}
	const FString& InSkeletonAssetPath = GetAssetPath(SkeletonPathFromInput, Asset);
	if (InSkeletonAssetPath.IsEmpty())
	{
		Context.Error(FText::Format(LOCTEXT("InvalidSkeletonPathOrBoundAsset", "invalid skeleton path or no skeleton bound asset : {0}"), FText::FromString(SkeletonPathFromInput)), this);
		SetOutputsOnFailure();
		return;
	}

	TObjectPtr<USkeleton> OutSkeleton = Cast<USkeleton>(GetOrCreateAsset(Context, InSkeletonAssetPath, USkeleton::StaticClass()));
	if (!OutSkeleton)
	{
		// error logging already handled by GetOrCreateAsset
		SetOutputsOnFailure();
		return;
	}

	UpdateSkeleton(Context, OutSkeleton);

	SetValue(Context, OutSkeleton, &SkeletonAsset);

	//-------------------------------------------------------------------
	// skeletalMesh section (optional)
	//-------------------------------------------------------------------
	const FString SkeletalMeshPathFromInput = GetValue(Context, &SkeletalMeshAssetPath);
	const FString& InSkeletalMeshAssetPath = GetAssetPath(SkeletalMeshPathFromInput, Asset);
	if (InSkeletalMeshAssetPath.IsEmpty())
	{
		Context.Warning(FText::Format(LOCTEXT("InvalidSkeletalMeshPathOrBoundAsset", "invalid skeletalMesh path or no skeletalMesh bound asset - only saving a skeleton : {0}"), FText::FromString(SkeletalMeshPathFromInput)));
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	TObjectPtr<USkeletalMesh> OutSkeletalMesh = Cast<USkeletalMesh>(GetOrCreateAsset(Context, InSkeletalMeshAssetPath, USkeletalMesh::StaticClass()));
	if (!OutSkeletalMesh)
	{
		// error logging already handled by GetOrCreateAsset
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	UpdateSkeletalMesh(Context, OutSkeletalMesh, OutSkeleton);

	// finally set the skeletal mesh to the output
	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);

#else
	Context.Error(TEXT("Creation of skeletal mesh asset only supported in Editor"), this);
	SetOutputsOnFailure();
#endif
}

int32 FDataflowGuidesToJointsNode::GetRefSkeletonRootIndex(const FReferenceSkeleton& RefSkeleton) const
{
	int32 RootIndex = INDEX_NONE;
	const TArray<FMeshBoneInfo>& BoneInfos = RefSkeleton.GetRefBoneInfo();
	if (BoneInfos.Num() > 0)
	{
		RootIndex = 0;
		while (BoneInfos[RootIndex].ParentIndex != INDEX_NONE)
		{
			RootIndex = BoneInfos[RootIndex].ParentIndex;
		}
	}
	return RootIndex;
}

void FDataflowGuidesToJointsNode::ClearSkeleton(USkeleton* InOutSkeleton) const
{
	// to fully clear a skeleton we need to const cast the reference skeleton 
	FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(InOutSkeleton->GetReferenceSkeleton());
	RefSkeleton.Empty();
	RefSkeleton.RebuildRefSkeleton(InOutSkeleton, true);
}

namespace DataflowGuidesToJointsNode::Private
{
	void AddRefBoneRecursivelyToSkeleton(const FReferenceSkeleton& RefSkeleton, const int32 RefBoneIndex, FReferenceSkeletonModifier& Edit)
	{
		if (RefBoneIndex != INDEX_NONE)
		{
			const FName BoneName = RefSkeleton.GetBoneName(RefBoneIndex);

			int32 ParentIndex = INDEX_NONE;
			const int32 RefParentIndex = RefSkeleton.GetParentIndex(RefBoneIndex);
			if (RefParentIndex != INDEX_NONE)
			{
				const FName RefParentName = RefSkeleton.GetBoneName(RefParentIndex);
				ParentIndex = Edit.FindBoneIndex(RefParentName);
			}
		
			const FTransform BoneTransform = RefSkeleton.GetRefBonePose().IsValidIndex(RefBoneIndex) 
				? RefSkeleton.GetRefBonePose()[RefBoneIndex]
				: FTransform::Identity;

			FMeshBoneInfo BoneInfo(BoneName, BoneName.ToString(), ParentIndex);
			Edit.Add(BoneInfo, BoneTransform, false/*bAllowMultipleRoots*/);

			TArray<int32> Children;
			RefSkeleton.GetDirectChildBones(RefBoneIndex, Children);
			for (const int32& Child : Children)
			{
				AddRefBoneRecursivelyToSkeleton(RefSkeleton, Child, Edit);
			}
		}
	}
}

void FDataflowGuidesToJointsNode::InitializeSkeletonFromRefSkeleton(const FReferenceSkeleton& RefSkeleton, USkeleton* InOutSkeleton) const
{
	ClearSkeleton(InOutSkeleton);
	const int32 RefRootIndex = GetRefSkeletonRootIndex(RefSkeleton);
	if (RefRootIndex != INDEX_NONE)
	{
		FReferenceSkeletonModifier Edit(InOutSkeleton);
		DataflowGuidesToJointsNode::Private::AddRefBoneRecursivelyToSkeleton(RefSkeleton, RefRootIndex, Edit);
	}
}

void FDataflowGuidesToJointsNode::UpdateSkeleton(UE::Dataflow::FContext& Context, USkeleton* InOutSkeleton) const
{
	ClearSkeleton(InOutSkeleton);

	// Get Parent node , search the base skeletal mesh, if not found create one 
	const FString InParentBone = GetValue(Context, &ParentBone);
	const FString InBoneBaseName = GetValue(Context, &BoneBaseName);
	const TObjectPtr<const USkeletalMesh> InBaseSkeletalMesh = GetValue(Context, &BaseSkeletalMesh);

	int32 InParentBoneIndex = INDEX_NONE;
	FTransform InParentBoneTransform = FTransform::Identity;
	if (IsValid(InBaseSkeletalMesh))
	{
		// initialize the skeleton with the ref skeleton
		const FReferenceSkeleton& RefSkeleton = InBaseSkeletalMesh->GetRefSkeleton();
		InitializeSkeletonFromRefSkeleton(RefSkeleton, InOutSkeleton);

		InParentBoneIndex = RefSkeleton.FindRawBoneIndex(FName(InParentBone));
		if (InParentBoneIndex == INDEX_NONE)
		{
			const FText InfoMessage = FText::Format(LOCTEXT("InvalidParentBoneName", "Bone {0} does not exists in Skeletal mesh {1}, use the actual root bone if present"), FText::FromString(InParentBone), FText::FromString(InBaseSkeletalMesh->GetPathName()));
			Context.Info(InfoMessage, this);
		}
		else
		{
			InParentBoneTransform = RefSkeleton.GetBoneAbsoluteTransform(InParentBoneIndex);
		}
	}

	FReferenceSkeletonModifier Edit(InOutSkeleton);

	// Add a root if we haven't found one yet 
	if (InParentBoneIndex == INDEX_NONE)
	{
		InParentBoneIndex = GetRefSkeletonRootIndex(InOutSkeleton->GetReferenceSkeleton());
		if (InParentBoneIndex == INDEX_NONE)
		{
			static const FName DefaultRootBoneName(TEXT("root"));
			FMeshBoneInfo BoneInfo(DefaultRootBoneName, DefaultRootBoneName.ToString(), INDEX_NONE);
			Edit.Add(BoneInfo, FTransform::Identity, false/*bAllowMultipleRoots*/);
			InParentBoneIndex = Edit.FindBoneIndex(DefaultRootBoneName);
		}
		else
		{
			InParentBoneTransform = InOutSkeleton->GetReferenceSkeleton().GetBoneAbsoluteTransform(InParentBoneIndex);
		}
	}

	// add the curves 
	const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
	const FDataflowCurveSelection& InCurveSelection = GetValue(Context, &CurveSelection);

	// consider all curves when selection is not valid 
	const bool bValidSelection = InCurveSelection.IsValidForCollection(InCollection);;
	const bool bUseAllCurves = (!bValidSelection);

	GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(InCollection);
	if (CurvesFacade.IsValid())
	{
		TArray<FVector3f> CurvePoints;
		const int32 NumCurves = CurvesFacade.GetNumCurves();
		for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			if (bUseAllCurves || InCurveSelection.IsSelected(CurveIndex))
			{
				CurvesFacade.GetCurvePointPositions(CurveIndex, CurvePoints);

				// Now add to the skeleton
				FTransform BoneParentTransform = InParentBoneTransform;
				int32 BoneParentIndex = InParentBoneIndex;
				const int32 NumPoints = CurvePoints.Num();
				for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
				{
					const FString BoneNameStr = FString::Format(TEXT("{0}_{1}_{2}"), { InBoneBaseName, FString::FromInt(CurveIndex), FString::FromInt(PointIndex) });
					const FName UniqueBoneName = GenerateUniqueBoneName(FName(BoneNameStr), Edit.GetReferenceSkeleton());
					const FVector BonePosition = FVector(CurvePoints[PointIndex]);
					const FTransform AbsoluteBoneTransform(BonePosition);
					const FTransform LocalBoneTransform = AbsoluteBoneTransform.GetRelativeTransform(BoneParentTransform);
					FMeshBoneInfo BoneInfo(UniqueBoneName, UniqueBoneName.ToString(), BoneParentIndex);
					Edit.Add(BoneInfo, LocalBoneTransform, false);
					BoneParentIndex = Edit.FindBoneIndex(UniqueBoneName);
					BoneParentTransform = AbsoluteBoneTransform;
				}
			}
		}
	}
}

FName FDataflowGuidesToJointsNode::GenerateUniqueBoneName(FName BaseName, const FReferenceSkeleton& RefSkeleton) const
{
	FName UniqueName = BaseName;
	int32 BoneIndex = RefSkeleton.FindRawBoneIndex(UniqueName);
	while (BoneIndex != INDEX_NONE)
	{
		UniqueName.SetNumber(UniqueName.GetNumber() + 1);
		BoneIndex = RefSkeleton.FindRawBoneIndex(UniqueName);
	}
	return UniqueName;
}

void FDataflowGuidesToJointsNode::UpdateSkeletalMesh(UE::Dataflow::FContext& Context, USkeletalMesh* InOutSkeletalMesh, USkeleton* InOutSkeleton) const
{
	InOutSkeletalMesh->PreEditChange(nullptr);
	{
		InOutSkeletalMesh->Clear();
		// Make sure that we create non-transient assets
		InOutSkeletalMesh->ClearFlags(RF_Transient);

		bool bIsCopied = false;
		if (bCopyGeometryFromBaseSkeletalMesh)
		{
			if (const TObjectPtr<USkeletalMesh> InBaseSkeletalMesh = GetValue(Context, &BaseSkeletalMesh))
			{
				TArray<uint8> Buffer;
				 
				FObjectWriter Writer(Buffer);
				InBaseSkeletalMesh->Serialize(Writer);

				FObjectReader Reader(Buffer);
				InOutSkeletalMesh->Serialize(Reader);

				InOutSkeletalMesh->SetSkeleton(InBaseSkeletalMesh->GetSkeleton());
				InOutSkeletalMesh->SetRefSkeleton(InBaseSkeletalMesh->GetRefSkeleton());
				InOutSkeletalMesh->CalculateInvRefMatrices();
				InOutSkeletalMesh->SetMaterials(InBaseSkeletalMesh->GetMaterials());

				// make sure tyhe render data is up to date 
				const FSkeletalMeshModel* SrcModel = InBaseSkeletalMesh->GetImportedModel();
				FSkeletalMeshModel* DstModel = InOutSkeletalMesh->GetImportedModel();
				if (SrcModel && DstModel)
				{
					DstModel->LODModels.Empty();
					for (int32 LODIdx = 0; LODIdx < SrcModel->LODModels.Num(); LODIdx++)
					{
						FSkeletalMeshLODModel* NewLODModel = new FSkeletalMeshLODModel();
						FSkeletalMeshLODModel::CopyStructure(NewLODModel, &SrcModel->LODModels[LODIdx]);
						DstModel->LODModels.Add(NewLODModel);
					}
				}

				InOutSkeletalMesh->Build();
#if WITH_EDITOR
				FSkinnedAssetCompilingManager::Get().FinishCompilation({ InOutSkeletalMesh });
#endif
				bIsCopied = true;
			}
		}

		InOutSkeletalMesh->SetSkeleton(InOutSkeleton);
		InOutSkeletalMesh->SetRefSkeleton(InOutSkeleton->GetReferenceSkeleton());
		InOutSkeleton->RebuildLinkup(InOutSkeletalMesh);
		InOutSkeletalMesh->CalculateInvRefMatrices();

		if (!bIsCopied)
		{
			FMeshDescription MeshDescription;
			const FVertexID VId0 = MeshDescription.CreateVertex();
			const FVertexID VId1 = MeshDescription.CreateVertex();
			const FVertexID VId2 = MeshDescription.CreateVertex();
			const int32 VInst0 = MeshDescription.CreateVertexInstance(VId0);
			const int32 VInst1 = MeshDescription.CreateVertexInstance(VId1);
			const int32 VInst2 = MeshDescription.CreateVertexInstance(VId2);
			TVertexAttributesRef<FVector3f> Positions = MeshDescription.GetVertexPositions();
			Positions[VId0] = FVector3f(0.0f, 0.0f, 0.0f);
			Positions[VId1] = FVector3f(0.1f, 0.0f, 0.0f);
			Positions[VId2] = FVector3f(0.1f, 0.1f, 0.0f);
			const FPolygonGroupID PolyGroupID = MeshDescription.CreatePolygonGroup();
			MeshDescription.CreateTriangle(PolyGroupID, { VInst0, VInst1, VInst2 });

			// add bone weights information
			FSkeletalMeshAttributes SkeletalMeshAttributes{ MeshDescription };
			SkeletalMeshAttributes.Register(/*bKeepExistingAttribute*/true);
			FSkinWeightsVertexAttributesRef VertexSkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();
			UE::AnimationCore::FBoneWeights Weights;
			Weights.SetBoneWeight(0, 1.0f);
			VertexSkinWeights.Set(VId0, Weights);
			VertexSkinWeights.Set(VId1, Weights);
			VertexSkinWeights.Set(VId2, Weights);

			TArray<const FMeshDescription*> MeshDescriptions = { &MeshDescription };
			// Update the skeletal mesh from the description
			FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
			//Parameters.Materials = SkeletalMaterials;
			Parameters.bRecomputeNormals = true;
			Parameters.bRecomputeTangents = false;

			FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
				InOutSkeletalMesh,
				MeshDescriptions,
				InOutSkeleton->GetReferenceSkeleton(),
				Parameters);

		}

		// update the relationship between the asset
		InOutSkeleton->SetPreviewMesh(InOutSkeletalMesh);
		InOutSkeleton->RecreateBoneTree(InOutSkeletalMesh);
	}

	// make sure the new bones are properly referenced in the skeletal mesh LODs
	if (InOutSkeletalMesh->GetImportedModel())
	{
		for (FSkeletalMeshLODModel& LODModel : InOutSkeletalMesh->GetImportedModel()->LODModels)
		{
			USkeletalMesh::CalculateRequiredBones(LODModel, InOutSkeletalMesh->GetRefSkeleton(), nullptr);
		}
	}

	InOutSkeletalMesh->InvalidateDeriveDataCacheGUID();
	InOutSkeletalMesh->InitResources();
	InOutSkeletalMesh->PostEditChange();

	// final rebuild for good measure 
	InOutSkeletalMesh->Build();
#if WITH_EDITOR
	FSkinnedAssetCompilingManager::Get().FinishCompilation({ InOutSkeletalMesh });
#endif
}

#if WITH_EDITOR
void FDataflowGuidesToJointsNode::OnDoubleClicked(UE::Dataflow::FContext* Context) const
{
	if (Context)
	{
		const FString& InAssetPath = GetValue(*Context, &SkeletalMeshAssetPath);
		if (TObjectPtr<USkeletalMesh> LoadedSkeletalMesh = ::LoadObject<USkeletalMesh>(nullptr, InAssetPath))
		{
			if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetEditorSubsystem->OpenEditorForAssets({ LoadedSkeletalMesh });
			}
		}
		else
		{
			FNotificationInfo NotificationInfo(LOCTEXT("SkeletalMeshEditorFailedNoAsset", "Failed to open SkeletalMesh Editor : No Skeletal Mesh Asset"));
			NotificationInfo.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE