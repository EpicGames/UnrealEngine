// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "Dataflow/DataflowSelection.h"
//#include "Dataflow/Skeleton.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "UObject/ObjectPtr.h"

#include "GuidesToJointsNode.generated.h"


/** 
* Generate joints/bones from a guide selection and save the resulting skeleton asset on disk 
*/
USTRUCT(meta = (Experimental, DataflowGroom))
struct FDataflowGuidesToJointsNode : public FDataflowTerminalNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGuidesToJointsNode, "GuidesToJoints", "Terminal", "Bones Rigged SkeletonMesh")

public:
	FDataflowGuidesToJointsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
#if WITH_EDITOR
	virtual void OnDoubleClicked(UE::Dataflow::FContext* Context) const override;
#endif
	//~ End FDataflowNode interface

	//~ Begin FDataflowTerminalNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	//~ End FDataflowTerminalNode interface

	void ClearSkeleton(USkeleton* InOutSkeleton) const;
	void UpdateSkeleton(UE::Dataflow::FContext& Context, USkeleton* InOutSkeleton) const;
	void InitializeSkeletonFromRefSkeleton(const FReferenceSkeleton& RefSkeleton, USkeleton* InOutSkeleton) const;
	int32 GetRefSkeletonRootIndex(const FReferenceSkeleton& RefSkeleton) const;

	void UpdateSkeletalMesh(UE::Dataflow::FContext& Context, USkeletalMesh* InOutSkeletalMesh, USkeleton* InOutSkeleton) const;
	FName GenerateUniqueBoneName(FName BaseName, const FReferenceSkeleton& RefSkeleton) const;

	/** Managed array collection source of the guides - Required */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Path to the skeletal mesh asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString SkeletalMeshAssetPath;

	/** Path to the skeleton asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString SkeletonAssetPath;

	/** Curve selection to use to generate bones/joints - if none specified all curve will be considered */
	UPROPERTY(meta = (DataflowInput))
	FDataflowCurveSelection CurveSelection;

	/** Base name for the bones created from the curve */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FDataflowStringTypes BoneBaseName;

	/** Optional name of parent bone from the input skeletal mesh to use as the root of the generate bones */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DataflowInput))
	FDataflowStringTypes ParentBone;

	/** Skeletal Mesh to use a base*/
	UPROPERTY(EditAnywhere, Category = Base, meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> BaseSkeletalMesh = nullptr;

	/** Option to copy the geometry from the base skeletal mesh */
	UPROPERTY(EditAnywhere, Category = Base)
	bool bCopyGeometryFromBaseSkeletalMesh = false;

	/** Created Skeletal Mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset = nullptr;

	/** Created Skeleton */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeleton> SkeletonAsset = nullptr;
};
