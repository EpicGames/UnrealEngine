// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsDataflowModule.h"

#include "AttachGuidesRootsNode.h"
#include "BuildGuidesLODsNode.h"
#include "GetGroomAssetNode.h"
#include "GroomAssetTerminalNode.h"
#include "GetGroomAttributesNodes.h"
#include "ResampleGuidesPointsNode.h"
#include "BuildGroomSkinningNodes.h"
#include "BuildGroomSplineSkinningNode.h"
#include "GenerateGuidesCurvesNode.h"
#include "SmoothGuidesCurvesNode.h"
#include "AverageCurvesNode.h"
#include "GuidesToJointsNode.h"
#include "GroomDataflowVisualization.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "HairStrandsDataflow"

void FHairStrandsDataflowModule::StartupModule()
{
	// Deprecated nodes
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetGroomAssetDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateGuidesCurvesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FResampleGuidesPointsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSmoothGuidesCurvesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransferSkinWeightsGroomNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAttachGuidesRootsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBuildGuidesLODsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGroomAssetTerminalDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetGroomAttributesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBuildGroomSplineSkinWeightsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertLinearToSplineSkinWeightsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertSplineToLinearSkinWeightsNode);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBuildSplineSkinWeightsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FLinearToSplineSkinWeightsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSplineToLinearSkinWeightsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetGroomAssetDataflowNode_v2);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGroomAssetToCollectionDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGroomAssetTerminalDataflowNode_v2);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FResampleCurvePointsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransferGeometrySkinWeightsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateCurveGeometryDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetCurveAttributesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSmoothCurvePointsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAttachCurveRootsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBuildCurveWeightsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBuildCurveLODsDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowAverageCurves);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGuidesToJointsNode);

	UE::Dataflow::FDataflowSimulationVisualizationRegistry::GetInstance().RegisterVisualization(
		MakeUnique<UE::Groom::FGroomDataflowSimulationVisualization>());

	// register node category for groom asset
	UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("Groom", UGroomAsset);
	UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("GeometryCollection", UGroomAsset);
}

void FHairStrandsDataflowModule::ShutdownModule()
{
	UE::Dataflow::FDataflowSimulationVisualizationRegistry::GetInstance().DeregisterVisualization(
		UE::Groom::FGroomDataflowSimulationVisualization::Name);
}

IMPLEMENT_MODULE(FHairStrandsDataflowModule, HairStrandsDataflow)

#undef LOCTEXT_NAMESPACE
