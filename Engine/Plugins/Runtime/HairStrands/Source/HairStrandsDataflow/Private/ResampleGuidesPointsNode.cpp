// Copyright Epic Games, Inc. All Rights Reserved.

#include "ResampleGuidesPointsNode.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResampleGuidesPointsNode)

#define LOCTEXT_NAMESPACE "ResampleGuidesPointsNode"

namespace UE::Groom::Private
{
	int32 ComputeCurvesPoints(const GeometryCollection::Facades::FCollectionCurveGeometryFacade& CurvesFacade, const FDataflowCurveSelection& CurveSelection, const int32 GuidePoints) 
	{
		const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());

		int32 PointOffset = 0, NumPoints = 0;
		for(int32 CurveIndex = 0, NumCurves = CurvesFacade.GetNumCurves(); CurveIndex < NumCurves; ++CurveIndex)
		{
			if(!bValidSelection || (bValidSelection && (CurveSelection.IsSelected(CurveIndex))))
			{
				NumPoints += GuidePoints;
			}
			else
			{
				NumPoints += CurvesFacade.GetCurvePointOffsets()[CurveIndex] - PointOffset;
			}
			PointOffset = CurvesFacade.GetCurvePointOffsets()[CurveIndex];
		}
		return NumPoints;
	}

	void BuildResampledPoints(const TArray<FVector3f>& PointRestPositions, const TArray<int32>& CurveObjectIndices, const TArray<int32>& CurvePointOffsets,
		const TArray<int32>& ObjectPointSamples, const int32 CurveIndex, const int32 PointOffset, const int32 SampleOffset,
					TArray<FVector3f>& SamplePositions, const int32 GuidePoints) 
	{
		const int32 ObjectIndex = CurveObjectIndices[CurveIndex];
		const int32 PointsCount = (CurvePointOffsets[CurveIndex]-PointOffset)-1;
		const int32 SamplesCount = (GuidePoints != 0) ? GuidePoints-1 : !ObjectPointSamples.IsEmpty() ? ObjectPointSamples[ObjectIndex]-1 : 0;

		float CurveLength = 0.0;
		TArray<float> EdgeLengths; EdgeLengths.Init(0.0f, PointsCount);
		for(int32 EdgeIndex = 0; EdgeIndex < PointsCount; ++EdgeIndex)
		{
			EdgeLengths[EdgeIndex] = (PointRestPositions[EdgeIndex+PointOffset+1] -
				PointRestPositions[EdgeIndex+PointOffset]).Length();
			CurveLength += EdgeLengths[EdgeIndex];
		}

		SamplePositions[SampleOffset + 1] = PointRestPositions[PointOffset];
		for(int32 SampleIndex = 1; SampleIndex < SamplesCount-1; ++SampleIndex)
		{
			const float SampleCoord = static_cast<float>(SampleIndex) / (SamplesCount - 1.0f);
			const float SampleLength = CurveLength * SampleCoord;

			float LocalLength = 0.0;
			for(int32 EdgeIndex = 0; EdgeIndex < PointsCount; ++EdgeIndex)
			{
				LocalLength += EdgeLengths[EdgeIndex];
				if(LocalLength >= SampleLength)
				{
					const int32 PrevPoint = EdgeIndex+PointOffset;
					const int32 NextPoint = PrevPoint+1;
					
					const float SampleAlpha = (LocalLength - SampleLength) / EdgeLengths[EdgeIndex];
					SamplePositions[SampleOffset + SampleIndex + 1] = PointRestPositions[PrevPoint] * SampleAlpha +
						PointRestPositions[NextPoint] * (1.0-SampleAlpha);
					break;
				}
			}
		}
		SamplePositions[SampleOffset+SamplesCount] = PointRestPositions[PointOffset+PointsCount];
		SamplePositions[SampleOffset] = 2 * SamplePositions[SampleOffset+1] - SamplePositions[SampleOffset+2];
	}

	void ResampleCurvesPoints(GeometryCollection::Facades::FCollectionCurveGeometryFacade& CurvesFacade, const FDataflowCurveSelection& CurveSelection, const int32 GuidePoints, bool bForSimulation)
	{
		if(CurvesFacade.IsValid())
		{
			const int32 NumSamples = ComputeCurvesPoints(CurvesFacade, CurveSelection, GuidePoints);
			const int32 NumCurves = CurvesFacade.GetNumCurves();

			TArray<FVector3f> PointSampledPositions;
			PointSampledPositions.Init(FVector3f::Zero(), NumSamples);

			TArray<int32> CurveSampleOffsets;
			CurveSampleOffsets.Init(0, NumCurves);

			const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());

			TArray<FVector3f> ResampledPoints; // used only when bForSimulation = false
			int32 PointOffset = 0;
			int32 SampleOffset = 0;
			for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				const int32 NumPoints = CurvesFacade.GetCurvePointOffsets()[CurveIndex]-PointOffset;
				if(!bValidSelection || (bValidSelection && CurveSelection.IsSelected(CurveIndex)))
				{
					if (bForSimulation)
					{
						UE::Groom::Private::BuildResampledPoints(CurvesFacade.GetPointRestPositions(), CurvesFacade.GetCurveGeometryIndices(), CurvesFacade.GetCurvePointOffsets(),
							{}, CurveIndex, PointOffset, SampleOffset, PointSampledPositions, GuidePoints);
					}
					else
					{
						ResampledPoints.Reset();
						CurvesFacade.GetCurveResampledPositions(CurveIndex, GuidePoints, ResampledPoints);
						const int32 NumResampledPoints = FMath::Min(GuidePoints, ResampledPoints.Num());
						for (int32 Index = 0; Index < NumResampledPoints; ++Index)
						{
							PointSampledPositions[SampleOffset + Index] = ResampledPoints[Index];
						}
					}
					SampleOffset += GuidePoints;
				}
				else
				{
					for(int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
					{
						PointSampledPositions[PointIndex + SampleOffset] = CurvesFacade.GetPointRestPositions()[PointIndex + PointOffset];
					}
					SampleOffset += NumPoints;
				}
				CurveSampleOffsets[CurveIndex] = SampleOffset;
				PointOffset = CurvesFacade.GetCurvePointOffsets()[CurveIndex];
			}
			const TArray<int32> GeometryCurveOffsets = CurvesFacade.GetGeometryCurveOffsets();
			const TArray<FString> GeometryGroupNames = CurvesFacade.GetGeometryGroupNames();
			const TArray<float> GeometryCurveThickness = CurvesFacade.GetGeometryCurveThickness();
			const TArray<int32> CurveSourceIndices = CurvesFacade.GetCurveSourceIndices();
			
			CurvesFacade.InitCurvesCollection(PointSampledPositions, CurveSampleOffsets, GeometryCurveOffsets, GeometryGroupNames, GeometryCurveThickness, CurveSourceIndices);
		}
	}

	static int32 ClosestPowerOfTwo(const int32 InputValue)
	{
		if(InputValue > 0)
		{
			const float LogValue = FMath::Log2(static_cast<float>(InputValue));
			const int32 LowerExp = FMath::FloorToInt(LogValue);
			const int32 UpperExp = FMath::CeilToInt(LogValue);

			const int32 LowerValue = 1 << LowerExp;
			const int32 UpperValue = 1 << UpperExp;

			return FMath::Clamp((InputValue-LowerValue < UpperValue-InputValue) ? LowerValue : UpperValue, 4, 64);
		}
		return 2;
	}
}

void FResampleGuidesPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FResampleCurvePointsDataflowNode::FResampleCurvePointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CurveSelection);
	RegisterInputConnection(&NumPoints).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}

void FResampleCurvePointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GuidesCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowCurveSelection& DataflowSelection = GetValue<FDataflowCurveSelection>(Context, &CurveSelection);
		
		const int32 InNumPoints = GetValue<int32>(Context, &NumPoints);
		int32 GuidePoints = InNumPoints;
		if (bResampleForSimulation)
		{
			GuidePoints = IsConnected(&NumPoints)
				? UE::Groom::Private::ClosestPowerOfTwo(InNumPoints)
				: static_cast<int32>(PointsCount);
		}
		if (GuidePoints < 2)
		{
			GuidePoints = 2;
			Context.Warning(LOCTEXT("NumPointToolSmall", "Number of points cannot be smaller than 2, setting it to 2 for evaluation"));
		}

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GuidesCollection);
		
		if(CurvesFacade.IsValid() && GuidePoints > 0)
		{
			UE::Groom::Private::ResampleCurvesPoints(CurvesFacade, DataflowSelection, GuidePoints, bResampleForSimulation);
		}
		
		SetValue(Context, MoveTemp(GuidesCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE