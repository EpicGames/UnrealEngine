// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothGuidesCurvesNode.h"

#include "AnimAssetFindReplace.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "Views/SInteractiveCurveEditorView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmoothGuidesCurvesNode)

namespace UE::Groom::Private
{
	FORCEINLINE void BuildSmoothedPoints(const TArray<FVector3f>& PointRestPositions, const int32 PointOffset, const int32 PointEnd, const float SmoothingFactor, TArray<FVector3f>& PointSmoothedPositions)
	{
		if((PointRestPositions.Num() >= PointEnd) && (PointEnd > PointOffset+1))
		{ 
			FVector3f DirM1 = PointRestPositions[PointOffset + 1] - PointRestPositions[PointOffset];
			FVector3f DirM2 = DirM1;

			const float Gamma1 = 2.0 * (1.0 - SmoothingFactor);
			const float Gamma2 = -(1.0 - SmoothingFactor) * (1.0 - SmoothingFactor);
			const float Gamma3 = SmoothingFactor * SmoothingFactor;

			PointSmoothedPositions[PointOffset] = PointRestPositions[PointOffset];

			for (int32 PointIndex = PointOffset; PointIndex < PointEnd - 1; ++PointIndex)
			{
				const FVector3f DirM3 = PointRestPositions[PointIndex + 1] - PointRestPositions[PointIndex];
				const FVector3f DirMi = Gamma1 * DirM1 + Gamma2 * DirM2 + Gamma3 * DirM3;

				PointSmoothedPositions[PointIndex + 1] = PointSmoothedPositions[PointIndex] + DirMi;

				DirM2 = DirM1;
				DirM1 = DirMi;
			}
		}
	}

	FORCEINLINE void SmoothCurvesPoints(GeometryCollection::Facades::FCollectionCurveGeometryFacade& CurvesFacade, const FDataflowCurveSelection& CurveSelection, float SmoothingFactor)
	{
		if(CurvesFacade.IsValid())
		{
			const int32 NumPoints = CurvesFacade.GetNumPoints();
			const int32 NumCurves = CurvesFacade.GetNumCurves();

			TArray<FVector3f> PointSmoothedPositions;
			PointSmoothedPositions.Init(FVector3f::Zero(), NumPoints);

			const bool bValidSelection = CurveSelection.IsValidForCollection(CurvesFacade.GetManagedArrayCollection());;

			int32 PointOffset = 0;
			for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				if(!bValidSelection || (bValidSelection && (CurveSelection.IsSelected(CurveIndex))))
				{
					BuildSmoothedPoints(CurvesFacade.GetPointRestPositions(), PointOffset,
					CurvesFacade.GetCurvePointOffsets()[CurveIndex], SmoothingFactor, PointSmoothedPositions);
				}
				else
				{
					for(int32 PointIndex = PointOffset; PointIndex < CurvesFacade.GetCurvePointOffsets()[CurveIndex]; ++PointIndex)
					{
						PointSmoothedPositions[PointIndex] = CurvesFacade.GetPointRestPositions()[PointIndex];
					}
				}
				PointOffset = CurvesFacade.GetCurvePointOffsets()[CurveIndex];
			}

			const TArray<int32> CurvePointOffsets = CurvesFacade.GetCurvePointOffsets();
			const TArray<int32> GeometryCurveOffsets = CurvesFacade.GetGeometryCurveOffsets();
			const TArray<FString> GeometryGroupNames = CurvesFacade.GetGeometryGroupNames();
			const TArray<float> GeometryCurveThickness = CurvesFacade.GetGeometryCurveThickness();
			const TArray<int32> CurveSourceIndices = CurvesFacade.GetCurveSourceIndices();
			
			CurvesFacade.InitCurvesCollection(PointSmoothedPositions, CurvePointOffsets, GeometryCurveOffsets, GeometryGroupNames, GeometryCurveThickness, CurveSourceIndices);
		}
	}
}

void FSmoothGuidesCurvesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FSmoothCurvePointsDataflowNode::FSmoothCurvePointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CurveSelection);
	RegisterInputConnection(&SmoothingFactor).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}

void FSmoothCurvePointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection GeometryCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const float SmoothedFactor = GetValue<float>(Context, &SmoothingFactor);
		
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurvesFacade(GeometryCollection);

		if(CurvesFacade.IsValid() && SmoothedFactor > 0.0f)
		{
			const FDataflowCurveSelection& DataflowSelection = GetValue<FDataflowCurveSelection>(Context, &CurveSelection);
			UE::Groom::Private::SmoothCurvesPoints(CurvesFacade, DataflowSelection, 1.0-SmoothedFactor);
		}
		
		SetValue(Context, MoveTemp(GeometryCollection), &Collection);
	}
}


