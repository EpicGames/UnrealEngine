// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowCore.h"

#include "AttachGuidesRootsNode.generated.h"

/** Attach the guides roots by setting their kinematic weights to 1.0f */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use AttachCurveRoots node instead.") FAttachGuidesRootsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FAttachGuidesRootsDataflowNode, "AttachGuidesRoots", "Groom", "")
	DATAFLOW_NODE_RENDER_TYPE("GuidesRender", FName("FGroomCollection"), "Collection")

public:
	
	FAttachGuidesRootsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&KinematicWeightsKey);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;
	
	/** Group index on which the roots will be attached. -1 will attach all the groups */
	UPROPERTY(EditAnywhere, Category="Groups", meta = (DisplayName = "Group Index"))
	int32 GroupIndex = INDEX_NONE;
	
	/** Point Kinematic weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Kinematic Weights", DataflowOutput))
	FCollectionAttributeKey KinematicWeightsKey;
};

/** Attach the curve roots by setting their kinematic weights to 1.0f */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FAttachCurveRootsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FAttachCurveRootsDataflowNode, "AttachCurveRoots", "Groom", "")

public:
	
	FAttachCurveRootsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Curve selection to focus the geometry transfer spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Curve Selection"))
	FDataflowCurveSelection CurveSelection;
	
	/** Vertex kinematic weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Kinematic Weights", DataflowOutput))
	FCollectionAttributeKey KinematicWeightsKey;
};

/** Build a weight map that will be identical on every curves */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FBuildCurveWeightsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FBuildCurveWeightsDataflowNode, "BuildCurveWeights", "Groom", "")

public:
	
	FBuildCurveWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&CurveSelection);
		RegisterInputConnection(&WeightsAttribute);
		RegisterOutputConnection(&Collection, &Collection);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Curve selection to focus the geometry transfer spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Curve Selection"))
	FDataflowCurveSelection CurveSelection;

	/** Weight value along each curves */
	UPROPERTY(EditAnywhere, Category = "Curves", meta = (DisplayName = "Curve Weights", ViewMinInput = "0.0", ViewMaxInput = "1.0", ViewMinOutput = "0.0", ViewMaxOutput = "1.0", TimeLineLength = "1.0", XAxisName = "Curve Coordinate (0,1)", YAxisName = "Weight value", ToolTip = "This curve determines the weight value from root to tip. \n The X axis range is [0,1], 0 mapping the root and 1 the tip"))
	FRuntimeFloatCurve CurveWeights;
	
	/** Vertex kinematic weights key to be used in other nodes if necessary */
	UPROPERTY(EditAnywhere, Category = "Curves", meta = (DisplayName = "Weights Attribute", DataflowInput))
	FCollectionAttributeKey WeightsAttribute;
};

