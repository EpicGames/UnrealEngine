// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowCore.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GetGroomAssetNode.h"

#include "BuildGroomSkinningNodes.generated.h"

namespace UE::Groom::Private
{
	/** Retrieve the bone indices key */
	FCollectionAttributeKey GetBoneIndicesKey();

	/** Retrieve the bone weights key */
	FCollectionAttributeKey GetBoneWeightsKey();
}

/** Build the curves skinning by transferring the indices weights from a skelmesh */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FTransferSkinWeightsGroomNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransferSkinWeightsGroomNode, "TransferSkinWeights", "Groom", "")

public:
	
	FTransferSkinWeightsGroomNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	/** SkeletalMesh used to transfer the skinning weights. Will be stored onto the groom asset */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	/** LOD used to transfer the weights */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DisplayName = "LOD Index"))
	int32 LODIndex = 0;
	
	/** Group index on which the dats will be transfered. -1 will transfer on all the groups */
	UPROPERTY(EditAnywhere, Category="Groom Groups", meta = (DisplayName = "Group Index"))
	int32 GroupIndex = INDEX_NONE;

	/** The relative transform between the skeletal mesh and the groom asset. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh")
	FTransform RelativeTransform;

	/** Type of curves to use to fill the groom collection (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Groom Groups", meta = (DisplayName = "Curves Type"))
	EGroomCollectionType CurvesType = EGroomCollectionType::Guides;
	
	/** Bone indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Indices", DataflowOutput))
	FCollectionAttributeKey BoneIndicesKey;

	/** Bone weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Weights", DataflowOutput))
	FCollectionAttributeKey BoneWeightsKey;
};

/** Build the curves skinning by transferring the indices weights from a skelmesh geometry */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FTransferGeometrySkinWeightsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransferGeometrySkinWeightsDataflowNode, "TransferLinearSkinWeights", "Groom", "")

public:
	FTransferGeometrySkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Vertex selection to focus the geometry transfer spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vertex Selection"))
	FDataflowVertexSelection VertexSelection;

	/** SkeletalMesh used to transfer the skinning weights. Will be stored onto the groom asset */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DataflowInput, DataflowPassthrough = "SkeletalMesh"))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	/** LOD used to transfer the weights */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DataflowInput, DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/** The relative transform between the skeletal mesh and the groom asset. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta = (DataflowInput, DisplayName = "Relative Transform"))
	FTransform RelativeTransform;

	/** Bone indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Indices", DataflowOutput))
	FCollectionAttributeKey BoneIndicesKey;

	/** Bone weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Weights", DataflowOutput))
	FCollectionAttributeKey BoneWeightsKey;
};
