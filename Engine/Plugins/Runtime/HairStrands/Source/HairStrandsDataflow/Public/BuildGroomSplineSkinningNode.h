// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Engine/SkeletalMesh.h"
#include "GetGroomAssetNode.h"

#include "BuildGroomSplineSkinningNode.generated.h"

namespace UE::Groom::Private
{
	/** Retrieve the spline param key */
	FCollectionAttributeKey GetSplineParamKey();

	/** Retrieve the spline bones key */
	FCollectionAttributeKey GetSplineBonesKey();
}

/** Build spline skinning data from skeletal mesh */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FBuildGroomSplineSkinWeightsNode : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FBuildGroomSplineSkinWeightsNode, "BuildSplineSkinWeights", "Groom", "")

public:
	FBuildGroomSplineSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	/** Spline Skinning Parameter key */
	UPROPERTY(meta = (DisplayName = "Spline Parameter Key", DataflowOutput))
	FCollectionAttributeKey SplineParamKey;

	/** Spline Bones key containing root and end spline bone. To be used in other nodes if necessary */
	UPROPERTY(meta = (DataflowOutput))
	FCollectionAttributeKey SplineBoneKey;


	/** SkeletalMesh used for spline skinning. Will be stored onto the groom asset */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DataFlowInput))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	/** Root bones to be used for spline skinning. Uses all bones if empty. Note that branches in the skeleton is currently not supported. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh")
	TArray<FString> RootBones;

	/** LOD used to build skinning weights */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/** Number of spline samples per bone segment. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh")
	int32 SamplesPerSegment = 64;

	/** Groom group index to build skinning weights for. -1 will build all groups */
	UPROPERTY(EditAnywhere, Category="Groom Groups")
	int32 GroupIndex = INDEX_NONE;

	/** Type of curves to build skinning for (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Groom Groups")
	EGroomCollectionType CurvesType = EGroomCollectionType::Guides;

	/** Spline skinning parameter attribute name */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Spline Parameter"))
	FString SplineParamName = "SplineParam";

	/** Spline bones attribute name. Used for storing root and end spline bone for each vertex. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes")
	FString SplineBonesParamName = "SplineBones";
};

/** Convert spline skinning data to linear data */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FConvertSplineToLinearSkinWeightsNode : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertSplineToLinearSkinWeightsNode, "SplineToLinearSkinWeights", "Groom", "")

public:
	FConvertSplineToLinearSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Spline skinning parameter attribute name */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Spline Parameter"))
	FString SplineParamName = "SplineParam";

	/** Spline bones attribute name. Used for storing root and end spline bone for each vertex. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Spline Bones"))
	FString SplineBonesName = "SplineBones";

	/**Spline skinning parameter key */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Spline Parameter Key"))
	FCollectionAttributeKey SplineParamKey;

	/** Spline bones key. Used for storing root and end spline bone for each vertex. */
	UPROPERTY(meta = (DataflowInput))
	FCollectionAttributeKey SplineBonesKey;

	/** Groom group index for converting skinning weight. -1 will convert all the groups */
	UPROPERTY(EditAnywhere, Category = "Groom Groups")
	int32 GroupIndex = INDEX_NONE;

	/** Linear skinning bone indices key */
	UPROPERTY(meta = (DataflowOutput))
	FCollectionAttributeKey BoneIndicesKey;

	/** Linear skinning bone weights key */
	UPROPERTY(meta = (DataflowOutput))
	FCollectionAttributeKey BoneWeightsKey;

	/** Type of curves to convert skinning for (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes")
	EGroomCollectionType CurvesType = EGroomCollectionType::Guides;
};

/** Convert linear skinning data to spline skinning data */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FConvertLinearToSplineSkinWeightsNode : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertLinearToSplineSkinWeightsNode, "LinearToSplineSkinWeights", "Groom", "")

public:
	FConvertLinearToSplineSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Groom group index for converting skinning weights. -1 will convert all groups */
	UPROPERTY(EditAnywhere, Category = "Groom Groups")
	int32 GroupIndex = INDEX_NONE;

	/** Spline Skinning Parameter key */
	UPROPERTY(meta = (DisplayName = "Spline Parameter Key", DataflowOutput))
	FCollectionAttributeKey SplineParamKey;

	/** Spline Bones key containing root and end spline bone. */
	UPROPERTY(meta = (DataflowOutput))
	FCollectionAttributeKey SplineBoneKey;

	/** Spline skinning parameter attribute name. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Spline Parameter"))
	FString SplineParamName = "SplineParam";

	/** Spline bones parameter attribute name. Contains root and end spline bone for each vertex. */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes", meta = (DisplayName = "Spline Bones"))
	FString SplineBonesName = "SplineBones";

	/** Type of curves to convert skinning for (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Vertex Attributes")
	EGroomCollectionType CurvesType = EGroomCollectionType::Guides;
};

namespace UE::Groom::Private
{
	struct FCollectionSplineAttributes
	{
		static const FName VertexSplineParamAttribute;
		static const FName VertexSplineBonesAttribute;
	};
}

/** Build spline skinning data from skeletal mesh */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FBuildSplineSkinWeightsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FBuildSplineSkinWeightsDataflowNode, "BuildSplineSkinWeights", "Groom", "")

public: 
	FBuildSplineSkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Vertex selection to focus the geometry transfer spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vertex Selection"))
	FDataflowVertexSelection VertexSelection;
	
	/** SkeletalMesh used to compute spline skinning weights. Will be stored onto the groom asset */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DataFlowInput))
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;
	
	/** LOD used to build skinning weights */
	UPROPERTY(EditAnywhere, Category="Skeletal Mesh", meta = (DataflowInput, DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/** Root bones to be used for spline skinning. Uses all bones if empty. Note that branches in the skeleton is currently not supported. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta = (DataFlowInput))
	TArray<FString> RootBones;

	/** Number of spline samples per bone segment. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh", meta = (DataflowInput))
	int32 SamplesPerSegment = 64;

	/** Spline Skinning Parameter key */
	UPROPERTY(meta = (DisplayName = "Spline Parameter", DataflowOutput))
	FCollectionAttributeKey SplineParamKey;

	/** Spline Bones key containing root and end spline bone. To be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Spline Bones", DataflowOutput))
	FCollectionAttributeKey SplineBonesKey;
};

/** Convert spline skinning data to linear data */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FSplineToLinearSkinWeightsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()

	DATAFLOW_NODE_DEFINE_INTERNAL(FSplineToLinearSkinWeightsDataflowNode, "SplineToLinearSkinWeights", "Groom", "")
	
public: 
	FSplineToLinearSkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Vertex selection to process vertices subset */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vertex Selection"))
	FDataflowVertexSelection VertexSelection;

	/** Spline skinning parameter key */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Spline Parameter"))
	FCollectionAttributeKey SplineParamKey;

	/** Spline bones key. Used for storing root and end spline bone for each vertex. */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Spline Bones"))
	FCollectionAttributeKey SplineBonesKey;

	/** Bone indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Indices", DataflowOutput))
	FCollectionAttributeKey BoneIndicesKey;

	/** Bone weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Weights", DataflowOutput))
	FCollectionAttributeKey BoneWeightsKey;

	/** Get spline parameter key */
	FCollectionAttributeKey GetSplineParamKey(UE::Dataflow::FContext& Context) const;

	/** Get spline bones key */
	FCollectionAttributeKey GetSplineBonesKey(UE::Dataflow::FContext& Context) const;
};

/** Convert linear skinning data to spline skinning data */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FLinearToSplineSkinWeightsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FLinearToSplineSkinWeightsDataflowNode, "LinearToSplineSkinWeights", "Groom", "")

public: 
	FLinearToSplineSkinWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private: 
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Vertex selection to process vertices subset */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vertex Selection"))
	FDataflowVertexSelection VertexSelection;
	
	/** Bone indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Indices", DataflowIndices))
	FCollectionAttributeKey BoneIndicesKey;

	/** Bone weights key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Bone Weights", DataflowWeights))
	FCollectionAttributeKey BoneWeightsKey;

	/**Spline skinning parameter key */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Spline Parameter"))
	FCollectionAttributeKey SplineParamKey;

	/** Spline bones key. Used for storing root and end spline bone for each vertex. */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Spline Bones"))
	FCollectionAttributeKey SplineBonesKey;
};

