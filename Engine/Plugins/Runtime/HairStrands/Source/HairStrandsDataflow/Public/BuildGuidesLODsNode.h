// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "BuildGuidesLODsNode.generated.h"


/** Builds the guides LODs */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FBuildGuidesLODsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FBuildGuidesLODsDataflowNode, "BuildGuidesLODs", "Groom", "")
	DATAFLOW_NODE_RENDER_TYPE("GuidesRender", FName("FGroomCollection"), "Collection")

public:
	
	FBuildGuidesLODsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&CurveParentsKey);
		RegisterOutputConnection(&CurveLodsKey);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store data */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;
	
	/** Curve parent indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Curve Parents", DataflowOutput))
	FCollectionAttributeKey CurveParentsKey;

	/** Curve lods indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Curve Lods", DataflowOutput))
	FCollectionAttributeKey CurveLodsKey;
};

/** Builds the curves LODs */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FBuildCurveLODsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FBuildCurveLODsDataflowNode, "BuildCurveLODs", "Groom", "")

public:
	
	FBuildCurveLODsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store data */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Curve selection to focus the geometry LODs spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Curve Selection"))
	FDataflowCurveSelection CurveSelection;
	
	/** Curve parent indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Curve Parents", DataflowOutput))
	FCollectionAttributeKey CurveParentsKey;

	/** Curve lods indices key to be used in other nodes if necessary */
	UPROPERTY(meta = (DisplayName = "Curve Lods", DataflowOutput))
	FCollectionAttributeKey CurveLodsKey;
};

