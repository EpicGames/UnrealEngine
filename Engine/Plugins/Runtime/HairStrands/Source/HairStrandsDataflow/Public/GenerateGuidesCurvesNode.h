// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"

#include "GenerateGuidesCurvesNode.generated.h"

/** Build the guides curves from the strands */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FGenerateGuidesCurvesDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateGuidesCurvesDataflowNode, "GenerateGuidesCurves", "Groom", "")
	DATAFLOW_NODE_RENDER_TYPE("GuidesRender", FName("FGroomCollection"), "Collection")

public:
	
	FGenerateGuidesCurvesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	/** Max number of guides */
	UPROPERTY(EditAnywhere, Category="Geometry", meta = (DisplayName = "Guides Count"))
	int32 GuidesCount = 0;
};

/** Build the curve geometry from a collection containing curves */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGenerateCurveGeometryDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateCurveGeometryDataflowNode, "GenerateCurveGeometry", "Groom", "")

public:
	
	FGenerateCurveGeometryDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Source Curves"))
	FManagedArrayCollection SourceCurves;

	/** Curve selection to focus the curves generation spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Curve Selection"))
	FDataflowCurveSelection CurveSelection;

	/** Max number of guides */
	UPROPERTY(EditAnywhere, Category="Geometry", meta = (DataflowInput, DisplayName = "Guides Count"))
	int32 CurveCount = 0;

	/** Flag to check if we can merge guides or not */
	UPROPERTY(EditAnywhere, Category="Geometry", meta = (DisplayName = "Merge Curves"))
	bool bMergeCurves = false;
};

