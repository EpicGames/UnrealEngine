// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "GetGroomAssetNode.generated.h"

UENUM()
enum class EGroomCollectionType : uint8
{
	/** Strands type (Rendering Mesh) */
	Strands UMETA(DisplayName = "Strands"),

	/** Guides type (Simulation Mesh)*/
	Guides UMETA(DisplayName = "Guides"),
};

/** Get the groom asset */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FGetGroomAssetDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetGroomAssetDataflowNode, "GetGroomAsset", "Groom", "")

public:
	
	FGetGroomAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Collection);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Input asset to read the guides from */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayName = "GroomAsset"))
	TObjectPtr<const UGroomAsset> GroomAsset = nullptr;

	/** Type of curves to use to fill the groom collection (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayName = "Display Curves"))
	EGroomCollectionType CurvesType = EGroomCollectionType::Strands;

	/** Managed array collection used to store the guides */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;
};

/** Get the groom asset */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGetGroomAssetDataflowNode_v2 : public FDataflowNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetGroomAssetDataflowNode_v2, "GetGroomAsset", "Groom", "")

public:
	
	FGetGroomAssetDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&GroomAsset);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Groom asset that will be used in the dataflow graph */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DataflowOutput, DisplayName = "Groom Asset"))
	TObjectPtr<const UGroomAsset> GroomAsset = nullptr;
};

/** Transform a groom asset to a collection */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGroomAssetToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGroomAssetToCollectionDataflowNode, "GroomAssetToCollection", "Groom", "")

public:
	
	FGroomAssetToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Input asset to read the guides from */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DataflowInput, DisplayName = "Groom Asset"))
	TObjectPtr<const UGroomAsset> GroomAsset = nullptr;

	/** Type of curves to use to fill the groom collection (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayName = "Curves Type"), meta = (DataflowInput))
	EGroomCollectionType CurvesType = EGroomCollectionType::Guides;

	/** Curves thickness for geometry generation */
	UPROPERTY(EditAnywhere, Category = "Geometry", meta = (DataflowInput, DisplayName = "Curves Thickness"))
	float CurvesThickness = 0.5;

	/** Managed array collection used to store the curves */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;
};

