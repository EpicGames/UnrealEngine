// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "GetGroomAssetNode.h"

#include "GetGroomAttributesNodes.generated.h"

/** Enum to pick groom attribute type */
UENUM()
enum class EGroomAttributeType : uint8
{
	KinematicWeights UMETA(DisplayName = "Kinematic Weights"),
	BoneIndices UMETA(DisplayName = "Bone Indices"),
	BoneWeights UMETA(DisplayName = "Bone Weights"),
	CurveParents UMETA(DisplayName = "Curve Parents"),
	CurveLods UMETA(DisplayName = "Curve Lods")
};

/** Get the kinematic weights attributes names */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FGetGroomAttributesDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetGroomAttributesDataflowNode, "GetGroomAttributes", "Groom", "")

public:
	
	FGetGroomAttributesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&AttributeKey);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Attribute key to build */
	UPROPERTY(meta = (DisplayName = "Attribute Key", DataflowOutput))
	FCollectionAttributeKey AttributeKey;

	/** Type of curves to use (guides/strands) */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayName = "Curves Type"))
	EGroomCollectionType CurvesType = EGroomCollectionType::Guides;

	/** Type of attribute to use */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayName = "Attribute Type"))
	EGroomAttributeType AttributeType = EGroomAttributeType::KinematicWeights;
};


/** Get the kinematic weights attributes names */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FGetCurveAttributesDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetCurveAttributesDataflowNode, "GetCurveAttributes", "Groom", "")

public:
	
	FGetCurveAttributesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Attribute key to build */
	UPROPERTY(meta = (DisplayName = "Attribute Key", DataflowOutput))
	FCollectionAttributeKey AttributeKey;
	
	/** Type of attribute to use */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayName = "Attribute Type"))
	EGroomAttributeType AttributeType = EGroomAttributeType::KinematicWeights;
};
