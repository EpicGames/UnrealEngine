// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

#include "GroomAssetTerminalNode.generated.h"

class UDataflow;

USTRUCT(meta = (Experimental, DataflowGroom, DataflowTerminal, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FGroomAssetTerminalDataflowNode : public FDataflowTerminalNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGroomAssetTerminalDataflowNode, "GroomAssetTerminal", "Groom", "")

public:

	FGroomAssetTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	//~ Begin FDataflowTerminalNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowTerminalNode interface

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;
	
	/** List of attribute keys that will be used to save matching attributes in the collection. */
	UPROPERTY()
	TArray<FCollectionAttributeKey> AttributeKeys;

	UPROPERTY(EditAnyWhere, Category = Attributes, EditFixedSize)
	TArray<FName> AttributeNames;

private :

	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;

	static constexpr int32 NumOtherInputs = 1;
};

USTRUCT(meta = (Experimental, DataflowGroom, DataflowTerminal))
struct FGroomAssetTerminalDataflowNode_v2 : public FDataflowTerminalNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGroomAssetTerminalDataflowNode_v2, "GroomAssetTerminal", "Groom", "")

public:

	FGroomAssetTerminalDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private :

	//~ Begin FDataflowTerminalNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return !AttributeKeys.IsEmpty(); }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowTerminalNode interface

	UPROPERTY(meta = (DataflowInput, DisplayName = "Strands Collection"))
	FManagedArrayCollection StrandsCollection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Guides Collection"))
	FManagedArrayCollection GuidesCollection;

	/** List of attribute keys that will be used to save matching attributes in the collection. */
	UPROPERTY()
	TArray<FCollectionAttributeKey> AttributeKeys;
	
	/** Attributes names used for the keys */
	UPROPERTY(EditAnyWhere, Category = Attributes, EditFixedSize)
	TArray<FName> AttributeNames;
	
	/** Get the connection reference matching an attribute key index */
	UE::Dataflow::TConnectionReference<FCollectionAttributeKey> GetConnectionReference(int32 Index) const;

	//~ Begin FDataflowTerminalNode interface
	virtual bool ShouldInvalidateOnPropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent) const override;
	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual bool SupportsDropConnectionOnNode(FName TypeName, UE::Dataflow::FPin::EDirection Direction) const override;
	virtual const FDataflowConnection* OnDropConnectionOnNode(const FDataflowConnection& DroppedConnection) override;
	//~ End FDataflowTerminalNode interface

	/** Sync input names with the keys */
	void SyncInputNames();

	/** Generate a unique input name */
	FName GenerateUniqueInputName(FName BaseName) const;

	TWeakObjectPtr<UDataflow> DataflowAssetWeakPtr;

	static constexpr int32 NumOtherInputs = 2;
};

