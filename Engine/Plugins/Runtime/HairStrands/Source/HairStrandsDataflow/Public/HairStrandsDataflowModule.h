// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/** Hair dataflow construction module public interface */
class FHairStrandsDataflowModule : public IModuleInterface
{
public :
	
	//~ IModuleInterface begin interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ IModuleInterface end interface
};
