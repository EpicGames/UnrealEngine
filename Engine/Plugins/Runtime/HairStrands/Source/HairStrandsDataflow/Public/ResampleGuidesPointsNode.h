// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "ResampleGuidesPointsNode.generated.h"

/** Enum to pick the number of points per guide */
UENUM(BlueprintType)
enum class EGroomNumPoints : uint8
{
	/** Default behavior coming from the strands size in the physics settings */
	Default  = 0 UMETA(DisplayName = "Default"),
	
	/** 4 points / curve */
	Size4  = 4 UMETA(DisplayName = "4 Points"),

	/** 8 points / curve */
	Size8  = 8 UMETA(DisplayName = "8 Points"),

	/** 16 points / curve */
	Size16  = 16 UMETA(DisplayName = "16 Points"),

	/** 32 points / curve */
	Size32  = 32 UMETA(DisplayName = "32 Points"),

	/** 64 points / curve */
	Size64 = 64 UMETA(DisplayName = "64 Points"),
};

/** Resample the groom guides with a fixed number of points (in physics strands size) */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FResampleGuidesPointsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FResampleGuidesPointsDataflowNode, "ResampleGuidesPoints", "Groom", "")
	DATAFLOW_NODE_RENDER_TYPE("GuidesRender", FName("FGroomCollection"), "Collection")

public:
	
	FResampleGuidesPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	/** Max number of points */
	UPROPERTY(EditAnywhere, Category="Geometry", meta = (DisplayName = "Points Count"))
	EGroomNumPoints PointsCount = EGroomNumPoints::Default;
};

/** Resample the curves with a fixed number of points */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FResampleCurvePointsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FResampleCurvePointsDataflowNode, "ResampleCurvePoints", "Groom", "")

public:
	
	FResampleCurvePointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store datas */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Curve selection to focus the guides generation spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Curve Selection"))
	FDataflowCurveSelection CurveSelection;

	/** 
	* When true, the resampling is done in the context of simulation
	* - Favors power of two point count ( optimized for GPU solvers )
	* - Extend the begginning of the curve to maintain stiffness
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bResampleForSimulation = true;

	/** Max number of points */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "Points Count", EditCondition="bResampleForSimulation", EditConditionHides))
	EGroomNumPoints PointsCount = EGroomNumPoints::Size16;
	
	/** Max number of points (to be able to plug into a variable since enum is not supported in dataflow) */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DataflowInput, DisplayName = "Points Count", ClampMin = "2", UIMin = "2", EditCondition = "!bResampleForSimulation", EditConditionHides))
	int32 NumPoints = 16;
};


