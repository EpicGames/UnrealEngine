// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "ChaosLog.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "SmoothGuidesCurvesNode.generated.h"


/** Smooth the guides for more stable simulation */
USTRUCT(meta = (Experimental, DataflowGroom, Deprecated="5.7"))
struct UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FSmoothGuidesCurvesDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FSmoothGuidesCurvesDataflowNode, "SmoothGuidesCurves", "Groom", "")
	DATAFLOW_NODE_RENDER_TYPE("GuidesRender", FName("FGroomCollection"), "Collection")

public:
	
	FSmoothGuidesCurvesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store data */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection"))
	FManagedArrayCollection Collection;

	/** Smoothing factor between 0 and 1 */
	UPROPERTY(EditAnywhere, Category="Geometry", meta = (DisplayName = "Smoothing Factor", ClampMin="0", ClampMax="1", UIMin="0", UIMax="1"))
	float SmoothingFactor = 0.0f;
};

/** Smooth the curves for more stable deformation */
USTRUCT(meta = (Experimental, DataflowGroom))
struct FSmoothCurvePointsDataflowNode : public FDataflowNode
{
	GENERATED_BODY()
	
	DATAFLOW_NODE_DEFINE_INTERNAL(FSmoothCurvePointsDataflowNode, "SmoothCurvePoints", "Groom", "")

public:
	
	FSmoothCurvePointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** Managed array collection to be used to store data */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough  = "Collection", DataflowRenderGroups = "Surface"))
	FManagedArrayCollection Collection;

	/** Curve selection to focus the guides smoothing spatially */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Curve Selection"))
	FDataflowCurveSelection CurveSelection;

	/** Smoothing factor between 0 and 1 */
	UPROPERTY(EditAnywhere, Category="Geometry", meta = (DataflowInput, DisplayName = "Smoothing Factor", ClampMin="0", ClampMax="1", UIMin="0", UIMax="1"))
	float SmoothingFactor = 0.0f;
};

