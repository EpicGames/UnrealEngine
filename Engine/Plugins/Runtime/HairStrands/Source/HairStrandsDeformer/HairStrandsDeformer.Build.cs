// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
namespace UnrealBuildTool.Rules
{
	public class HairStrandsDeformer : ModuleRules
	{
		public HairStrandsDeformer(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"NiagaraCore",
					"Niagara",
					"RenderCore",
					"Renderer",
					"RHI",
					"ComputeFramework",
					"OptimusCore",
					"HairStrandsCore",
					"HairStrandsSolver", 
					"Chaos",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Shaders",
				});
			
			PrivateIncludePaths.AddRange(
				new string[] {
					Path.Combine(EngineDirectory,"Source/Runtime/Renderer/Private"),
				}
			);
		}
	}
}
