// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomAttributeRead.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "GroomInstance.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "OptimusDataTypeRegistry.h"
#include "GroomAsset.h"
#include "OptimusNode.h"
#include "SystemTextures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomAttributeRead)


FString UOptimusGroomAttributeReadDataInterface::GetDisplayName() const
{
	return TEXT("Groom Attribute");
}

FName UOptimusGroomAttributeReadDataInterface::GetGroomAttributeName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusGroomAttributeReadDataInterface, GroomAttributeName);
}

TArray<FOptimusCDIPropertyPinDefinition> UOptimusGroomAttributeReadDataInterface::GetPropertyPinDefinitions() const
{
	TArray<FOptimusCDIPropertyPinDefinition> PropertyPinDefinitions;

	const FOptimusDataTypeHandle NameType = FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass());
	
	PropertyPinDefinitions.Add(
	{GetGroomAttributeName(), NameType}
	);

	return PropertyPinDefinitions;
}

#if WITH_EDITOR
void UOptimusGroomAttributeReadDataInterface::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{	
		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UOptimusGroomAttributeReadDataInterface, GroomAttributeGroup) ||
			PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UOptimusGroomAttributeReadDataInterface, GroomAttributeType))
		{
			OnPinDefinitionChangedDelegate.Execute();
		}
	}
}
#endif

FORCEINLINE void AddGroupDefinitions(const EOptimusGroomAttributeTypes GroomAttributeType, const FName& ContextName, const FString& CountFunctionName, TArray<FOptimusCDIPinDefinition>& Defs)
{
	Defs.Add({ "NumAttributeValues", CountFunctionName, false, {"NumAttributeValues"} });

	// List all mapping between the deformer graph pin types and the hlsl ones
	
	if(GroomAttributeType == EOptimusGroomAttributeTypes::Bool)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedBool", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Int)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedInt", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::IntVector2)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedInt2", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::IntVector3)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedInt3", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::IntVector4)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedInt4", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Uint)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedUint", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Float)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Vector2)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat2", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Vector3)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat3", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Vector4)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat4", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::LinearColor)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat4", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Quat)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat4", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Rotator)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat3", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Transform)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat4x4", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Matrix3x4)
	{
		Defs.Add({ "ValueTypedData", "ReadValueTypedFloat3x4", ContextName, CountFunctionName, false, "ValueTypedData" });
	}
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomAttributeReadDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	AddGroupDefinitions(GroomAttributeType, UOptimusGroomExecDataInterface::GetExecutionDomainName(GroomAttributeGroup), "ReadNumValues", Defs);
	return Defs;
}

void UOptimusGroomAttributeReadDataInterface::RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode)
{
	if(InNode)
	{
		OnPinDefinitionChangedDelegate.BindUObject(InNode, &UOptimusNode::RecreatePinsFromPinDefinitions);
	}
}
 
TSubclassOf<UActorComponent> UOptimusGroomAttributeReadDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomAttributeReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumValues"))
		.AddReturnType(EShaderFundamentalType::Uint);

	// List all the hlsl return types
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedInt"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedInt2"))
		.AddReturnType(EShaderFundamentalType::Int, 2)
		.AddParam(EShaderFundamentalType::Uint);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedInt3"))
		.AddReturnType(EShaderFundamentalType::Int, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedInt4"))
		.AddReturnType(EShaderFundamentalType::Int, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedUint"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedBool"))
		.AddReturnType(EShaderFundamentalType::Bool)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedFloat"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedFloat2"))
		.AddReturnType(EShaderFundamentalType::Float, 2)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedFloat3"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedFloat4"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedFloat4x4"))
		.AddReturnType(EShaderFundamentalType::Float, 4, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueTypedFloat3x4"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomAttributeReadParameters, )
	SHADER_PARAMETER(uint32, NumValues)
	SHADER_PARAMETER(uint32, BaseValueIndex)
	SHADER_PARAMETER(uint32, TotalValueCount)
	SHADER_PARAMETER(uint32, bHasIndexMapping)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float>, ValueTypedData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, IndexMappingBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomAttributeReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomAttributeReadParameters>(UID);
}

TCHAR const* UOptimusGroomAttributeReadDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomAttributeRead.ush");

TCHAR const* UOptimusGroomAttributeReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomAttributeReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomAttributeReadDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{}

void UOptimusGroomAttributeReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomAttributeReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomAttributeReadDataProvider* Provider = NewObject<UOptimusGroomAttributeReadDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	Provider->GroomAttributeGroup = GroomAttributeGroup;
	Provider->GroomAttributeName = GroomAttributeName;
	Provider->GroomAttributeType = GroomAttributeType;
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomAttributeReadDataProvider::GetRenderProxy()
{
	return new FOptimusGroomAttributeReadProviderProxy(MeshComponent, GroomAttributeName, GroomAttributeGroup, GroomAttributeType);
}

FOptimusGroomAttributeReadProviderProxy::FOptimusGroomAttributeReadProviderProxy(UMeshComponent* MeshComponent,
	const FName& InAtributeName, const EOptimusGroomExecDomain InAttributeGroup, const EOptimusGroomAttributeTypes InAttributeType)
{
	GroomAttributeName = InAtributeName;;
	GroomAttributeGroup = InAttributeGroup;
	GroomAttributeType = InAttributeType;
	
	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroomComponents(MeshComponent, GroomComponents);
	UE::Groom::Private::GroomComponentsToInstances(GroomComponents, GroupInstances);
	UE::Groom::Private::GetGroomInvocationElementGroups(GroomComponents,
		UOptimusGroomExecDataInterface::GetExecutionDomainName(GroomAttributeGroup), GroupElements, true);

	// Retrieve the rendering elements (not the source ones)
	TArray<TPair<UGroomAsset*, UE::Groom::Private::FGroupElements>> InvocationElements;
	UE::Groom::Private::GetGroomInvocationElementGroups(GroomComponents,
		UOptimusGroomExecDataInterface::GetExecutionDomainName(GroomAttributeGroup), InvocationElements, false);

	// Compute the number of elements for the base indices
	NumElements.SetNumZeroed(InvocationElements.Num());
	int32 AssetIndex = 0;
	for (TPair<UGroomAsset*, UE::Groom::Private::FGroupElements>& Elements : InvocationElements)
	{
		NumElements[AssetIndex].Reset();
		for (int32 GroupIndex = 0, NumGroups = Elements.Value.GroupOffsets.Num()-1; GroupIndex < NumGroups; ++GroupIndex)
		{
			NumElements[AssetIndex].Add(Elements.Value.GroupOffsets[GroupIndex+1]-Elements.Value.GroupOffsets[GroupIndex]);
		}
		++AssetIndex;
	}
}

bool FOptimusGroomAttributeReadProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != GroupInstances.Num())
	{
		return false;
	}
	const FName GroupName = UOptimusGroomExecDataInterface::GetExecutionDomainName(GroomAttributeGroup);
	
	if((GroomAttributeGroup == EOptimusGroomExecDomain::GuidesPoints) || (GroomAttributeGroup == EOptimusGroomExecDomain::GuidesEdges) ||
		(GroomAttributeGroup == EOptimusGroomExecDomain::GuidesCurves) || (GroomAttributeGroup == EOptimusGroomExecDomain::GuidesObjects))
	{
		if(!UE::Groom::Private::HaveGuidesInstanceResources(GroupInstances))
		{
			return false;
		}
	}
	else if ((GroomAttributeGroup == EOptimusGroomExecDomain::ControlPoint) || (GroomAttributeGroup == EOptimusGroomExecDomain::StrandsEdges) ||
		(GroomAttributeGroup == EOptimusGroomExecDomain::Curve) || (GroomAttributeGroup == EOptimusGroomExecDomain::StrandsObjects))
	{
		if(!UE::Groom::Private::HaveStrandsInstanceResources(GroupInstances))
		{
			return false;
		}
	}
	for (const TPair<UGroomAsset*, UE::Groom::Private::FGroupElements>& AssetPair : GroupElements)
	{
		if(!AssetPair.Key)
		{
			return false;
		}
		
		if(!AssetPair.Key->GetDataflowSettings().GetRestCollection()->HasAttribute(GroomAttributeName, GroupName))
		{
			return false;
		}
	}
	
	return true;
}

void FOptimusGroomAttributeReadProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
}

template<typename DataType>
FORCEINLINE void ConvertAttributeData(const DataType& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData;
}

template<typename DataType>
FORCEINLINE int32 NumAttributeValues()
{
	return 1;;
}

template<>
FORCEINLINE void ConvertAttributeData<FTransform3f>(const FTransform3f& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	const FMatrix44f InputMatrix = InputData.ToMatrixWithScale();
	DataValues[DataIndex++] = InputMatrix.M[0][0];
	DataValues[DataIndex++] = InputMatrix.M[0][1];
	DataValues[DataIndex++] = InputMatrix.M[0][2];
	DataValues[DataIndex++] = InputMatrix.M[0][3];

	DataValues[DataIndex++] = InputMatrix.M[1][0];
	DataValues[DataIndex++] = InputMatrix.M[1][1];
	DataValues[DataIndex++] = InputMatrix.M[1][2];
	DataValues[DataIndex++] = InputMatrix.M[1][3];

	DataValues[DataIndex++] = InputMatrix.M[2][0];
	DataValues[DataIndex++] = InputMatrix.M[2][1];
	DataValues[DataIndex++] = InputMatrix.M[2][2];
	DataValues[DataIndex++] = InputMatrix.M[2][3];

	DataValues[DataIndex++] = InputMatrix.M[3][0];
	DataValues[DataIndex++] = InputMatrix.M[3][1];
	DataValues[DataIndex++] = InputMatrix.M[3][2];
	DataValues[DataIndex++] = InputMatrix.M[3][3];
}

template<>
FORCEINLINE int32 NumAttributeValues<FTransform3f>()
{
	return 16;
}

template<>
FORCEINLINE void ConvertAttributeData<FLinearColor>(const FLinearColor& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.R;
	DataValues[DataIndex++] = InputData.G;
	DataValues[DataIndex++] = InputData.B;
	DataValues[DataIndex++] = InputData.A;
}

template<>
FORCEINLINE int32 NumAttributeValues<FLinearColor>()
{
	return 4;
}

template<>
FORCEINLINE void ConvertAttributeData<FQuat4f>(const FQuat4f& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
	DataValues[DataIndex++] = InputData.Z;
	DataValues[DataIndex++] = InputData.W;
}

template<>
FORCEINLINE int32 NumAttributeValues<FQuat4f>()
{
	return 4;
}

template<>
FORCEINLINE void ConvertAttributeData<FRotator3f>(const FRotator3f& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.Pitch;
	DataValues[DataIndex++] = InputData.Yaw;
	DataValues[DataIndex++] = InputData.Roll;
}

template<>
FORCEINLINE int32 NumAttributeValues<FRotator3f>()
{
	return 3;
}

template<>
FORCEINLINE void ConvertAttributeData<FIntVector4>(const FIntVector4& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
	DataValues[DataIndex++] = InputData.Z;
	DataValues[DataIndex++] = InputData.W;
}

template<>
FORCEINLINE int32 NumAttributeValues<FIntVector4>()
{
	return 4;
}

template<>
FORCEINLINE void ConvertAttributeData<FIntVector3>(const FIntVector3& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
	DataValues[DataIndex++] = InputData.Z;
}

template<>
FORCEINLINE int32 NumAttributeValues<FIntVector3>()
{
	return 3;
}

template<>
FORCEINLINE void ConvertAttributeData<FIntVector2>(const FIntVector2& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
}

template<>
FORCEINLINE int32 NumAttributeValues<FIntVector2>()
{
	return 2;
}

template<>
FORCEINLINE void ConvertAttributeData<FVector4f>(const FVector4f& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
	DataValues[DataIndex++] = InputData.Z;
	DataValues[DataIndex++] = InputData.W;
}

template<>
FORCEINLINE int32 NumAttributeValues<FVector4f>()
{
	return 4;
}

template<>
FORCEINLINE void ConvertAttributeData<FVector3f>(const FVector3f& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
	DataValues[DataIndex++] = InputData.Z;
}

template<>
FORCEINLINE int32 NumAttributeValues<FVector3f>()
{
	return 3;
}

template<>
FORCEINLINE void ConvertAttributeData<FVector2f>(const FVector2f& InputData, TArray<float>& DataValues, int32& DataIndex)
{
	DataValues[DataIndex++] = InputData.X;
	DataValues[DataIndex++] = InputData.Y;
}

template<>
FORCEINLINE int32 NumAttributeValues<FVector2f>()
{
	return 2;
}

template<typename DataType>
static void CreateInternalBuffers(const UGroomAsset* GroomAsset, const UE::Groom::Private::FGroupElements& GroupElements, const FName& AttributeName,
	const EOptimusGroomExecDomain& AttributeGroup, FRDGBuilder& GraphBuilder, TArray<FRDGBufferSRVRef>& AttributeResources,
		TArray<FRDGBufferSRVRef>& IndexMappings, TArray<uint32>& bHasIndexMapping)
{
	const FName GroupName = UOptimusGroomExecDataInterface::GetExecutionDomainName(AttributeGroup);
	
	if(GroomAsset && GroomAsset->GetDataflowSettings().GetRestCollection() &&
     		GroomAsset->GetDataflowSettings().GetRestCollection()->HasAttribute(AttributeName, GroupName))
    {
     	const TManagedArray<DataType>& AssetAttributeValues =
     			GroomAsset->GetDataflowSettings().GetRestCollection()->GetAttribute<DataType>(AttributeName,GroupName);

		const bool bIsGuide = ((AttributeGroup == EOptimusGroomExecDomain::GuidesPoints) || (AttributeGroup == EOptimusGroomExecDomain::GuidesEdges) ||
					(AttributeGroup == EOptimusGroomExecDomain::GuidesCurves) || (AttributeGroup == EOptimusGroomExecDomain::GuidesObjects));
		const bool bIsStrand = ((AttributeGroup == EOptimusGroomExecDomain::ControlPoint) || (AttributeGroup == EOptimusGroomExecDomain::StrandsEdges) ||
			   (AttributeGroup == EOptimusGroomExecDomain::Curve) || (AttributeGroup == EOptimusGroomExecDomain::StrandsObjects));

		int32 InstanceIndex = 0;
     	for(int32 GroupIndex : GroupElements.GroupIndices)
     	{
     		if(const FHairGroupInstance* GroupInstance = GroupElements.GroupInstances[InstanceIndex])
     		{
     			const FString ResourceType = bIsGuide ? FString("Guides.") : FString("Strands.");
     			const FString BufferName = FString("Hair.Deformer.") + ResourceType + AttributeName.ToString();
     			
     			if(FHairStrandsRestResource* RestResource = bIsGuide ? GroupInstance->Guides.RestResource : bIsStrand ? GroupInstance->Strands.RestResource : nullptr)
     			{
     				// For now we only use index mapping for strands since there is no randomization for guides
     				const bool bCurveMapping = ((AttributeGroup == EOptimusGroomExecDomain::Curve) && ((RestResource->BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasCurveMapping) != 0));
     				const bool bPointMapping = ((AttributeGroup == EOptimusGroomExecDomain::ControlPoint) && ((RestResource->BulkData.Header.Flags & FHairStrandsBulkData::DataFlags_HasPointMapping) != 0));
     				
     				IndexMappings.Add( RegisterAsSRV(GraphBuilder, bCurveMapping ? RestResource->CurveMappingBuffer : RestResource->PointMappingBuffer));
     				bHasIndexMapping.Add(bCurveMapping || bPointMapping);
     				
     				FRDGExternalBuffer* ExternalBuffer = RestResource->ExternalBuffers.Find(BufferName);
     				if(!ExternalBuffer)
     				{
     					const uint32 NumAttributes = GroupElements.GroupOffsets[GroupIndex+1]-GroupElements.GroupOffsets[GroupIndex];
     				
			        	TArray<float> GroupAttributeValues;
			        	GroupAttributeValues.SetNum(NumAttributes*NumAttributeValues<DataType>());
	 
			        	int32 DataIndex = 0;
			        	for(uint32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
			        	{
			        		ConvertAttributeData<DataType>(AssetAttributeValues[AttributeIndex+GroupElements.GroupOffsets[GroupIndex]], GroupAttributeValues, DataIndex);
			        	}
			        	FRDGBufferRef TransientBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(
			        		sizeof(float), GroupAttributeValues.Num()), *BufferName);
     					ExternalBuffer = &RestResource->ExternalBuffers.Add(BufferName);

     					ExternalBuffer->Buffer = GraphBuilder.ConvertToExternalBuffer(TransientBuffer);
     					
			        	GraphBuilder.QueueBufferUpload(TransientBuffer, GroupAttributeValues.GetData(),
			        		sizeof(float) * GroupAttributeValues.Num(), ERDGInitialDataFlags::None);
			        }
     				AttributeResources.Add(RegisterAsSRV(GraphBuilder, *ExternalBuffer));
     			}
     		}
     		++InstanceIndex;
     	}
    }
}

static void AddGroupResources(UGroomAsset* GroomAsset, const UE::Groom::Private::FGroupElements& GroupElements, const EOptimusGroomAttributeTypes GroomAttributeType,
const FName& GroomAttributeName,  const EOptimusGroomExecDomain& GroomAttributeGroup, FRDGBuilder& GraphBuilder, TArray<FRDGBufferSRVRef>& GroomAttributeResources,
	TArray<FRDGBufferSRVRef>& IndexMappingResources, TArray<uint32>& bHasIndexMapping)
{
	if(GroomAttributeType == EOptimusGroomAttributeTypes::Bool)
	{
		CreateInternalBuffers<bool>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Int || GroomAttributeType == EOptimusGroomAttributeTypes::Uint)
	{
		CreateInternalBuffers<int32>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::IntVector2)
	{
		CreateInternalBuffers<FIntVector2>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::IntVector3)
	{
		CreateInternalBuffers<FIntVector3>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::IntVector4)
	{
		CreateInternalBuffers<FIntVector4>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Float)
	{
		CreateInternalBuffers<float>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Vector2)
	{
		CreateInternalBuffers<FVector2f>(GroomAsset, GroupElements,  GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Vector3 || GroomAttributeType == EOptimusGroomAttributeTypes::Rotator)
	{
		CreateInternalBuffers<FVector3f>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Vector4)
	{
		CreateInternalBuffers<FVector4f>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::LinearColor)
	{
		CreateInternalBuffers<FLinearColor>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Quat)
	{
		CreateInternalBuffers<FQuat4f>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
	else if(GroomAttributeType == EOptimusGroomAttributeTypes::Transform || GroomAttributeType == EOptimusGroomAttributeTypes::Matrix3x4)
	{
		CreateInternalBuffers<FTransform3f>(GroomAsset, GroupElements, GroomAttributeName,
			GroomAttributeGroup, GraphBuilder, GroomAttributeResources, IndexMappingResources,  bHasIndexMapping);
	}
}

void FOptimusGroomAttributeReadProviderProxy::CreateInternalBuffers(FRDGBuilder& GraphBuilder)
{
	for (const TPair<UGroomAsset*,UE::Groom::Private::FGroupElements>& AssetPair : GroupElements)
	{
		AddGroupResources(AssetPair.Key, AssetPair.Value, GroomAttributeType, GroomAttributeName,
				GroomAttributeGroup, GraphBuilder, AttributeValuesResources,
				IndexMappingResources, bHasIndexMapping);
	}
}

void FOptimusGroomAttributeReadProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	if (!FallbackStructuredSRV) { FallbackStructuredSRV  = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 16u)); }

	AttributeValuesResources.Reset();
	IndexMappingResources.Reset();
	bHasIndexMapping.Reset();
	
	CreateInternalBuffers(GraphBuilder);
}

void FOptimusGroomAttributeReadProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(InDispatchData.NumInvocations == AttributeValuesResources.Num());
	
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	int32 InvocationIndex = 0;
	int32 BaseValueIndex = 0;
	for (int32 AssetIndex = 0, NumAssets = GroupElements.Num(); AssetIndex < NumAssets; ++AssetIndex)
	{
		const UE::Groom::Private::FGroupElements& AssetElements = GroupElements[AssetIndex].Value;
		for (int32 InstanceIndex = 0, NumInstances = GroupElements[AssetIndex].Value.GroupIndices.Num(); InstanceIndex < NumInstances; ++InstanceIndex)
		{
			const int32 GroupIndex = GroupElements[AssetIndex].Value.GroupIndices[InstanceIndex];
			
			FParameters& Parameters = ParameterArray[InvocationIndex];
			if (const FHairGroupInstance* GroupInstance = AssetElements.GroupInstances[InstanceIndex])
			{
				const int32 ElementCount = NumElements[AssetIndex][GroupIndex];

				Parameters.NumValues = AssetElements.GroupOffsets[GroupIndex+1] - AssetElements.GroupOffsets[GroupIndex];
				Parameters.BaseValueIndex = BaseValueIndex;
				Parameters.ValueTypedData = AttributeValuesResources[InvocationIndex];

				Parameters.bHasIndexMapping = bHasIndexMapping[InvocationIndex];
				Parameters.IndexMappingBuffer = IndexMappingResources[InvocationIndex];

				BaseValueIndex += ElementCount;
			}
			else
			{
				Parameters.NumValues = 0;
				Parameters.BaseValueIndex = BaseValueIndex;
				Parameters.ValueTypedData = FallbackStructuredSRV;

				Parameters.bHasIndexMapping = 0;
				Parameters.IndexMappingBuffer = FallbackStructuredSRV;
			}
			++InvocationIndex;
		}
	}

	InvocationIndex = 0;
	for (const TPair<UGroomAsset*, UE::Groom::Private::FGroupElements>& AssetPair : GroupElements)
	{
		for (int32 GroupIndex : AssetPair.Value.GroupIndices)
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.TotalValueCount = BaseValueIndex;
			++InvocationIndex;
		}
	}

}
