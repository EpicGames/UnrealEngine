// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeformerGroomDomainsExec.h"
#include "RenderGraphDefinitions.h"
#include "OptimusComputeDataInterface.h"
#include "HairStrandsInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "DeformerGroomAttributeRead.generated.h"

namespace UE::Groom::Private
{
	struct FGroupElements;
}

class FOptimusGroomAttributeReadParameters;
class UGroomComponent;
class UGroomAsset;
struct FHairGroupInstance;
class UMeshComponent;

/** List of attribute types on which you could paint in dataflow */
UENUM()
enum class EOptimusGroomAttributeTypes : uint8
{
	None = 0 UMETA(Hidden),
	Bool UMETA(DisplayName = "Bool"),
	Int UMETA(DisplayName = "Int"),
	IntVector2 UMETA(DisplayName = "Int Vector 2"),
	IntVector3 UMETA(DisplayName = "Int Vector 3"),
	IntVector4 UMETA(DisplayName = "Int Vector 4"),
	Uint UMETA(DisplayName = "Uint"),
	Float UMETA(DisplayName = "Float"),
	Vector2 UMETA(DisplayName = "Vector 2"),
	Vector3 UMETA(DisplayName = "Vector 3"),
	Vector4 UMETA(DisplayName = "Vector 4"),
	LinearColor UMETA(DisplayName = "Linear Color"),
	Quat UMETA(DisplayName = "Quat"),
	Rotator UMETA(DisplayName = "Rotator"),
	Transform UMETA(DisplayName = "Transform"),
	Matrix3x4 UMETA(DisplayName = "Matrix 3x4"),
};

/** Compute Framework Data Interface for reading groom strands. */
UCLASS(Category = ComputeFramework)
class UOptimusGroomAttributeReadDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	virtual TArray<FOptimusCDIPropertyPinDefinition> GetPropertyPinDefinitions() const override;
	virtual bool CanPinDefinitionChange() override { return true; };
	virtual void RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode) override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("GroomAttributeRead"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	/** File holding the hlsl implementation */
	static TCHAR const* TemplateFilePath;

	static FName GetGroomAttributeName();

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Groom Attribute")
	FName GroomAttributeName = NAME_None;

	/** Attribute group */
	UPROPERTY(EditAnywhere, Category = "Groom Attribute")
	EOptimusGroomExecDomain GroomAttributeGroup = EOptimusGroomExecDomain::GuidesPoints;

	/** Attribute type */
	UPROPERTY(EditAnywhere, Category = "Groom Attribute")
	EOptimusGroomAttributeTypes GroomAttributeType = EOptimusGroomAttributeTypes::Float;

	/** Delegate to change the pin definition context based on the group selected */
	FOnPinDefinitionChanged OnPinDefinitionChangedDelegate;
};

/** Compute Framework Data Provider for reading groom strands. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGroomAttributeReadDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMeshComponent> MeshComponent = nullptr;

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Groom Attribute")
	FName GroomAttributeName = NAME_None;

	/** Attribute group */
	UPROPERTY(EditAnywhere, Category = "Groom Attribute")
	EOptimusGroomExecDomain GroomAttributeGroup = EOptimusGroomExecDomain::GuidesPoints;

	/** Attribute type */
	UPROPERTY(EditAnywhere, Category = "Groom Attribute")
	EOptimusGroomAttributeTypes GroomAttributeType = EOptimusGroomAttributeTypes::Float;

	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusGroomAttributeReadProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGroomAttributeReadProviderProxy(UMeshComponent* InGroomComponent,
		const FName& InAtributeName, const EOptimusGroomExecDomain InAttributeGroup, const EOptimusGroomAttributeTypes InAttributeType);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	virtual void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FOptimusGroomAttributeReadParameters;

	/** Create internal buffers */
	void CreateInternalBuffers(FRDGBuilder& GraphBuilder);

	/** List of instances (invocations) used in that data interface */
	TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;

	/** Groom assets group indices the proxy instance is linked to */
	TArray<TPair<UGroomAsset*, UE::Groom::Private::FGroupElements>> GroupElements;

	/** Num elements per assets that will be used in deformer for each invocation (usually larger than the number of source elements) */
	TArray<TArray<int32>> NumElements;
	
	/** Attribute Resources used to dispatch CS on GPU */
	TArray<FRDGBufferSRVRef> AttributeValuesResources;

	/** Index mappings resources */
	TArray<FRDGBufferSRVRef> IndexMappingResources;

	/** Check if we need to use the index mappings */
	TArray<uint32> bHasIndexMapping;

	/** Attribute name */
	FName GroomAttributeName = NAME_None;

	/** Attribute group */
	EOptimusGroomExecDomain GroomAttributeGroup = EOptimusGroomExecDomain::GuidesPoints;

	/** Attribute type */
	EOptimusGroomAttributeTypes GroomAttributeType = EOptimusGroomAttributeTypes::Float;

	/** Fallback resources */
	FRDGBufferSRVRef FallbackStructuredSRV = nullptr;
};
