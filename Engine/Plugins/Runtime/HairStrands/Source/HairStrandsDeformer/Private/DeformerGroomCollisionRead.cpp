// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomCollisionRead.h"

#include "CachedGeometry.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "OptimusDataTypeRegistry.h"
#include "GroomSolverComponent.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SystemTextures.h"
#include "SkeletalRenderPublic.h"
//#include "Editor/UnrealEdTypes.h"

FString UOptimusGroomCollisionReadDataInterface::GetDisplayName() const
{
	return TEXT("Groom Collision");
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomCollisionReadDataInterface::GetPinDefinitions() const
{
	static const FName CollisionVertices(UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Vertices);
	static const FName CollisionTriangles(UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Triangles);

	TArray<FOptimusCDIPinDefinition> Defs;
	
	Defs.Add({ "NumCollisionVertices",   "ReadNumVertices", true, "NumCollisionVertices" });
	Defs.Add({ "VertexGlobalPosition", "ReadVertexGlobalPosition", CollisionVertices,   "ReadNumVertices", true, "VertexGlobalPosition"  });
	Defs.Add({ "NumCollisionTriangles",   "ReadNumTriangles", true, "NumCollisionTriangles" });
	Defs.Add({ "TriangleVertexIndices", "ReadTriangleVertexIndices", CollisionTriangles,   "ReadNumTriangles", true, "TriangleVertexIndices"  });

	return Defs;
}
 
TSubclassOf<UActorComponent> UOptimusGroomCollisionReadDataInterface::GetRequiredComponentClass() const
{
	return UGroomSolverComponent::StaticClass();
}

void UOptimusGroomCollisionReadDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{}

void UOptimusGroomCollisionReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumTriangles"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadVertexGlobalPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);
		
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadTriangleVertexIndices"))
		.AddReturnType(EShaderFundamentalType::Int, 3)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomCollisionReadParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumTriangles)
	SHADER_PARAMETER(uint32, VertexOffset)
	SHADER_PARAMETER(uint32, TriangleOffset)
	SHADER_PARAMETER(uint32, BaseVertexIndex)
	SHADER_PARAMETER(uint32, BaseTriangleIndex)
	SHADER_PARAMETER(uint32, TotalNumVertices)
	SHADER_PARAMETER(uint32, TotalNumTriangles)
	SHADER_PARAMETER_SRV(Buffer<float3>, VertexGlobalPositions)
	SHADER_PARAMETER_SRV(Buffer<uint3>, TriangleVertexIndices)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomCollisionReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomCollisionReadParameters>(UID);
}

TCHAR const* UOptimusGroomCollisionReadDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomCollisionRead.ush");

TCHAR const* UOptimusGroomCollisionReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomCollisionReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomCollisionReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomCollisionReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomCollisionReadDataProvider* Provider = NewObject<UOptimusGroomCollisionReadDataProvider>();
	Provider->SolverComponent = Cast<UGroomSolverComponent>(InBinding);
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomCollisionReadDataProvider::GetRenderProxy()
{
	return new FOptimusGroomCollisionReadDataProviderProxy(SolverComponent);
}

FOptimusGroomCollisionReadDataProviderProxy::FOptimusGroomCollisionReadDataProviderProxy(UGroomSolverComponent* SolverComponent)
{
	if(SolverComponent)
	{
		CollisionObjects = SolverComponent->GetCollisionComponents();
	}
}

bool FOptimusGroomCollisionReadDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	int32 NumSections = 0;
	for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : CollisionObjects)
	{
		if(CollisionComponent.Key)
		{
			if(const USkinnedMeshComponent* SkinnedMeshComponent = Cast<const USkinnedMeshComponent>(CollisionComponent.Key))
			{
				if(SkinnedMeshComponent->MeshObject)
				{
					const FSkeletalMeshRenderData& MeshRenderData = SkinnedMeshComponent->MeshObject->GetSkeletalMeshRenderData();
					NumSections += MeshRenderData.LODRenderData[SkinnedMeshComponent->MeshObject->GetLOD()].RenderSections.Num();
				}
			}
		}
	}
	if (InValidationData.NumInvocations != NumSections)
	{
		return false;
	}
	
	return true;
}

void FOptimusGroomCollisionReadDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	if (!FallbackStructuredSRV) { FallbackStructuredSRV  = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 16u)); }

	CollisionResources.Reset();
	for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : CollisionObjects)
	{
		if(CollisionComponent.Key)
		{
			if(const USkinnedMeshComponent* SkinnedMeshComponent = Cast<const USkinnedMeshComponent>(CollisionComponent.Key))
			{
				if(SkinnedMeshComponent->MeshObject)
				{
					FCachedGeometry CachedGeometry;
					SkinnedMeshComponent->MeshObject->GetCachedGeometry(GraphBuilder, CachedGeometry);

					const FSkeletalMeshRenderData& MeshRenderData = SkinnedMeshComponent->MeshObject->GetSkeletalMeshRenderData();
					check(CachedGeometry.Sections.Num() == MeshRenderData.LODRenderData[SkinnedMeshComponent->MeshObject->GetLOD()].RenderSections.Num());

					for(int32 SectionIndex = 0; SectionIndex < CachedGeometry.Sections.Num(); ++SectionIndex)
					{
						FCollisionResources& CollisionResource = CollisionResources.AddDefaulted_GetRef();
						
						CollisionResource.VertexPositions = CachedGeometry.Sections[SectionIndex].PositionBuffer;
						CollisionResource.TriangleIndices = CachedGeometry.Sections[SectionIndex].IndexBuffer;
						CollisionResource.NumTriangles = CachedGeometry.Sections[SectionIndex].NumPrimitives;
						CollisionResource.NumVertices = CachedGeometry.Sections[SectionIndex].NumVertices;
						CollisionResource.TriangleOffset = CachedGeometry.Sections[SectionIndex].IndexBaseIndex;
						CollisionResource.VertexOffset = CachedGeometry.Sections[SectionIndex].VertexBaseIndex;
					}
				} 
			}
			else if(const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(CollisionComponent.Key))
			{
				if(StaticMeshComponent->GetStaticMesh())
				{
					if(const FStaticMeshRenderData* MeshRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData())
					{
						const int32 ValidLod = MeshRenderData->GetCurrentFirstLODIdx(0);
						if(MeshRenderData->LODResources.IsValidIndex(ValidLod))
						{
							for(int32 SectionIndex = 0; SectionIndex < MeshRenderData->LODResources[ValidLod].Sections.Num(); ++SectionIndex)
							{
								FCollisionResources& CollisionResource = CollisionResources.AddDefaulted_GetRef();

								const FStaticMeshSection& SectionData = MeshRenderData->LODResources[ValidLod].Sections[SectionIndex];
						
								CollisionResource.VertexPositions = MeshRenderData->LODResources[ValidLod].VertexBuffers.PositionVertexBuffer.GetSRV();
								CollisionResource.TriangleIndices = GraphBuilder.RHICmdList.CreateShaderResourceView(MeshRenderData->LODResources[ValidLod].IndexBuffer.IndexBufferRHI,
									FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Typed).SetFormat(
										MeshRenderData->LODResources[ValidLod].IndexBuffer.Is32Bit() ? PF_R32_UINT : PF_R16_UINT));
								CollisionResource.NumTriangles = SectionData.NumTriangles;
								CollisionResource.NumVertices = SectionData.MaxVertexIndex - SectionData.MinVertexIndex;
								CollisionResource.TriangleOffset = SectionData.FirstIndex;
								CollisionResource.VertexOffset = SectionData.MinVertexIndex;
							}
						}
					}
				}
			}
		}
	}
}

void FOptimusGroomCollisionReadDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(InDispatchData.NumInvocations == CollisionResources.Num());
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	int32 TotalNumVertices = 0;
	int32 TotalNumTriangles = 0;
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		const FCollisionResources& CollisionResource = CollisionResources[InvocationIndex];
		Parameters.VertexOffset = CollisionResource.VertexOffset;
		Parameters.TriangleOffset = CollisionResource.TriangleOffset;
		Parameters.TriangleVertexIndices = CollisionResource.TriangleIndices;
		Parameters.VertexGlobalPositions = CollisionResource.VertexPositions;
		Parameters.NumTriangles = CollisionResource.NumTriangles;
		Parameters.NumVertices =  CollisionResource.NumVertices;
		Parameters.BaseTriangleIndex = TotalNumTriangles;
		Parameters.BaseVertexIndex = TotalNumVertices;

		TotalNumVertices += CollisionResource.NumVertices;
		TotalNumTriangles += CollisionResource.NumTriangles;
	}

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		
		Parameters.TotalNumTriangles = TotalNumTriangles;
		Parameters.TotalNumVertices = TotalNumVertices;
	}
}
