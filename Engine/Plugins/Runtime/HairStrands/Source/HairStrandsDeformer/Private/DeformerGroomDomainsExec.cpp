// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomDomainsExec.h"

#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "GroomComponent.h"
#include "OptimusExecutionDomain.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomDomainsExec)

FString UOptimusGroomExecDataInterface::GetDisplayName() const
{
	return TEXT("Execute Groom");
}

FName UOptimusGroomExecDataInterface::GetCategory() const
{
	return CategoryName::ExecutionDataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomExecDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "NumThreads", "ReadNumThreads" });
	return Defs;
}

TSubclassOf<UActorComponent> UOptimusGroomExecDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomExecDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumThreads"))
		.AddReturnType(EShaderFundamentalType::Int, 3);
}

BEGIN_SHADER_PARAMETER_STRUCT(FGroomExecDataInterfaceParameters, )
	SHADER_PARAMETER(FIntVector, NumThreads)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomExecDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FGroomExecDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusGroomExecDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomDomainsExec.ush");

TCHAR const* UOptimusGroomExecDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomExecDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomExecDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomExecDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomExecDataProvider* Provider = NewObject<UOptimusGroomExecDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	Provider->Domain = Domain;
	return Provider;
}

FName UOptimusGroomExecDataInterface::GetExecutionDomainName(const EOptimusGroomExecDomain ExecutionDomain) 
{
	if (ExecutionDomain == EOptimusGroomExecDomain::ControlPoint)
	{
		return UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Points;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::Curve)
	{
		return UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Curves;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::StrandsEdges)
	{
		return UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Edges;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::StrandsObjects)
	{
		return UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Objects;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::GuidesPoints)
	{
		return UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::GuidesCurves)
	{
		return UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Curves;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::GuidesEdges)
	{
		return UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Edges;
	}
	else if (ExecutionDomain == EOptimusGroomExecDomain::GuidesObjects)
	{
		return UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Objects;
	}

	return NAME_None;
}

FName UOptimusGroomExecDataInterface::GetSelectedExecutionDomainName() const
{
	return GetExecutionDomainName(Domain);
}

FComputeDataProviderRenderProxy* UOptimusGroomExecDataProvider::GetRenderProxy()
{
	return new FOptimusGroomExecDataProviderProxy(MeshComponent, Domain);
}

FOptimusGroomExecDataProviderProxy::FOptimusGroomExecDataProviderProxy(UMeshComponent* MeshComponent, EOptimusGroomExecDomain InDomain)
{
	UE::Groom::Private::GatherGroupInstances(MeshComponent, GroupInstances);

	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroomComponents(MeshComponent, GroomComponents);
	UE::Groom::Private::GroomComponentsToInstances(GroomComponents, GroupInstances);
	UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents,UOptimusGroomExecDataInterface::GetExecutionDomainName(InDomain), GroupCounts);
	
	Domain = InDomain;
}

int32 FOptimusGroomExecDataProviderProxy::GetDispatchThreadCount(TArray<FIntVector>& ThreadCounts) const
{
	ThreadCounts.Reset();
	ThreadCounts.Reserve(GroupCounts.Num());
	
	for (const int32& NumThreads : GroupCounts)
	{	
		ThreadCounts.Add(FIntVector(NumThreads, 1, 1));
	}
	return GroupCounts.Num();
}

bool FOptimusGroomExecDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != GroupInstances.Num())
	{
		return false;
	}
	if (GroupCounts.Num() == 0)
	{
		return false;
	}
	if(Domain == EOptimusGroomExecDomain::ControlPoint || Domain == EOptimusGroomExecDomain::Curve ||
		Domain == EOptimusGroomExecDomain::StrandsEdges || Domain == EOptimusGroomExecDomain::StrandsObjects)
	{
		if(!UE::Groom::Private::HaveStrandsInstanceResources(GroupInstances))
		{
			return false;
			
		}
	}
	if(Domain == EOptimusGroomExecDomain::GuidesPoints || Domain == EOptimusGroomExecDomain::GuidesCurves ||
		Domain == EOptimusGroomExecDomain::GuidesEdges || Domain == EOptimusGroomExecDomain::GuidesObjects)
	{
		if(!UE::Groom::Private::HaveGuidesInstanceResources(GroupInstances))
		{
			return false;
		}
	}
	return true;
}

void FOptimusGroomExecDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	TArray<FIntVector> ThreadCounts;
	GetDispatchThreadCount(ThreadCounts);
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	
	if(ThreadCounts.Num() == ParameterArray.Num())
	{
		for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
		{
			ParameterArray[InvocationIndex].NumThreads = ThreadCounts[InvocationIndex];
		}
	}
}
