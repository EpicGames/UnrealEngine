// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeprecatedExecutionDataInterface.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "DeformerGroomDomainsExec.generated.h"

class FGroomExecDataInterfaceParameters;
class UMeshComponent;
struct FHairGroupInstance;

UENUM()
enum class EOptimusGroomExecDomain : uint8
{
	None = 0 UMETA(Hidden),
	/** Run kernel with one thread per strands points. */
	ControlPoint = 1 UMETA(DisplayName = "StrandsPoints"),
	/** Run kernel with one thread per strands curves. */
	Curve UMETA(DisplayName = "StrandsCurves"),
	/** Run kernel with one thread per strands edges. */
	StrandsEdges UMETA(DisplayName = "StrandsEdges"),
	/** Run kernel with one thread per strands objects. */
	StrandsObjects UMETA(DisplayName = "StrandsObjects"),
	/** Run kernel with one thread per guides points. */
	GuidesPoints UMETA(DisplayName = "GuidesPoints"),
	/** Run kernel with one thread per guides curves. */
	GuidesCurves UMETA(DisplayName = "GuidesCurves"),
	/** Run kernel with one thread per guides edges. */
	GuidesEdges UMETA(DisplayName = "GuidesEdges"),
	/** Run kernel with one thread per guides objects. */
	GuidesObjects UMETA(DisplayName = "GuidesObjects"),
};

/** Compute Framework Data Interface for executing kernels over a skinned mesh domain. */
UCLASS(Category = ComputeFramework)
class UOptimusGroomExecDataInterface :
	public UOptimusComputeDataInterface,
	public IOptimusDeprecatedExecutionDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual FName GetCategory() const override;
	virtual bool IsVisible() const override {return false;};
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("GroomExec"); }
	virtual bool IsExecutionInterface() const override { return true; }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	//~ Begin IOptimusDeprecatedExecutionDataInterface Interface
	virtual FName GetSelectedExecutionDomainName() const override;
	//~ End IOptimusDeprecatedExecutionDataInterface Interface

	/** static function to retrieve the execution domain name */
	static FName GetExecutionDomainName(const EOptimusGroomExecDomain ExecutionDomain);

	/** Groom execution domain */
	UPROPERTY(EditAnywhere, Category = Execution)
	EOptimusGroomExecDomain Domain = EOptimusGroomExecDomain::ControlPoint;

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for executing kernels over a groom execution. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGroomExecDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UMeshComponent> MeshComponent = nullptr;

	/** Groom execution domain */
	UPROPERTY()
	EOptimusGroomExecDomain Domain = EOptimusGroomExecDomain::ControlPoint;

	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusGroomExecDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGroomExecDataProviderProxy(UMeshComponent* MeshComponent, EOptimusGroomExecDomain InDomain);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual int32 GetDispatchThreadCount(TArray<FIntVector>& ThreadCounts) const override;
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FGroomExecDataInterfaceParameters;

	/** List of instances (invocations) used in that data interface */
	TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;

	/** Number of elements for each invocation */
	TArray<int32> GroupCounts;

	/** Groom execution domain */
	EOptimusGroomExecDomain Domain = EOptimusGroomExecDomain::ControlPoint;
};
