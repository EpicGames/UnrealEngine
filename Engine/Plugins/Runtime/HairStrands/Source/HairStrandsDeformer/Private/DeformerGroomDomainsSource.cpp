// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "GroomComponent.h"
#include "GroomSolverComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomDomainsSource)

#define LOCTEXT_NAMESPACE "DeformersGroomDomainsSource"

FName UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Edges("StrandsEdges");
FName UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Curves("StrandsCurves");
FName UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Objects("StrandsObjects");
FName UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Points("StrandsPoints");

FName UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Edges("GuidesEdges");
FName UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Curves("GuidesCurves");
FName UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Objects("GuidesObjects");
FName UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points("GuidesPoints");

FName UOptimusGroomAssetComponentSource::FMeshesExecutionDomains::Bones("MeshesBones");
FName UOptimusGroomAssetComponentSource::FMeshesExecutionDomains::Vertices("MeshesVertices");

FText UOptimusGroomAssetComponentSource::GetDisplayName() const
{
	return LOCTEXT("GroomAssetComponent", "Groom Asset Component");
}

TSubclassOf<UActorComponent> UOptimusGroomAssetComponentSource::GetComponentClass() const
{
	return UMeshComponent::StaticClass();
}

bool UOptimusGroomAssetComponentSource::IsUsableAsPrimarySource() const
{
	return GetComponentClass()->IsChildOf<UMeshComponent>();
}

TArray<FName> UOptimusGroomAssetComponentSource::GetExecutionDomains() const
{
	return {FStrandsExecutionDomains::Edges, FStrandsExecutionDomains::Objects, FStrandsExecutionDomains::Curves, FStrandsExecutionDomains::Points,
				FGuidesExecutionDomains::Edges, FGuidesExecutionDomains::Objects, FGuidesExecutionDomains::Curves, FGuidesExecutionDomains::Points,
				FMeshesExecutionDomains::Bones, FMeshesExecutionDomains::Vertices};
}

int32 UOptimusGroomAssetComponentSource::GetLodIndex(const UActorComponent* InComponent) const
{
	return 0;
}

uint32 UOptimusGroomAssetComponentSource::GetDefaultNumInvocations(const UActorComponent* ActorComponent, int32 LodIndex) const
{
	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroomComponents(ActorComponent, GroomComponents);
	
	uint32 NumInvocations = 0;
	for(const UGroomComponent* GroomComponent : GroomComponents)
	{
		const uint32 NumGroups = GroomComponent->GetGroupCount();
		for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
		{
			if(GroomComponent->IsDeformationEnable(GroupIndex) && GroomComponent->GetGroupInstance(GroupIndex))
			{
				++NumInvocations;
			}
		}
	}
	return NumInvocations;
}

bool UOptimusGroomAssetComponentSource::GetComponentElementCountsForExecutionDomain(
	FName DomainName, const UActorComponent* ActorComponent, int32 LodIndex, TArray<int32>& InvocationElementCounts) const
{
	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroomComponents(ActorComponent, GroomComponents);

	InvocationElementCounts.Reset();
	const uint32 TotalCount = UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents, DomainName, InvocationElementCounts);
	return true;//TotalCount != 0;
}

FName UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Edges("SolverEdges");
FName UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Curves("SolverCurves");
FName UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Objects("SolverObjects");
FName UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Points("SolverPoints");

FName UOptimusGroomSolverComponentSource::FDynamicExecutionDomains::Points("DynamicPoints");
FName UOptimusGroomSolverComponentSource::FDynamicExecutionDomains::Curves("DynamicCurves");
FName UOptimusGroomSolverComponentSource::FDynamicExecutionDomains::Edges("DynamicEdges");

FName UOptimusGroomSolverComponentSource::FKinematicExecutionDomains::Points("KinematicPoints");
FName UOptimusGroomSolverComponentSource::FKinematicExecutionDomains::Curves("KinematicCurves");
FName UOptimusGroomSolverComponentSource::FKinematicExecutionDomains::Edges("KinematicEdges");

FText UOptimusGroomSolverComponentSource::GetDisplayName() const
{
	return LOCTEXT("GroomSolverComponent", "Groom Solver Component");
}

TSubclassOf<UActorComponent> UOptimusGroomSolverComponentSource::GetComponentClass() const
{
	return UGroomSolverComponent::StaticClass();
}

bool UOptimusGroomSolverComponentSource::IsUsableAsPrimarySource() const
{
	return GetComponentClass()->IsChildOf<UGroomSolverComponent>();
}

TArray<FName> UOptimusGroomSolverComponentSource::GetExecutionDomains() const
{
	return {FSolverExecutionDomains::Points, FSolverExecutionDomains::Edges, FSolverExecutionDomains::Curves, FSolverExecutionDomains::Objects,
		FDynamicExecutionDomains::Points, FDynamicExecutionDomains::Curves, FKinematicExecutionDomains::Points, FKinematicExecutionDomains::Curves,
	FDynamicExecutionDomains::Edges, FKinematicExecutionDomains::Edges};
}

int32 UOptimusGroomSolverComponentSource::GetLodIndex(const UActorComponent* InComponent) const
{
	return 0;
}

uint32 UOptimusGroomSolverComponentSource::GetDefaultNumInvocations(const UActorComponent* ActorComponent, int32 LodIndex) const
{
	return 1;
}

bool UOptimusGroomSolverComponentSource::GetComponentElementCountsForExecutionDomain(
	FName DomainName,const UActorComponent* ActorComponent, int32 LodIndex, TArray<int32>& InvocationElementCounts) const
{
	if(const UGroomSolverComponent* GroomSolver = Cast<UGroomSolverComponent>(ActorComponent))
	{
		TArray<const UGroomComponent*> GroomComponents;
		for(const TObjectPtr<UGroomComponent>& GroomComponent : GroomSolver->GetGroomComponents())
		{
			GroomComponents.Add(GroomComponent);
		}
		TArray<int32> LocalElementCounts;
		uint32 TotalCount = 0;
		
		if(DomainName == FSolverExecutionDomains::Objects)
		{
			TotalCount = UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents, UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Objects, LocalElementCounts);
		}
		else if(DomainName == FSolverExecutionDomains::Curves)
		{
			TotalCount = UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents, UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Curves, LocalElementCounts);
		}
		else if (DomainName == FSolverExecutionDomains::Edges)
		{
			TotalCount = UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents, UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Edges, LocalElementCounts);
		}
		else if (DomainName == FSolverExecutionDomains::Points)
		{
			TotalCount = UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents, UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points, LocalElementCounts);
		}
		else if (DomainName == FDynamicExecutionDomains::Curves)
		{
			TotalCount = GroomSolver->GetSolverSettings().CurveDynamicIndices.Num();
		}
		else if (DomainName == FDynamicExecutionDomains::Points)
		{
			TotalCount = GroomSolver->GetSolverSettings().PointDynamicIndices.Num();
		}
		else if (DomainName == FDynamicExecutionDomains::Edges)
		{
			TotalCount = GroomSolver->GetSolverSettings().PointDynamicIndices.Num() - GroomSolver->GetSolverSettings().CurveDynamicIndices.Num();
		}
		else if (DomainName == FKinematicExecutionDomains::Curves)
		{
			TotalCount = GroomSolver->GetSolverSettings().CurveKinematicIndices.Num();
		}
		else if (DomainName == FKinematicExecutionDomains::Points)
		{
			TotalCount = GroomSolver->GetSolverSettings().PointKinematicIndices.Num();
		}
		else if (DomainName == FKinematicExecutionDomains::Edges)
		{
			TotalCount = GroomSolver->GetSolverSettings().PointKinematicIndices.Num() - GroomSolver->GetSolverSettings().CurveKinematicIndices.Num();
		}
		TotalCount = FMath::Max(TotalCount, 1u);
		InvocationElementCounts.Init(TotalCount,1);
		return TotalCount != 0;
	}
	return false;
}

FName UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Vertices("CollisionVertices");
FName UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Triangles("CollisionTriangles");

FText UOptimusGroomCollisionComponentSource::GetDisplayName() const
{
	return LOCTEXT("GroomCollisionComponent", "Groom Collision Component");
}

TSubclassOf<UActorComponent> UOptimusGroomCollisionComponentSource::GetComponentClass() const
{
	return UGroomSolverComponent::StaticClass();
}

bool UOptimusGroomCollisionComponentSource::IsUsableAsPrimarySource() const
{
	return GetComponentClass()->IsChildOf<UGroomSolverComponent>();
}

TArray<FName> UOptimusGroomCollisionComponentSource::GetExecutionDomains() const
{
	return {FCollisionExecutionDomains::Vertices, FCollisionExecutionDomains::Triangles};
}

int32 UOptimusGroomCollisionComponentSource::GetLodIndex(const UActorComponent* InComponent) const
{
	int32 LODIndex = 0;
	if(const UGroomSolverComponent* GroomSolver = Cast<UGroomSolverComponent>(InComponent))
	{
		int32 LODCount = 1;
		for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : GroomSolver->GetCollisionComponents())
		{
			if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(CollisionComponent.Key))
			{
				LODIndex += SkinnedMeshComponent->GetPredictedLODLevel() * LODCount;
				LODCount *= SkinnedMeshComponent->GetNumLODs();
			}
		}
	}
	
	return LODIndex;
}

uint32 UOptimusGroomCollisionComponentSource::GetDefaultNumInvocations(const UActorComponent* ActorComponent, int32 LodIndex) const
{
	if(const UGroomSolverComponent* GroomSolver = Cast<UGroomSolverComponent>(ActorComponent))
	{
		TArray<TPair<TObjectPtr<UMeshComponent>, int32>> CollisionComponents;
		for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : GroomSolver->GetCollisionComponents())
		{
			CollisionComponents.Add(CollisionComponent);
		}
		return UE::Groom::Private::GetCollisionNumInvocations(CollisionComponents);
	}
	
	return 0;
}

bool UOptimusGroomCollisionComponentSource::GetComponentElementCountsForExecutionDomain(
	FName DomainName,const UActorComponent* ActorComponent, int32 LodIndex, TArray<int32>& InvocationElementCounts) const
{
	if(const UGroomSolverComponent* GroomSolver = Cast<UGroomSolverComponent>(ActorComponent))
	{
		TArray<TPair<TObjectPtr<UMeshComponent>, int32>> CollisionComponents;
		for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : GroomSolver->GetCollisionComponents())
		{
			CollisionComponents.Add(CollisionComponent);
		}
		InvocationElementCounts.Reset();
		const uint32 TotalCount = UE::Groom::Private::GetCollisionInvocationElementCounts(CollisionComponents, DomainName, InvocationElementCounts);
		return true;
	}
	return false;
}


#undef LOCTEXT_NAMESPACE
