// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComponentSource.h"
#include "DeformerGroomDomainsSource.generated.h"

class UActorComponent;
class UGroomComponent;
class UGroomSolverComponent;

/** Execution domain that will be used by the groom asset optimus data interfaces */
UCLASS()
class UOptimusGroomAssetComponentSource : public UOptimusComponentSource
{
	GENERATED_BODY()
public:

	struct FGuidesExecutionDomains
	{
		static FName Edges;
		static FName Curves;
		static FName Objects;
		static FName Points;
	};

	struct FStrandsExecutionDomains
	{
		static FName Edges;
		static FName Curves;
		static FName Objects;
		static FName Points;
	};

	struct FMeshesExecutionDomains
	{
		static FName Bones;
		static FName Vertices;
	};
	
	//~ Begin  UOptimusComponentSource implementations
	virtual FText GetDisplayName() const override;
	virtual FName GetBindingName() const override { return FName("Groom Asset"); }
	virtual TSubclassOf<UActorComponent> GetComponentClass() const override;
	virtual TArray<FName> GetExecutionDomains() const override;
	virtual int32 GetLodIndex(const UActorComponent* InComponent) const override;
	virtual uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const override;
	virtual bool GetComponentElementCountsForExecutionDomain(FName InDomainName, const UActorComponent* InComponent, int32 InLodIndex, TArray<int32>& OutInvocationElementCounts) const override;
	virtual bool IsUsableAsPrimarySource() const override;
	//~ End UOptimusComputeDataInterface Interface
};

/** Execution domain that will be used by the groom solver optimus data interfaces */
UCLASS()
class UOptimusGroomSolverComponentSource : public UOptimusComponentSource
{
	GENERATED_BODY()
public:

	static constexpr int32 GroupSize = 64;

	struct FSolverExecutionDomains
	{
		static FName Edges;
		static FName Curves;
		static FName Objects;
		static FName Points;
	};

	struct FDynamicExecutionDomains
	{
		static FName Points;
		static FName Curves;
		static FName Edges;
	};

	struct FKinematicExecutionDomains
	{
		static FName Points;
		static FName Curves;
		static FName Edges;
	};
	
	//~ Begin  UOptimusComponentSource implementations
	virtual FText GetDisplayName() const override;
	virtual FName GetBindingName() const override { return FName("Groom Solver"); }
	virtual TSubclassOf<UActorComponent> GetComponentClass() const override;
	virtual TArray<FName> GetExecutionDomains() const override;
	virtual int32 GetLodIndex(const UActorComponent* InComponent) const override;
	virtual uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const override;
	virtual bool GetComponentElementCountsForExecutionDomain(FName InDomainName, const UActorComponent* InComponent, int32 InLodIndex, TArray<int32>& OutInvocationElementCounts) const override;
	virtual bool IsUsableAsPrimarySource() const override;
	//~ End UOptimusComputeDataInterface Interface
};

/** Execution domain that will be used by the groom collision optimus data interfaces */
UCLASS()
class UOptimusGroomCollisionComponentSource : public UOptimusComponentSource
{
	GENERATED_BODY()
public:

	static constexpr int32 GroupSize = 64;
	
	struct FCollisionExecutionDomains
	{
		static FName Vertices;
		static FName Triangles;
		static FName Implicits;
	};
	
	//~ Begin  UOptimusComponentSource implementations
	virtual FText GetDisplayName() const override;
	virtual FName GetBindingName() const override { return FName("Groom Collision"); }
	virtual TSubclassOf<UActorComponent> GetComponentClass() const override;
	virtual TArray<FName> GetExecutionDomains() const override;
	virtual int32 GetLodIndex(const UActorComponent* InComponent) const override;
	virtual uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const override;
	virtual bool GetComponentElementCountsForExecutionDomain(FName InDomainName, const UActorComponent* InComponent, int32 InLodIndex, TArray<int32>& OutInvocationElementCounts) const override;
	virtual bool IsUsableAsPrimarySource() const override;
	//~ End UOptimusComputeDataInterface Interface
};





