// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomGuidesRead.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "GroomInstance.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "HairStrandsInterpolation.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "OptimusValueContainerStruct.h"
#include "SystemTextures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomGuidesRead)


FString UOptimusGroomGuidesReadDataInterface::GetDisplayName() const
{
	return TEXT("Groom Guides");
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomGuidesReadDataInterface::GetPinDefinitions() const
{
	static const FName GuidesPoints(UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points);
	static const FName GuidesCurves(UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Curves);

	TArray<FOptimusCDIPinDefinition> Defs;

	// Deformation buffers
	Defs.Add({ "NumGuidePoints",   "ReadNumPoints", false, "NumGuidesPoints" });
	Defs.Add({ "NumGuideCurves",   "ReadNumCurves", false, "NumGuidesCurves" });
	Defs.Add({ "Position",         "ReadPointRestPosition",          GuidesPoints,  "ReadNumPoints", false, "PointRestPosition"});
	Defs.Add({ "CurveOffsetPoint", "ReadCurvePointOffset",  GuidesCurves,          "ReadNumCurves", false, "CurvePointOffset" });
	Defs.Add({ "CurveNumPoint",    "ReadCurveNumPoints",     GuidesCurves,          "ReadNumCurves", false, "CurveNumPoints"});
	Defs.Add({ "PointCurveIndex", "ReadPointCurveIndex", GuidesPoints,          "ReadNumPoints", false, "PointCurveIndex"  });
	Defs.Add({ "CurveRestTransform", "ReadCurveRestTransform", GuidesCurves,   "ReadNumCurves", false, "CurveRestTransform"  });
	Defs.Add({ "CurveDeformedTransform", "ReadCurveDeformedTransform", GuidesCurves,          "ReadNumCurves", true, "CurveDeformedTransform"  });
	Defs.Add({ "ObjectRestTransform", "ReadObjectRestTransform", false, "ObjectRestTransform"  });
	Defs.Add({ "ObjectDeformedTransform", "ReadObjectDeformedTransform", true, "ObjectDeformedTransform"  });
	Defs.Add({ "GuidesObjectIndex", "ReadObjectIndex", true, "GuidesObjectIndex"  });
	
	return Defs;
}
 
TSubclassOf<UActorComponent> UOptimusGroomGuidesReadDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomGuidesReadDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("ENABLE_SKINNED_TRANSFORM"), 2);
}

void UOptimusGroomGuidesReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumCurves"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectIndex"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointRestPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurvePointOffset"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointCurveIndex"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
		
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveRestTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveDeformedTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);
		
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectRestTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectDeformedTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4);
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomGuidesReadParameters, )
	SHADER_PARAMETER(uint32, PointCount)
	SHADER_PARAMETER(uint32, CurveCount)
	SHADER_PARAMETER(uint32, BasePointIndex)
	SHADER_PARAMETER(uint32, BaseCurveIndex)
	SHADER_PARAMETER(uint32, TotalPointCount)
	SHADER_PARAMETER(uint32, TotalCurveCount)
	SHADER_PARAMETER(uint32, ObjectIndex)
	SHADER_PARAMETER(FVector3f, RestPositionOffset)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointRestPositions)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurvePointOffsets)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointCurveIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveMapping)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointMapping)
	SHADER_PARAMETER(FMatrix44f, ObjectRestTransform)
	SHADER_PARAMETER(FMatrix44f, ObjectDeformedTransform)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TriangleRestPositions)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TriangleDeformedPositions)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CurveBarycentricCoordinates)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CurveTriangleIndices)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomGuidesReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomGuidesReadParameters>(UID);
}

TCHAR const* UOptimusGroomGuidesReadDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomGuidesRead.ush");

TCHAR const* UOptimusGroomGuidesReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomGuidesReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomGuidesReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomGuidesReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomGuidesReadDataProvider* Provider = NewObject<UOptimusGroomGuidesReadDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomGuidesReadDataProvider::GetRenderProxy()
{
	return new FOptimusGroomGuidesReadDataProviderProxy(MeshComponent);
}

FOptimusGroomGuidesReadDataProviderProxy::FOptimusGroomGuidesReadDataProviderProxy(UMeshComponent* MeshComponent)
{
	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroomComponents(MeshComponent, GroomComponents);
	UE::Groom::Private::GroomComponentsToInstances(GroomComponents, GroupInstances);
}

bool FOptimusGroomGuidesReadDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != GroupInstances.Num())
	{
		return false;
	}
	if(!UE::Groom::Private::HaveGuidesInstanceResources(GroupInstances) || !UE::Groom::Private::HaveGuidesSkinnedResources(GroupInstances))
	{
		return false;
	}
	
	return true;
}

struct FOptimusGroomGuidesReadPermutationIds
{
	uint32 EnableSkinnedTransform = 0;

	FOptimusGroomGuidesReadPermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		static FString Name(TEXT("ENABLE_SKINNED_TRANSFORM"));
		static uint32 Hash = GetTypeHash(Name);
		EnableSkinnedTransform = PermutationVector.GetPermutationBits(Name, Hash, 1);
	}
};

void FOptimusGroomGuidesReadDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	const FOptimusGroomGuidesReadPermutationIds PermutationIds(InOutPermutationData.PermutationVector);

	int32 InvocationIndex = 0;
	for (const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		if (GroupInstance)
		{
			InOutPermutationData.PermutationIds[InvocationIndex] |= ((GroupInstance->BindingType == EHairBindingType::Skinning) ? PermutationIds.EnableSkinnedTransform : 0);
		}
		++InvocationIndex;
	}
}

void FOptimusGroomGuidesReadDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	InstanceResources.Reset();
	BindingResources.Reset();

	int32 InvocationIndex = 0;
	for (const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		if (GroupInstance)
		{
			{
				FInstanceResources& Resource = InstanceResources.AddDefaulted_GetRef();
				
				Resource.PointRestPositions	= RegisterAsSRV(GraphBuilder, GroupInstance->Guides.RestResource->PositionBuffer);
				Resource.CurvePointOffsets = RegisterAsSRV(GraphBuilder, GroupInstance->Guides.RestResource->CurveBuffer);
				Resource.PointCurveIndices = RegisterAsSRV(GraphBuilder, GroupInstance->Guides.RestResource->PointToCurveBuffer);
				Resource.CurveMapping = RegisterAsSRV(GraphBuilder, GroupInstance->Guides.RestResource->CurveMappingBuffer);
				Resource.PointMapping = RegisterAsSRV(GraphBuilder, GroupInstance->Guides.RestResource->PointMappingBuffer);
			}
			{
				FBindingResources& Resource = BindingResources.AddDefaulted_GetRef();
				if(GroupInstance->BindingType == EHairBindingType::Skinning)
				{
					FHairStrandsLODRestRootResource* RestLODDatas = GroupInstance->Guides.RestRootResource->LODs[GroupInstance->HairGroupPublicData->MeshLODIndex];
					FHairStrandsLODDeformedRootResource* DeformedLODDatas = GroupInstance->Guides.DeformedRootResource->LODs[GroupInstance->HairGroupPublicData->MeshLODIndex];
			
					Resource.CurveTriangleIndices = RegisterAsSRV(GraphBuilder, RestLODDatas->RootToUniqueTriangleIndexBuffer);
					Resource.TriangleRestPositions = RegisterAsSRV(GraphBuilder, RestLODDatas->RestUniqueTrianglePositionBuffer);
					Resource.TriangleDeformedPositions = RegisterAsSRV(GraphBuilder, DeformedLODDatas->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current));
					Resource.CurveBarycentricCoordinates = RegisterAsSRV(GraphBuilder, RestLODDatas->RootBarycentricBuffer);
				}
				{
					Resource.ObjectDeformedTransform = FMatrix44f(GroupInstance->GetCurrentLocalToWorld().ToMatrixWithScale().GetTransposed());
					Resource.ObjectRestTransform = FMatrix44f(GroupInstance->GetRestLocalToWorld().ToMatrixWithScale().GetTransposed());
				}
			}
		}
		++InvocationIndex;
	}
}

void FOptimusGroomGuidesReadDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(InDispatchData.NumInvocations == GroupInstances.Num());
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	uint32 BasePointIndex = 0;
	uint32 BaseCurveIndex = 0;
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			
			Parameters.PointCount = GroupInstance->Guides.RestResource->GetPointCount();
			Parameters.CurveCount = GroupInstance->Guides.RestResource->GetCurveCount();
			
			// Used to get the local element indices for the current group, since index supplied by the compute kernel goes from 0 to NumElementsPerGroup * NumGroups
			Parameters.BasePointIndex = BasePointIndex;
			BasePointIndex += Parameters.PointCount;
			Parameters.BaseCurveIndex = BaseCurveIndex;
			BaseCurveIndex += Parameters.CurveCount;
			
			Parameters.ObjectIndex = InvocationIndex;
			Parameters.RestPositionOffset = FVector3f(GroupInstance->Guides.RestResource->GetPositionOffset());

			{
				FInstanceResources& Resource = InstanceResources[InvocationIndex];
				Parameters.PointRestPositions = Resource.PointRestPositions;
				Parameters.CurvePointOffsets = Resource.CurvePointOffsets;
				Parameters.PointCurveIndices = Resource.PointCurveIndices;
				Parameters.CurveMapping = Resource.CurveMapping;
				Parameters.PointMapping = Resource.PointMapping;
			}

			{
				FBindingResources& Resource = BindingResources[InvocationIndex];
				if(GroupInstance->BindingType == EHairBindingType::Skinning)
				{
					Parameters.CurveTriangleIndices = Resource.CurveTriangleIndices;
					Parameters.CurveBarycentricCoordinates = Resource.CurveBarycentricCoordinates;
					Parameters.TriangleRestPositions = Resource.TriangleRestPositions;
					Parameters.TriangleDeformedPositions = Resource.TriangleDeformedPositions;
				}
				{
					Parameters.ObjectDeformedTransform = Resource.ObjectDeformedTransform;
					Parameters.ObjectRestTransform = Resource.ObjectRestTransform;
				}
			}
		}
	}

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.TotalPointCount = BasePointIndex;
			Parameters.TotalCurveCount = BaseCurveIndex;
		}
	}
}
