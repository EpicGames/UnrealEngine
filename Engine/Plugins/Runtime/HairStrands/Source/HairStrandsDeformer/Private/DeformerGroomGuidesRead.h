// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "OptimusComputeDataInterface.h"
#include "HairStrandsInterface.h"
#include "IOptimusDeformerInstanceAccessor.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "DeformerGroomGuidesRead.generated.h"

class FOptimusGroomGuidesReadParameters;
class UGroomComponent;
struct FHairGroupInstance;
class UGroomAsset;
class UMeshComponent;

/** Compute Framework Data Interface for reading groom guides. */
UCLASS(Category = ComputeFramework)
class UOptimusGroomGuidesReadDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("GroomGuidesRead"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	/** File holding the hlsl implementation */
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading groom guides. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGroomGuidesReadDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMeshComponent> MeshComponent = nullptr;

	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusGroomGuidesReadDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGroomGuidesReadDataProviderProxy(UMeshComponent* MeshComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	virtual void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FOptimusGroomGuidesReadParameters;

	/** List of instances (invocations) used in that data interface */
	TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;

	/** Binding Resources used to dispatch CS on GPU */
	struct FBindingResources
	{
		FMatrix44f ObjectRestTransform = FMatrix44f::Identity;
		FMatrix44f ObjectDeformedTransform = FMatrix44f::Identity;
		FRDGBufferSRVRef TriangleRestPositions = nullptr;
		FRDGBufferSRVRef TriangleDeformedPositions = nullptr;
		FRDGBufferSRVRef CurveBarycentricCoordinates = nullptr;
		FRDGBufferSRVRef CurveTriangleIndices = nullptr;
	};
	TArray<FBindingResources> BindingResources;

	/** Instance resources used to dispatch CS on GPU */
	struct FInstanceResources
	{
		FRDGBufferSRVRef CurvePointOffsets = nullptr;
		FRDGBufferSRVRef PointRestPositions = nullptr;
		FRDGBufferSRVRef PointCurveIndices = nullptr;
		FRDGBufferSRVRef CurveMapping = nullptr;
		FRDGBufferSRVRef PointMapping = nullptr;
	};
	TArray<FInstanceResources> InstanceResources;
};
