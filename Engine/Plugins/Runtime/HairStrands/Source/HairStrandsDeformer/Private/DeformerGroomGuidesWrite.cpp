// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomGuidesWrite.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "RenderGraphBuilder.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "GroomInstance.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "HairStrandsInterpolation.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomGuidesWrite)

FString UOptimusGroomGuidesWriteDataInterface::GetDisplayName() const
{
	return TEXT("Write Groom Guides");
}

FName UOptimusGroomGuidesWriteDataInterface::GetCategory() const
{
	return CategoryName::OutputDataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomGuidesWriteDataInterface::GetPinDefinitions() const
{
	static const FName GuidesPoints(UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points);

	TArray<FOptimusCDIPinDefinition> Defs;
	
	// Deformation buffers
	Defs.Add({ "PointDeformedPosition", "WritePointDeformedPosition", GuidesPoints, "ReadNumPoints", true, "PointDeformedPosition" });

	return Defs;
}

TSubclassOf<UActorComponent> UOptimusGroomGuidesWriteDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomGuidesWriteDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);
}

void UOptimusGroomGuidesWriteDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointDeformedPosition"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 3);
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomGuidesWriteParameters, )
	SHADER_PARAMETER(uint32, RegisteredIndex)
	SHADER_PARAMETER(uint32, PointCount)
	SHADER_PARAMETER(uint32, TotalPointCount)
	SHADER_PARAMETER(uint32, BasePointIndex)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeformedPositionOffset)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointRestPositions)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, PointDeformedPositions)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomGuidesWriteDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomGuidesWriteParameters>(UID);
}

TCHAR const* UOptimusGroomGuidesWriteDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomGuidesWrite.ush");

TCHAR const* UOptimusGroomGuidesWriteDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomGuidesWriteDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomGuidesWriteDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomGuidesWriteDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomGuidesWriteDataProvider* Provider = NewObject<UOptimusGroomGuidesWriteDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	Provider->OutputMask = InOutputMask;
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomGuidesWriteDataProvider::GetRenderProxy()
{
	return new FOptimusGroomGuidesWriteProviderProxy(MeshComponent, OutputMask);
}

FOptimusGroomGuidesWriteProviderProxy::FOptimusGroomGuidesWriteProviderProxy(UMeshComponent* MeshComponent, uint64 InOutputMask)
{
	OutputMask = InOutputMask;
	UE::Groom::Private::GatherGroupInstances(MeshComponent, GroupInstances);
}

bool FOptimusGroomGuidesWriteProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != GroupInstances.Num())
	{
		return false;
	}
	if(!UE::Groom::Private::HaveGuidesInstanceResources(GroupInstances))
	{
		return false;
	}
	return true;
}

void FOptimusGroomGuidesWriteProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	InstanceResources.Reset();
	for (const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		if (GroupInstance)
		{
			FInstanceResources& Resource = InstanceResources.AddDefaulted_GetRef();
			
			Resource.DeformedPositionOffset = Register(GraphBuilder, GroupInstance->Guides.DeformedResource->GetPositionOffsetBuffer(FHairStrandsDeformedResource::EFrameType::Current), ERDGImportedBufferFlags::CreateSRV).SRV;
			Resource.PointRestPositions = Register(GraphBuilder, GroupInstance->Guides.RestResource->PositionBuffer, ERDGImportedBufferFlags::CreateSRV).SRV;
			Resource.PointDeformedPositions = Register(GraphBuilder, GroupInstance->Guides.DeformedResource->GetDeformerBuffer(GraphBuilder), ERDGImportedBufferFlags::CreateUAV).UAV;

			//Resource.PointDeformedPositions = Register(GraphBuilder, GroupInstance->Guides.DeformedResource->GetBuffer(FHairStrandsDeformedResource::Current), ERDGImportedBufferFlags::CreateUAV).UAV;
		}	
	}
}

void FOptimusGroomGuidesWriteProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	uint32 BasePointIndex = 0;
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			FInstanceResources& Resource = InstanceResources[InvocationIndex];
			
			Parameters.PointCount = GroupInstance->Guides.RestResource->GetPointCount();
			Parameters.BasePointIndex = BasePointIndex;
			BasePointIndex += Parameters.PointCount;
			Parameters.RegisteredIndex = GroupInstance->RegisteredIndex;
			
			Parameters.DeformedPositionOffset 	=  Resource.DeformedPositionOffset;
			Parameters.PointRestPositions = Resource.PointRestPositions;
			Parameters.PointDeformedPositions = Resource.PointDeformedPositions;
		}
	}
	
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.TotalPointCount = BasePointIndex;
		}
	}
}
