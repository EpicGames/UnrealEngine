// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "DeformerGroomGuidesWrite.generated.h"

class FOptimusGroomGuidesWriteParameters;
struct FHairGroupInstance;
class FRDGBuffer;
class FRDGBufferSRV;
class FRDGBufferUAV;
class UGroomComponent;
class UMeshComponent;

/** Compute Framework Data Interface for writing groom strands. */
UCLASS(Category = ComputeFramework)
class UOptimusGroomGuidesWriteDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual FName GetCategory() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("GroomGuidesWrite"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	/** File holding the hlsl implementation */
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for writing skinned mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGroomGuidesWriteDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMeshComponent> MeshComponent = nullptr;

	/** Output mask used to know the buffer format */
	uint64 OutputMask = 0;

	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusGroomGuidesWriteProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGroomGuidesWriteProviderProxy(UMeshComponent* MeshComponent, uint64 InOutputMask);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FOptimusGroomGuidesWriteParameters;

	/** List of instances (invocations) used in that data interface */
	TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;

	/** Output mask for gpu buffer format*/
	uint64 OutputMask = 0;

	/** Instance resources used to dispatch CS on GPU */
	struct FInstanceResources
	{
		FRDGBufferSRVRef DeformedPositionOffset = nullptr;
		FRDGBufferSRVRef PointRestPositions = nullptr;
		FRDGBufferUAVRef PointDeformedPositions = nullptr;
	};
	TArray<FInstanceResources> InstanceResources;
};