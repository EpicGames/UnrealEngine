// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GroomComponent.h"
#include "GroomSolverComponent.h"
#include "RenderGraphBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "SkeletalRenderPublic.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "DeformerGroomDomainsSource.h"

namespace UE::Groom::Private
{
	FORCEINLINE bool HasDeformationEnabledOrHasMeshDeformer(const UGroomComponent* GroomComponent, const int32 GroupIndex)
	{
		return GroomComponent->IsDeformationEnable(GroupIndex);
	}

	FORCEINLINE USkeletalMeshComponent* GetGroupSkelMesh(const UGroomComponent* GroomComponent, const int32 GroupIndex, int32& MeshLOD)
	{
		if (GroomComponent)
		{
			TArray<UActorComponent*> ActorComponents;
			if (AActor* RootActor = GroomComponent->GetAttachmentRootActor())
			{
				RootActor->GetComponents(USkeletalMeshComponent::StaticClass(), ActorComponents);
			}
			for (UActorComponent* ActorComponent : ActorComponents)
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ActorComponent))
				{
					if (GroomComponent->GroomAsset && (SkeletalMeshComponent->GetSkeletalMeshAsset() ==
						GroomComponent->GroomAsset->GetDataflowSettings().GetSkeletalMesh(GroupIndex)))
					{
						MeshLOD = GroomComponent->GroomAsset->GetDataflowSettings().GetMeshLOD(GroupIndex);
						return SkeletalMeshComponent;
					}
				}
			}
		}
		MeshLOD = INDEX_NONE;
		return nullptr;
	}
	
	FORCEINLINE void GroomComponentsToInstances(const TArray<const UGroomComponent*>& GroomComponents, TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
 	{
 		GroupInstances.Reset();
 		for (const UGroomComponent* GroomComponent : GroomComponents)
 		{
 			if (GroomComponent)
 			{
 				const uint32 NumGroups = GroomComponent->GetGroupCount();
 				for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
 				{
 					if(HasDeformationEnabledOrHasMeshDeformer(GroomComponent, GroupIndex) && GroomComponent->GetGroupInstance(GroupIndex))
 					{
						const TRefCountPtr<const FHairGroupInstance> GroupInstance(GroomComponent->GetGroupInstance(GroupIndex));
 						GroupInstances.Add(GroupInstance);
 					}
 				}
 			}
 		}
 	}
	FORCEINLINE void GroomComponentsToSkelmeshes(const TArray<const UGroomComponent*>& GroomComponents, TArray<const FSkeletalMeshObject*>& SkeletalMeshes, TArray<FMatrix44f>& SkeletalTransforms,
		TArray<TArray<FMatrix44f>>& BonesRefToLocals, TArray<TArray<FMatrix44f>>& BindTransforms, TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		SkeletalMeshes.Reset();
		SkeletalTransforms.Reset();
		BonesRefToLocals.Reset();
		BindTransforms.Reset();
		GroupInstances.Reset();

		TArray<FTransform> BonesTransforms;
		TArray<FMatrix44f> RefToLocals;
		TArray<FMatrix44f> RefBases;
		int32 MeshLOD = INDEX_NONE;
		
		for (const UGroomComponent* GroomComponent : GroomComponents)
		{
			if (GroomComponent)
			{
				const uint32 NumGroups = GroomComponent->GetGroupCount();
				for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
				{
					if (GroomComponent->IsDeformationEnable(GroupIndex) && GroomComponent->GetGroupInstance(GroupIndex))
					{
						if (USkeletalMeshComponent* SkelMesh =  GetGroupSkelMesh(GroomComponent, GroupIndex, MeshLOD))
						{
							if (USkinnedAsset* SkinnedAsset = SkelMesh->GetSkinnedAsset())
							{
								const FSkeletalMeshRenderData* const RenderData = SkinnedAsset->GetResourceForRendering();
								if (RenderData && RenderData->LODRenderData.IsValidIndex(MeshLOD))
								{
									// Get the matching active indices used for skinning
									TArray<FBoneIndexType> GuidesBones = RenderData->LODRenderData[MeshLOD].ActiveBoneIndices;
								
									// Get inv ref pose matrices
									const TArray<FMatrix44f>* RefBasesInvMatrix = &SkelMesh->GetSkinnedAsset()->GetRefBasesInvMatrix();

									// Get the component space transforms
									BonesTransforms.Init(FTransform::Identity, RefBasesInvMatrix->Num());
								
									SkelMesh->GetSkinnedAsset()->FillComponentSpaceTransforms(
										SkelMesh->GetBoneSpaceTransforms(), GuidesBones, BonesTransforms);

									// Fill ref to local and bind matrices
									RefToLocals.Init(FMatrix44f::Identity, RefBasesInvMatrix->Num());
									RefBases.Init(FMatrix44f::Identity, RefBasesInvMatrix->Num());
								
									for (int32 BoneIndex = 0; BoneIndex < RefToLocals.Num(); ++BoneIndex)
									{
										RefToLocals[BoneIndex] = (*RefBasesInvMatrix)[BoneIndex] * (FMatrix44f)BonesTransforms[BoneIndex].ToMatrixWithScale();
										RefBases[BoneIndex] = (*RefBasesInvMatrix)[BoneIndex].Inverse();
									}
							
									const FTransform& BonesTransform = SkelMesh->GetComponentTransform();
									const FTransform& GroupTransform = GroomComponent->GetGroupInstance(GroupIndex)->GetCurrentLocalToWorld();
		
									SkeletalTransforms.Add(FMatrix44f(BonesTransform.ToMatrixWithScale() * GroupTransform.ToInverseMatrixWithScale()));
									BonesRefToLocals.Add(RefToLocals);
									BindTransforms.Add(RefBases);
									SkeletalMeshes.Add(SkelMesh->MeshObject);
									GroupInstances.Add(TRefCountPtr<const FHairGroupInstance>(GroomComponent->GetGroupInstance(GroupIndex)));
								}
							}
						}
					}
				}
			}
		}
	}
	
	FORCEINLINE void GatherGroomComponents(const UActorComponent* ActorComponent, TArray<const UGroomComponent*>& GroomComponents)
	{
		GroomComponents.Reset();
		if (const UGroomComponent* GroomComponent = Cast<UGroomComponent>(ActorComponent))
		{
			GroomComponents.Add(GroomComponent);
		}
		else if (const UGroomSolverComponent* GroomSolver = Cast<UGroomSolverComponent>(ActorComponent))
		{
			for (const TObjectPtr<UGroomComponent>& SolverGroom : GroomSolver->GetGroomComponents())
			{
				GroomComponents.Add(SolverGroom);
			}
		}
	}
	
	FORCEINLINE void GatherGroupInstances(const UActorComponent* ActorComponent, TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		TArray<const UGroomComponent*> GroomComponents;
		GatherGroomComponents(ActorComponent, GroomComponents);

		GroomComponentsToInstances(GroomComponents, GroupInstances);
	}

	FORCEINLINE void GatherGroupSkelmeshes(const UActorComponent* ActorComponent, TArray<const FSkeletalMeshObject*>& SkeletalMeshes, TArray<FMatrix44f>& SkeletalTransforms,
		TArray<TArray<FMatrix44f>>& BonesRefToLocals, TArray<TArray<FMatrix44f>>& BindTransforms, TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		TArray<const UGroomComponent*> GroomComponents;
		GatherGroomComponents(ActorComponent, GroomComponents);

		GroomComponentsToSkelmeshes(GroomComponents, SkeletalMeshes, SkeletalTransforms, BonesRefToLocals, BindTransforms, GroupInstances);
	}

	template<typename DataType>
	static bool HaveValidInstanceResources(const DataType& InstanceData)
	{
		if(InstanceData.RestResource && InstanceData.DeformedResource)
		{
			if(InstanceData.RestResource->IsInitialized() && InstanceData.RestResource->bIsInitialized &&
			   InstanceData.DeformedResource->IsInitialized() && InstanceData.DeformedResource->bIsInitialized)
			{
				return true;
			}
		}
		return false;
	}

	static bool HaveGuidesInstanceResources(const TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		for(TRefCountPtr<const FHairGroupInstance> GroupInstance : GroupInstances)
		{
			if(!GroupInstance || (GroupInstance && !HaveValidInstanceResources(GroupInstance->Guides)))
			{
				return false;
			}
		}
		return true;
	}

	FORCEINLINE bool HaveStrandsInstanceResources(const TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		for(TRefCountPtr<const FHairGroupInstance> GroupInstance : GroupInstances)
		{
			if(!GroupInstance || (GroupInstance && !HaveValidInstanceResources(GroupInstance->Strands)))
			{
				return false;
			}
		}
		return true;
	}

	template<typename DataType>
	static bool HaveValidSkinnedResources(const DataType& SkinnedData, const int32 LODIndex)
	{
		if(SkinnedData.HasValidRootData() && (SkinnedData.RestRootResource->GetRootCount() > 0) &&
		  (SkinnedData.RestRootResource->LODs.Num() == SkinnedData.DeformedRootResource->LODs.Num()))
		{
			if(SkinnedData.RestRootResource->LODs.IsValidIndex(LODIndex))
			{
				FHairStrandsLODRestRootResource* RestLODDatas = SkinnedData.RestRootResource->LODs[LODIndex];
				FHairStrandsLODDeformedRootResource* DeformedLODDatas = SkinnedData.DeformedRootResource->LODs[LODIndex];

				return (RestLODDatas && RestLODDatas->IsValid() && DeformedLODDatas && DeformedLODDatas->IsValid());
			}
		}
		return false;
	}

	static bool HaveGuidesSkinnedResources(const TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		for(TRefCountPtr<const FHairGroupInstance> GroupInstance : GroupInstances)
		{
			if(GroupInstance && GroupInstance->HairGroupPublicData)
			{
				if(GroupInstance->BindingType == EHairBindingType::Skinning)
				{
					if(!HaveValidSkinnedResources(GroupInstance->Guides, GroupInstance->HairGroupPublicData->MeshLODIndex ))
					{
						return false;
					}
				}
			}
			else
			{
				return false;
			}
		}
		return true;
	}

	FORCEINLINE bool HaveStrandsSkinnedResources(const TArray<TRefCountPtr<const FHairGroupInstance>>& GroupInstances)
	{
		for(TRefCountPtr<const FHairGroupInstance> GroupInstance : GroupInstances)
		{
			if(GroupInstance && GroupInstance->HairGroupPublicData)
			{
				if(GroupInstance->BindingType == EHairBindingType::Skinning)
				{
					if(!HaveValidSkinnedResources(GroupInstance->Strands, GroupInstance->HairGroupPublicData->MeshLODIndex ))
					{
						return false;
					}
				}
			}
			else
			{
				return false;
			}
		}
		return true;
	}

	static void GetGroomGroupElementOffsets(const UGroomComponent* GroomComponent, const FName DomainName, TArray<int32>& GroupOffsets, const bool bSourceElements = false)
	{
		if(GroomComponent) 
		{
			const uint32 NumGroups = GroomComponent->GetGroupCount();
			int32 MeshLOD = INDEX_NONE;
			
			GroupOffsets.Init(0, NumGroups+1);
			int32 NumElements = 0;
			for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
			{
				GroupOffsets[GroupIndex] = NumElements;
				
				if(DomainName == UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Objects ||
				   DomainName == UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Objects)
				{
					NumElements++;
				}
				if(const USkeletalMeshComponent* SkelMesh =  GetGroupSkelMesh(GroomComponent,GroupIndex, MeshLOD))
				{
					if (SkelMesh->MeshObject)
					{
						if(DomainName == UOptimusGroomAssetComponentSource::FMeshesExecutionDomains::Bones)
						{
							NumElements += SkelMesh->MeshObject->GetReferenceToLocalMatrices().Num();
						}
						else if(DomainName == UOptimusGroomAssetComponentSource::FMeshesExecutionDomains::Vertices)
						{
							NumElements += 0; // @todo : compute the total number of vertices to modify
						}
					}
				}
				if (const TObjectPtr<UGroomAsset> GroomAsset = GroomComponent->GroomAsset)
				{
					if (GroomAsset->GetHairGroupsPlatformData().IsValidIndex(GroupIndex))
					{
						const FHairGroupPlatformData& GroupData = GroomAsset->GetHairGroupsPlatformData()[GroupIndex];
						if(DomainName == UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Curves)
						{
							NumElements += bSourceElements ? GroupData.Strands.BulkData.GetNumSourceCurves() : GroupData.Strands.BulkData.GetNumCurves();
						}
						else if (DomainName == UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Edges)
						{
							NumElements += bSourceElements ? GroupData.Strands.BulkData.GetNumSourcePoints() - GroupData.Strands.BulkData.GetNumSourceCurves() :
									GroupData.Strands.BulkData.GetNumPoints() - GroupData.Strands.BulkData.GetNumCurves();
						}
						else if (DomainName == UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Points)
						{
							NumElements += bSourceElements ? GroupData.Strands.BulkData.GetNumSourcePoints() : GroupData.Strands.BulkData.GetNumPoints();
						}
						else if(DomainName == UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Curves)
						{
							NumElements += bSourceElements ? GroupData.Guides.BulkData.GetNumSourceCurves() : GroupData.Guides.BulkData.GetNumCurves();
						}
						else if (DomainName == UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Edges)
						{
							NumElements += bSourceElements ? GroupData.Guides.BulkData.GetNumSourcePoints() - GroupData.Guides.BulkData.GetNumSourceCurves() :
								GroupData.Guides.BulkData.GetNumPoints() - GroupData.Guides.BulkData.GetNumCurves();
						}
						else if (DomainName == UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points)
						{
							NumElements += bSourceElements ? GroupData.Guides.BulkData.GetNumSourcePoints() : GroupData.Guides.BulkData.GetNumPoints();
						}
					}
				}
			}
			GroupOffsets[NumGroups] = NumElements;
		}
	}

	static int32 GetGroomInvocationElementCounts(const TArray<const UGroomComponent*>& GroomComponents, const FName DomainName, TArray<int32>& InvocationCounts, const bool bSourceElements = false)
	{
		TArray<int32> GroupOffsets;
		int32 TotalCount = 0;
		InvocationCounts.Reset();
		for(const UGroomComponent* GroomComponent : GroomComponents)
		{
			if(GroomComponent)
			{
				GetGroomGroupElementOffsets(GroomComponent, DomainName, GroupOffsets, bSourceElements);

				const uint32 NumGroups = GroomComponent->GetGroupCount();
				for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
				{
					if(HasDeformationEnabledOrHasMeshDeformer(GroomComponent, GroupIndex) && GroomComponent->GetGroupInstance(GroupIndex))
					{
						InvocationCounts.Add(GroupOffsets[GroupIndex+1]-GroupOffsets[GroupIndex]);
						TotalCount += InvocationCounts.Last();
					}
				}
			}
		}
		return TotalCount;
	}

	static int32 FillCollisionOverlappingComponents(const TSet<TObjectPtr<UGroomComponent>>& GroomComponents,
		const TMap<TObjectPtr<UMeshComponent>, int32>& CollisionComponents, TArray<TPair<TObjectPtr<UMeshComponent>, int32>>& OverlappingComponents)
	{
		OverlappingComponents.Reset();
		for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : CollisionComponents)
		{
			if(CollisionComponent.Key)
			{
				const FBoxSphereBounds CollisionBounds = CollisionComponent.Key->GetLocalBounds().TransformBy(CollisionComponent.Key->GetComponentTransform());
				for(const UGroomComponent* GroomComponent : GroomComponents)
				{
					if(GroomComponent)
					{
						const uint32 NumGroups = GroomComponent->GetGroupCount();
						for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
						{
							if (TRefCountPtr<const FHairGroupInstance> GroupInstance = TRefCountPtr<const FHairGroupInstance>(GroomComponent->GetGroupInstance(GroupIndex)))
							{
								if(FBoxSphereBounds::BoxesIntersect(GroupInstance->GetBounds(), CollisionBounds))
								{
									OverlappingComponents.Add(CollisionComponent);
								}
							}
						}
					}
				}
			}
		}
		return OverlappingComponents.Num();
	}

	static int32 GetCollisionNumInvocations(const TArray<TPair<TObjectPtr<UMeshComponent>, int32>>& CollisionComponents)
	{
		int32 NumInvocations = 0;
		
		for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : CollisionComponents)
		{
			if(CollisionComponent.Key)
			{
				if(const USkinnedMeshComponent* SkinnedMeshComponent = Cast<const USkinnedMeshComponent>(CollisionComponent.Key))
				{
					if(SkinnedMeshComponent->MeshObject)
					{	
						const FSkeletalMeshRenderData& MeshRenderData = SkinnedMeshComponent->MeshObject->GetSkeletalMeshRenderData();
						{
							const int32 ValidLod = SkinnedMeshComponent->GetPredictedLODLevel();
							if(MeshRenderData.LODRenderData.IsValidIndex(ValidLod))
							{
								NumInvocations += MeshRenderData.LODRenderData[ValidLod].RenderSections.Num();
							}
						}
					}
				}
				else if(const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(CollisionComponent.Key))
				{
					if(StaticMeshComponent->GetStaticMesh())
					{
						if(const FStaticMeshRenderData* MeshRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData())
						{
							const int32 ValidLod = MeshRenderData->GetCurrentFirstLODIdx(0); 
							if(MeshRenderData->LODResources.IsValidIndex(ValidLod))
							{
								NumInvocations += MeshRenderData->LODResources[ValidLod].Sections.Num();
							}
						}
					}
				}
			}
		}
		return NumInvocations;		
	}
	
	static int32 GetCollisionInvocationElementCounts(const TArray<TPair<TObjectPtr<UMeshComponent>, int32>>& CollisionComponents, const FName DomainName, TArray<int32>& InvocationCounts)
	{
		InvocationCounts.Reset();
		int32 TotalCount = 0;
		for(const TPair<TObjectPtr<UMeshComponent>, int32>& CollisionComponent : CollisionComponents)
		{
			if(CollisionComponent.Key)
			{
				if(const USkinnedMeshComponent* SkinnedMeshComponent = Cast<const USkinnedMeshComponent>(CollisionComponent.Key))
				{
					if(SkinnedMeshComponent->MeshObject)
					{
						const FSkeletalMeshRenderData& MeshRenderData = SkinnedMeshComponent->MeshObject->GetSkeletalMeshRenderData();
						{
							const int32 ValidLod = SkinnedMeshComponent->GetPredictedLODLevel();
							if(MeshRenderData.LODRenderData.IsValidIndex(ValidLod))
							{
								const FSkeletalMeshLODRenderData& LodRenderData = MeshRenderData.LODRenderData[ValidLod];
								
								const int32 NumSections = LodRenderData.RenderSections.Num();
								for(int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
								{
									if(DomainName == UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Vertices)
									{
										InvocationCounts.Add(LodRenderData.RenderSections[SectionIndex].NumVertices);
										TotalCount += InvocationCounts.Last();
									}
									else if(DomainName == UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Triangles)
									{
										InvocationCounts.Add(LodRenderData.RenderSections[SectionIndex].NumTriangles);
										TotalCount += InvocationCounts.Last();
									}
								}
							}
						}
					}
				}
				else if(const UStaticMeshComponent* StaticMeshComponent = Cast<const UStaticMeshComponent>(CollisionComponent.Key))
				{
					if(StaticMeshComponent->GetStaticMesh())
					{
						if(const FStaticMeshRenderData* MeshRenderData = StaticMeshComponent->GetStaticMesh()->GetRenderData())
						{
							const int32 ValidLod = MeshRenderData->GetCurrentFirstLODIdx(0);
							if(MeshRenderData->LODResources.IsValidIndex(ValidLod))
							{
								const FStaticMeshLODResources& LodRenderData = MeshRenderData->LODResources[ValidLod];

								const int32 NumSections = LodRenderData.Sections.Num();
								for(int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
								{
									if(DomainName == UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Vertices)
									{
										InvocationCounts.Add(LodRenderData.Sections[SectionIndex].MaxVertexIndex - LodRenderData.Sections[SectionIndex].MinVertexIndex);
										TotalCount += InvocationCounts.Last();
									}
									else if(DomainName == UOptimusGroomCollisionComponentSource::FCollisionExecutionDomains::Triangles)
									{
										InvocationCounts.Add(LodRenderData.Sections[SectionIndex].NumTriangles);
										TotalCount += InvocationCounts.Last();
									}
								}
							}
						}
					}
				}
			}
		}
		return TotalCount;
	}

	struct FGroupElements
	{
		TArray<int32> GroupIndices;
		TArray<int32> GroupOffsets;
		TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;
	};
	
	FORCEINLINE void GetGroomInvocationElementGroups(const TArray<const UGroomComponent*>& GroomComponents, const FName DomainName, TArray<TPair<UGroomAsset*, FGroupElements>>& InvocationGroups,  const bool bSourceElements = false)
	{
		InvocationGroups.Reset();
		for(const UGroomComponent* GroomComponent : GroomComponents)
		{
			if(GroomComponent)
			{
				FGroupElements GroupData;
				GetGroomGroupElementOffsets(GroomComponent, DomainName, GroupData.GroupOffsets, bSourceElements);
				
				const uint32 NumGroups = GroomComponent->GetGroupCount();
				for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
				{ 
					if(HasDeformationEnabledOrHasMeshDeformer(GroomComponent, GroupIndex) && GroomComponent->GetGroupInstance(GroupIndex))
					{
						GroupData.GroupIndices.Add(GroupIndex);
						GroupData.GroupInstances.Add(GroomComponent->GetGroupInstance(GroupIndex));
					}
				}
				InvocationGroups.Add(MakeTuple(GroomComponent->GroomAsset, GroupData));
			}
			
		}
	}
}
