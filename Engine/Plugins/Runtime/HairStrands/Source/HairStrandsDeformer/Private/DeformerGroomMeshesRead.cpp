// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomMeshesRead.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "SystemTextures.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomMeshesRead)

FString UOptimusGroomMeshesReadDataInterface::GetDisplayName() const
{
	return TEXT("Groom Meshes");
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomMeshesReadDataInterface::GetPinDefinitions() const
{
	static const FName MeshesBones(UOptimusGroomAssetComponentSource::FMeshesExecutionDomains::Bones);

	TArray<FOptimusCDIPinDefinition> Defs;

	// Bones buffers
	Defs.Add({ "NumMeshesBones",   "ReadNumBones", false, "NumMeshesBones" });
	Defs.Add({ "BoneTransformMatrix", "ReadBoneTransformMatrix", MeshesBones,   "ReadNumBones", true, "BoneTransformMatrix"  });
	Defs.Add({ "BindTransformMatrix", "ReadBindTransformMatrix", MeshesBones,   "ReadNumBones", true, "BindTransformMatrix" });

	return Defs;
}
 
TSubclassOf<UActorComponent> UOptimusGroomMeshesReadDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomMeshesReadDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{}

void UOptimusGroomMeshesReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumBones"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneTransformMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBindTransformMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomMeshesReadParameters, )
	SHADER_PARAMETER(uint32, NumBones)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, BoneTransformMatrices)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomMeshesReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomMeshesReadParameters>(UID);
}

TCHAR const* UOptimusGroomMeshesReadDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomMeshesRead.ush");

TCHAR const* UOptimusGroomMeshesReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomMeshesReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomMeshesReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomMeshesReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomMeshesReadDataProvider* Provider = NewObject<UOptimusGroomMeshesReadDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomMeshesReadDataProvider::GetRenderProxy()
{
	return new FOptimusGroomMeshesReadDataProviderProxy(MeshComponent);
}

FOptimusGroomMeshesReadDataProviderProxy::FOptimusGroomMeshesReadDataProviderProxy(UMeshComponent* MeshComponent)
{
	UE::Groom::Private::GatherGroupSkelmeshes(MeshComponent, SkeletalMeshObjects, SkeletalMeshTransforms, BonesRefToLocals, BindTransforms, GroupInstances);
}

bool FOptimusGroomMeshesReadDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != SkeletalMeshObjects.Num())
	{
		return false;
	}
	
	return true;
}

static void CreateInternalBuffers(const FSkeletalMeshObject* SkeletalMeshObject, const FMatrix44f& SkeletalMeshTransform, const TArray<FMatrix44f>& BoneRefToLocals, const TArray<FMatrix44f>& BindTransforms,
	const FHairGroupInstance* GroupInstance, FRDGBuilder& GraphBuilder, TArray<FRDGBufferSRVRef>& AttributeResources)
{
	if(SkeletalMeshObject && GroupInstance)
	{
		const FString BufferName("Hair.Deformer.Strands.BoneTransformMatrices");
		if(FHairStrandsDeformedResource* DeformedResource = GroupInstance->Strands.DeformedResource)
		{
			FRDGBufferRef TransientBuffer = nullptr;
			FRDGExternalBuffer* ExternalBuffer = DeformedResource->ExternalBuffers.Find(BufferName);
			if(ExternalBuffer)
			{
				if(!GraphBuilder.FindExternalBuffer(ExternalBuffer->Buffer))
				{
					TransientBuffer = GraphBuilder.RegisterExternalBuffer(ExternalBuffer->Buffer);
				}
			}
			else
			{
				TransientBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(
					sizeof(FVector4f), BoneRefToLocals.Num() * 3 * 2), *BufferName);
				ExternalBuffer = &DeformedResource->ExternalBuffers.Add(BufferName);
				
				ExternalBuffer->Buffer = GraphBuilder.ConvertToExternalBuffer(TransientBuffer);
			}
			if(TransientBuffer)
			{
				const int32 NumBones = BoneRefToLocals.Num();
	
				TArray<FVector4f> BoneVectors;
				BoneVectors.Init(FVector4f::Zero(), NumBones * 3 * 2);

				// Bone transforms
				int32 VectorIndex = 0;
				for(int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					const FMatrix44f LocalMatrix = BoneRefToLocals[BoneIndex] * SkeletalMeshTransform;
					LocalMatrix.To3x4MatrixTranspose(static_cast<float*>(&BoneVectors[VectorIndex].X));
					 
					VectorIndex += 3;
				}

				// Bind transforms
				for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
				{
					const FMatrix44f LocalMatrix = BindTransforms[BoneIndex] * SkeletalMeshTransform;
					LocalMatrix.To3x4MatrixTranspose(static_cast<float*>(&BoneVectors[VectorIndex].X));

					VectorIndex += 3;
				}

				GraphBuilder.QueueBufferUpload(TransientBuffer, BoneVectors.GetData(),
					sizeof(FVector4f) * BoneVectors.Num(), ERDGInitialDataFlags::None);
			}
			AttributeResources.Add(RegisterAsSRV(GraphBuilder,*ExternalBuffer));
		}
	}
}

void FOptimusGroomMeshesReadDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	if (!FallbackStructuredSRV) { FallbackStructuredSRV  = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 16u)); }

	BoneMatricesResources.Reset();

	int32 SkelIndex = 0;
	for (const FSkeletalMeshObject* SkeletalMeshObject : SkeletalMeshObjects)
	{
		CreateInternalBuffers(SkeletalMeshObject, SkeletalMeshTransforms[SkelIndex], BonesRefToLocals[SkelIndex], BindTransforms[SkelIndex], GroupInstances[SkelIndex], GraphBuilder, BoneMatricesResources);
		++SkelIndex;
	}
}

void FOptimusGroomMeshesReadDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(InDispatchData.NumInvocations == SkeletalMeshObjects.Num());
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		if (const FSkeletalMeshObject* SkeletalMeshObject = SkeletalMeshObjects[InvocationIndex])
		{
			Parameters.NumBones = BonesRefToLocals[InvocationIndex].Num();
			Parameters.BoneTransformMatrices = BoneMatricesResources[InvocationIndex];
		}
		else
		{
			Parameters.NumBones = 0;
			Parameters.BoneTransformMatrices = FallbackStructuredSRV;
		}
	}
}
