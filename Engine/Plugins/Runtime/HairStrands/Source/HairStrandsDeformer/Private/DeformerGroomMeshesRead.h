// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "OptimusComputeDataInterface.h"
#include "HairStrandsInterface.h"
#include "IOptimusDeformerInstanceAccessor.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "DeformerGroomMeshesRead.generated.h"

class FOptimusGroomMeshesReadParameters;
class UGroomComponent;
struct FHairGroupInstance;
class UGroomAsset;
class UMeshComponent;
class FSkeletalMeshObject;

/** Compute Framework Data Interface for reading groom meshes. */
UCLASS(Category = ComputeFramework)
class UOptimusGroomMeshesReadDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("GroomMeshesRead"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	/** File holding the hlsl implementation */
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading groom meshes. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGroomMeshesReadDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMeshComponent> MeshComponent = nullptr;

	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusGroomMeshesReadDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGroomMeshesReadDataProviderProxy(UMeshComponent* MeshComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FOptimusGroomMeshesReadParameters;
	
	/** Skeletal mesh objects to extract the bones from */
	TArray<const FSkeletalMeshObject*> SkeletalMeshObjects;

	/** Skeletal mesh transforms in group space */
	TArray<FMatrix44f> SkeletalMeshTransforms;

	/** Bones refs to locals matrices */
	TArray<TArray<FMatrix44f>> BonesRefToLocals;

	/** Bind Transforms */
	TArray<TArray<FMatrix44f>> BindTransforms;

	/** List of instances (invocations) used in that data interface */
	TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;
	
	/** Bones Resources used to dispatch CS on GPU */
	TArray<FRDGBufferSRVRef> BoneMatricesResources;

	/** Fallback resources */
	FRDGBufferSRVRef FallbackStructuredSRV = nullptr;
};
