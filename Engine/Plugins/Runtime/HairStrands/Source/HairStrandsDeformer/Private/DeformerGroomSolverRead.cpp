// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomSolverRead.h"

#include "DeformerGroomDomainsExec.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "GroomSolverComponent.h"
#include "GroomInstance.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "HairStrandsInterpolation.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "OptimusValueContainerStruct.h"
#include "ScenePrivate.h"
#include "SystemTextures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomSolverRead)

FString UOptimusGroomSolverReadDataInterface::GetDisplayName() const
{
	return TEXT("Groom Solver");
}

FName UOptimusGroomSolverReadDataInterface::GetResetSimulationTriggerName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusGroomSolverReadDataInterface, ResetSimulationTrigger);
}

TArray<FOptimusCDIPropertyPinDefinition> UOptimusGroomSolverReadDataInterface::GetPropertyPinDefinitions() const
{
	TArray<FOptimusCDIPropertyPinDefinition> PropertyPinDefinitions;

	const FOptimusDataTypeHandle NameType = FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass());
	
	PropertyPinDefinitions.Add(
		{GetResetSimulationTriggerName(), NameType}
	);

	return PropertyPinDefinitions;
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomSolverReadDataInterface::GetPinDefinitions() const
{
	static const FName SolverObjects(UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Objects);
	static const FName SolverPoints(UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Points);
	static const FName SolverCurves(UOptimusGroomSolverComponentSource::FSolverExecutionDomains::Curves);

	static const FName DynamicPoints(UOptimusGroomSolverComponentSource::FDynamicExecutionDomains::Points);
	static const FName DynamicCurves(UOptimusGroomSolverComponentSource::FDynamicExecutionDomains::Curves);
	static const FName KinematicPoints(UOptimusGroomSolverComponentSource::FKinematicExecutionDomains::Points);
	static const FName KinematicCurves(UOptimusGroomSolverComponentSource::FKinematicExecutionDomains::Curves);

	TArray<FOptimusCDIPinDefinition> Defs;

	Defs.Add({ "NumSolverPoints",  "ReadNumSolverPoints", true, "NumSolverPoints" });
	Defs.Add({ "NumSolverCurves",  "ReadNumSolverCurves", true, "NumSolverCurves" });
	Defs.Add({ "NumSolverObjects",  "ReadNumSolverObjects", true, "NumSolverObjects" });
	Defs.Add({ "NumDynamicPoints",  "ReadNumDynamicPoints", true, "NumDynamicPoints" });
	Defs.Add({ "NumDynamicCurves",  "ReadNumDynamicCurves", true, "NumDynamicCurves" });
	Defs.Add({ "NumKinematicPoints",  "ReadNumKinematicPoints", true, "NumKinematicPoints" });
	Defs.Add({ "NumKinematicCurves",  "ReadNumKinematicCurves", true, "NumKinematicCurves" });
	Defs.Add({ "BoundingBoxMin",  "ReadBoundingBoxMin", true, "BoundingBoxMin" });
	Defs.Add({ "BoundingBoxMax",  "ReadBoundingBoxMax", true, "BoundingBoxMax" });
	Defs.Add({ "ObjectPointOffset", "ReadObjectPointOffset", SolverObjects,          "ReadNumSolverObjects", true, "ObjectPointOffset"  });
	Defs.Add({ "ObjectNumPoints", "ReadObjectNumPoints", SolverObjects,          "ReadNumSolverObjects", true, "ObjectNumPoints"  });
	Defs.Add({ "ObjectCurveOffset", "ReadObjectCurveOffset", SolverObjects,          "ReadNumSolverObjects", true, "ObjectCurveOffset"  });
	Defs.Add({ "ObjectNumCurves", "ReadObjectNumCurves", SolverObjects,          "ReadNumSolverObjects", true, "ObjectNumCurves"  });
	Defs.Add({ "PointObjectIndex", "ReadPointObjectIndex", SolverPoints,          "ReadNumSolverPoints", true, "PointObjectIndex"  });
	Defs.Add({ "CurveObjectIndex", "ReadCurveObjectIndex", SolverCurves,          "ReadNumSolverCurves", true, "CurveObjectIndex"  });
	Defs.Add({ "DynamicPointIndex", "ReadDynamicPointIndex", DynamicPoints,          "ReadNumDynamicPoints", true, "DynamicPointIndex"  });
	Defs.Add({ "DynamicCurveIndex", "ReadDynamicCurveIndex", DynamicCurves,          "ReadNumDynamicCurves", true, "DynamicCurveIndex"  });
	Defs.Add({ "KinematicPointIndex", "ReadKinematicPointIndex", KinematicPoints,          "ReadNumKinematicPoints", true, "KinematicPointIndex"  });
	Defs.Add({ "KinematicCurveIndex", "ReadKinematicCurveIndex", KinematicCurves,          "ReadNumKinematicCurves", true, "KinematicCurveIndex"  });
	Defs.Add({ "ObjectDistanceLod", "ReadObjectDistanceLod", SolverObjects,          "ReadObjectDistanceLod", true, "ObjectDistanceLod"  });

#ifdef RAYTRACING_SOLVER_BINDING
	Defs.Add({ "WorldAccelerationStructure", "ReadWorldAccelerationStructure", false, "WorldAccelerationStructure"  });
#endif

	return Defs;
}

void UOptimusGroomSolverReadDataInterface::RegisterTypes() 
{
#ifdef RAYTRACING_SOLVER_BINDING
	FOptimusDataTypeRegistry::Get().RegisterType(  
		FName("FAccelerationStructure"),
		FText::FromString(TEXT("FAccelerationStructure")),
		FShaderValueType::Get(FName("FAccelerationStructure"), { FShaderValueType::FStructElement(FName("ValidTLAS"), FShaderValueType::Get(EShaderFundamentalType::Int)) }),
		FName("FAccelerationStructure"),
		nullptr,
		FLinearColor(0.3f, 0.7f, 0.4f, 1.0f),
		EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);
#endif
}
 
TSubclassOf<UActorComponent> UOptimusGroomSolverReadDataInterface::GetRequiredComponentClass() const
{
	return UGroomSolverComponent::StaticClass();
}

void UOptimusGroomSolverReadDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
}

void UOptimusGroomSolverReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumSolverPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumSolverCurves"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumSolverObjects"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumDynamicPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumDynamicCurves"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumKinematicPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumKinematicCurves"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoundingBoxMin"))
		.AddReturnType(EShaderFundamentalType::Float,3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoundingBoxMax"))
		.AddReturnType(EShaderFundamentalType::Float,3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectPointOffset"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
		
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectCurveOffset"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
		
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectNumCurves"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointObjectIndex"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);
		
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveObjectIndex"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDynamicPointIndex"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDynamicCurveIndex"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadKinematicPointIndex"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadKinematicCurveIndex"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadObjectDistanceLod"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

#ifdef RAYTRACING_SOLVER_BINDING
	if(FOptimusDataTypeHandle DataType = FOptimusDataTypeRegistry::Get().FindType(FName("FAccelerationStructure")))
	{
		FShaderValueTypeHandle AccelerationStructureType = DataType->ShaderValueType;

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("ReadWorldAccelerationStructure"))
			.AddReturnType(AccelerationStructureType);
	}

#endif
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomSolverReadParameters, )
	SHADER_PARAMETER(uint32, NumSolverPoints)
	SHADER_PARAMETER(uint32, NumSolverCurves)
	SHADER_PARAMETER(uint32, NumSolverObjects)
	SHADER_PARAMETER(uint32, NumDynamicPoints)
	SHADER_PARAMETER(uint32, NumDynamicCurves)
	SHADER_PARAMETER(uint32, NumKinematicPoints)
	SHADER_PARAMETER(uint32, NumKinematicCurves)
	SHADER_PARAMETER(FVector3f, BoundingBoxMin)
	SHADER_PARAMETER(FVector3f, BoundingBoxMax)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectPointOffsets)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectCurveOffsets)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectNumPoints)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectNumCurves)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, PointObjectIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, CurveObjectIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, DynamicPointIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, DynamicCurveIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, KinematicPointIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, KinematicCurveIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectDistanceLods)
#ifdef RAYTRACING_SOLVER_BINDING
	SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, WorldAccelerationStructure)
#endif
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomSolverReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomSolverReadParameters>(UID);
}

TCHAR const* UOptimusGroomSolverReadDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomSolverRead.ush");

TCHAR const* UOptimusGroomSolverReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomSolverReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomSolverReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomSolverReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomSolverReadDataProvider* Provider = NewObject<UOptimusGroomSolverReadDataProvider>();
	Provider->SolverComponent = Cast<UGroomSolverComponent>(InBinding);
	Provider->WeakDataInterface = this;
	return Provider;
}

void UOptimusGroomSolverReadDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	DeformerInstance = InInstance;
}

FComputeDataProviderRenderProxy* UOptimusGroomSolverReadDataProvider::GetRenderProxy()
{
	static constexpr int32 MaxResetCount = 4;
	bool bResetSimulationTrigger = false;
	if (const UOptimusGroomSolverReadDataInterface* DataInterface = WeakDataInterface.Get())
	{
		FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride(
						DataInterface,
						UOptimusGroomSolverReadDataInterface::GetResetSimulationTriggerName()
						);

		TValueOrError<FName, EPropertyBagResult> Value = ValueContainer.Value.GetValueName(FOptimusValueContainerStruct::ValuePropertyName);
		if (Value.HasValue())
		{
			const FName& ResetSimulationGraphName = Value.GetValue();

			TArray<const UGroomComponent*> GroomComponents;
			UE::Groom::Private::GatherGroomComponents(SolverComponent, GroomComponents);

			for(const UGroomComponent* GroomComponent : GroomComponents)
			{
				if(GroomComponent->bResetSimulation)
				{
					bResetSimulationCount = 0;
					break;
				}
			}
			if(bResetSimulationCount > 0)
			{
				TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;
				UE::Groom::Private::GroomComponentsToInstances(GroomComponents, GroupInstances);
				
				if(!UE::Groom::Private::HaveGuidesInstanceResources(GroupInstances) || !UE::Groom::Private::HaveGuidesSkinnedResources(GroupInstances))
				{
					bResetSimulationCount = 0;
				}
			} 
			bResetSimulationTrigger = (bResetSimulationCount < MaxResetCount);
			if(bResetSimulationTrigger)
			{
				DeformerInstance->EnqueueTriggerGraph(ResetSimulationGraphName);
				bResetSimulationCount++;
			}
		}
	}
	return new FOptimusGroomSolverReadDataProviderProxy(SolverComponent, bResetSimulationTrigger);
}

FOptimusGroomSolverReadDataProviderProxy::FOptimusGroomSolverReadDataProviderProxy(UGroomSolverComponent* SolverComponent, const bool bResetSimulation)
{
	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroomComponents(SolverComponent, GroomComponents);

	UE::Groom::Private::GroomComponentsToInstances(GroomComponents, GroupInstances);
	
	UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents,
		UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Points, PointsCounts);

	UE::Groom::Private::GetGroomInvocationElementCounts(GroomComponents,
		UOptimusGroomAssetComponentSource::FGuidesExecutionDomains::Curves, CurvesCounts);
	
	SolverSettings = SolverComponent->GetSolverSettings();
	bResetSimulationTrigger = bResetSimulation;

#ifdef RAYTRACING_SOLVER_BINDING
	SceneInterface = SolverComponent->GetScene();
#endif
}

bool FOptimusGroomSolverReadDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if(!UE::Groom::Private::HaveGuidesInstanceResources(GroupInstances) || !UE::Groom::Private::HaveGuidesSkinnedResources(GroupInstances))
	{
		return false;
	}
	if(bResetSimulationTrigger)
	{
		return false;
	}
	
	return true;
}

void FOptimusGroomSolverReadDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
}

FORCEINLINE void FillElementsData(const TArray<int32>& ElementsCounts, 
	TArray<uint32>& ObjectOffsets, TArray<uint32>& ObjectCounts, TArray<int32>& ElementObjects, int32& NumElements)
{
	const uint32 NumObjects = ElementsCounts.Num();
	
	ObjectOffsets.Init(0, NumObjects+1);
	ObjectCounts.Init(0, NumObjects);
	uint32 ElementOffset = 0;
	for (uint32 ObjectIndex = 0; ObjectIndex < NumObjects; ++ObjectIndex)
	{
		ObjectCounts[ObjectIndex] = static_cast<uint32>(ElementsCounts[ObjectIndex]);
		ObjectOffsets[ObjectIndex] = ElementOffset;
		ElementOffset += ElementsCounts[ObjectIndex];
	}
	
	ObjectOffsets[NumObjects] = ElementOffset;
	NumElements = ElementOffset;

	ElementObjects.Init(INDEX_NONE, NumElements);
	for (uint32 ObjectIndex = 0; ObjectIndex < NumObjects; ++ObjectIndex)
	{
		for(uint32 ElementIndex = ObjectOffsets[ObjectIndex],
			ElementEnd = ObjectOffsets[ObjectIndex] + ObjectCounts[ObjectIndex]; ElementIndex < ElementEnd; ++ElementIndex)
		{
			ElementObjects[ElementIndex] = ObjectIndex;
		}
	}
}

void FOptimusGroomSolverReadDataProviderProxy::CreateInternalBuffers(FRDGBuilder& GraphBuilder)
{
	TArray<uint32> ObjectPointOffsets;
	TArray<uint32> ObjectNumPoints;
	TArray<int32> PointObjectIndices;

	FillElementsData(PointsCounts, ObjectPointOffsets,  ObjectNumPoints,
		PointObjectIndices, NumPoints);

	FString BufferName = FString("Hair.Deformer.ObjectPointOffsets");
	ObjectPointOffsetsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ObjectPointOffsets.Num()), *BufferName);
	ObjectPointOffsetsResource = GraphBuilder.CreateSRV(ObjectPointOffsetsBuffer);
	GraphBuilder.QueueBufferUpload(ObjectPointOffsetsBuffer, ObjectPointOffsets.GetData(), sizeof(uint32) * ObjectPointOffsets.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.ObjectNumPoints");
	ObjectNumPointsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ObjectNumPoints.Num()), *BufferName);
	ObjectNumPointsResource = GraphBuilder.CreateSRV(ObjectNumPointsBuffer);
	GraphBuilder.QueueBufferUpload(ObjectNumPointsBuffer, ObjectNumPoints.GetData(), sizeof(uint32) * ObjectNumPoints.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.PointObjectIndices");
	PointObjectIndicesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), PointObjectIndices.Num()), *BufferName);
	PointObjectIndicesResource = GraphBuilder.CreateSRV(PointObjectIndicesBuffer);
	GraphBuilder.QueueBufferUpload(PointObjectIndicesBuffer, PointObjectIndices.GetData(), sizeof(int32) * PointObjectIndices.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.DynamicPointIndices");
	DynamicPointIndicesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), SolverSettings.PointDynamicIndices.Num()), *BufferName);
	DynamicPointIndicesResource = GraphBuilder.CreateSRV(DynamicPointIndicesBuffer);
	GraphBuilder.QueueBufferUpload(DynamicPointIndicesBuffer, SolverSettings.PointDynamicIndices.GetData(), sizeof(int32) * SolverSettings.PointDynamicIndices.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.KinematicPointIndices");
	KinematicPointIndicesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), SolverSettings.PointKinematicIndices.Num()), *BufferName);
	KinematicPointIndicesResource = GraphBuilder.CreateSRV(KinematicPointIndicesBuffer);
	GraphBuilder.QueueBufferUpload(KinematicPointIndicesBuffer, SolverSettings.PointKinematicIndices.GetData(), sizeof(int32) * SolverSettings.PointKinematicIndices.Num(), ERDGInitialDataFlags::None);
	
	TArray<uint32> ObjectCurveOffsets;
	TArray<uint32> ObjectNumCurves;
	TArray<int32> CurveObjectIndices;
	
	FillElementsData(CurvesCounts, ObjectCurveOffsets,  ObjectNumCurves,
	CurveObjectIndices, NumCurves);

	BufferName = FString("Hair.Deformer.ObjectCurveOffsets");
	ObjectCurveOffsetsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ObjectCurveOffsets.Num()), *BufferName);
	ObjectCurveOffsetsResource = GraphBuilder.CreateSRV(ObjectCurveOffsetsBuffer);
	GraphBuilder.QueueBufferUpload(ObjectCurveOffsetsBuffer, ObjectCurveOffsets.GetData(), sizeof(uint32) * ObjectCurveOffsets.Num(), ERDGInitialDataFlags::None);
	
	BufferName = FString("Hair.Deformer.ObjectNumCurves");
	ObjectNumCurvesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ObjectNumCurves.Num()), *BufferName);
	ObjectNumCurvesResource = GraphBuilder.CreateSRV(ObjectNumCurvesBuffer);
	GraphBuilder.QueueBufferUpload(ObjectNumCurvesBuffer, ObjectNumCurves.GetData(), sizeof(uint32) * ObjectNumCurves.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.CurveObjectIndices");
	CurveObjectIndicesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), CurveObjectIndices.Num()), *BufferName);
	CurveObjectIndicesResource = GraphBuilder.CreateSRV(CurveObjectIndicesBuffer);
	GraphBuilder.QueueBufferUpload(CurveObjectIndicesBuffer, CurveObjectIndices.GetData(), sizeof(int32) * CurveObjectIndices.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.DynamicCurveIndices");
	DynamicCurveIndicesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), SolverSettings.CurveDynamicIndices.Num()), *BufferName);
	DynamicCurveIndicesResource = GraphBuilder.CreateSRV(DynamicCurveIndicesBuffer);
	GraphBuilder.QueueBufferUpload(DynamicCurveIndicesBuffer, SolverSettings.CurveDynamicIndices.GetData(), sizeof(int32) * SolverSettings.CurveDynamicIndices.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.KinematicCurveIndices");
	KinematicCurveIndicesBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), SolverSettings.CurveKinematicIndices.Num()), *BufferName);
	KinematicCurveIndicesResource = GraphBuilder.CreateSRV(KinematicCurveIndicesBuffer);
	GraphBuilder.QueueBufferUpload(KinematicCurveIndicesBuffer, SolverSettings.CurveKinematicIndices.GetData(), sizeof(int32) * SolverSettings.CurveKinematicIndices.Num(), ERDGInitialDataFlags::None);

	BufferName = FString("Hair.Deformer.ObjectDistanceLods");
	ObjectDistanceLodsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), SolverSettings.ObjectDistanceLods.Num()), *BufferName);
	ObjectDistanceLodsResource = GraphBuilder.CreateSRV(ObjectDistanceLodsBuffer);
	GraphBuilder.QueueBufferUpload(ObjectDistanceLodsBuffer, SolverSettings.ObjectDistanceLods.GetData(), sizeof(uint32) * SolverSettings.ObjectDistanceLods.Num(), ERDGInitialDataFlags::None);

}

void FOptimusGroomSolverReadDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	CreateInternalBuffers(GraphBuilder);

#ifdef RAYTRACING_SOLVER_BINDING
	if(SceneInterface)
	{
		if (const FScene* RenderScene = SceneInterface->GetRenderScene())
		{
			AccelerationStructure = RenderScene->RayTracingScene.CreateLayerViewRHI(GraphBuilder.RHICmdList, ERayTracingSceneLayer::Base);
		}
	}
#endif
}

void FOptimusGroomSolverReadDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{ 
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	check(!ParameterArray.IsEmpty());

	FBox BoundingBox(ForceInitToZero);
	for(const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		BoundingBox += GroupInstance->GetBounds().GetBox();
	}

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];

		Parameters.NumSolverPoints = NumPoints;
		Parameters.NumSolverCurves = NumCurves;
		Parameters.NumSolverObjects = GroupInstances.Num();

		Parameters.BoundingBoxMin = FVector3f(BoundingBox.Min);
		Parameters.BoundingBoxMax = FVector3f(BoundingBox.Max);

		Parameters.NumDynamicPoints = SolverSettings.PointDynamicIndices.Num();
		Parameters.NumDynamicCurves = SolverSettings.CurveDynamicIndices.Num();
		Parameters.NumKinematicPoints = SolverSettings.PointKinematicIndices.Num();
		Parameters.NumKinematicCurves = SolverSettings.CurveKinematicIndices.Num();

		Parameters.ObjectPointOffsets = ObjectPointOffsetsResource;
		Parameters.ObjectCurveOffsets = ObjectCurveOffsetsResource;

		Parameters.ObjectNumPoints = ObjectNumPointsResource;
		Parameters.ObjectNumCurves = ObjectNumCurvesResource;

		Parameters.PointObjectIndices = PointObjectIndicesResource;
		Parameters.CurveObjectIndices = CurveObjectIndicesResource;

		Parameters.DynamicPointIndices = DynamicPointIndicesResource;
		Parameters.DynamicCurveIndices = DynamicCurveIndicesResource;

		Parameters.KinematicPointIndices = KinematicPointIndicesResource;
		Parameters.KinematicCurveIndices = KinematicCurveIndicesResource;

		Parameters.ObjectDistanceLods = ObjectDistanceLodsResource;

#ifdef RAYTRACING_SOLVER_BINDING
		Parameters.WorldAccelerationStructure = AccelerationStructure;
#endif
	}
}

