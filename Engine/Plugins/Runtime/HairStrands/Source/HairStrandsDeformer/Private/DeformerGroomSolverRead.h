// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GroomSolverComponent.h"
#include "RenderGraphDefinitions.h"
#include "OptimusComputeDataInterface.h"
#include "HairStrandsInterface.h"
#include "IOptimusDeformerInstanceAccessor.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "DeformerGroomSolverRead.generated.h"

namespace UE::Groom::Private
{
	struct FGroupElements;
}

class FOptimusGroomSolverReadParameters;
class UGroomComponent;
struct FHairGroupInstance;
class UGroomAsset;

/** Compute Framework Data Interface for reading groom guides. */
UCLASS(Category = ComputeFramework)
class UOptimusGroomSolverReadDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:

	static FName GetResetSimulationTriggerName();
	
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	virtual TArray<FOptimusCDIPropertyPinDefinition> GetPropertyPinDefinitions() const override;
	virtual void RegisterTypes() override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("GroomSolverRead"); }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	virtual bool CanSupportUnifiedDispatch() const override { return true; }
	//~ End UComputeDataInterface Interface

private:
	/** File holding the hlsl implementation */
	static TCHAR const* TemplateFilePath;

	UPROPERTY(EditAnywhere, Category = "Simulation Trigger")
	FName ResetSimulationTrigger = NAME_None;
};

/** Compute Framework Data Provider for reading groom guides. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGroomSolverReadDataProvider : public UComputeDataProvider, public IOptimusDeformerInstanceAccessor
{
	GENERATED_BODY()

public:

	//~ Begin UComputeDataProvider Interface
	virtual void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;
	//~ End UComputeDataProvider Interface
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UGroomSolverComponent> SolverComponent = nullptr;

	//~ Begin UComputeDataProvider Interface
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	/** Deformer instance using this provider */
	UPROPERTY()
	TObjectPtr<UOptimusDeformerInstance> DeformerInstance = nullptr;
	
	/** Data interface from which this provider has been created */
	TWeakObjectPtr<const UOptimusGroomSolverReadDataInterface> WeakDataInterface;

	/** Reset simulation count */
	int32 bResetSimulationCount = 0;
};

class FOptimusGroomSolverReadDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGroomSolverReadDataProviderProxy(UGroomSolverComponent* SolverComponent, const bool bResetSimulation);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	virtual void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FOptimusGroomSolverReadParameters;

	/** Create internal buffers */
	void CreateInternalBuffers(FRDGBuilder& GraphBuilder);

	/** Solver settings used in the solver */
	FGroomSolverSettings SolverSettings;

	/** Groom solver buffers built from the deformer */
	FRDGBufferRef ObjectPointOffsetsBuffer = nullptr;
	FRDGBufferRef ObjectCurveOffsetsBuffer = nullptr;
	FRDGBufferRef ObjectNumPointsBuffer = nullptr;
	FRDGBufferRef ObjectNumCurvesBuffer = nullptr;
	FRDGBufferRef PointObjectIndicesBuffer = nullptr;
	FRDGBufferRef CurveObjectIndicesBuffer = nullptr;
	FRDGBufferRef DynamicPointIndicesBuffer = nullptr;
	FRDGBufferRef DynamicCurveIndicesBuffer = nullptr;
	FRDGBufferRef KinematicPointIndicesBuffer = nullptr;
	FRDGBufferRef KinematicCurveIndicesBuffer = nullptr;
	FRDGBufferRef ObjectDistanceLodsBuffer = nullptr;

	/** Invocation points count */
	TArray<int32> PointsCounts;

	/** Invocation curves count */
	TArray<int32> CurvesCounts;

	/** Total number of points */
	int32 NumPoints = 0;

	/** Total number of curves */
	int32 NumCurves = 0;

	/** Groom solver resources built from the deformer */
	FRDGBufferSRVRef ObjectPointOffsetsResource = nullptr;
	FRDGBufferSRVRef ObjectCurveOffsetsResource = nullptr;
	FRDGBufferSRVRef ObjectNumPointsResource = nullptr;
	FRDGBufferSRVRef ObjectNumCurvesResource = nullptr;
	FRDGBufferSRVRef PointObjectIndicesResource = nullptr;
	FRDGBufferSRVRef CurveObjectIndicesResource = nullptr;
	FRDGBufferSRVRef DynamicPointIndicesResource = nullptr;
	FRDGBufferSRVRef DynamicCurveIndicesResource = nullptr;
	FRDGBufferSRVRef KinematicPointIndicesResource = nullptr;
	FRDGBufferSRVRef KinematicCurveIndicesResource = nullptr;
	FRDGBufferSRVRef ObjectDistanceLodsResource = nullptr;

	/** List of instances (invocations) used in that data interface */
	TArray<TRefCountPtr<const FHairGroupInstance>> GroupInstances;

	/** Reset simulation trigger */
	bool bResetSimulationTrigger = false;

#ifdef RAYTRACING_SOLVER_BINDING
	
	/** Scene interface used to retrieve the TLAS */
	FSceneInterface* SceneInterface;

	/** Acceleration structure used for geometric collisions */
	FRHIShaderResourceView* AccelerationStructure = nullptr;

#endif
};
