// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomStrandsRead.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "GroomInstance.h"
#include "RenderGraphBuilder.h"
#include "GlobalRenderResources.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "HairStrandsInterpolation.h"
#include "SystemTextures.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "GroomVisualizationData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomStrandsRead)

FString UOptimusGroomStrandsReadDataInterface::GetDisplayName() const
{
	return TEXT("Groom Strands");
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomStrandsReadDataInterface::GetPinDefinitions() const
{
	static const FName StrandsPoints(UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Points);
	static const FName StrandsCurves(UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Curves);

	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "NumControlPoints", "ReadNumPoints", false, {"NumStrandsPoints"} });
	Defs.Add({ "NumCurves", "ReadNumCurves", false, {"NumStrandsCurves"} });

	// Deformation buffers
	Defs.Add({ "Position", "ReadPointRestPosition", StrandsPoints, "ReadNumPoints", false, "PointRestPosition" });
	Defs.Add({ "CurveOffsetPoint", "ReadCurvePointOffset", StrandsCurves,          "ReadNumCurves", false, "CurvePointOffset"  });
	Defs.Add({ "CurveNumPoint", "ReadCurveNumPoints", StrandsCurves,          "ReadNumCurves", false, "CurveNumPoints"  });
	Defs.Add({ "PointCurveIndex", "ReadPointCurveIndex", StrandsPoints,          "ReadNumPoints", false, "PointCurveIndex"  });
	Defs.Add({ "CurveRestTransform", "ReadCurveRestTransform", StrandsCurves,   "ReadNumCurves", false, "CurveRestTransform"  });
	Defs.Add({ "CurveDeformedTransform", "ReadCurveDeformedTransform", StrandsCurves,          "ReadNumCurves", true, "CurveDeformedTransform"  });
	Defs.Add({ "CurveSourceIndex", "ReadCurveSourceIndex", StrandsCurves,          "ReadNumCurves", true, "CurveSourceIndex"  });
	Defs.Add({ "PointSourceIndex", "ReadPointSourceIndex", StrandsPoints,          "ReadNumPoints", true, "PointSourceIndex"  });

	// Interpolation buffers
	Defs.Add({ "GuideIndex", "ReadPointGuideIndices", StrandsPoints,   "ReadNumPoints", false, "PointGuideIndices"  });
	Defs.Add({ "PointGuideWeights", "ReadPointGuideWeights", StrandsPoints,   "ReadNumPoints", false, "PointGuideWeights"  });
	
	// Geometry buffers
	Defs.Add({ "CoordU", "ReadPointCoordU", StrandsPoints,   "ReadNumPoints", false, "PointCurveCoordU"  });
	Defs.Add({ "Length", "ReadPointLength", StrandsPoints,   "ReadNumPoints", false, "PointCurveLength"  });
	
	// Material buffers
	Defs.Add({ "Radius", "ReadPointRadius", StrandsPoints, "ReadNumPoints", true, "PointMaterialRadius" });
	Defs.Add({ "RootUV", "ReadPointRootUV", StrandsPoints, "ReadNumPoints", true, "PointMaterialRootUV" });
	Defs.Add({ "Seed", "ReadPointSeed", StrandsPoints,   "ReadNumPoints", true, "PointMaterialSeed"  });
	Defs.Add({ "ClumpId", "ReadPointClumpId", StrandsPoints,   "ReadNumPoints", true, "PointMaterialClumpId"  });
	Defs.Add({ "Color", "ReadPointColor", StrandsPoints,   "ReadNumPoints", true, "PointMaterialColor"  });
	Defs.Add({ "Roughness", "ReadPointRoughness", StrandsPoints,   "ReadNumPoints", true, "PointMaterialRoughness"  });
	Defs.Add({ "AO", "ReadPointAO", StrandsPoints,   "ReadNumPoints", true, "PointMaterialAO"  });
	
	return Defs;
}
 
TSubclassOf<UActorComponent> UOptimusGroomStrandsReadDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomStrandsReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumCurves"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointRestPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointRadius"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointCoordU"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointLength"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointRootUV"))
		.AddReturnType(EShaderFundamentalType::Float, 2)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointSeed"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointClumpId"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointColor"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointRoughness"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointAO"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurvePointOffset"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointGuideIndices"))
		.AddReturnType(EShaderFundamentalType::Int, 2)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointGuideWeights"))
		.AddReturnType(EShaderFundamentalType::Float, 2)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointCurveIndex"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveRestTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveDeformedTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCurveSourceIndex"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPointSourceIndex"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
	
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomStrandsReadParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCommonParameters, Common)
	SHADER_PARAMETER(uint32, BasePointIndex)
	SHADER_PARAMETER(uint32, BaseCurveIndex)
	SHADER_PARAMETER(uint32, BaseGuidePointIndex)
	SHADER_PARAMETER(uint32, TotalPointCount)
	SHADER_PARAMETER(uint32, TotalCurveCount)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceResourceParameters, Resources)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceInterpolationParameters, Interpolation)
	SHADER_PARAMETER(FMatrix44f, RigidRestTransform)
	SHADER_PARAMETER(FMatrix44f, RigidDeformedTransform)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TriangleRestPositions)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TriangleDeformedPositions)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CurveBarycentricCoordinates)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CurveTriangleIndices)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomStrandsReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomStrandsReadParameters>(UID);
}

TCHAR const* UOptimusGroomStrandsReadDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomStrandsRead.ush");

TCHAR const* UOptimusGroomStrandsReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomStrandsReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomStrandsReadDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("ENABLE_SKINNED_TRANSFORM"), 2);
}

void UOptimusGroomStrandsReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomStrandsReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomStrandsReadDataProvider* Provider = NewObject<UOptimusGroomStrandsReadDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomStrandsReadDataProvider::GetRenderProxy()
{
	return new FOptimusGroomStrandsReadProviderProxy(MeshComponent);
}

FOptimusGroomStrandsReadProviderProxy::FOptimusGroomStrandsReadProviderProxy(UMeshComponent* MeshComponent)
{
	UE::Groom::Private::GatherGroupInstances(MeshComponent, GroupInstances);
}

bool FOptimusGroomStrandsReadProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != GroupInstances.Num())
	{
		return false;
	}
	if(!UE::Groom::Private::HaveStrandsInstanceResources(GroupInstances) || !UE::Groom::Private::HaveStrandsSkinnedResources(GroupInstances))
	{
		return false;
	}
	
	return true;
}

struct FOptimusGroomStrandsReadPermutationIds
{
	uint32 EnableSkinnedTransform = 0;

	FOptimusGroomStrandsReadPermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		static FString Name(TEXT("ENABLE_SKINNED_TRANSFORM"));
		static uint32 Hash = GetTypeHash(Name);
		EnableSkinnedTransform = PermutationVector.GetPermutationBits(Name, Hash, 1);
	}
};

void FOptimusGroomStrandsReadProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	const FOptimusGroomStrandsReadPermutationIds PermutationIds(InOutPermutationData.PermutationVector);

	int32 InvocationIndex = 0;
	for (const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		if (GroupInstance)
		{
			InOutPermutationData.PermutationIds[InvocationIndex] |= ((GroupInstance->BindingType == EHairBindingType::Skinning) ? PermutationIds.EnableSkinnedTransform : 0);
		}
		++InvocationIndex;
	}
}

void FOptimusGroomStrandsReadProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	if (!FallbackByteAddressSRV) { FallbackByteAddressSRV = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 16u)); }

	GroupResources.Reset();
	GroupInterpolations.Reset();
	BindingResources.Reset();
	
	for (const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		if (GroupInstance)
		{
			{
				FHairStrandsInstanceResourceParameters& Resource = GroupResources.AddDefaulted_GetRef();
				Resource.PositionBuffer	= RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->PositionBuffer);
				Resource.PositionOffsetBuffer = RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->PositionOffsetBuffer);
				Resource.CurveBuffer = RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->CurveBuffer);
				Resource.PointToCurveBuffer	= RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->PointToCurveBuffer);
				Resource.CurveAttributeBuffer = RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->CurveAttributeBuffer);
				Resource.PointAttributeBuffer = RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->PointAttributeBuffer);
				Resource.CurveMappingBuffer = RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->CurveMappingBuffer);
				Resource.PointMappingBuffer = RegisterAsSRV(GraphBuilder, GroupInstance->Strands.RestResource->PointMappingBuffer);
			}

			{
				FHairStrandsInstanceInterpolationParameters& Resource = GroupInterpolations.AddDefaulted_GetRef();
				Resource.CurveInterpolationBuffer = GroupInstance->Strands.InterpolationResource ? RegisterAsSRV(GraphBuilder, GroupInstance->Strands.InterpolationResource->CurveInterpolationBuffer) : nullptr;
				Resource.PointInterpolationBuffer = GroupInstance->Strands.InterpolationResource ? RegisterAsSRV(GraphBuilder, GroupInstance->Strands.InterpolationResource->PointInterpolationBuffer) : nullptr;
			}
			{
				FBindingResources& Resource = BindingResources.AddDefaulted_GetRef();
				if(GroupInstance->BindingType == EHairBindingType::Skinning)
				{
					FHairStrandsLODRestRootResource* RestLODDatas = GroupInstance->Strands.RestRootResource->LODs[GroupInstance->HairGroupPublicData->MeshLODIndex];
					FHairStrandsLODDeformedRootResource* DeformedLODDatas = GroupInstance->Strands.DeformedRootResource->LODs[GroupInstance->HairGroupPublicData->MeshLODIndex];
				
					Resource.CurveTriangleIndices = RegisterAsSRV(GraphBuilder, RestLODDatas->RootToUniqueTriangleIndexBuffer);
					Resource.TriangleRestPositions = RegisterAsSRV(GraphBuilder, RestLODDatas->RestUniqueTrianglePositionBuffer);
					Resource.TriangleDeformedPositions = RegisterAsSRV(GraphBuilder, DeformedLODDatas->GetDeformedUniqueTrianglePositionBuffer(FHairStrandsLODDeformedRootResource::Current));
					Resource.CurveBarycentricCoordinates = RegisterAsSRV(GraphBuilder, RestLODDatas->RootBarycentricBuffer);
				}
				else
				{
					Resource.RigidDeformedTransform = FMatrix44f(GroupInstance->Debug.RigidCurrentLocalToWorld.ToMatrixWithScale());
					Resource.RigidRestTransform = FMatrix44f(GroupInstance->Debug.RigidRestLocalToWorld.ToMatrixWithScale());
				}
			}
		}
	}
}

void FOptimusGroomStrandsReadProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	check(InDispatchData.NumInvocations == GroupInstances.Num());
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	uint32 BasePointIndex = 0;
	uint32 BaseCurveIndex = 0;
	uint32 BaseGuidePointIndex = 0;
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			
			const FHairGroupPublicData::FVertexFactoryInput VFInput = ComputeHairStrandsVertexInputData(GroupInstance, EGroomViewMode::None);
			Parameters.Common = VFInput.Strands.Common;

			// Used to get the local element indices for the current group, since index supplied by the compute kernel goes from 0 to NumElementsPerGroup * NumGroups
			Parameters.BasePointIndex = BasePointIndex;
			BasePointIndex += Parameters.Common.PointCount;
			Parameters.BaseCurveIndex = BaseCurveIndex;
			BaseCurveIndex += Parameters.Common.CurveCount;

			Parameters.BaseGuidePointIndex = BaseGuidePointIndex;
			BaseGuidePointIndex += GroupInstance->Guides.RestResource->GetPointCount();
			
			Parameters.Resources = GroupResources[InvocationIndex];
			
			if (GroupInterpolations[InvocationIndex].CurveInterpolationBuffer != nullptr)
			{
				Parameters.Interpolation = GroupInterpolations[InvocationIndex];
			}
			else
			{
				Parameters.Interpolation.CurveInterpolationBuffer = FallbackByteAddressSRV;
				Parameters.Interpolation.PointInterpolationBuffer = FallbackByteAddressSRV;
			}
			FBindingResources& Resource = BindingResources[InvocationIndex];
			if(GroupInstance->BindingType == EHairBindingType::Skinning)
			{          
				Parameters.CurveTriangleIndices = Resource.CurveTriangleIndices;
				Parameters.CurveBarycentricCoordinates = Resource.CurveBarycentricCoordinates;
				Parameters.TriangleRestPositions = Resource.TriangleRestPositions;
				Parameters.TriangleDeformedPositions = Resource.TriangleDeformedPositions;
			}
			else
			{
				Parameters.RigidDeformedTransform = Resource.RigidDeformedTransform;
				Parameters.RigidRestTransform = Resource.RigidRestTransform;
			}
		}
	}
	
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.TotalPointCount = BasePointIndex;	
			Parameters.TotalCurveCount = BaseCurveIndex;	
		}
	}
}
