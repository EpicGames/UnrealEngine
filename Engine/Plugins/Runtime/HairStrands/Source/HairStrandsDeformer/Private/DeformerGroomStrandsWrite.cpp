// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformerGroomStrandsWrite.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "RenderGraphBuilder.h"
#include "ShaderParameterMetadataBuilder.h"
#include "GroomComponent.h"
#include "GroomInstance.h"
#include "DeformerGroomDomainsSource.h"
#include "DeformerGroomInterfaceUtils.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "HairStrandsInterpolation.h"
#include "GroomVisualizationData.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformerGroomStrandsWrite)

FString UOptimusGroomStrandsWriteDataInterface::GetDisplayName() const
{
	return TEXT("Write Groom Strands");
}

FName UOptimusGroomStrandsWriteDataInterface::GetCategory() const
{
	return CategoryName::OutputDataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UOptimusGroomStrandsWriteDataInterface::GetPinDefinitions() const
{
	static const FName StrandsPoints(UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Points);
	static const FName StrandsCurves(UOptimusGroomAssetComponentSource::FStrandsExecutionDomains::Curves);

	TArray<FOptimusCDIPinDefinition> Defs;
	
	// Deformation buffers
	Defs.Add({ "Position", "WritePointDeformedPosition", StrandsPoints, "ReadNumPoints", true, "PointDeformedPosition" });
	Defs.Add({ "PositionAndRadius", "WritePointPositionRadius", StrandsPoints, "ReadNumPoints", true, "PointPositionRadius" });

	// Material buffers
	Defs.Add({ "Radius", "WritePointRadius", StrandsPoints, "ReadNumPoints", true, "PointMaterialRadius" });
	Defs.Add({ "RootUV", "WriteCurveRootUV", StrandsCurves, "ReadNumCurves", true, "CurveMaterialRootUV" });
	Defs.Add({ "Seed", "WriteCurveSeed", StrandsCurves, "ReadNumCurves", true, "CurveMaterialSeed" });
	Defs.Add({ "ClumpId", "WriteCurveClumpId", StrandsCurves, "ReadNumCurves", true, "CurveMaterialClumpId" });
	Defs.Add({ "Color", "WritePointColor", StrandsPoints, "ReadNumPoints", true, "PointMaterialColor" });
	Defs.Add({ "Roughness", "WritePointRoughness", StrandsPoints, "ReadNumPoints", true, "PointMaterialRoughness" });
	Defs.Add({ "AO", "WritePointAO", StrandsPoints, "ReadNumPoints", true, "PointMaterialAO" });

	return Defs;
}

TSubclassOf<UActorComponent> UOptimusGroomStrandsWriteDataInterface::GetRequiredComponentClass() const
{
	return UMeshComponent::StaticClass();
}

void UOptimusGroomStrandsWriteDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumPoints"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumCurves"))
		.AddReturnType(EShaderFundamentalType::Uint);
}

void UOptimusGroomStrandsWriteDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointDeformedPosition"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointRadius"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointPositionRadius"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 4);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteCurveRootUV"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 2);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteCurveSeed"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteCurveClumpId"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointColor"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointRoughness"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePointAO"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float);
}

BEGIN_SHADER_PARAMETER_STRUCT(FOptimusGroomStrandsWriteParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCommonParameters, Common)
	SHADER_PARAMETER(uint32, BasePointIndex)
	SHADER_PARAMETER(uint32, BaseCurveIndex)
	SHADER_PARAMETER(uint32, TotalPointCount)
	SHADER_PARAMETER(uint32, TotalCurveCount)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeformedPositionOffset)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointRestPositions)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutPointDeformedPositions)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutCurveAttributeBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutPointAttributeBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusGroomStrandsWriteDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FOptimusGroomStrandsWriteParameters>(UID);
}

TCHAR const* UOptimusGroomStrandsWriteDataInterface::TemplateFilePath = TEXT("/Plugin/Runtime/HairStrands/Private/Deformers/DeformerGroomStrandsWrite.ush");

TCHAR const* UOptimusGroomStrandsWriteDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusGroomStrandsWriteDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusGroomStrandsWriteDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusGroomStrandsWriteDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGroomStrandsWriteDataProvider* Provider = NewObject<UOptimusGroomStrandsWriteDataProvider>();
	Provider->MeshComponent = Cast<UMeshComponent>(InBinding);
	Provider->OutputMask = InOutputMask;
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusGroomStrandsWriteDataProvider::GetRenderProxy()
{
	return new FOptimusGroomStrandsWriteProviderProxy(MeshComponent, OutputMask);
}

FOptimusGroomStrandsWriteProviderProxy::FOptimusGroomStrandsWriteProviderProxy(UMeshComponent* MeshComponent, uint64 InOutputMask)
{
	OutputMask = InOutputMask;

	TArray<const UGroomComponent*> GroomComponents;
	UE::Groom::Private::GatherGroupInstances(MeshComponent, GroupInstances);
}

bool FOptimusGroomStrandsWriteProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (InValidationData.NumInvocations != GroupInstances.Num())
	{
		return false;
	}
	if(!UE::Groom::Private::HaveStrandsInstanceResources(GroupInstances))
	{
		return false;
	}

	return true;
}

void FOptimusGroomStrandsWriteProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	if (!FallbackPositionBufferSRV) { FallbackPositionBufferSRV = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 16u));}
	if (!FallbackPositionBufferUAV) { FallbackPositionBufferUAV = GraphBuilder.CreateUAV(GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(16u), TEXT("Groom.Deformer.FallbackDeformedPositionBuffer")), ERDGUnorderedAccessViewFlags::SkipBarrier);}
	if (!FallbackAttributeBufferUAV) { FallbackAttributeBufferUAV = GraphBuilder.CreateUAV(GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(16u), TEXT("Groom.Deformer.FallbackDeformedAttributeBuffer")), ERDGUnorderedAccessViewFlags::SkipBarrier);}

	for (const FHairGroupInstance* GroupInstance : GroupInstances)
	{
		if (GroupInstance)
		{
			FDispatchResources& Resource = GroupResources.AddDefaulted_GetRef();
			
			// Positions / Radius
			if ((OutputMask & 0x7) && GroupInstance->Strands.RestResource && GroupInstance->Strands.DeformedResource)
			{
				Resource.DeformedPositionOffset 	= Register(GraphBuilder, GroupInstance->Strands.RestResource->PositionOffsetBuffer, ERDGImportedBufferFlags::CreateSRV).SRV;
				Resource.PointRestPositions 		= Register(GraphBuilder, GroupInstance->Strands.RestResource->PositionBuffer, ERDGImportedBufferFlags::CreateSRV).SRV;
				Resource.OutPointDeformedPositions 	= Register(GraphBuilder, GroupInstance->Strands.DeformedResource->GetDeformerBuffer(GraphBuilder), ERDGImportedBufferFlags::CreateUAV).UAV;
			}
			
			// Always copy the attributes from the rest asset, so that if deformer write different attribute at different tick, it all remains consistent with the source data
			if(GroupInstance->Strands.RestResource && GroupInstance->Strands.DeformedResource)
			{
				// Curve Attributes
				if (OutputMask & 0x38)
				{
					FRDGExternalBuffer OutCurveAttributeBufferExt = GroupInstance->Strands.DeformedResource->GetDeformerCurveAttributeBuffer(GraphBuilder);
					if (OutCurveAttributeBufferExt.Buffer)
					{
						FRDGImportedBuffer DstCurveAttributeBuffer = Register(GraphBuilder, OutCurveAttributeBufferExt, ERDGImportedBufferFlags::CreateUAV);
						FRDGImportedBuffer SrcCurveAttributeBuffer = Register(GraphBuilder, GroupInstance->Strands.RestResource->CurveAttributeBuffer, ERDGImportedBufferFlags::None);
						if (SrcCurveAttributeBuffer.Buffer)
						{
							AddCopyBufferPass(GraphBuilder, DstCurveAttributeBuffer.Buffer, SrcCurveAttributeBuffer.Buffer);
						}
						Resource.OutCurveAttributeBuffer = DstCurveAttributeBuffer.UAV;
					}
				}

				// Point Attributes
				if (OutputMask & 0x1c0)
				{
					FRDGExternalBuffer OutPointAttributeBufferExt = GroupInstance->Strands.DeformedResource->GetDeformerPointAttributeBuffer(GraphBuilder);
					if (OutPointAttributeBufferExt.Buffer)
					{
						FRDGImportedBuffer DstPointAttributeBuffer = Register(GraphBuilder, OutPointAttributeBufferExt, ERDGImportedBufferFlags::CreateUAV);
						FRDGImportedBuffer SrcPointAttributeBuffer = Register(GraphBuilder, GroupInstance->Strands.RestResource->PointAttributeBuffer, ERDGImportedBufferFlags::None);
						if(SrcPointAttributeBuffer.Buffer)
						{
							AddCopyBufferPass(GraphBuilder, DstPointAttributeBuffer.Buffer, SrcPointAttributeBuffer.Buffer);
						}
						Resource.OutPointAttributeBuffer = DstPointAttributeBuffer.UAV;
					}
				}
			}
		}	
	}
}

void FOptimusGroomStrandsWriteProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	
	uint32 BasePointIndex = 0;
	uint32 BaseCurveIndex = 0;
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FDispatchResources& Resource = GroupResources[InvocationIndex];

			const bool bValid = Resource.OutPointDeformedPositions != nullptr && Resource.PointRestPositions != nullptr && Resource.DeformedPositionOffset != nullptr;
			const FHairGroupPublicData::FVertexFactoryInput VFInput = ComputeHairStrandsVertexInputData(GroupInstance, EGroomViewMode::None);		

			FParameters& Parameters = ParameterArray[InvocationIndex];
			
			Parameters.Common 					= VFInput.Strands.Common;
			
			// Used to get the local element indices for the current group, since index supplied by the compute kernel goes from 0 to NumElementsPerGroup * NumGroups
			Parameters.BasePointIndex = BasePointIndex;
			BasePointIndex += Parameters.Common.PointCount;
			Parameters.BaseCurveIndex = BaseCurveIndex;
			BaseCurveIndex += Parameters.Common.CurveCount;
			
			Parameters.DeformedPositionOffset 	= bValid ? Resource.DeformedPositionOffset : FallbackPositionBufferSRV;
			Parameters.PointRestPositions 		= bValid ? Resource.PointRestPositions : FallbackPositionBufferSRV;
			Parameters.OutPointDeformedPositions= bValid ? Resource.OutPointDeformedPositions : FallbackPositionBufferUAV;
			Parameters.OutPointAttributeBuffer 	= bValid ? Resource.OutPointAttributeBuffer : FallbackAttributeBufferUAV;
			Parameters.OutCurveAttributeBuffer 	= bValid ? Resource.OutCurveAttributeBuffer : FallbackAttributeBufferUAV;
		}
	}

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		if (const FHairGroupInstance* GroupInstance = GroupInstances[InvocationIndex])
		{
			FParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.TotalPointCount = BasePointIndex;	
			Parameters.TotalCurveCount = BaseCurveIndex;	
		}
	}	
}
