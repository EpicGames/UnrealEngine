// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusComputeDataInterface.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

/**
 *  FHairStrandsDeformerModule
 */
class FHairStrandsDeformerModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
};

void FHairStrandsDeformerModule::StartupModule()
{
	//UOptimusComputeDataInterface::RegisterAllTypes();
}

void FHairStrandsDeformerModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHairStrandsDeformerModule, HairStrandsDeformer);
