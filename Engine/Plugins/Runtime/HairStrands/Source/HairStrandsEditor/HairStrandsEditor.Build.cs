// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairStrandsEditor : ModuleRules
	{
		public HairStrandsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"EditorFramework",
					"Engine",
					"GeometryCache",
					"HairStrandsCore",
					"DataflowCore",
					"DataflowEngine",
					"InputCore",
					"MainFrame",
					"Slate",
					"SlateCore",
					"Projects",
					"ToolMenus",
					"ContentBrowser",
					"UnrealEd",
					"AssetDefinition",
					"AssetTools",
					"EditorInteractiveToolsFramework",
					"AdvancedPreviewScene",
					"InputCore",
					"Renderer",
					"PropertyEditor",
					"RHI",
					"LevelSequence",
					"MovieScene",
					"MovieSceneTools",
					"SequencerCore",
					"Sequencer",
					"HairCardGeneratorFramework",
					"CommonMenuExtensions",
					"RenderCore",
					"DataflowEditor",
					"JsonUtilities",
					"Json",
					"ApplicationCore",
					"ToolWidgets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Analytics"
				});

			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"FBX"
			);
		}
	}
}