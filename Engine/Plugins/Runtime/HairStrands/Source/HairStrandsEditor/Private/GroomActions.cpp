// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomActions.h"
#include "GroomAsset.h"

#include "EditorFramework/AssetImportData.h"
#include "HairStrandsCore.h"
#include "GeometryCache.h"
#include "GroomAssetImportData.h"
#include "GroomBuilder.h"
#include "GroomDeformerBuilder.h"
#include "GroomImportOptions.h"
#include "GroomImportOptionsWindow.h"
#include "GroomCustomAssetEditorToolkit.h"
#include "GroomCreateBindingOptions.h"
#include "GroomCreateBindingOptionsWindow.h"
#include "GroomCreateFollicleMaskOptions.h"
#include "GroomCreateFollicleMaskOptionsWindow.h"
#include "GroomCreateStrandsTexturesOptions.h"
#include "GroomCreateStrandsTexturesOptionsWindow.h"
#include "AssetCompilingManager.h"
#include "HairStrandsImporter.h"
#include "HairStrandsTranslator.h"
#include "ToolMenuSection.h"
#include "Misc/ScopedSlowTask.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GroomBindingBuilder.h"
#include "GroomTextureBuilder.h"
#include "GroomBindingAsset.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ObjectTools.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Materials/Material.h"

#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "HairStrandsFactory.h"
#include "ToolMenus.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dialog/SMessageDialog.h"

#include "ThumbnailRendering/SceneThumbnailInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomActions)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FLinearColor UAssetDefinition_GroomAsset::GetAssetColor() const
{
	return FColor::White;
}

UThumbnailInfo* UAssetDefinition_GroomAsset::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_GroomAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	// #ueent_todo: Will need a custom editor at some point, for now just use the Properties editor
	for (UGroomAsset* GroomAsset : OpenArgs.LoadObjects<UGroomAsset>())
	{
		if (GroomAsset != nullptr)
		{
			if(!GroomAsset->GetOnCreateGroomDataflow().IsBound())
			{
				GroomAsset->GetOnCreateGroomDataflow().AddLambda(
					[](UGroomAsset* GroomAsset)
					{
						if (UE::Groom::BuildGroomDataflowAsset(GroomAsset))
						{
							if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
							{
								// note : we cannot use CloseAllEditorsForAsset for the Dataflow Editor , the persona based editor seem to have an issue with calling this
								// and will crash if because of an order of destruction issue with the toolkit. the path to solve it is unclear and may be risky
								// As an alternative let's not reload the dataflow editor and leverage the in place dataflow asset change feature that already exists
								bool bDataflowEditorAlreadyOpened = false;

								TArray<IAssetEditorInstance*> EditorInstances = AssetEditorSubsystem->FindEditorsForAssetAndSubObjects(GroomAsset);
								for (IAssetEditorInstance* EditorInstance : EditorInstances)
								{
									if (EditorInstance->GetEditorName() != "DataflowEditor")
									{
										EditorInstance->CloseWindow(EAssetEditorCloseReason::CloseAllEditorsForAsset);
									}
									else if (!bDataflowEditorAlreadyOpened && EditorInstance->GetEditorName() == "DataflowEditor")
									{
										if (FDataflowEditorToolkit* DataflowEditorToolkit = static_cast<FDataflowEditorToolkit*>(EditorInstance))
										{
											DataflowEditorToolkit->OnDataflowAssetChanged();
										}
										bDataflowEditorAlreadyOpened = true;
									}
									AssetEditorSubsystem->OnAssetEditorRequestClose().Broadcast(GroomAsset, EAssetEditorCloseReason::CloseAllEditorsForAsset);
								}

								if (!bDataflowEditorAlreadyOpened)
								{
									// Open a new editor to be able to edit the groom asset with dataflow
									AssetEditorSubsystem->OpenEditorForAsset(GroomAsset);
								}
							}
						}
					} );
			}
			if(!GroomAsset->GetDataflowSettings().GetDataflowAsset())
			{
				TSharedRef<FGroomCustomAssetEditorToolkit> NewCustomAssetEditor(new FGroomCustomAssetEditorToolkit());
				NewCustomAssetEditor->InitCustomAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, GroomAsset);
			}
			else
			{
				UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
				AssetEditor->RegisterToolCategories({"General"});
				
				const TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr,
					TEXT("/HairStrands/BP_PreviewGroom.BP_PreviewGroom_C"), nullptr, LOAD_None, nullptr);

				UMaterial* HairMaterial = Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(),nullptr,
					TEXT("/HairStrands/Materials/HairDataflowMaterial.HairDataflowMaterial"), nullptr, LOAD_None, nullptr));
				
				GroomAsset->GetDataflowSettings().GetDataflowAsset()->Material = HairMaterial;
				AssetEditor->Initialize({ GroomAsset}, ActorClass);

				auto ResetDataflowSimulation = [GroomAsset]()
				{
					if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
					{
						TArray<IAssetEditorInstance*> EditorInstances = AssetEditorSubsystem->FindEditorsForAssetAndSubObjects(GroomAsset);
						for (IAssetEditorInstance* EditorInstance : EditorInstances)
						{
							if (EditorInstance->GetEditorName() == "DataflowEditor")
							{
								if (FDataflowEditorToolkit* DataflowEditorToolkit = static_cast<FDataflowEditorToolkit*>(EditorInstance))
								{
									DataflowEditorToolkit->ResetDataflowSimulation();
								}
							}
						}
					}
				};

				GroomAsset->GetOnGroomAssetResourcesChanged().AddLambda(ResetDataflowSimulation);
				GroomAsset->GetOnGroomAssetChanged().AddLambda(ResetDataflowSimulation);
			}
		}
	}

	return EAssetCommandResult::Handled;
}

void UAssetDefinition_GroomAsset::GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const
{
	for (UObject* Asset : TypeAssets)
	{
		const UGroomAsset* GroomAsset = CastChecked<UGroomAsset>(Asset);
		if (GroomAsset && GroomAsset->AssetImportData)
		{
			GroomAsset->AssetImportData->ExtractFilenames(OutSourceFilePaths);
		}
	}
}

namespace MenuExtension_GroomAsset
{
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Groom build/rebuild

bool CanRebuild(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	for (UGroomAsset* GroomAsset : CBContext->LoadSelectedObjects<UGroomAsset>())
	{
		if (GroomAsset && GroomAsset->IsValid() && GroomAsset->CanRebuildFromDescription())
		{
			return true;
		}
	}
	return false;
}

void ExecuteRebuild(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	for (UGroomAsset* GroomAsset : CBContext->LoadSelectedObjects<UGroomAsset>())
	{
		if (GroomAsset && GroomAsset->IsValid() && GroomAsset->CanRebuildFromDescription() && GroomAsset->AssetImportData)
		{
			UGroomAssetImportData* GroomAssetImportData = Cast<UGroomAssetImportData>(GroomAsset->AssetImportData);
			if (GroomAssetImportData && GroomAssetImportData->ImportOptions)
			{
				FString Filename(GroomAssetImportData->GetFirstFilename());

				// Duplicate the options to prevent dirtying the asset when they are modified but the rebuild is cancelled
				UGroomImportOptions* CurrentOptions = DuplicateObject<UGroomImportOptions>(GroomAssetImportData->ImportOptions, nullptr);
			
				const uint32 GroupCount = GroomAsset->GetNumHairGroups();
				UGroomHairGroupsPreview* GroupsPreview = NewObject<UGroomHairGroupsPreview>();
				{
					FHairDescription HairDescription = GroomAsset->GetHairDescription();
					FHairDescriptionGroups HairDescriptionGroups;
					FGroomBuilder::BuildHairDescriptionGroups(HairDescription, HairDescriptionGroups);

					for (uint32 GroupIndex = 0; GroupIndex < GroupCount; GroupIndex++)
					{
						FGroomHairGroupPreview& OutGroup = GroupsPreview->Groups.AddDefaulted_GetRef();
						OutGroup.GroupIndex 	= GroupIndex;
						OutGroup.GroupID		= GroomAsset->GetHairGroupsInfo()[GroupIndex].GroupID;
						OutGroup.GroupName		= GroomAsset->GetHairGroupsInfo()[GroupIndex].GroupName;
						OutGroup.CurveCount 	= GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.BulkData.GetNumCurves();
						OutGroup.GuideCount 	= GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Guides.BulkData.GetNumCurves();
						OutGroup.Attributes 	= HairDescriptionGroups.HairGroups[GroupIndex].GetHairAttributes();
						OutGroup.AttributeFlags = HairDescriptionGroups.HairGroups[GroupIndex].GetHairAttributeFlags();
						OutGroup.InterpolationSettings = GroomAsset->GetHairGroupsInterpolation()[GroupIndex];
					}
				}
				TSharedPtr<SGroomImportOptionsWindow> GroomOptionWindow = SGroomImportOptionsWindow::DisplayRebuildOptions(CurrentOptions, GroupsPreview, nullptr/*GroupsMapping*/, Filename);

				if (!GroomOptionWindow->ShouldImport())
				{
					continue;
				}

				// Apply new interpolation settings to the groom, prior to rebuilding the groom
				bool bEnableRigging = false;
				for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
				{
					GroomAsset->GetHairGroupsInterpolation()[GroupIndex] = GroupsPreview->Groups[GroupIndex].InterpolationSettings;
					bEnableRigging |= GroomAsset->GetHairGroupsInterpolation()[GroupIndex].InterpolationSettings.GuideType == EGroomGuideType::Rigged;
				}

				bool bSucceeded = GroomAsset->CacheDerivedDatas();
				if (bSucceeded)
				{
					if(bEnableRigging)
					{
						GroomAsset->SetRiggedSkeletalMesh(FGroomDeformerBuilder::CreateSkeletalMesh(GroomAsset));
					}
					// Move the transient ImportOptions to the asset package and set it on the GroomAssetImportData for serialization
					CurrentOptions->Rename(nullptr, GroomAssetImportData);
					for (const FGroomHairGroupPreview& GroupPreview : GroupsPreview->Groups)
					{
						CurrentOptions->InterpolationSettings[GroupPreview.GroupIndex] = GroupPreview.InterpolationSettings;
					}
					GroomAssetImportData->ImportOptions = CurrentOptions;
					GroomAsset->MarkPackageDirty();
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Binding

bool CanCreateBindingAsset(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	for (const FAssetData& SelectedAsset : CBContext->SelectedAssets)
	{
		if (SelectedAsset.IsValid())
		{
			return true;
		}
	}
	return false;
}

void ExecuteCreateBindingAsset(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	for (UGroomAsset* GroomAsset : CBContext->LoadSelectedObjects<UGroomAsset>())
	{
		if (GroomAsset && GroomAsset->IsValid())
		{
			// Duplicate the options to prevent dirtying the asset when they are modified but the rebuild is cancelled
			UGroomCreateBindingOptions* CurrentOptions = NewObject<UGroomCreateBindingOptions>();
			if (CurrentOptions)
			{
				CurrentOptions->GroomAsset = GroomAsset;
			}
			TSharedPtr<SGroomCreateBindingOptionsWindow> GroomOptionWindow = SGroomCreateBindingOptionsWindow::DisplayCreateBindingOptions(CurrentOptions);

			if (!GroomOptionWindow->ShouldCreate())
			{
				continue;
			}
			else if (CurrentOptions && 
				    ((CurrentOptions->GroomBindingType == EGroomBindingMeshType::SkeletalMesh && CurrentOptions->TargetSkeletalMesh) ||
					(CurrentOptions->GroomBindingType == EGroomBindingMeshType::GeometryCache && CurrentOptions->TargetGeometryCache)))
			{
				GroomAsset->ConditionalPostLoad();

				UGroomBindingAsset* BindingAsset = nullptr;
				if (CurrentOptions->GroomBindingType == EGroomBindingMeshType::SkeletalMesh)
				{
					CurrentOptions->TargetSkeletalMesh->ConditionalPostLoad();
					if (CurrentOptions->SourceSkeletalMesh)
					{
						CurrentOptions->SourceSkeletalMesh->ConditionalPostLoad();
					}
					BindingAsset = FHairStrandsCore::CreateGroomBindingAsset(CurrentOptions->GroomBindingType, GroomAsset, CurrentOptions->SourceSkeletalMesh, CurrentOptions->TargetSkeletalMesh, CurrentOptions->NumInterpolationPoints, CurrentOptions->MatchingSection, CurrentOptions->TargetBindingAttribute);
				}
				else
				{
					CurrentOptions->TargetGeometryCache->ConditionalPostLoad();
					if (CurrentOptions->SourceGeometryCache)
					{
						CurrentOptions->SourceGeometryCache->ConditionalPostLoad();
					}
					BindingAsset = FHairStrandsCore::CreateGroomBindingAsset(CurrentOptions->GroomBindingType, GroomAsset, CurrentOptions->SourceGeometryCache, CurrentOptions->TargetGeometryCache, CurrentOptions->NumInterpolationPoints, CurrentOptions->MatchingSection, CurrentOptions->TargetBindingAttribute);
				}

				if (BindingAsset)
				{
					BindingAsset->Build();
#if WITH_EDITOR
					FAssetCompilingManager::Get().FinishCompilationForObjects({BindingAsset});
#endif
					if (BindingAsset->IsValid())
					{
						TArray<UObject*> CreatedObjects;
						CreatedObjects.Add(BindingAsset);

						FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
						ContentBrowserModule.Get().SyncBrowserToAssets(CreatedObjects);
					#if WITH_EDITOR
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(CreatedObjects);
					#endif
					}
					else
					{
						FNotificationInfo Info(LOCTEXT("FailedToCreateBinding", "Failed to create groom binding. See Output Log for details"));
						Info.ExpireDuration = 5.0f;
						FSlateNotificationManager::Get().AddNotification(Info);

						if (ObjectTools::DeleteSingleObject(BindingAsset))
						{
							CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
						}
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Follicle

bool CanCreateFollicleTexture(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	for (const FAssetData& SelectedAsset : CBContext->SelectedAssets)
	{
		if (SelectedAsset.IsValid())
		{
			return true;
		}
	}
	return false;
}

void ExecuteCreateFollicleTexture(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	TArray<UGroomAsset*> GroomAssets = CBContext->LoadSelectedObjects<UGroomAsset>();
	if (GroomAssets.Num() == 0)
	{
		return;
	}

	// Duplicate the options to prevent dirtying the asset when they are modified but the rebuild is cancelled
	UGroomCreateFollicleMaskOptions* CurrentOptions = NewObject<UGroomCreateFollicleMaskOptions>();
	if (!CurrentOptions)
	{
		return;
	}

	for (UGroomAsset* GroomAsset : GroomAssets)
	{
		if (GroomAsset && GroomAsset->IsValid())
		{
			FFollicleMaskOptions& Items = CurrentOptions->Grooms.AddDefaulted_GetRef();;
			Items.Groom   = GroomAsset;
			Items.Channel = EFollicleMaskChannel::R;
		}
	}

	if (CurrentOptions->Grooms.Num() == 0)
	{
		return;
	}
	
	TSharedPtr<SGroomCreateFollicleMaskOptionsWindow> GroomOptionWindow = SGroomCreateFollicleMaskOptionsWindow::DisplayCreateFollicleMaskOptions(CurrentOptions);

	if (!GroomOptionWindow->ShouldCreate())
	{
		return;
	}
	else 
	{
		TArray<FFollicleInfo> Infos;
		for (FFollicleMaskOptions& Option : CurrentOptions->Grooms)
		{
			if (Option.Groom)
			{
				Option.Groom->ConditionalPostLoad();

				FFollicleInfo& Info = Infos.AddDefaulted_GetRef();
				Info.GroomAsset			= Option.Groom;
				Info.Channel			= FFollicleInfo::EChannel(uint8(Option.Channel));
				Info.KernelSizeInPixels = FMath::Max(2, CurrentOptions->RootRadius);
			}
		}

		const uint32 Resolution = FMath::RoundUpToPowerOfTwo(CurrentOptions->Resolution);
		UTexture2D* FollicleTexture = FGroomTextureBuilder::CreateGroomFollicleMaskTexture(CurrentOptions->Grooms[0].Groom, Resolution);
		if (FollicleTexture)
		{
			FGroomTextureBuilder::BuildFollicleTexture(Infos, FollicleTexture, false);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Strands Textures

bool CanCreateStrandsTextures(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	for (const FAssetData& SelectedAsset : CBContext->SelectedAssets)
	{
		if (SelectedAsset.IsValid())
		{
			return true;
		}
	}
	return false;
}

void ExecuteCreateStrandsTextures(const FToolMenuContext& InContext)
{
	const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	TArray<UGroomAsset*> GroomAssets = CBContext->LoadSelectedObjects<UGroomAsset>();
	if (GroomAssets.Num() == 0)
	{
		return;
	}

	// Duplicate the options to prevent dirtying the asset when they are modified but the rebuild is cancelled
	for (UGroomAsset* GroomAsset : GroomAssets)
	{
		if (GroomAsset && GroomAsset->IsValid())
		{
			UGroomCreateStrandsTexturesOptions* CurrentOptions = nullptr;
			UGroomAssetImportData* GroomAssetImportData = Cast<UGroomAssetImportData>(GroomAsset->AssetImportData);
			if (GroomAssetImportData)
			{
				// Duplicate the options to prevent dirtying the asset when they are modified but the rebuild is cancelled
				if (GroomAssetImportData->HairStrandsTexturesOptions)
				{
					CurrentOptions = DuplicateObject<UGroomCreateStrandsTexturesOptions>(GroomAssetImportData->HairStrandsTexturesOptions, nullptr);
				}
			}
			else
			{
				// Create UGroomAssetImportData and copy existing values if any
				GroomAssetImportData = NewObject<UGroomAssetImportData>();
				GroomAssetImportData->Rename(nullptr, GroomAsset);
				GroomAssetImportData->SourceData = GroomAsset->AssetImportData->SourceData;
				
				// Create/Initialize import setings
				const uint32 GroupCount = GroomAsset->GetNumHairGroups();
				GroomAssetImportData->ImportOptions = NewObject<UGroomImportOptions>();
				GroomAssetImportData->ImportOptions->Rename(nullptr, GroomAssetImportData);
				GroomAssetImportData->ImportOptions->InterpolationSettings.SetNum(GroupCount);
				for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
				{
					GroomAssetImportData->ImportOptions->InterpolationSettings[GroupIndex] = GroomAsset->GetHairGroupsInterpolation()[GroupIndex];
				}
			}

			if (CurrentOptions == nullptr)
			{
				CurrentOptions = NewObject<UGroomCreateStrandsTexturesOptions>();
			}

			TSharedPtr<SGroomCreateStrandsTexturesOptionsWindow> GroomOptionWindow = SGroomCreateStrandsTexturesOptionsWindow::DisplayCreateStrandsTexturesOptions(CurrentOptions);
			if (!GroomOptionWindow->ShouldCreate())
			{
				return;
			}
			else
			{
				GroomAsset->ConditionalPostLoad();

				// Create debug data for the groom asset for tracing hair geometry when redering strands texture.
				if (!GroomAsset->HasDebugData())
				{
					GroomAsset->CreateDebugData();
				}

				float SignDirection = 1;
				float MaxDistance = CurrentOptions->TraceDistance;
				switch (CurrentOptions->TraceType)
				{
				case EStrandsTexturesTraceType::TraceOuside:		SignDirection =  1; break;
				case EStrandsTexturesTraceType::TraceInside:		SignDirection = -1; break;
				case EStrandsTexturesTraceType::TraceBidirectional: SignDirection = 0;  MaxDistance *= 2; break;
				}

				UStaticMesh* StaticMesh = nullptr;
				USkeletalMesh* SkeletalMesh = nullptr;
				switch (CurrentOptions->MeshType)
				{
				case EStrandsTexturesMeshType::Static: StaticMesh = CurrentOptions->StaticMesh; break;
				case EStrandsTexturesMeshType::Skeletal: SkeletalMesh = CurrentOptions->SkeletalMesh; break;
				}
				if (SkeletalMesh == nullptr && StaticMesh == nullptr)
				{
					return;
				}
				
				FStrandsTexturesInfo Info;
				Info.Layout = CurrentOptions->Layout;
				Info.GroomAsset  = GroomAsset;
				Info.TracingDirection = SignDirection;
				Info.MaxTracingDistance = MaxDistance;
				Info.Resolution = FMath::RoundUpToPowerOfTwo(FMath::Max(256, CurrentOptions->Resolution));
				Info.LODIndex = FMath::Max(0, CurrentOptions->LODIndex);
				Info.SectionIndex = FMath::Max(0, CurrentOptions->SectionIndex);
				Info.UVChannelIndex= FMath::Max(0, CurrentOptions->UVChannelIndex);
				Info.SkeletalMesh = SkeletalMesh;
				Info.StaticMesh = StaticMesh;
				Info.Dilation = FMath::Clamp(CurrentOptions->Dilation, 0, 64);
				if (CurrentOptions->GroupIndex.Num())
				{
					Info.GroupIndices = CurrentOptions->GroupIndex;
				}
				else
				{
					for (int32 GroupIndex = 0; GroupIndex < GroomAsset->GetNumHairGroups(); ++GroupIndex)
					{
						Info.GroupIndices.Add(GroupIndex);
					}
				}

				FStrandsTexturesOutput Output;
				const uint32 TextureCount = GetHairTextureLayoutTextureCount(Info.Layout);
				if (CurrentOptions->GeneratedTextures.Num() != TextureCount)
				{
					Output = FGroomTextureBuilder::CreateGroomStrandsTexturesTexture(GroomAsset, Info.Resolution, Info.Layout);
					CurrentOptions->GeneratedTextures = Output.Textures;
				}
				else
				{
					Output.Textures = CurrentOptions->GeneratedTextures;
				}

				if (Output.IsValid())
				{
					FGroomTextureBuilder::BuildStrandsTextures(Info, Output);

					// Save settings used for generating these textures
					if (GroomAssetImportData)
					{
						if (!GroomAssetImportData->HairStrandsTexturesOptions || GroomAssetImportData->HairStrandsTexturesOptions != CurrentOptions)
						{
							// Move the transient ImportOptions to the asset package and set it on the GroomAssetImportData for serialization
							CurrentOptions->Rename(nullptr, GroomAssetImportData);
							GroomAssetImportData->HairStrandsTexturesOptions = CurrentOptions;
							GroomAsset->AssetImportData = GroomAssetImportData;
							GroomAsset->MarkPackageDirty();
						}
					}
				}
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Actions registration

static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UGroomAsset::StaticClass());
		
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			{
				const TAttribute<FText> Label = LOCTEXT("RebuildGroomAsset", "Rebuild Groom Asset");
				const TAttribute<FText> ToolTip = LOCTEXT("RebuildGroomTooltip", "Rebuild the groom with new build settings");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRebuild);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanRebuild);
				InSection.AddMenuEntry("GroomAsset_RebuildGroom", Label, ToolTip, Icon, UIAction);					
			}

			{
				const TAttribute<FText> Label = LOCTEXT("CreateBindingAsset", "Create Binding Asset");
				const TAttribute<FText> ToolTip = LOCTEXT("CreateBindingAssetTooltip", "Create a binding asset between a skeletal mesh and a groom asset");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateBindingAsset);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanCreateBindingAsset);
				InSection.AddMenuEntry("GroomAsset_CreateBindingAsset", Label, ToolTip, Icon, UIAction);
			}

			{
				const TAttribute<FText> Label = LOCTEXT("CreateFollicleTexture", "Create Follicle Texture");
				const TAttribute<FText> ToolTip = LOCTEXT("CreateFollicleTextureTooltip", "Create a follicle texture for the selected groom assets");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateFollicleTexture);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanCreateFollicleTexture);
				InSection.AddMenuEntry("GroomAsset_CreateFollicleTexture", Label, ToolTip, Icon, UIAction);
			}

			{
				const TAttribute<FText> Label = LOCTEXT("CreateStrandsTextures", "Create Strands Textures");
				const TAttribute<FText> ToolTip = LOCTEXT("CreateStrandsTexturesTooltip", "Create projected strands textures onto meshes");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateStrandsTextures);
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanCreateStrandsTextures);
				InSection.AddMenuEntry("GroomAsset_CreateStrandsTextures", Label, ToolTip, Icon, UIAction);
			}
		}));
	}));
});

} // namespace MenuExtension_GroomAsset
#undef LOCTEXT_NAMESPACE
