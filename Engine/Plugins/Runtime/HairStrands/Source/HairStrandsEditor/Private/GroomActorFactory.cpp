// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomActorFactory.h"
#include "GroomAsset.h"
#include "GroomActor.h"
#include "GroomComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomActorFactory)

#define LOCTEXT_NAMESPACE "GroomActorFactory"

UGroomActorFactory::UGroomActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("DefaultName", "Groom");
	NewActorClass = AGroomActor::StaticClass();
	bUseSurfaceOrientation = false;
	bShowInEditorQuickMenu = true;
}

bool UGroomActorFactory::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( !AssetData.IsValid() || !AssetData.IsInstanceOf( UGroomAsset::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoGroomAsset", "A valid groom asset must be specified.");
		return false;
	}

	return true;
}

void UGroomActorFactory::PostSpawnActor( UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UGroomAsset* GroomAsset = CastChecked<UGroomAsset>(Asset);

	// Change properties
	AGroomActor* GroomActor = CastChecked<AGroomActor>( NewActor );
	UGroomComponent* GroomComponent = GroomActor->GetGroomComponent();
	check(GroomComponent);

	GroomComponent->UnregisterComponent();
	GroomComponent->SetGroomAsset(GroomAsset);
	GroomComponent->PostLoad();
	GroomComponent->RegisterComponent();
}

UObject* UGroomActorFactory::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	AGroomActor* SMA = CastChecked<AGroomActor>(Instance);

	check(SMA->GetGroomComponent());
	return SMA->GetGroomComponent()->GroomAsset;
}

FQuat UGroomActorFactory::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}


#undef LOCTEXT_NAMESPACE 
