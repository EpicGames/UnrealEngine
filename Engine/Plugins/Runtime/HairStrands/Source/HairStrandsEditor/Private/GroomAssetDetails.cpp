// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomAssetDetails.h"
#include "Widgets/Input/SCheckBox.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IDetailGroup.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "GroomComponent.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorDirectories.h"
#include "UnrealEdGlobals.h"
#include "IDetailsView.h"
#include "MaterialList.h"
#include "PropertyCustomizationHelpers.h"
#include "Interfaces/IMainFrameModule.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Rendering/SkeletalMeshModel.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "EditorFramework/AssetImportData.h"
#include "Logging/LogMacros.h"
#include "IHairCardGenerator.h"

#include "MeshDescription.h"
#include "MeshAttributes.h"
#include "MeshAttributeArray.h"

#include "Widgets/Input/STextComboBox.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "IDocumentation.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "ComponentReregisterContext.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "SKismetInspector.h"
#include "PropertyEditorDelegates.h"
#include "PropertyCustomizationHelpers.h"
#include "GroomCustomAssetEditorToolkit.h"
#include "IPropertyUtilities.h"
#include "JsonObjectConverter.h"

#include "Styling/AppStyle.h"
#include "GroomEditorStyle.h"

#define LOCTEXT_NAMESPACE "GroomRenderingDetails"

DEFINE_LOG_CATEGORY_STATIC(LogGroomAssetDetails, Log, All);

FText GetHairAttributeLocText(EHairAttribute In, uint32 InFlags);

///////////////////////////////////////////////////////////////////////////////////////////////////////////
// Array panel for hair strands infos
TSharedRef<SUniformGridPanel> MakeHairStrandsAttributeInfoGrid(const FSlateFontInfo& DetailFontInfo, const uint32 InAttributes, const uint32 InAttributeFlags)
{
	TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);

	const FLinearColor AttributeColor = FLinearColor(FColor::Yellow);

	uint32 SlotIndex = 0;
	for (uint32 AttributeIt = 0, AttributeCount = uint32(EHairAttribute::Count); AttributeIt<AttributeCount; ++AttributeIt)
	{
		const EHairAttribute Attribute = (EHairAttribute)AttributeIt;
		if (HasHairAttribute(InAttributes, Attribute))
		{
			Grid->AddSlot(0, SlotIndex++) // x, y
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Font(DetailFontInfo)
				.ColorAndOpacity(AttributeColor)
				.Text(GetHairAttributeLocText(Attribute, InAttributeFlags))
			];
		}
	}

	return Grid;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////
// Array panel for hair strands curve infos
TSharedRef<SUniformGridPanel> MakeHairStrandsCurveInfoGrid(const FSlateFontInfo& DetailFontInfo, const FHairStrandsBulkData::FHeader& InHeader)
{
	TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);
	
	// Min. point per curve
	Grid->AddSlot(0, 1) // x, y
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_CPPerCurve", "Pt/Curve"))
	];
	Grid->AddSlot(1, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_MinCPPerCurveText", "Min"))
	];
	Grid->AddSlot(2, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_MaxCPPerCurveText", "Max"))
	];
	Grid->AddSlot(3, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_AvgCPPerCurveText", "Avg"))
	];

	// Max. point per curve
	Grid->AddSlot(0, 2) // x, y
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText())
	];
	Grid->AddSlot(1, 2) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(InHeader.MinPointPerCurve))
	];
	Grid->AddSlot(2, 2) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(InHeader.MaxPointPerCurve))
	];
	Grid->AddSlot(3, 2) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(InHeader.AvgPointPerCurve))
	];

	return Grid;
}

void AddHairStrandsCurveWarning(IDetailChildrenBuilder& ChildrenBuilder, const FSlateFontInfo& DetailFontInfo, const FHairStrandsBulkData::FHeader& InHeader)
{
	// Warning if group has trimmed curves or trimmed points
	const FLinearColor ErrorColor = FLinearColor(FColor::Red);
	if (InHeader.Flags & uint32(FHairStrandsBulkData::DataFlags_HasTrimmedCurve))
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairStrandsCurveWarning_Curve", "HairStrandsCurveWarning"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(STextBlock)
			.Font(DetailFontInfo)
			.ColorAndOpacity(ErrorColor)
			.Text(LOCTEXT("HairInfo_TrimmedCurve", "Group has > 4M curves"))
		];
	}
	if (InHeader.Flags & uint32(FHairStrandsBulkData::DataFlags_HasTrimmedPoint))
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairStrandsCurveWarning_Point", "HairStrandsPointWarning"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(STextBlock)
			.Font(DetailFontInfo)
			.ColorAndOpacity(ErrorColor)
			.Text(LOCTEXT("HairInfo_TrimmedPoint", "Group has curve with >255 points."))
		];
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////
// Array panel for hair strands infos
TSharedRef<SUniformGridPanel> MakeHairStrandsLODInfoGrid(const FSlateFontInfo& DetailFontInfo, const FHairLODInfo& LODInfo)
{
	TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);

	// Header
	Grid->AddSlot(1, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_CurveLOD", "Curves"))
	];

	Grid->AddSlot(2, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_PointLOD", "Points"))
	];

	// Strands
	Grid->AddSlot(1, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(LODInfo.CurveCount))
	];
	Grid->AddSlot(2, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(LODInfo.PointCount))
	];
	   
	return Grid;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////
// Array panel for hair strands infos
TSharedRef<SUniformGridPanel> MakeHairStrandsInfoGrid(const FSlateFontInfo& DetailFontInfo, FHairGroupInfoWithVisibility& CurrentAsset, float MaxRadius)
{
	TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);

	// Header
	Grid->AddSlot(1, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Curves", "Curves"))
	];

	Grid->AddSlot(2, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Points", "Points"))
	];

	// Strands
	Grid->AddSlot(0, 1) // x, y
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Strands", "Strands"))
	];
	Grid->AddSlot(1, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(CurrentAsset.NumCurves))
	];
	Grid->AddSlot(2, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(CurrentAsset.NumCurveVertices))
	];

	// Guides
	Grid->AddSlot(0, 2) // x, y
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Guides", "Guides"))
	];
	Grid->AddSlot(1, 2) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(CurrentAsset.NumGuides))
	];
	Grid->AddSlot(2, 2) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(CurrentAsset.NumGuideVertices))
	];

	// Width (mm)
	Grid->AddSlot(0, 3) // x, y
	.HAlign(HAlign_Left)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Width", "Max. Width"))
	];
	Grid->AddSlot(1, 3) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText())
	];
	Grid->AddSlot(2, 3) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(MaxRadius*2.0f))
	];

	return Grid;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////
// Array panel for hair cards infos

TSharedRef<SUniformGridPanel> MakeHairCardsInfoGrid(const FSlateFontInfo& DetailFontInfo, FHairGroupCardsInfo& CardsInfo)
{
	TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);

	// Header
	Grid->AddSlot(0, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairCardsInfo_Curves", "Cards"))
	];

	Grid->AddSlot(1, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairCardsInfo_Vertices", "Vertices"))
	];

	// Strands
	Grid->AddSlot(0, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(CardsInfo.NumCards))
	];
	Grid->AddSlot(1, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(CardsInfo.NumCardVertices))
	];

	return Grid;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Copy/Paste LOD

static bool IsValidLODGroup(UGroomAsset* InAsset, int32 InGroupIndex, int32 InLODIndex)
{
	return InAsset->GetHairGroupsLOD().IsValidIndex(InGroupIndex) && InAsset->GetHairGroupsLOD()[InGroupIndex].LODs.IsValidIndex(InLODIndex);
}

static void CopyLODToJson(UGroomAsset* InAsset, int32 InGroupIndex, int32 InLODIndex, TSharedRef<FJsonObject> OutRootJsonObject)
{
	if (IsValidLODGroup(InAsset, InGroupIndex, InLODIndex))
	{
		FJsonObjectConverter::UStructToJsonObject(FHairLODSettings::StaticStruct(), &InAsset->GetHairGroupsLOD()[InGroupIndex].LODs[InLODIndex], OutRootJsonObject, 0, 0);
	}
}

static void PasteLODFromJson(UGroomAsset* InAsset, int32 InGroupIndex, int32 InLODIndex, TSharedRef<FJsonObject> InRootJsonObject)
{
	if (IsValidLODGroup(InAsset, InGroupIndex, InLODIndex))
	{
		FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairLODSettings::StaticStruct(), &InAsset->GetHairGroupsLOD()[InGroupIndex].LODs[InLODIndex], 0, 0);
	}
}

void FGroomRenderingDetails::OnCopyLODItem(int32 InGroupIndex, int32 InLODIndex)
{
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	CopyLODToJson(GroomAsset, InGroupIndex, InLODIndex, RootJsonObject);

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FGroomRenderingDetails::OnCanCopyLODItem(int32 InGroupIndex, int32 InLODIndex) const
{
	return IsValidLODGroup(GroomAsset, InGroupIndex, InLODIndex);
}

void FGroomRenderingDetails::OnPasteLODItem(int32 InGroupIndex, int32 InLODIndex)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		FProperty* Property = FHairGroupsLOD::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FHairGroupsLOD, LODs));
		if (Property)
		{
			GroomAsset->PreEditChange(Property);

			FScopedTransaction Transaction(LOCTEXT("GroomAssetChangedPasteLOD", "GroomAsset editor: Pasted LOD"));
			GroomAsset->Modify();
			PasteLODFromJson(GroomAsset, InGroupIndex, InLODIndex, RootJsonObject.ToSharedRef());

			CallPostEditChange(Property);
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Copy/Paste Group

static bool IsValidGroup(UGroomAsset* InAsset, int32 InIndex, EMaterialPanelType InType)
{
	switch (InType)
	{
		case EMaterialPanelType::Strands: 		return InAsset->GetHairGroupsRendering().IsValidIndex(InIndex);
		case EMaterialPanelType::Cards:			return InAsset->GetHairGroupsCards().IsValidIndex(InIndex);
		case EMaterialPanelType::Meshes:		return InAsset->GetHairGroupsMeshes().IsValidIndex(InIndex);
		case EMaterialPanelType::Physics:		return InAsset->GetHairGroupsPhysics().IsValidIndex(InIndex);
		case EMaterialPanelType::Interpolation:	return InAsset->GetHairGroupsInterpolation().IsValidIndex(InIndex);
		case EMaterialPanelType::LODs:			return InAsset->GetHairGroupsLOD().IsValidIndex(InIndex);
	}
	return false;
	
}
static void CopyGroupToJson(UGroomAsset* InAsset, int32 InIndex, EMaterialPanelType InType, TSharedRef<FJsonObject> OutRootJsonObject)
{
	if (!IsValidGroup(InAsset, InIndex, InType)) return;

	switch (InType)
	{
		case EMaterialPanelType::Strands:		FJsonObjectConverter::UStructToJsonObject(FHairGroupsRendering::StaticStruct(), &InAsset->GetHairGroupsRendering()[InIndex], OutRootJsonObject, 0, 0); break;
		case EMaterialPanelType::Cards:			FJsonObjectConverter::UStructToJsonObject(FHairGroupsCardsSourceDescription::StaticStruct(), &InAsset->GetHairGroupsCards()[InIndex], OutRootJsonObject, 0, 0); break;
		case EMaterialPanelType::Meshes:		FJsonObjectConverter::UStructToJsonObject(FHairGroupsMeshesSourceDescription::StaticStruct(), &InAsset->GetHairGroupsMeshes()[InIndex], OutRootJsonObject, 0, 0); break;
		case EMaterialPanelType::Physics:		FJsonObjectConverter::UStructToJsonObject(FHairGroupsPhysics::StaticStruct(), &InAsset->GetHairGroupsPhysics()[InIndex], OutRootJsonObject, 0, 0); break;
		case EMaterialPanelType::Interpolation:	FJsonObjectConverter::UStructToJsonObject(FHairGroupsInterpolation::StaticStruct(), &InAsset->GetHairGroupsInterpolation()[InIndex], OutRootJsonObject, 0, 0); break;
		case EMaterialPanelType::LODs:			FJsonObjectConverter::UStructToJsonObject(FHairGroupsLOD::StaticStruct(), &InAsset->GetHairGroupsLOD()[InIndex], OutRootJsonObject, 0, 0); break;
	};
}

static void PasteGroupFromJson(UGroomAsset* InAsset, int32 InIndex, EMaterialPanelType InType, TSharedRef<FJsonObject> InRootJsonObject)
{
	if (!IsValidGroup(InAsset, InIndex, InType)) return;

	switch (InType)
	{
		case EMaterialPanelType::Strands: 		FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairGroupsRendering::StaticStruct(), &InAsset->GetHairGroupsRendering()[InIndex], 0, 0); break;
		case EMaterialPanelType::Cards: 		FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairGroupsCardsSourceDescription::StaticStruct(), &InAsset->GetHairGroupsCards()[InIndex], 0, 0); break;
		case EMaterialPanelType::Meshes:  		FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairGroupsMeshesSourceDescription::StaticStruct(), &InAsset->GetHairGroupsMeshes()[InIndex], 0, 0); break;
		case EMaterialPanelType::Physics:  		FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairGroupsPhysics::StaticStruct(), &InAsset->GetHairGroupsPhysics()[InIndex], 0, 0); break;
		case EMaterialPanelType::Interpolation: FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairGroupsInterpolation::StaticStruct(), &InAsset->GetHairGroupsInterpolation()[InIndex], 0, 0); break;
		case EMaterialPanelType::LODs: 			FJsonObjectConverter::JsonObjectToUStruct(InRootJsonObject, FHairGroupsLOD::StaticStruct(), &InAsset->GetHairGroupsLOD()[InIndex], 0, 0); break;
	};
}

static FProperty* GetGroupPropertyFromType(EMaterialPanelType InType)
{
	switch (InType)
	{
		case EMaterialPanelType::Strands: 		return UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsRenderingMemberName());
		case EMaterialPanelType::Cards:			return UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsCardsMemberName());
		case EMaterialPanelType::Meshes:		return UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsMeshesMemberName());
		case EMaterialPanelType::Physics:		return UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsPhysicsMemberName());
		case EMaterialPanelType::Interpolation:	return UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsInterpolationMemberName());
		case EMaterialPanelType::LODs:			return UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsLODMemberName());
	};
	return nullptr;
}

void FGroomRenderingDetails::OnCopyGroupItem(int32 CurrentSlot, EMaterialPanelType InType)
{
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	CopyGroupToJson(GroomAsset, CurrentSlot, PanelType, RootJsonObject);

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FGroomRenderingDetails::OnCanCopyGroupItem(int32 CurrentSlot, EMaterialPanelType InType) const
{
	return IsValidGroup(GroomAsset, CurrentSlot, InType);
}

void FGroomRenderingDetails::OnPasteGroupItem(int32 CurrentSlot, EMaterialPanelType InType)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		if (FProperty* Property = GetGroupPropertyFromType(PanelType))
		{
			GroomAsset->PreEditChange(Property);

			FScopedTransaction Transaction(LOCTEXT("GroomAssetChangedPasteGroup", "GroomAsset editor: Pasted group"));
			GroomAsset->Modify();
			PasteGroupFromJson(GroomAsset, CurrentSlot, InType, RootJsonObject.ToSharedRef());
			CallPostEditChange(Property);
		}
	}
}

void FGroomRenderingDetails::CallPostEditChange(FProperty* PropertyChanged/*=nullptr*/)
{
	if (PropertyChanged)
	{
		FPropertyChangedEvent PropertyUpdateStruct(PropertyChanged);
		GroomAsset->PostEditChangeProperty(PropertyUpdateStruct);
	}
	else
	{
		GroomAsset->Modify();
		GroomAsset->PostEditChange();
	}
	GroomDetailLayout->ForceRefreshDetails();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Details view

FGroomRenderingDetails::FGroomRenderingDetails(IGroomCustomAssetEditorToolkit* InToolkit, EMaterialPanelType Type)
{
	if (InToolkit)
	{
		GroomAsset = InToolkit->GetCustomAsset();
		Toolkit = InToolkit;
	}
	bDeleteWarningConsumed = false;
	PanelType = Type;
}

FGroomRenderingDetails::~FGroomRenderingDetails()
{

}

TSharedRef<IDetailCustomization> FGroomRenderingDetails::MakeInstance(IGroomCustomAssetEditorToolkit* InToolkit, EMaterialPanelType Type)
{
	return MakeShareable(new FGroomRenderingDetails(InToolkit, Type));
}

FName GetCategoryName(EMaterialPanelType Type)
{
	switch (Type) 
	{
	case EMaterialPanelType::Strands:		return FName(TEXT("Strands"));
	case EMaterialPanelType::Cards:			return FName(TEXT("Cards"));
	case EMaterialPanelType::Meshes:		return FName(TEXT("Meshes"));
	case EMaterialPanelType::Interpolation: return FName(TEXT("Interpolation"));
	case EMaterialPanelType::LODs:			return FName(TEXT("LODs"));
	case EMaterialPanelType::Physics:		return FName(TEXT("Physics"));
	case EMaterialPanelType::Bindings:		return FName(TEXT("Bindings"));
	case EMaterialPanelType::Dataflow:		return FName(TEXT("Dataflow"));
	}
	return FName(TEXT("Unknown"));
}

void FGroomRenderingDetails::ApplyChanges()
{
	GroomDetailLayout->ForceRefreshDetails();
}

void FGroomRenderingDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailLayout.GetSelectedObjects();
	check(SelectedObjects.Num() <= 1); // The OnGenerateCustomWidgets delegate will not be useful if we try to process more than one object.

	FName CategoryName = GetCategoryName(PanelType);
	GroomDetailLayout = &DetailLayout;
	if(SelectedObjects.Num() > 0)
	{
		if (UGroomAsset* LocalGroomAsset = Cast<UGroomAsset>(SelectedObjects[0].Get()))
		{
			GroomAsset = LocalGroomAsset;

			IDetailCategoryBuilder& HairGroupCategory = DetailLayout.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::TypeSpecific);
			CustomizeStrandsGroupProperties(DetailLayout, HairGroupCategory);
		}
		else if (UGroomBindingAssetList* LocalGroomBindingList = Cast<UGroomBindingAssetList>(SelectedObjects[0].Get()))
		{
			GroomBindingAssetList = LocalGroomBindingList;

			IDetailCategoryBuilder& HairGroupCategory = DetailLayout.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::TypeSpecific);
			CustomizeStrandsGroupProperties(DetailLayout, HairGroupCategory);
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Custom widget for material slot for hair rendering

void FGroomRenderingDetails::AddNewGroupButton(IDetailCategoryBuilder& FilesCategory, FProperty* Property, const FText& HeaderText)
{
	// Add a button for adding element to the hair groups array
	FilesCategory.AddCustomRow(FText::FromString(TEXT("AddGroup")))
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 2.f, 10.f, 2.f)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(HeaderText)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &FGroomRenderingDetails::OnAddGroup, Property)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
		]
	];
}

static void AddLODModeProperties(const IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& FilesCategory)
{
	const TSharedRef<IPropertyHandle> LODModeProperty = DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass());
	const TSharedRef<IPropertyHandle> LODBiasProperty = DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass());

	FilesCategory.AddProperty(LODModeProperty);
	FilesCategory.AddProperty(LODBiasProperty);
}

void FGroomRenderingDetails::CustomizeStrandsGroupProperties(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& FilesCategory)
{
	switch (PanelType)
	{
	case EMaterialPanelType::Cards:
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	case EMaterialPanelType::Meshes:
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	case EMaterialPanelType::Strands:
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	case EMaterialPanelType::Physics:
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	case EMaterialPanelType::Interpolation:
	{
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	case EMaterialPanelType::LODs:
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
//		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	case EMaterialPanelType::Dataflow:
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		//DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		//DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}
	break;
	}

	switch (PanelType)
	{
		case EMaterialPanelType::Cards:		
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass());
			AddNewGroupButton(FilesCategory, Property->GetProperty(), FText::FromString(TEXT("Add Card asset")));
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForHairGroup, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("Cards assets")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::Meshes:	
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass());
			AddNewGroupButton(FilesCategory, Property->GetProperty(), FText::FromString(TEXT("Add Mesh asset")));
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForHairGroup, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("Meshes assets")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::Strands:	
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass());
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForHairGroup, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("Strands Groups")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::Physics:
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass());
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForHairGroup, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("Physics Groups")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::Interpolation:
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass());
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForHairGroup, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("Interpolation Groups")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::LODs:
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass());
			AddLODModeProperties(DetailLayout, FilesCategory);
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForHairGroup, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("LOD Groups")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::Bindings:
		{
			TSharedRef<IPropertyHandle> Property = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomBindingAssetList, Bindings), UGroomBindingAssetList::StaticClass());
			if (Property->IsValidHandle())
			{
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForBindingAsset, &DetailLayout));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("Bindings")));
				FilesCategory.AddCustomBuilder(PropertyBuilder, false);
			}
		}
		break;
		case EMaterialPanelType::Dataflow:
		{
			FilesCategory.AddProperty(DetailLayout.GetProperty(FGroomDataflowSettings::GetDataflowAssetMemberName(), UGroomAsset::StaticClass()));
			FilesCategory.AddProperty(DetailLayout.GetProperty(FGroomDataflowSettings::GetDataflowTerminalMemberName(), UGroomAsset::StaticClass()));
		}
		break;
	}
}

FReply FGroomRenderingDetails::OnAddGroup(FProperty* Property)
{
	check(GroomAsset);
	switch (PanelType)
	{
	case EMaterialPanelType::Cards:
	{
		FScopedTransaction Transaction(FText::FromString(TEXT("AddCardsGroup")));
		GroomAsset->GetHairGroupsCards().AddDefaulted();

		const int32 LODCount = GroomAsset->GetHairGroupsCards().Num();
		if (LODCount > 1)
		{
			const FHairGroupsCardsSourceDescription& Prev = GroomAsset->GetHairGroupsCards()[LODCount - 2];
			FHairGroupsCardsSourceDescription& Current = GroomAsset->GetHairGroupsCards()[LODCount - 1];

			Current.GroupIndex = Prev.GroupIndex;
			Current.LODIndex = FMath::Min(Prev.LODIndex + 1, 7);
		}
		else
		{
			FHairGroupsCardsSourceDescription& Current = GroomAsset->GetHairGroupsCards()[LODCount - 1];
			Current.LODIndex = 0;
		}

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayAdd);
		GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
	}
	break;
	case EMaterialPanelType::Meshes:
	{
		FScopedTransaction Transaction(FText::FromString(TEXT("AddMeshesGroup")));
		GroomAsset->GetHairGroupsMeshes().AddDefaulted();

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayAdd);
		GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
	}
	break;
	}

	return FReply::Handled();
}

FReply FGroomRenderingDetails::OnSelectBinding(int32 BindingIndex, FProperty* Property)
{
	check(GroomBindingAssetList);

	// If user click twice onto the same binding index, we disable the binding;
	BindingIndex = Toolkit->GetActiveBindingIndex() == BindingIndex ? -1 : BindingIndex;
	Toolkit->PreviewBinding(BindingIndex);
	ApplyChanges();
	return FReply::Handled();
}

FName& FGroomRenderingDetails::GetMaterialSlotName(int32 GroupIndex)
{
	check(GroomAsset);
	switch (PanelType)
	{
		case EMaterialPanelType::Cards:		return GroomAsset->GetHairGroupsCards()[GroupIndex].MaterialSlotName;
		case EMaterialPanelType::Meshes:	return GroomAsset->GetHairGroupsMeshes()[GroupIndex].MaterialSlotName;
		case EMaterialPanelType::Strands:	return GroomAsset->GetHairGroupsRendering()[GroupIndex].MaterialSlotName;
	}

	static FName Default;
	return Default;
}

const FName& FGroomRenderingDetails::GetMaterialSlotName(int32 GroupIndex) const
{
	static const FName Default(TEXT("Invalid"));
	check(GroomAsset);
	switch (PanelType)
	{
		case EMaterialPanelType::Cards:		return GroupIndex < GroomAsset->GetHairGroupsCards().Num() ? GroomAsset->GetHairGroupsCards()[GroupIndex].MaterialSlotName : Default;
		case EMaterialPanelType::Meshes:	return GroupIndex < GroomAsset->GetHairGroupsMeshes().Num() ? GroomAsset->GetHairGroupsMeshes()[GroupIndex].MaterialSlotName : Default;
		case EMaterialPanelType::Strands:	return GroupIndex < GroomAsset->GetHairGroupsRendering().Num() ? GroomAsset->GetHairGroupsRendering()[GroupIndex].MaterialSlotName : Default;
	}

	return Default;
}

int32 FGroomRenderingDetails::GetGroupCount() const
{
	check(GroomAsset);
	switch (PanelType)
	{
		case EMaterialPanelType::Cards:			return GroomAsset->GetHairGroupsCards().Num();
		case EMaterialPanelType::Meshes:		return GroomAsset->GetHairGroupsMeshes().Num();
		case EMaterialPanelType::Strands:		return GroomAsset->GetHairGroupsRendering().Num();
		case EMaterialPanelType::Physics:		return GroomAsset->GetHairGroupsPhysics().Num();
		case EMaterialPanelType::Interpolation:	return GroomAsset->GetHairGroupsInterpolation().Num();
		case EMaterialPanelType::LODs:			return GroomAsset->GetHairGroupsLOD().Num();
	}

	return 0;
}

FReply FGroomRenderingDetails::OnRemoveGroupClicked(int32 GroupIndex, FProperty* Property)
{
	check(GroomAsset);
	switch (PanelType)
	{
	case EMaterialPanelType::Cards:		
	{
		if (GroupIndex < GroomAsset->GetHairGroupsCards().Num())
		{
			FScopedTransaction Transaction(FText::FromString(TEXT("RemoveCardsGroup")));
			GroomAsset->GetHairGroupsCards().RemoveAt(GroupIndex);

			FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayRemove);
			GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	break;
	case EMaterialPanelType::Meshes:
	{
		if (GroupIndex < GroomAsset->GetHairGroupsMeshes().Num())
		{
			FScopedTransaction Transaction(FText::FromString(TEXT("RemoveMeshesGroup")));
			GroomAsset->GetHairGroupsMeshes().RemoveAt(GroupIndex);

			FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayRemove);
			GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	break;
	}

	return FReply::Handled();
}

void FGroomRenderingDetails::SetMaterialSlot(int32 GroupIndex, int32 MaterialIndex)
{
	if (!GroomAsset || GroupIndex < 0)
	{
		return;
	}

	int32 GroupCount = GetGroupCount();
	if (MaterialIndex == INDEX_NONE)
	{
		GetMaterialSlotName(GroupIndex) = NAME_None;
	}
	else if (GroupIndex < GroupCount && MaterialIndex >= 0 && MaterialIndex < GroomAsset->GetHairGroupsMaterials().Num())
	{
		GetMaterialSlotName(GroupIndex) = GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName;
	}

	GroomAsset->MarkMaterialsHasChanged();
}

TSharedRef<SWidget> FGroomRenderingDetails::OnGenerateStrandsMaterialMenuPicker(int32 GroupIndex)
{
	if (GroomAsset == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	const int32 MaterialCount = GroomAsset->GetHairGroupsMaterials().Num();
	if(MaterialCount == 0)
	{
		return SNullWidget::NullWidget;
	}
	FMenuBuilder MenuBuilder(true, NULL);

	// Default material
	{
		int32 MaterialIt = INDEX_NONE;
		FText DefaultString = FText::FromString(TEXT("Default"));
		FUIAction Action(FExecuteAction::CreateSP(this, &FGroomRenderingDetails::SetMaterialSlot, GroupIndex, MaterialIt));
		MenuBuilder.AddMenuEntry(DefaultString, FText::GetEmpty(), FSlateIcon(), Action);
	}

	// Add a menu item for material
	for (int32 MaterialIt = 0; MaterialIt < MaterialCount; ++MaterialIt)
	{
		FText MaterialString = FText::FromString(FString::FromInt(MaterialIt) + TEXT(" - ") + GroomAsset->GetHairGroupsMaterials()[MaterialIt].SlotName.ToString());
		FUIAction Action(FExecuteAction::CreateSP(this, &FGroomRenderingDetails::SetMaterialSlot, GroupIndex, MaterialIt));
		MenuBuilder.AddMenuEntry(MaterialString, FText::GetEmpty(), FSlateIcon(), Action);
	}

	return MenuBuilder.MakeWidget();
}

FText FGroomRenderingDetails::GetStrandsMaterialName(int32 GroupIndex) const
{
	const FName& MaterialSlotName = GetMaterialSlotName(GroupIndex);
	const int32 MaterialIndex = GroomAsset->GetMaterialIndex(MaterialSlotName);
	FText MaterialString = FText::FromString(TEXT("Default"));
	if (MaterialIndex != INDEX_NONE)
	{
		MaterialString = FText::FromString(FString::FromInt(MaterialIndex) + TEXT(" - ") + GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName.ToString());
	}
	return MaterialString;
}

TSharedRef<SWidget> FGroomRenderingDetails::OnGenerateStrandsMaterialPicker(int32 GroupIndex, IDetailLayoutBuilder* DetailLayoutBuilder)
{
	return CreateMaterialSwatch(DetailLayoutBuilder->GetThumbnailPool(), GroupIndex);
}

bool FGroomRenderingDetails::IsStrandsMaterialPickerEnabled(int32 GroupIndex) const
{
	return true;
}


template <typename T>
bool AssignIfDifferent(T& Dest, const T& Src, bool bSetValue)
{
	const bool bHasChanged = Dest != Src;
	if (bHasChanged && bSetValue)
	{
		Dest = Src;
	}
	return bHasChanged;
}
	
#define HAIR_RESET0(GroomMemberName, StructTypeName, MemberName)										{ if (PropertyName == GET_MEMBER_NAME_CHECKED(StructTypeName, MemberName)) { bHasChanged = AssignIfDifferent(GroomAsset->GroomMemberName[GroupIndex].MemberName, Default.MemberName, bSetValue); } }
#define HAIR_RESET1(GroomMemberName, StructTypeName, StructMemberName, MemberName)						{ if (PropertyName == GET_MEMBER_NAME_CHECKED(StructTypeName, MemberName)) { bHasChanged = AssignIfDifferent(GroomAsset->GroomMemberName[GroupIndex].StructMemberName.MemberName, Default.MemberName, bSetValue); } }
#define HAIR_RESET2(GroomMemberName, StructTypeName, StructMemberName, SubStructMemberName, MemberName)	{ if (PropertyName == GET_MEMBER_NAME_CHECKED(StructTypeName, MemberName)) { bHasChanged = AssignIfDifferent(GroomAsset->GroomMemberName[GroupIndex].StructMemberName.SubStructMemberName.MemberName, Default.MemberName, bSetValue); } }

bool FGroomRenderingDetails::CommonResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, int32 LODIndex, bool bSetValue)
{
	bool bHasChanged = false;
	if (ChildHandle == nullptr || GroomAsset == nullptr || GroupIndex < 0)
	{
		return bHasChanged;
	}

	FName PropertyName = ChildHandle->GetProperty()->GetFName();

	// For cards & meshes the incoming index is actually the cards/mesh description index, not the group index
	// For the rest, the group index refers to the actual group index.
	const bool bIsCardDescIndexValid = GroupIndex < GroomAsset->GetHairGroupsCards().Num();
	const bool bIsMeshDescIndexValid = GroupIndex < GroomAsset->GetHairGroupsMeshes().Num();
	const bool bIsGroupIndexValid = GroupIndex < GroomAsset->GetNumHairGroups();
		
	// Hair strands
	if (bIsGroupIndexValid)
	{
		{
			FHairGeometrySettings Default;
			HAIR_RESET1(GetHairGroupsRendering(), FHairGeometrySettings, GeometrySettings, HairWidth);
			HAIR_RESET1(GetHairGroupsRendering(), FHairGeometrySettings, GeometrySettings, HairRootScale);
			HAIR_RESET1(GetHairGroupsRendering(), FHairGeometrySettings, GeometrySettings, HairTipScale);
		}

		{
			FHairShadowSettings Default;
			HAIR_RESET1(GetHairGroupsRendering(), FHairShadowSettings, ShadowSettings, bVoxelize);
			HAIR_RESET1(GetHairGroupsRendering(), FHairShadowSettings, ShadowSettings, bUseHairRaytracingGeometry);
			HAIR_RESET1(GetHairGroupsRendering(), FHairShadowSettings, ShadowSettings, HairRaytracingRadiusScale);
			HAIR_RESET1(GetHairGroupsRendering(), FHairShadowSettings, ShadowSettings, HairRaytracingRadiusScale);
		}

		{
			FHairAdvancedRenderingSettings Default;
			HAIR_RESET1(GetHairGroupsRendering(), FHairAdvancedRenderingSettings, AdvancedSettings, bScatterSceneLighting);
			HAIR_RESET1(GetHairGroupsRendering(), FHairAdvancedRenderingSettings, AdvancedSettings, bUseStableRasterization);
		}
	}

	// Interpolation
	if (bIsGroupIndexValid)
	{
		{
			FHairDecimationSettings Default;
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairDecimationSettings, DecimationSettings, CurveDecimation);
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairDecimationSettings, DecimationSettings, VertexDecimation);
		}

		{
			FHairInterpolationSettings Default;
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairInterpolationSettings, InterpolationSettings, GuideType);
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairInterpolationSettings, InterpolationSettings, HairToGuideDensity);
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairInterpolationSettings, InterpolationSettings, bRandomizeGuide);
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairInterpolationSettings, InterpolationSettings, bUseUniqueGuide);
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairInterpolationSettings, InterpolationSettings, RiggedGuideNumCurves);
			HAIR_RESET1(GetHairGroupsInterpolation(), FHairInterpolationSettings, InterpolationSettings, RiggedGuideNumPoints);
		}
	}

	// LODs
	if (bIsGroupIndexValid)
	{
		if (LODIndex>=0 && LODIndex < GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num())
		{
			FHairLODSettings Default;
			HAIR_RESET1(GetHairGroupsLOD(), FHairLODSettings, LODs[LODIndex], CurveDecimation);
			HAIR_RESET1(GetHairGroupsLOD(), FHairLODSettings, LODs[LODIndex], VertexDecimation);
			HAIR_RESET1(GetHairGroupsLOD(), FHairLODSettings, LODs[LODIndex], AngularThreshold);
			HAIR_RESET1(GetHairGroupsLOD(), FHairLODSettings, LODs[LODIndex], ScreenSize);
			HAIR_RESET1(GetHairGroupsLOD(), FHairLODSettings, LODs[LODIndex], ThicknessScale);
			HAIR_RESET1(GetHairGroupsLOD(), FHairLODSettings, LODs[LODIndex], GeometryType);
		}
	}

	// Cards
	if (bIsCardDescIndexValid)
	{
		{
			FHairGroupCardsTextures Default;
			HAIR_RESET1(GetHairGroupsCards(), FHairGroupCardsTextures, Textures, Layout);
			HAIR_RESET1(GetHairGroupsCards(), FHairGroupCardsTextures, Textures, Textures);
		}
	}

	// Meshes
	if (bIsMeshDescIndexValid)
	{
		{
			FHairGroupsMeshesSourceDescription Default;
			HAIR_RESET0(GetHairGroupsMeshes(), FHairGroupsMeshesSourceDescription, ImportedMesh);
			HAIR_RESET0(GetHairGroupsMeshes(), FHairGroupsMeshesSourceDescription, GroupIndex);
			HAIR_RESET0(GetHairGroupsMeshes(), FHairGroupsMeshesSourceDescription, LODIndex);
		}

		{
			FHairGroupCardsTextures Default;
			HAIR_RESET1(GetHairGroupsMeshes(), FHairGroupCardsTextures, Textures, Layout);
			HAIR_RESET1(GetHairGroupsMeshes(), FHairGroupCardsTextures, Textures, Textures);
		}
	}

	// Physics
	if (bIsGroupIndexValid)
	{
		{
			FHairSolverSettings Default;
			HAIR_RESET1(GetHairGroupsPhysics(), FHairSolverSettings, SolverSettings, EnableSimulation);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairSolverSettings, SolverSettings, NiagaraSolver);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairSolverSettings, SolverSettings, CustomSystem);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairSolverSettings, SolverSettings, SubSteps);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairSolverSettings, SolverSettings, IterationCount);
		}

		{
			FHairExternalForces Default;
			HAIR_RESET1(GetHairGroupsPhysics(), FHairExternalForces, ExternalForces, GravityVector);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairExternalForces, ExternalForces, AirDrag);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairExternalForces, ExternalForces, AirVelocity);
		}

		{
			FHairBendConstraint Default;
			HAIR_RESET2(GetHairGroupsPhysics(), FHairBendConstraint, MaterialConstraints, BendConstraint, SolveBend);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairBendConstraint, MaterialConstraints, BendConstraint, ProjectBend);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairBendConstraint, MaterialConstraints, BendConstraint, BendDamping);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairBendConstraint, MaterialConstraints, BendConstraint, BendStiffness);
//			HAIR_RESET2(GetHairGroupsPhysics(), FHairBendConstraint, MaterialConstraints, BendConstraint, BendScale);
		}

		{
			FHairStretchConstraint Default;
			HAIR_RESET2(GetHairGroupsPhysics(), FHairStretchConstraint, MaterialConstraints, StretchConstraint, SolveStretch);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairStretchConstraint, MaterialConstraints, StretchConstraint, ProjectStretch);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairStretchConstraint, MaterialConstraints, StretchConstraint, StretchDamping);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairStretchConstraint, MaterialConstraints, StretchConstraint, StretchStiffness);
//			HAIR_RESET2(GetHairGroupsPhysics(), FHairStretchConstraint, MaterialConstraints, StretchConstraint, StretchScale);
		}

		{
			FHairCollisionConstraint Default;
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, SolveCollision);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, ProjectCollision);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, StaticFriction);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, KineticFriction);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, StrandsViscosity);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, GridDimension);
			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, CollisionRadius);
//			HAIR_RESET2(GetHairGroupsPhysics(), FHairCollisionConstraint, MaterialConstraints, CollisionConstraint, RadiusScale);
		}
		

		{
			FHairStrandsParameters Default;
			HAIR_RESET1(GetHairGroupsPhysics(), FHairStrandsParameters, StrandsParameters, StrandsSize);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairStrandsParameters, StrandsParameters, StrandsDensity);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairStrandsParameters, StrandsParameters, StrandsSmoothing);
			HAIR_RESET1(GetHairGroupsPhysics(), FHairStrandsParameters, StrandsParameters, StrandsThickness);
//			HAIR_RESET1(GetHairGroupsPhysics(), FHairStrandsParameters, StrandsParameters, ThicknessScale);
		}
	}

	if (bSetValue && bHasChanged)
	{
		FPropertyChangedEvent PropertyChangedEvent(ChildHandle->GetProperty(), EPropertyChangeType::ValueSet);
		GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
	}
	return bHasChanged;
}

bool FGroomRenderingDetails::ShouldResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, int32 LODIndex)
{
	return CommonResetToDefault(ChildHandle, GroupIndex, LODIndex, false);
}

void FGroomRenderingDetails::ResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, int32 LODIndex)
{
	CommonResetToDefault(ChildHandle, GroupIndex, LODIndex, true);
}

IDetailPropertyRow& FGroomRenderingDetails::AddPropertyWithCustomReset(TSharedPtr<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& Builder, int32 GroupIndex, int32 LODIndex)
{	
	FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FGroomRenderingDetails::ShouldResetToDefault, GroupIndex, LODIndex);
	FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FGroomRenderingDetails::ResetToDefault, GroupIndex, LODIndex);
	FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
	return Builder.AddProperty(PropertyHandle.ToSharedRef()).OverrideResetToDefault(ResetOverride);
}

void FGroomRenderingDetails::ExpandStructForLOD(TSharedRef<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& ChildrenBuilder, int32 GroupIndex, int32 LODIndex, bool bOverrideReset)
{
	uint32 ChildrenCount = 0;
	PropertyHandle->GetNumChildren(ChildrenCount);
	for (uint32 ChildIt = 0; ChildIt < ChildrenCount; ++ChildIt)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIt);
		const FName ChildPropertyName = ChildHandle->GetProperty()->GetFName();

		const bool bIsManualLODStrandsProperties = 
			(ChildPropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, CurveDecimation) ||
			 ChildPropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, VertexDecimation) ||
			 ChildPropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, AngularThreshold) ||
			 ChildPropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, ThicknessScale));
		const bool bIsScreenSize = ChildPropertyName == GET_MEMBER_NAME_CHECKED(FHairLODSettings, ScreenSize);

		// If the geometry type is not strands, then bypass the display of the strands related property
		if (GroomAsset->GetGeometryType(GroupIndex, LODIndex) != EGroomGeometryType::Strands && bIsManualLODStrandsProperties)
		{
			continue;
		}

		// If using AutoLOD, then the following property are not editable, since they are dedicated to Manual LOD mode
		const bool bAutoLOD = GroomAsset->GetLODMode() == EGroomLODMode::Auto;
		const bool bIsVisible = bAutoLOD ? !bIsManualLODStrandsProperties && !bIsScreenSize : true;

		if (bOverrideReset)
		{
			FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FGroomRenderingDetails::ShouldResetToDefault, GroupIndex, LODIndex);
			FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FGroomRenderingDetails::ResetToDefault, GroupIndex, LODIndex);
			FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
			IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef()).OverrideResetToDefault(ResetOverride);
			Row.IsEnabled(bIsVisible);
		}
		else
		{
			IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
			Row.IsEnabled(bIsVisible);
		}
	}
}

void FGroomRenderingDetails::ExpandStruct(TSharedPtr<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& ChildrenBuilder, int32 GroupIndex, int32 LODIndex, bool bOverrideReset)
{

	uint32 ChildrenCount = 0;
	PropertyHandle->GetNumChildren(ChildrenCount);
	for (uint32 ChildIt = 0; ChildIt < ChildrenCount; ++ChildIt)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIt);
		if (bOverrideReset)
		{
			FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FGroomRenderingDetails::ShouldResetToDefault, GroupIndex, LODIndex);
			FResetToDefaultHandler ResetHandler     = FResetToDefaultHandler::CreateSP(this, &FGroomRenderingDetails::ResetToDefault, GroupIndex, LODIndex);
			FResetToDefaultOverride ResetOverride   = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
			ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef()).OverrideResetToDefault(ResetOverride);
		}
		else
		{
			ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

void FGroomRenderingDetails::ExpandStruct(TSharedRef<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& ChildrenBuilder, int32 GroupIndex, int32 LODIndex, bool bOverrideReset)
{
	uint32 ChildrenCount = 0;
	PropertyHandle->GetNumChildren(ChildrenCount);
	for (uint32 ChildIt = 0; ChildIt < ChildrenCount; ++ChildIt)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIt);
		if (bOverrideReset)
		{
			FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FGroomRenderingDetails::ShouldResetToDefault, GroupIndex, LODIndex);
			FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FGroomRenderingDetails::ResetToDefault, GroupIndex, LODIndex);
			FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
			ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef()).OverrideResetToDefault(ResetOverride);
		}
		else
		{
			ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

void FGroomRenderingDetails::AddPropertySeparator(FName PropertyName, IDetailChildrenBuilder& ChildrenBuilder)
{
	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");
	float OtherMargin = 0.0f;
	float RightMargin = 10.0f;
	const FLinearColor SeparatorColor(0.05f, 0.05f,0.05f);
	ChildrenBuilder.AddCustomRow(LOCTEXT("Hair_Separator", "Separator"))
	.WholeRowContent()
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(GenericBrush)
			.ColorAndOpacity(SeparatorColor)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(RightMargin, OtherMargin, RightMargin, OtherMargin)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FLinearColor(0.9f, 0.9f,0.9f))
				.Text(FText::FromName(PropertyName))
			]						
		]
	];
}

FReply FGroomRenderingDetails::OnRemoveLODClicked(int32 GroupIndex, int32 LODIndex, FProperty* Property)
{
	check(GroomAsset);
	if (GroupIndex < GroomAsset->GetHairGroupsLOD().Num() && LODIndex >= 0 && LODIndex < GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num() && GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num() > 1)
	{
		FScopedTransaction Transaction(FText::FromString(TEXT("RemoveLOD")));

		GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.RemoveAt(LODIndex);

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayRemove);
		GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
	}
	return FReply::Handled();
}

FReply FGroomRenderingDetails::OnAddLODClicked(int32 GroupIndex, FProperty* Property)
{
	const int32 MaxLODCount = 8; 
	if (GroupIndex < GroomAsset->GetHairGroupsLOD().Num() && GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num() < MaxLODCount)
	{
		FScopedTransaction Transaction(FText::FromString(TEXT("AddLOD")));

		GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.AddDefaulted();
		const int32 LODCount = GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs.Num();
		if (LODCount > 1)
		{
			const FHairLODSettings& PrevLODSettings = GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs[LODCount-2];
			FHairLODSettings& LODSettings = GroomAsset->GetHairGroupsLOD()[GroupIndex].LODs[LODCount-1];

			// Prefill the LOD setting with basic preset
			LODSettings.VertexDecimation = PrevLODSettings.VertexDecimation * 0.5f;
			LODSettings.AngularThreshold = PrevLODSettings.AngularThreshold * 2.f;
			LODSettings.CurveDecimation = PrevLODSettings.CurveDecimation * 0.5f;
			LODSettings.ScreenSize = PrevLODSettings.ScreenSize * 0.5f;
			LODSettings.GeometryType = PrevLODSettings.GeometryType;
		}

		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ArrayAdd);
		GroomAsset->PostEditChangeProperty(PropertyChangedEvent);
	}
	return FReply::Handled();
}

FReply FGroomRenderingDetails::OnGenerateCardDataUsingPlugin(int32 GroupIndex)
{
	if (GroomAsset && GroomAsset->GetHairGroupsCards().IsValidIndex(GroupIndex))
	{
		TArray<IHairCardGenerator*> HairCardGeneratorPlugins = IModularFeatures::Get().GetModularFeatureImplementations<IHairCardGenerator>(IHairCardGenerator::ModularFeatureName);
		if (HairCardGeneratorPlugins.Num() > 0)
		{
			UE_CLOGF(HairCardGeneratorPlugins.Num() > 1, LogGroomAssetDetails, Warning, "There are more than one available hair-card generator options. Defaulting to the first one found.");

			const FScopedTransaction Transaction(LOCTEXT("GenerateHairCardsTransaction", "Generate hair cards."));

			// Use a copy so we can only apply changes on success
			FHairGroupsCardsSourceDescription HairCardsCopy = GroomAsset->GetHairGroupsCards()[GroupIndex];
			// Clear fields that are supposed to be set by the generation (in case it leaves any unset, and we don't cary over old settings)
			HairCardsCopy.Textures = FHairGroupCardsTextures();

			const bool bSuccess = HairCardGeneratorPlugins[0]->GenerateHairCardsForLOD(GroomAsset, HairCardsCopy);
			if (bSuccess)
			{
				GroomAsset->Modify();
				GroomAsset->GetHairGroupsCards()[GroupIndex] = HairCardsCopy;
			}
		}
	}
	return FReply::Handled();
}

void FGroomRenderingDetails::OnGenerateElementForLODs(TSharedRef<IPropertyHandle> StructProperty, int32 LODIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout, int32 GroupIndex)
{
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	FProperty* Property = StructProperty->GetProperty();

	const FLinearColor LODColorBlock = GetHairGroupDebugColor(GroupIndex) * 0.25f;
	const FLinearColor LODNameColor(FLinearColor::White);
	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");
	float OtherMargin = 2.0f;

	// LOD Bar with add button
	ChildrenBuilder.AddCustomRow(LOCTEXT("HairInfo_Separator", "Separator"))
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FGroomRenderingDetails::OnCopyLODItem, GroupIndex, LODIndex), FCanExecuteAction::CreateSP(this, &FGroomRenderingDetails::OnCanCopyLODItem, GroupIndex, LODIndex)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FGroomRenderingDetails::OnPasteLODItem, GroupIndex, LODIndex)))
	.WholeRowContent()
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(GenericBrush)
			.ColorAndOpacity(LODColorBlock)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(OtherMargin)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(LODNameColor)
				.Text(FText::Format(LOCTEXT("LOD", "LOD {0}"), FText::AsNumber(LODIndex)))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FGroomRenderingDetails::OnRemoveLODClicked, GroupIndex, LODIndex, Property)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
			
		]
	];

	// LOD Stats
	const FHairStrandsClusterBulkData& ClusterBulkData = GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.ClusterBulkData;
	if (ClusterBulkData.IsValid() && LODIndex < ClusterBulkData.Header.LODInfos.Num())
	{
		const FHairLODInfo& LODInfo = ClusterBulkData.Header.LODInfos[LODIndex];

		ChildrenBuilder.AddCustomRow(LOCTEXT("HairStrandsLODInfo_Array", "HairStrandsLODInfo"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			MakeHairStrandsLODInfoGrid(DetailFontInfo, LODInfo)
		];
	}
	
	// Rename the array entry name by its group name and adds all its existing properties
	StructProperty->SetPropertyDisplayName(LOCTEXT("LODProperties", "LOD Properties"));
	ExpandStructForLOD(StructProperty, ChildrenBuilder, GroupIndex, LODIndex, true); ///
}

TSharedRef<SWidget> FGroomRenderingDetails::MakeGroupNameButtonCustomization(int32 GroupIndex, FProperty* Property)
{
	switch (PanelType)
	{
	case EMaterialPanelType::LODs:
	{
		return SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &FGroomRenderingDetails::OnAddLODClicked, GroupIndex, Property)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			];
	}
	case EMaterialPanelType::Cards:
	{
		return SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &FGroomRenderingDetails::OnRemoveGroupClicked, GroupIndex, Property)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
			];
	}
	case EMaterialPanelType::Meshes:
	{
		return SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &FGroomRenderingDetails::OnRemoveGroupClicked, GroupIndex, Property)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
			];
	}
	}

	return SNullWidget::NullWidget;
}

static FName GetGroupName(const UGroomAsset* GroomAsset, int32 GroupIndex)
{
	if (GroomAsset && GroupIndex >= 0 && GroupIndex < GroomAsset->GetHairGroupsInfo().Num())
	{
		return GroomAsset->GetHairGroupsInfo()[GroupIndex].GroupName;
	}
	return NAME_None;
}

TSharedRef<SWidget> GetGroupNameWidget(const UGroomAsset* GroomAsset, int32 GroupIndex, const FLinearColor& GroupColor)
{
	FName GroupName = GetGroupName(GroomAsset, GroupIndex);
	if (GroupName != NAME_None)
	{
		return SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(GroupColor)
			.Text(FText::Format(LOCTEXT("GroupWithName", "Group ID {0} - {1}"), FText::AsNumber(GroupIndex), FText::FromName(GroupName)));
	}
	else
	{
		return SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(GroupColor)
			.Text(FText::Format(LOCTEXT("GroupWithoutName", "Group ID {0}"), FText::AsNumber(GroupIndex)));
	}
}

TSharedRef<SWidget> FGroomRenderingDetails::MakeGroupNameCustomization(int32 GroupIndex, const FLinearColor& GroupTextColor)
{
	switch (PanelType)
	{
	case EMaterialPanelType::Cards:
	{
		return SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(GroupTextColor)
			.Text(FText::Format(LOCTEXT("Cards", "Cards {0} "), FText::AsNumber(GroupIndex)));
	}
	case EMaterialPanelType::Meshes:
	{
		return SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(GroupTextColor)
			.Text(FText::Format(LOCTEXT("Meshes", "Meshes {0} "), FText::AsNumber(GroupIndex)));
	}
	default:
	{
		return GetGroupNameWidget(GroomAsset, GroupIndex, GroupTextColor);
	}
	}
}

// Hair group custom display
void FGroomRenderingDetails::OnGenerateElementForHairGroup(TSharedRef<IPropertyHandle> StructProperty, int32 GroupIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	const FLinearColor GroupColorBlock = GetHairGroupDebugColor(GroupIndex) * 0.75f;
	const FLinearColor GroupNameColor(FLinearColor::White);

	FProperty* Property = StructProperty->GetProperty();

	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");

	float OtherMargin = 2.0f;
	float RightMargin = 2.0f;
	if (PanelType != EMaterialPanelType::LODs && PanelType != EMaterialPanelType::Cards && PanelType != EMaterialPanelType::Meshes)
	{
		RightMargin = 10.0f;
	}

	ChildrenBuilder.AddCustomRow(LOCTEXT("HairInfo_Separator", "Separator"))
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FGroomRenderingDetails::OnCopyGroupItem, GroupIndex, PanelType), FCanExecuteAction::CreateSP(this, &FGroomRenderingDetails::OnCanCopyGroupItem, GroupIndex, PanelType)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FGroomRenderingDetails::OnPasteGroupItem, GroupIndex, PanelType)))
	.WholeRowContent()
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(GenericBrush)
			.ColorAndOpacity(GroupColorBlock)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(OtherMargin, OtherMargin, RightMargin, OtherMargin)
			[
				MakeGroupNameCustomization(GroupIndex, GroupNameColor)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				MakeGroupNameButtonCustomization(GroupIndex, Property)
			]
			
		]
	];

	if (GroomAsset != nullptr && GroupIndex>=0 && GroupIndex < GroomAsset->GetHairGroupsInfo().Num() && (PanelType == EMaterialPanelType::Strands || PanelType == EMaterialPanelType::Interpolation))
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairStrandsInfo_Array", "HairStrandsInfo"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			MakeHairStrandsInfoGrid(DetailFontInfo, GroomAsset->GetHairGroupsInfo()[GroupIndex], GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.BulkData.Header.MaxRadius)
		];
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairStrandsCurveInfo_Array", "HairStrandsCurveInfo"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			MakeHairStrandsCurveInfoGrid(DetailFontInfo, GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.BulkData.Header)
		];
		AddHairStrandsCurveWarning(ChildrenBuilder, DetailFontInfo, GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.BulkData.Header);
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairStrandsAttributeInfo_Array", "HairStrandsAttributeInfo"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			MakeHairStrandsAttributeInfoGrid(DetailFontInfo, GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.BulkData.Header.ImportedAttributes, GroomAsset->GetHairGroupsPlatformData()[GroupIndex].Strands.BulkData.Header.ImportedAttributeFlags)
		];
	}

	if (GroomAsset != nullptr && GroupIndex >= 0 && GroupIndex < GroomAsset->GetHairGroupsCards().Num() && (PanelType == EMaterialPanelType::Cards))
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairCardsInfo_Array", "HairCardsInfo"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			MakeHairCardsInfoGrid(DetailFontInfo, GroomAsset->GetHairGroupsCards()[GroupIndex].CardsInfo)
		];
	}

	// Display a material picker for strands/cards/meshes panel. This material picker allows to select material among the one valid for this current asset, i.e., 
	// materials which have been added by the user within the material panel
	if (PanelType == EMaterialPanelType::Strands || PanelType == EMaterialPanelType::Cards || PanelType == EMaterialPanelType::Meshes)
	{
		FResetToDefaultOverride ResetToDefaultOverride = FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateSP(this, &FGroomRenderingDetails::GetReplaceVisibility),
			FResetToDefaultHandler::CreateSP(this, &FGroomRenderingDetails::OnResetToBaseClicked)
		);

		ChildrenBuilder.AddCustomRow(LOCTEXT("HairGroup_Material", "Material"))
		.OverrideResetToDefault(ResetToDefaultOverride)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("HairRenderingGroup_Label_Material", "Material"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(this, &FGroomRenderingDetails::IsStrandsMaterialPickerEnabled, GroupIndex)
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		.MaxDesiredWidth(0.0f) // no maximum
		[
			OnGenerateStrandsMaterialPicker(GroupIndex, DetailLayout)
		];
	}

	// Rename the array entry name by its group name and adds all its existing properties
	StructProperty->SetPropertyDisplayName(LOCTEXT("GroupProperties", "Properties"));

	uint32 ChildrenCount = 0;
	StructProperty->GetNumChildren(ChildrenCount);
	for (uint32 ChildIt = 0; ChildIt < ChildrenCount; ++ChildIt)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructProperty->GetChildHandle(ChildIt);
		FName PropertyName = ChildHandle->GetProperty()->GetFName();

		switch (PanelType)
		{
		case EMaterialPanelType::Strands:
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsRendering, GeometrySettings) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsRendering, ShadowSettings) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsRendering, AdvancedSettings))
			{
				ExpandStruct(ChildHandle, ChildrenBuilder, GroupIndex, -1, true);
			}
			else 
			{
				ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
			}
		}
		break;
		case EMaterialPanelType::Cards:
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, CardsInfo))
			{
				// Not node display
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, Textures))
			{
				EHairTextureLayout LayoutType = EHairTextureLayout::Layout0;
				if (GroomAsset != nullptr && GroomAsset->GetHairGroupsCards().IsValidIndex(GroupIndex))
				{
					LayoutType = GroomAsset->GetHairGroupsCards()[GroupIndex].Textures.Layout;
				}

				IDetailGroup& TextureGroup = ChildrenBuilder.AddGroup(TEXT("HairCardsTextures"), LOCTEXT("HairCardsTextures", "Textures"));
				{
					// Layout type
					TextureGroup.AddPropertyRow(ChildHandle->GetChildHandle(0).ToSharedRef());

					// Textures
					const uint32 TextureCount = GroomAsset->GetHairGroupsCards()[GroupIndex].Textures.Textures.Num();
					TSharedPtr<IPropertyHandle> TextureArrayHandle = ChildHandle->GetChildHandle(1);
					for (uint32 TextureIt = 0; TextureIt < TextureCount; ++TextureIt)
					{
						TextureGroup.AddPropertyRow(TextureArrayHandle->GetChildHandle(TextureIt).ToSharedRef()).DisplayName(FTextStringHelper::CreateFromBuffer(GetHairTextureLayoutTextureName(LayoutType, TextureIt, true /*bDetail*/)));
					}
				}
			}
			else
			{
				IDetailPropertyRow& PropertyRow = AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
				
				if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsCardsSourceDescription, ImportedMesh))
				{
					TSharedPtr<SWidget> NameWidget;
					TSharedPtr<SWidget> ValueWidget;
					FDetailWidgetRow Row;
					PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

					TArray<IHairCardGenerator*> HairCardGeneratorPlugins = IModularFeatures::Get().GetModularFeatureImplementations<IHairCardGenerator>(IHairCardGenerator::ModularFeatureName);
					bool bHasProceduralGenerationPlugin = HairCardGeneratorPlugins.Num() > 0;
					TAttribute<EVisibility> ProceduralPropertyVisibility = TAttribute<EVisibility>::CreateLambda([bHasProceduralGenerationPlugin]()
						{
							return bHasProceduralGenerationPlugin ? EVisibility::Visible : EVisibility::Collapsed;
						}
					);

					const FSlateBrush* Brush = nullptr;
					if (const ISlateStyle* AppStyle = Toolkit->GetSlateStyle())
					{
						Brush = AppStyle->GetBrush("GroomEditor.GroomCardGenerator");
					}
					else
					{
						Brush = FAppStyle::GetBrush("ContentBrowser.AssetActions.ReimportAsset");
					}

					TSharedRef<SHorizontalBox> ProceduralGenButtons = SNew(SHorizontalBox).Visibility(ProceduralPropertyVisibility);
					if (bHasProceduralGenerationPlugin)
					{
						FText RegenToolTipText = LOCTEXT("GenerateNewCardsTooltip", "Generate new card assets (meshes and textures) using the current procedural settings. NOTE: This will overwrite preexisting card assets for this LOD.");
						ProceduralGenButtons->AddSlot()
							.AutoWidth()
							.Padding(0.0f, 2.0f, 0.0f, 4.0f)
							[
								SNew(SButton)
									.VAlign(VAlign_Center)
									.HAlign(HAlign_Center)
									.ContentPadding(0)
									.ButtonStyle(FAppStyle::Get(), "Button")
									.ToolTipText(RegenToolTipText)
									.OnClicked(this, &FGroomRenderingDetails::OnGenerateCardDataUsingPlugin, GroupIndex)
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(VAlign_Center)
										.HAlign(HAlign_Center)
										.Padding(-1.0f, 2.0f, 2.0f, 2.0f)
										[
											SNew(SImage)
												.Image(Brush)
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(VAlign_Center)
										.HAlign(HAlign_Center)
										.Padding(2.0f, 2.0f, -1.0f, 2.0f)
										[
											SNew(STextBlock)
												.Text(LOCTEXT("GenerateNewCardsButtonText","Generate Hair Cards"))
										]
									]
							];
					}

					PropertyRow.CustomWidget()
						.NameContent()
						[
							NameWidget.ToSharedRef()
						]
						.ValueContent()
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								ValueWidget.ToSharedRef()
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								ProceduralGenButtons
							]
						];
				}
			}
		}
		break;
		case EMaterialPanelType::Meshes:
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsMeshesSourceDescription, Textures))
			{
				EHairTextureLayout LayoutType = EHairTextureLayout::Layout0;
				if (GroomAsset != nullptr && GroomAsset->GetHairGroupsMeshes().IsValidIndex(GroupIndex))
				{
					LayoutType = GroomAsset->GetHairGroupsMeshes()[GroupIndex].Textures.Layout;
				}

				IDetailGroup& TextureGroup = ChildrenBuilder.AddGroup(TEXT("HairMeshesTextures"), LOCTEXT("HairMeshesTextures", "Textures"));
				{
					// Layout type
					TextureGroup.AddPropertyRow(ChildHandle->GetChildHandle(0).ToSharedRef());

					// Textures
					const uint32 TextureCount = GroomAsset->GetHairGroupsMeshes()[GroupIndex].Textures.Textures.Num();
					TSharedPtr<IPropertyHandle> TextureArrayHandle = ChildHandle->GetChildHandle(1);
					for (uint32 TextureIt = 0; TextureIt < TextureCount; ++TextureIt)
					{
						TextureGroup.AddPropertyRow(TextureArrayHandle->GetChildHandle(TextureIt).ToSharedRef()).DisplayName(FTextStringHelper::CreateFromBuffer(GetHairTextureLayoutTextureName(LayoutType, TextureIt, true /*bDetail*/)));
					}
				}
			}
			else
			{
				AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
			}
		}
		break;
		case EMaterialPanelType::Interpolation:
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsInterpolation, DecimationSettings) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsInterpolation, InterpolationSettings))
			{
				ExpandStruct(ChildHandle, ChildrenBuilder, GroupIndex, -1, true);
			}
			else
			{
				AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
			}
		}
		break;
		case EMaterialPanelType::LODs:
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsLOD, LODs))
			{
				// Add a custom builder for each LOD arrays within each group. This way we can customize this 'nested/inner' array
				TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(ChildHandle.ToSharedRef(), false, false, false));
				PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomRenderingDetails::OnGenerateElementForLODs, DetailLayout, GroupIndex));
				PropertyBuilder->SetDisplayName(FText::FromString(TEXT("LODs")));
				ChildrenBuilder.AddCustomBuilder(PropertyBuilder);
			}
			else
			{
				AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
			}
		}
		break;
		case EMaterialPanelType::Physics:
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsPhysics, SolverSettings) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsPhysics, ExternalForces) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsPhysics, StrandsParameters))
			{
				AddPropertySeparator(PropertyName, ChildrenBuilder);
				ExpandStruct(ChildHandle, ChildrenBuilder, GroupIndex, -1, true);
			}
			else if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupsPhysics, MaterialConstraints))
			{
				// Expand the constraint, so that each constraints type has its own block (so that custom value reset works correctly)
				uint32 SubChildrenCount = 0;
				ChildHandle->GetNumChildren(SubChildrenCount);
				for (uint32 SubChildIt = 0; SubChildIt < SubChildrenCount; ++SubChildIt)
				{
					TSharedPtr<IPropertyHandle> SubChildHandle = ChildHandle->GetChildHandle(SubChildIt);
					AddPropertySeparator(SubChildHandle->GetProperty()->GetFName(), ChildrenBuilder);
					ExpandStruct(SubChildHandle, ChildrenBuilder, GroupIndex, -1, true);
				}
			}
			else
			{
				AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
			}
		}
		break;
		case EMaterialPanelType::Bindings:
		{
			AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
		}
		break;
		case EMaterialPanelType::Dataflow:
		{
			AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex, -1);
		}
		break;
		default:
		{
			ChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
		break;
		}

	}
}

void FGroomRenderingDetails::OnGroomLODTypeChanged()
{
	GroomDetailLayout->ForceRefreshDetails();
}

void FGroomRenderingDetails::OnGroomTextureLayoutChanged()
{
	GroomDetailLayout->ForceRefreshDetails();
}

// Hair binding display
void FGroomRenderingDetails::OnGenerateElementForBindingAsset(TSharedRef<IPropertyHandle> StructProperty, int32 BindingIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
	FProperty* Property = StructProperty->GetProperty();
	ChildrenBuilder.AddProperty(StructProperty);

	const FLinearColor Color = BindingIndex == Toolkit->GetActiveBindingIndex() ? FLinearColor::White : FLinearColor(0.1f, 0.1f, 0.1f, 1.f);

	ChildrenBuilder.AddCustomRow(FText::FromString(TEXT("Preview")))
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &FGroomRenderingDetails::OnSelectBinding, BindingIndex, Property)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Visible"))
				.ColorAndOpacity(Color)
			]
		]
	];
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void FGroomRenderingDetails::OnSetObject(const FAssetData& AssetData)
{

}

FString FGroomRenderingDetails::OnGetObjectPath(int32 GroupIndex) const
{
	if (!GroomAsset || GroupIndex < 0)
		return FString();

	int32 MaterialIndex = INDEX_NONE;
	check(GroomAsset);
	switch (PanelType)
	{
	case EMaterialPanelType::Cards:
	{
		if (GroupIndex < GroomAsset->GetHairGroupsCards().Num())
		{
			MaterialIndex = GroomAsset->GetMaterialIndex(GroomAsset->GetHairGroupsCards()[GroupIndex].MaterialSlotName);
		}
	}
	break;
	case EMaterialPanelType::Meshes:
	{
		if (GroupIndex < GroomAsset->GetHairGroupsMeshes().Num())
		{
			MaterialIndex = GroomAsset->GetMaterialIndex(GroomAsset->GetHairGroupsMeshes()[GroupIndex].MaterialSlotName);
		}
	}
	break;
	case EMaterialPanelType::Strands:
	{
		if (GroupIndex < GroomAsset->GetHairGroupsRendering().Num())
		{
			MaterialIndex = GroomAsset->GetMaterialIndex(GroomAsset->GetHairGroupsRendering()[GroupIndex].MaterialSlotName);
		}
	}
	break;
	}

	if (MaterialIndex == INDEX_NONE)
		return FString();
	
	return GroomAsset->GetHairGroupsMaterials()[MaterialIndex].Material->GetPathName();
}

/**
 * Called to get the visibility of the replace button
 */
bool FGroomRenderingDetails::GetReplaceVisibility(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	return false;
}

/**
 * Called when reset to base is clicked
 */
void FGroomRenderingDetails::OnResetToBaseClicked(TSharedPtr<IPropertyHandle> PropertyHandle)
{

}

TSharedRef<SWidget> FGroomRenderingDetails::CreateMaterialSwatch( const TSharedPtr<FAssetThumbnailPool>& ThumbnailPool/*, const TArray<FAssetData>& OwnerAssetDataArray*/, int32 GroupIndex)
{
	FIntPoint ThumbnailSize(64, 64);

	const bool bDisplayCompactSize = false;
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f )
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew( SObjectPropertyEntryBox )
					.EnableContentPicker(false)
					.DisplayUseSelected(false)
					.ObjectPath(this, &FGroomRenderingDetails::OnGetObjectPath, GroupIndex)
					.AllowedClass(UMaterialInterface::StaticClass())
					.OnObjectChanged(this, &FGroomRenderingDetails::OnSetObject)
					.ThumbnailPool(ThumbnailPool)
					.DisplayCompactSize(bDisplayCompactSize)
					//.OwnerAssetDataArray(OwnerAssetDataArray)
					.CustomContentSlot()
					[
						SNew(SComboButton)
						.IsEnabled(this, &FGroomRenderingDetails::IsStrandsMaterialPickerEnabled, GroupIndex)
						.OnGetMenuContent(this, &FGroomRenderingDetails::OnGenerateStrandsMaterialMenuPicker, GroupIndex)
						.VAlign(VAlign_Center)
						.ContentPadding(2)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(this, &FGroomRenderingDetails::GetStrandsMaterialName, GroupIndex)
						]
					]
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE