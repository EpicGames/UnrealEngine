// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "EngineDefines.h"
#include "PropertyHandle.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Widgets/Input/SComboBox.h"

class UGroomAsset;
class UGroomBindingAssetList;
class IGroomCustomAssetEditorToolkit;
class IDetailLayoutBuilder;
class IDetailCategoryBuilder;
class FDetailWidgetRow;
class IDetailPropertyRow;

enum class EMaterialPanelType
{
	Strands,
	Cards,
	Meshes,
	Interpolation,
	LODs,
	Physics,
	Bindings,
	Dataflow
};

class FGroomRenderingDetails : public IDetailCustomization
{
	UGroomAsset* GroomAsset = nullptr;
	UGroomBindingAssetList* GroomBindingAssetList = nullptr;
	IGroomCustomAssetEditorToolkit* Toolkit = nullptr;

public:
	FGroomRenderingDetails(IGroomCustomAssetEditorToolkit* InToolkit, EMaterialPanelType Type);
	~FGroomRenderingDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(IGroomCustomAssetEditorToolkit* InToolkit, EMaterialPanelType Type);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:

	FName& GetMaterialSlotName(int32 GroupIndex);
	const FName& GetMaterialSlotName(int32 GroupIndex) const;
	int32 GetGroupCount() const;

	void AddNewGroupButton(IDetailCategoryBuilder& StrandsGroupFilesCategory, FProperty* Property, const FText& HeaderText);

	void ApplyChanges();
	void SetMaterialSlot(int32 GroupIndex, int32 MaterialIndex);
	TSharedRef<SWidget> OnGenerateStrandsMaterialMenuPicker(int32 GroupIndex);
	FText GetStrandsMaterialName(int32 GroupIndex) const;
	TSharedRef<SWidget> OnGenerateStrandsMaterialPicker(int32 GroupIndex, IDetailLayoutBuilder* DetailLayoutBuilder);
	bool IsStrandsMaterialPickerEnabled(int32 GroupIndex) const;

	// Custom widget for material slot
	void CustomizeStrandsGroupProperties(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& StrandsGroupFilesCategory);
	void OnGenerateElementForHairGroup(TSharedRef<IPropertyHandle> StructProperty, int32 GroupIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);
	void OnGenerateElementForLODs(TSharedRef<IPropertyHandle> StructProperty, int32 LODIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout, int32 GroupIndex);
	void OnGenerateElementForBindingAsset(TSharedRef<IPropertyHandle> StructProperty, int32 BindingIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);

	// Display custom thumbnail for material
	void OnSetObject(const FAssetData& AssetData);
	FString OnGetObjectPath(int32 GroupIndex) const;
	bool GetReplaceVisibility(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	void OnResetToBaseClicked(TSharedPtr<IPropertyHandle> PropertyHandle);
	TSharedRef<SWidget> CreateMaterialSwatch(const TSharedPtr<FAssetThumbnailPool>& ThumbnailPool, int32 GroupIndex);

	TSharedRef<SWidget> MakeGroupNameCustomization(int32 GroupIndex, const FLinearColor& GroupColor);
	TSharedRef<SWidget> MakeGroupNameButtonCustomization(int32 GroupIndex, FProperty* Property);

	FReply OnRemoveLODClicked(int32 GroupIndex, int32 LODIndex, FProperty* Property);
	FReply OnAddLODClicked(int32 GroupIndex, FProperty* Property);
	FReply OnRemoveGroupClicked(int32 GroupIndex, FProperty* Property);
	FReply OnAddGroup(FProperty* Property);
	FReply OnRefreshCards(int32 GroupIndex, FProperty* Property);
	FReply OnSaveCards(int32 GroupIndex, FProperty* Property);
	FReply OnGenerateCardDataUsingPlugin(int32 GroupIndex);
	FReply OnSelectBinding(int32 BindingIndex, FProperty* Property);
	void OnGroomLODTypeChanged();
	void OnGroomTextureLayoutChanged();

	void AddPropertySeparator(FName PropertyName, IDetailChildrenBuilder& ChildrenBuilder);
	void ExpandStructForLOD(TSharedRef<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& ChildrenBuilder, int32 GroupIndex, int32 LODIndex, bool bOverrideReset);
	void ExpandStruct(TSharedPtr<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& ChildrenBuilder, int32 GroupIndex, int32 LODIndex, bool bOverrideReset);
	void ExpandStruct(TSharedRef<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& ChildrenBuilder, int32 GroupIndex, int32 LODIndex, bool bOverrideReset);
	IDetailPropertyRow& AddPropertyWithCustomReset(TSharedPtr<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& Builder, int32 GroupIndex, int32 LODIndex);

	bool CommonResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, int32 LODIndex, bool bSetValue);
	bool ShouldResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, int32 LODInex);
	void ResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, int32 LODIndex);

private:
	IDetailLayoutBuilder* GroomDetailLayout = nullptr;
	EMaterialPanelType PanelType = EMaterialPanelType::Strands;
	bool bDeleteWarningConsumed; // This prevent showing the delete material slot warning dialog more then once per editor session

	// Copy/Paste group
	void OnCopyGroupItem(int32 CurrentSlot, EMaterialPanelType InType);
	bool OnCanCopyGroupItem(int32 CurrentSlot, EMaterialPanelType InType) const;
	void OnPasteGroupItem(int32 CurrentSlot, EMaterialPanelType InType);

	// Copy/Paste LOD
	void OnCopyLODItem(int32 CurrentSlot, int32 InLODIndex);
	bool OnCanCopyLODItem(int32 CurrentSlot, int32 InLODIndex) const;
	void OnPasteLODItem(int32 CurrentSlot, int32 InLODIndex);

	void CallPostEditChange(FProperty* PropertyChanged=nullptr);
};