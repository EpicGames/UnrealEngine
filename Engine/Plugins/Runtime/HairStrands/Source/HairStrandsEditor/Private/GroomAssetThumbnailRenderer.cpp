// Copyright Epic Games, Inc. All Rights Reserved.
#include "GroomAssetThumbnailRenderer.h"

#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "ThumbnailHelpers.h"

#include "GroomAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomAssetThumbnailRenderer)

bool UGroomAssetThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object))
	{
		return GroomAsset->IsValid();
	}
	return false;
}

void UGroomAssetThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UGroomAsset* GroomAsset = Cast<UGroomAsset>(Object); GroomAsset && GroomAsset->IsValid())
	{
		if (!ThumbnailScene.IsValid() || !ensure(ThumbnailScene->GetWorld()))
		{
			if (ThumbnailScene.IsValid())
			{
				FlushRenderingCommands();
				ThumbnailScene.Reset();
			}

			ThumbnailScene = MakeUnique<FGroomAssetThumbnailScene>();
		}

		ThumbnailScene->SetGroomAsset(GroomAsset);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->CleanupSceneAfterThumbnailRender();
	}
}

void UGroomAssetThumbnailRenderer::BeginDestroy()
{
	ThumbnailScene.Reset();
	Super::BeginDestroy();
}
