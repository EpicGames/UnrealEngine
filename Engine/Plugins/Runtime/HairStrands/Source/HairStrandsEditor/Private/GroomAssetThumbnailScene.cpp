// Copyright Epic Games, Inc. All Rights Reserved.
#include "GroomAssetThumbnailScene.h"

#include "AssetCompilingManager.h"

#include "GroomActor.h"
#include "GroomComponent.h"

#include "ThumbnailRendering/SceneThumbnailInfo.h"

FGroomAssetThumbnailScene::FGroomAssetThumbnailScene()
	:FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AGroomActor>(SpawnInfo);
	PreviewActor->GetRootComponent()->SetCanEverAffectNavigation(false);
	PreviewActor->GetRootComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FGroomAssetThumbnailScene::SetGroomAsset(UGroomAsset* GroomAsset)
{
	PreviewActor->GetGroomComponent()->SetGroomAsset(GroomAsset);
	
	if (ensure(GroomAsset))
	{
		PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
		PreviewActor->GetGroomComponent()->UpdateBounds();

		const FBoxSphereBounds GroomAssetBounds = PreviewActor->GetGroomComponent()->Bounds;
		// Center the mesh at the world origin then offset to put it on top of the plane
		const float BoundsZOffset = GetBoundsZOffset(GroomAssetBounds);
		PreviewActor->SetActorLocation(-GroomAssetBounds.Origin + FVector(0, 0, BoundsZOffset), false);
	}

	PreviewActor->GetGroomComponent()->MarkRenderStateDirty();
}

void FGroomAssetThumbnailScene::CleanupSceneAfterThumbnailRender()
{
	PreviewActor->GetGroomComponent()->SetGroomAsset(nullptr);
	PreviewActor->GetGroomComponent()->MarkRenderStateDirty();
}

void FGroomAssetThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetGroomComponent());
	check(PreviewActor->GetGroomComponent()->GroomAsset);

	const FBoxSphereBounds GroomAssetBounds = PreviewActor->GetGroomComponent()->Bounds;
	const float HalfFOVRadians = FMath::DegreesToRadians<float>(!FMath::IsNearlyZero(InFOVDegrees, SMALL_NUMBER) ? InFOVDegrees : 5.0f) * 0.5f;

	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = static_cast<float>(GroomAssetBounds.SphereRadius * 1.15);
	const float BoundsZOffset = GetBoundsZOffset(GroomAssetBounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = nullptr;
	if (USceneThumbnailInfo* GroomThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetGroomComponent()->GroomAsset->ThumbnailInfo))
	{
		ThumbnailInfo = GroomThumbnailInfo;
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}
	
	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}
