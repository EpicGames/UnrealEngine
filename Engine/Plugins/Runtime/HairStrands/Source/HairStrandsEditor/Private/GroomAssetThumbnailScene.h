// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "ThumbnailHelpers.h"
#include "UObject/ObjectPtr.h"

class UGroomAsset;
class AGroomActor;

class FGroomAssetThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	HAIRSTRANDSEDITOR_API FGroomAssetThumbnailScene();

	/** Sets the groom asset to use in the next CreateView() */
	HAIRSTRANDSEDITOR_API void SetGroomAsset(UGroomAsset* GroomAsset);


	HAIRSTRANDSEDITOR_API void CleanupSceneAfterThumbnailRender();
protected:
	// FThumbnailPreviewScene implementation
	HAIRSTRANDSEDITOR_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The static mesh actor used to display all static mesh thumbnails */
	TObjectPtr<AGroomActor> PreviewActor;
};
