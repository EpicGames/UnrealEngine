// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBindingActions.h"

#include "ContentBrowserMenuContexts.h"
#include "GroomAsset.h"
#include "GroomCustomAssetEditorToolkit.h"

#include "GeometryCache.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "ToolMenuSection.h"
#include "GroomBindingBuilder.h"
#include "ToolMenus.h"

#include "ThumbnailRendering/SceneThumbnailInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomBindingActions)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FLinearColor UAssetDefinition_GroomBindingAsset::GetAssetColor() const
{
	return FColor::White;
}

UThumbnailInfo* UAssetDefinition_GroomBindingAsset::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

namespace MenuExtension_GroomBindingAsset
{

void ExecuteRebuildBindingAsset(const FToolMenuContext& InContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
	{
		for (UGroomBindingAsset* BindingAsset : Context->LoadSelectedObjects<UGroomBindingAsset>())
		{
			if (BindingAsset->GetGroom() && BindingAsset->HasValidTarget())
			{
				BindingAsset->GetGroom()->ConditionalPostLoad();
				if (BindingAsset->GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
				{
					BindingAsset->GetTargetSkeletalMesh()->ConditionalPostLoad();
					if (BindingAsset->GetSourceSkeletalMesh())
					{
						BindingAsset->GetSourceSkeletalMesh()->ConditionalPostLoad();
					}
				}
				else
				{
					BindingAsset->GetTargetGeometryCache()->ConditionalPostLoad();
					if (BindingAsset->GetSourceGeometryCache())
					{
						BindingAsset->GetSourceGeometryCache()->ConditionalPostLoad();
					}
				}
				BindingAsset->BeginCacheDerivedDatas();
				BindingAsset->GetOutermost()->MarkPackageDirty();
			}
		}
	}
}

void GroomBindingRebuildAction(FToolMenuSection& InSection)
{
	const TAttribute<FText> Label = LOCTEXT("RebuildGroomBinding", "Rebuild");
	const TAttribute<FText> ToolTip = LOCTEXT("RebuildGroomBindingTooltip", "Rebuild the groom binding");
	const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions");

	FToolUIAction UIAction;
	UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRebuildBindingAsset);
	InSection.AddMenuEntry("GroomBinding_Rebuild", Label, ToolTip, Icon, UIAction);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Actions registration

static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UGroomBindingAsset::StaticClass());
		
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			GroomBindingRebuildAction(InSection);
		}));
	}));
});

} // namespace MenuExtension_GroomBindingAsset

#undef LOCTEXT_NAMESPACE
