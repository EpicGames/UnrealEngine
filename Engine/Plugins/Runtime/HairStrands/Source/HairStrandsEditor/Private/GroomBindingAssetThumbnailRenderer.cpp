// Copyright Epic Games, Inc. All Rights Reserved.
#include "GroomBindingAssetThumbnailRenderer.h"

#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "ThumbnailHelpers.h"

#include "ObjectTools.h"
#include "GroomBindingAsset.h"
#include "GroomAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomBindingAssetThumbnailRenderer)

bool UGroomBindingAssetThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (UGroomBindingAsset* GroomBindingAsset = Cast<UGroomBindingAsset>(Object); GroomBindingAsset->bIsValid)
	{
		if (!GroomBindingAsset->GetGroom() || !GroomBindingAsset->GetGroom()->IsValid())
		{
			ThumbnailTools::CacheEmptyThumbnail(Object->GetFullName(), Object->GetPackage());
			return false;
		}

		EGroomBindingMeshType BindingType = GroomBindingAsset->GetGroomBindingType();
		switch (BindingType)
		{

		case EGroomBindingMeshType::SkeletalMesh:
		{
			if (USkeletalMesh* TargetSkeletalMesh = GroomBindingAsset->GetTargetSkeletalMesh())
			{
				return true;
			}
			break;
		}
		case EGroomBindingMeshType::GeometryCache:
		{
			if (UGeometryCache* TargetGeometryCache = GroomBindingAsset->GetTargetGeometryCache())
			{
				return true;
			}
			break;
		}
		default:
			break;
		}
	}

	return false;
}

void UGroomBindingAssetThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UGroomBindingAsset* GroomBindingAsset = Cast<UGroomBindingAsset>(Object); GroomBindingAsset->bIsValid)
	{
		if (!ThumbnailScene.IsValid() || !ensure(ThumbnailScene->GetWorld()))
		{
			if (ThumbnailScene)
			{
				FlushRenderingCommands();
				ThumbnailScene.Reset();
			}
			ThumbnailScene = MakeUnique<FGroomBindingAssetThumbnailScene>();
		}

		ThumbnailScene->SetGroomBindingAsset(GroomBindingAsset);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->CleanupSceneAfterThumbnailRender();
	}
}

void UGroomBindingAssetThumbnailRenderer::BeginDestroy()
{
	ThumbnailScene.Reset();
	Super::BeginDestroy();
}
