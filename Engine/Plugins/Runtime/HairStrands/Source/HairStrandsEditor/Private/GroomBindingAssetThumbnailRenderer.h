// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "GroomBindingAssetThumbnailScene.h"
#include "Templates/UniquePtr.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "UObject/ObjectMacros.h"
#include "GroomBindingAssetThumbnailRenderer.generated.h"


class FCanvas;
class FRenderTarget;

UCLASS(config = Editor, MinimalAPI)
class UGroomBindingAssetThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

public:
	// Begin UThumbnailRenderer Object
	HAIRSTRANDSEDITOR_API virtual bool CanVisualizeAsset(UObject* Object);

	HAIRSTRANDSEDITOR_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject implementation
	HAIRSTRANDSEDITOR_API virtual void BeginDestroy() override;

private:
	TUniquePtr<FGroomBindingAssetThumbnailScene> ThumbnailScene;
};

