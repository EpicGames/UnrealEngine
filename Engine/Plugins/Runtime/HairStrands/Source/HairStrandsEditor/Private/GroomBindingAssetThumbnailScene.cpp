// Copyright Epic Games, Inc. All Rights Reserved.
#include "GroomBindingAssetThumbnailScene.h"

#include "GroomActor.h"
#include "GroomBindingAsset.h"
#include "GroomComponent.h"

#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/SkeletalMeshActor.h"

#include "GeometryCache.h"
#include "GeometryCacheActor.h"
#include "GeometryCacheComponent.h"

#include "ThumbnailRendering/SceneThumbnailInfo.h"

namespace GroomBindingAssetThumbnailScene::Private
{
	template<typename ActorClassType>
	ActorClassType* SpawnActorInThumbnailScene(FGroomBindingAssetThumbnailScene& ThumbnailScene)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.bNoFail = true;
		SpawnInfo.ObjectFlags = RF_Transient;
		ActorClassType* SpawnedActor = ThumbnailScene.GetWorld()->SpawnActor<ActorClassType>(SpawnInfo);
		SpawnedActor->GetRootComponent()->SetCanEverAffectNavigation(false);
		SpawnedActor->GetRootComponent()->SetMobility(EComponentMobility::Movable);
		SpawnedActor->SetActorEnableCollision(false);
		return SpawnedActor;
	}
}

FGroomBindingAssetThumbnailScene::FGroomBindingAssetThumbnailScene()
	:FThumbnailPreviewScene()
{
	using namespace GroomBindingAssetThumbnailScene::Private;
	
	bForceAllUsedMipsResident = false;

	PreviewGroomActor = SpawnActorInThumbnailScene<AGroomActor>(*this);
	PreviewSkeletalMeshActor = SpawnActorInThumbnailScene<ASkeletalMeshActor>(*this);
	PreviewGeometryCacheActor = SpawnActorInThumbnailScene<AGeometryCacheActor>(*this);
	
}

void FGroomBindingAssetThumbnailScene::SetGroomBindingAsset(UGroomBindingAsset* GroomBindingAsset)
{
	using namespace GroomBindingAssetThumbnailScene::Private;
	if (ensure(GroomBindingAsset))
	{
		CachedBindingAsset = GroomBindingAsset;
		UGroomAsset* GroomAsset = GroomBindingAsset->GetGroom();
		PreviewGroomActor->GetGroomComponent()->SetGroomAsset(GroomAsset);

		EGroomBindingMeshType BindingType = GroomBindingAsset->GetGroomBindingType();
		switch (BindingType)
		{

		case EGroomBindingMeshType::SkeletalMesh:
		{
			if (USkeletalMesh* TargetSkeletalMesh = GroomBindingAsset->GetTargetSkeletalMesh())
			{
				PreviewSkeletalMeshActor->GetSkeletalMeshComponent()->SetSkeletalMesh(TargetSkeletalMesh);
				PreviewRootActor = PreviewSkeletalMeshActor;
			}
			
			break;
		}
		case EGroomBindingMeshType::GeometryCache:
		{
			if (UGeometryCache* TargetGeometryCache = GroomBindingAsset->GetTargetGeometryCache())
			{
				PreviewGeometryCacheActor->GetGeometryCacheComponent()->SetGeometryCache(TargetGeometryCache);
				PreviewRootActor = PreviewGeometryCacheActor;
			}
			break;
		}
		default:
			break;
		}

		if (PreviewRootActor != nullptr)
		{
			PreviewGroomActor->AttachToActor(PreviewRootActor, FAttachmentTransformRules::SnapToTargetIncludingScale);
			PreviewGroomActor->GetGroomComponent()->UpdateBounds();
		}
		else
		{
			PreviewRootActor = PreviewGroomActor;
		}
		
		PreviewRootActor->SetActorLocation(FVector::ZeroVector, false);
		PreviewRootActor->GetRootComponent()->UpdateBounds();
		
		PreviewGroomActor->GetGroomComponent()->MarkRenderStateDirty();
		if (PreviewRootActor != PreviewGroomActor)
		{
			PreviewRootActor->GetRootComponent()->MarkRenderStateDirty();
		}
	}
}

void FGroomBindingAssetThumbnailScene::CleanupSceneAfterThumbnailRender()
{
	if (PreviewRootActor != nullptr && PreviewGroomActor != PreviewRootActor)
	{
		PreviewGroomActor->DetachFromActor(FDetachmentTransformRules::KeepRelativeTransform);
	}

	PreviewRootActor = nullptr;

	PreviewGroomActor->GetGroomComponent()->SetGroomAsset(nullptr);
	PreviewGroomActor->GetGroomComponent()->MarkRenderStateDirty();

	PreviewSkeletalMeshActor->GetSkeletalMeshComponent()->SetSkeletalMesh(nullptr);
	PreviewSkeletalMeshActor->GetSkeletalMeshComponent()->EmptyOverrideMaterials();
	PreviewSkeletalMeshActor->GetSkeletalMeshComponent()->MarkRenderStateDirty();

	PreviewGeometryCacheActor->GetGeometryCacheComponent()->SetGeometryCache(nullptr);
	PreviewGeometryCacheActor->GetGeometryCacheComponent()->MarkRenderStateDirty();

	CachedBindingAsset.Reset();
}

void FGroomBindingAssetThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewGroomActor);
	check(PreviewGroomActor->GetGroomComponent());
	check(PreviewGroomActor->GetGroomComponent()->GroomAsset);

	check(PreviewRootActor);
	check(PreviewRootActor->GetRootComponent());


	const FBoxSphereBounds PreviewBounds = PreviewRootActor->GetRootComponent()->Bounds;

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(!FMath::IsNearlyZero(InFOVDegrees, SMALL_NUMBER) ? InFOVDegrees : 5.0f) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = static_cast<float>(PreviewBounds.SphereRadius * 1.15);
	const float BoundsZOffset = GetBoundsZOffset(PreviewBounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	OutOrigin = -PreviewBounds.Origin;

	USceneThumbnailInfo* ThumbnailInfo = nullptr;
	if (CachedBindingAsset.IsValid())
	{
		if (USceneThumbnailInfo* BindingAssetThumbnailInfo = Cast<USceneThumbnailInfo>(CachedBindingAsset.Get()->ThumbnailInfo))
		{
			ThumbnailInfo = BindingAssetThumbnailInfo;
		}
	}

	if(ThumbnailInfo == nullptr)
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}
	
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}

bool FGroomBindingAssetThumbnailScene::ShouldClampOrbitZoom() const
{
	return false;
}