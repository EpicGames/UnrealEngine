// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "ThumbnailHelpers.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class AGeometryCacheActor;
class AGroomActor;
class AGroomActor;
class ASkeletalMeshActor;
class UGroomBindingAsset;

class FGroomBindingAssetThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	HAIRSTRANDSEDITOR_API FGroomBindingAssetThumbnailScene();

	/** Sets the groom binding to use in the next CreateView() */
	HAIRSTRANDSEDITOR_API void SetGroomBindingAsset(UGroomBindingAsset* GroomBindingAsset);

	/** Sets the groom binding to use in the next CreateView() */
	HAIRSTRANDSEDITOR_API void CleanupSceneAfterThumbnailRender();

protected:
	// FThumbnailPreviewScene implementation
	HAIRSTRANDSEDITOR_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	HAIRSTRANDSEDITOR_API virtual bool ShouldClampOrbitZoom() const override;

private:
	/** The groom actor used to display all groom asset thumbnails */
	TObjectPtr<AGroomActor> PreviewGroomActor;

	/** The skeletal mesh actor for which the binding is created that will be previewed */
	TObjectPtr<ASkeletalMeshActor> PreviewSkeletalMeshActor;

	/** The geometry cache actor for which the binding is created that will be previewed */
	TObjectPtr<AGeometryCacheActor> PreviewGeometryCacheActor;

	/** Actor used as the root of the scene for thumbnail */
	TObjectPtr<AActor> PreviewRootActor = nullptr;

	TWeakObjectPtr<UGroomBindingAsset> CachedBindingAsset;
};
