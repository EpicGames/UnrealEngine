// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBindingDetailsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "GroomBindingAsset.h"
#include "GroomCreateBindingOptions.h"
#include "GroomImportOptions.h"
#include "DetailWidgetRow.h"
#include "GroomBindingCompiler.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyCustomizationHelpers.h" 
#include "IDetailChildrenBuilder.h"

#define LOCTEXT_NAMESPACE "GroomBindingDetails"

///////////////////////////////////////////////////////////////////////////////////////////////////

template<typename T>
T* GetCutomizeDetailObject(IDetailLayoutBuilder& LayoutBuilder)
{
	T* Out = nullptr;

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = LayoutBuilder.GetSelectedObjects();
	check(SelectedObjects.Num() <= 1);
	if (SelectedObjects.Num() > 0)
	{
		Out = Cast<T>(SelectedObjects[0].Get());
	}
	return Out;
}

static TArray<FName> GetSkelMeshAttributes(USkeletalMesh* SkeletalMesh)
{
	TArray<FName> Out;
	if (SkeletalMesh)
	{
		if (FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(0/*LODIndex*/))
		{
			for (const FSkeletalMeshVertexAttributeInfo& Attribute : LODInfo->VertexAttributes)
			{
				if (Attribute.IsEnabledForRender())
				{
					Out.Add(Attribute.Name);
				}
			}
		}
	}
	return Out;
}

// Target attribute
static void AddBindingAttributeSelection(
	IDetailLayoutBuilder& LayoutBuilder, 
	TSharedRef<IPropertyHandle> InBindingAttributeProperty, 
	UObject* InObject, 
	FGroomBindingAttributeSelection* In, 
	EGroomBindingMeshType InBindingType, 
	USkeletalMesh* InTargetSkeletalMesh)
{
	if (In && InBindingAttributeProperty->IsValidHandle())
	{
		In->SelectedBindingAttribute = 0;
		In->BindingAttributeNames.Empty();
		In->BindingAttributeNames.Add(TEXT("No Attribute"));
		if (InBindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			In->BindingAttributeNames.Append(GetSkelMeshAttributes(InTargetSkeletalMesh));
		}

		IDetailPropertyRow& PropertyRow = LayoutBuilder.AddPropertyToCategory(InBindingAttributeProperty);
		FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget();

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, WidgetRow);

		TSharedRef<SComboBox<FName>> AttributeComboBox = SNew(SComboBox<FName>)
		.OptionsSource(&In->BindingAttributeNames)
		.OnSelectionChanged_Lambda([InObject, In, InBindingAttributeProperty](FName NewSelection, ESelectInfo::Type SelectInfo)
		{
			In->SelectedBindingAttribute = In->BindingAttributeNames.Find(NewSelection);
			
			if (UGroomBindingAsset* BindingAsset = Cast<UGroomBindingAsset>(InObject))
			{
				if (In->BindingAttributeNames.IsValidIndex(In->SelectedBindingAttribute))
				{
					BindingAsset->SetTargetBindingAttribute(In->BindingAttributeNames[In->SelectedBindingAttribute]);
				}
			}
			else if (UGroomCreateBindingOptions* BindingOptions = Cast<UGroomCreateBindingOptions>(InObject))
			{
				if (In->BindingAttributeNames.IsValidIndex(In->SelectedBindingAttribute))
				{
					BindingOptions->TargetBindingAttribute = In->BindingAttributeNames[In->SelectedBindingAttribute];
				}
			}
			
			FPropertyChangedEvent PropertyUpdate(InBindingAttributeProperty->GetProperty());
			InObject->PostEditChangeProperty(PropertyUpdate);
		})
		.OnGenerateWidget_Lambda([In](FName Item)
		{
			return SNew(STextBlock).Text(FText::FromName(Item));
		})
		.Content()
		[
			SNew(STextBlock)
			.Text_Lambda([In] 
			{ 
				FName SelectedName = In->BindingAttributeNames.IsValidIndex(In->SelectedBindingAttribute) ? In->BindingAttributeNames[In->SelectedBindingAttribute] : NAME_None;
				return FText::FromName(SelectedName);
			})
		];

		WidgetRow.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			AttributeComboBox
		];
	}
}

static void RefreshLayoutOnValueChanged(IDetailLayoutBuilder& LayoutBuilder, TSharedRef<IPropertyHandle> Property)
{
	Property->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&LayoutBuilder]()
	{
		LayoutBuilder.ForceRefreshDetails();
	}));
}

///////////////////////////////////////////////////////////////////////////////////////////////////

// Customization for UGroomBindingAsset
void FGroomBindingDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	UGroomBindingAsset* Asset = GetCutomizeDetailObject<UGroomBindingAsset>(LayoutBuilder);
	
	// GroomBindingType show/hide
	if (Asset->GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
	{
		LayoutBuilder.HideProperty(UGroomBindingAsset::GetSourceGeometryCacheMemberName());
		LayoutBuilder.HideProperty(UGroomBindingAsset::GetTargetGeometryCacheMemberName());
	}
	else
	{
		LayoutBuilder.HideProperty(UGroomBindingAsset::GetSourceSkeletalMeshMemberName());
		LayoutBuilder.HideProperty(UGroomBindingAsset::GetTargetSkeletalMeshMemberName());
	}

	// GroomBindingType
	TSharedRef<IPropertyHandle> GroomBindingType = LayoutBuilder.GetProperty(UGroomBindingAsset::GetGroomBindingTypeMemberName());
	RefreshLayoutOnValueChanged(LayoutBuilder, GroomBindingType);

	// Target Skeletal Mesh
	TSharedRef<IPropertyHandle> TargetSkeletalMesh = LayoutBuilder.GetProperty(UGroomBindingAsset::GetTargetSkeletalMeshMemberName());
	RefreshLayoutOnValueChanged(LayoutBuilder, TargetSkeletalMesh);

	// Target attribute
	TSharedRef<IPropertyHandle> TargetBindingAttribute = LayoutBuilder.GetProperty(UGroomBindingAsset::GetTargetBindingAttributeMemberName());
	AddBindingAttributeSelection(LayoutBuilder, TargetBindingAttribute, Asset, &BindingAttributeSelection, Asset->GetGroomBindingType(), Asset->GetTargetSkeletalMesh());
}

TSharedRef<IDetailCustomization> FGroomBindingDetailsCustomization::MakeInstance()
{
	return MakeShared<FGroomBindingDetailsCustomization>();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

// Customization for UGroomCreateBindingOptions
void FGroomCreateBindingDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	UGroomCreateBindingOptions* Options = GetCutomizeDetailObject<UGroomCreateBindingOptions>(LayoutBuilder);

	// GroomBindingType show/hide
	if (Options->GroomBindingType == EGroomBindingMeshType::SkeletalMesh)
	{
		LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, SourceGeometryCache));
		LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, TargetGeometryCache));
	}
	else
	{
		LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, SourceSkeletalMesh));
		LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, TargetSkeletalMesh));
	}

	// GroomBindingType
	TSharedRef<IPropertyHandle> GroomBindingType = LayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, GroomBindingType));
	RefreshLayoutOnValueChanged(LayoutBuilder, GroomBindingType);

	// Target Skeletal Mesh
	TSharedRef<IPropertyHandle> TargetSkeletalMesh = LayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, TargetSkeletalMesh));
	RefreshLayoutOnValueChanged(LayoutBuilder, TargetSkeletalMesh);

	// Target attribute
	TSharedRef<IPropertyHandle> TargetBindingAttribute = LayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, TargetBindingAttribute));
	AddBindingAttributeSelection(LayoutBuilder, TargetBindingAttribute, Options, &BindingAttributeSelection, Options->GroomBindingType, Options->TargetSkeletalMesh);
}

TSharedRef<IDetailCustomization> FGroomCreateBindingDetailsCustomization::MakeInstance()
{
	return MakeShared<FGroomCreateBindingDetailsCustomization>();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

// Customization for UGroomHairGroupsMapping
void FGroomHairGroomRemappingDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	UGroomHairGroupsMapping* Mapping = GetCutomizeDetailObject<UGroomHairGroupsMapping>(LayoutBuilder);

	LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomHairGroupsMapping, OldToNewGroupIndexMapping));
	LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomHairGroupsMapping, NewToOldGroupIndexMapping));
	LayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UGroomHairGroupsMapping, OldGroupNames));

	TSharedRef<IPropertyHandle> Property = LayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomHairGroupsMapping, NewGroupNames), UGroomHairGroupsMapping::StaticClass());
	if (Property->IsValidHandle())
	{
		TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShareable(new FDetailArrayBuilder(Property, false, false, false));
		PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomHairGroomRemappingDetailsCustomization::OnGenerateElementForBindingAsset, &LayoutBuilder, Mapping));

		FName CategoryName = FName(TEXT("GroupMapping"));
		IDetailCategoryBuilder& GroupMappingCategory = LayoutBuilder.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::TypeSpecific);
		GroupMappingCategory.AddCustomBuilder(PropertyBuilder, false);
	}
}

void FGroomHairGroomRemappingDetailsCustomization::OnGenerateElementForBindingAsset(TSharedRef<IPropertyHandle> StructProperty, int32 InNewGroupIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout, UGroomHairGroupsMapping* InMapping)
{
	FProperty* Property = StructProperty->GetProperty();
	const FLinearColor Color = FLinearColor::White;
	ChildrenBuilder.AddCustomRow(FText::FromString(TEXT("Preview")))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromName(InMapping->NewGroupNames[InNewGroupIndex]))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&InMapping->GetOldGroupNames())
		.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
		{
			return SNew(STextBlock)
			.Text(FText::FromString(*InItem));
		})
		.OnSelectionChanged_Lambda([InNewGroupIndex, InMapping](TSharedPtr<FString> InItem, ESelectInfo::Type SelectInfo)
		{
			FName Item = FName(*InItem);
			int32 OldGroupIndex = 0;
			for (FName OldName : InMapping->OldGroupNames)
			{
				if (OldName == Item)
				{
					break;
				}
				++OldGroupIndex;
			}
			InMapping->SetIndex(InNewGroupIndex, OldGroupIndex);
		})
		[
			SNew(STextBlock)
			.Text_Lambda([InNewGroupIndex, InMapping] 
			{ 
				FText OldText = FText::FromString(TEXT("Default"));
				int32 OldIndex = InMapping->NewToOldGroupIndexMapping[InNewGroupIndex];
				if (InMapping->OldGroupNames.IsValidIndex(OldIndex))
				{
					OldText = FText::FromName(InMapping->OldGroupNames[OldIndex]);
				}
				return OldText; 
			})
		]
	];
}

TSharedRef<IDetailCustomization> FGroomHairGroomRemappingDetailsCustomization::MakeInstance()
{
	return MakeShared<FGroomHairGroomRemappingDetailsCustomization>();
}

#undef LOCTEXT_NAMESPACE
