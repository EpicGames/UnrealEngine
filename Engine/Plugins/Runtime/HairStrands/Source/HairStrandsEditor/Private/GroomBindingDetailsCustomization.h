// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class UGroomHairGroupsMapping;
class IDetailChildrenBuilder;
class IPropertyHandle;

struct FGroomBindingAttributeSelection
{
	int32 SelectedBindingAttribute = INDEX_NONE;
	TArray<FName> BindingAttributeNames;
};

class FGroomBindingDetailsCustomization : public IDetailCustomization
{
public:
	/** Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	/** End IDetailCustomization interface */
	static TSharedRef<IDetailCustomization> MakeInstance();
	FGroomBindingAttributeSelection BindingAttributeSelection;
};

class FGroomCreateBindingDetailsCustomization : public IDetailCustomization
{
public:
	/** Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	/** End IDetailCustomization interface */
	static TSharedRef<IDetailCustomization> MakeInstance();
	FGroomBindingAttributeSelection BindingAttributeSelection;
};

class FGroomHairGroomRemappingDetailsCustomization : public IDetailCustomization
{
public:
	/** Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	void OnGenerateElementForBindingAsset(TSharedRef<IPropertyHandle> StructProperty, int32 InIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout, UGroomHairGroupsMapping* InMapping);
	/** End IDetailCustomization interface */
	static TSharedRef<IDetailCustomization> MakeInstance();
};
