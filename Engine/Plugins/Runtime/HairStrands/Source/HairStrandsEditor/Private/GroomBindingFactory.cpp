// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomBindingFactory.h"
#include "GroomBindingAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomBindingFactory)

UGroomBindingFactory::UGroomBindingFactory()
{
	SupportedClass = UGroomBindingAsset::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
	bText = false;
	bEditorImport = true;
}

UObject* UGroomBindingFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UGroomBindingAsset* GroomBinding = NewObject<UGroomBindingAsset>(InParent, InName, Flags | RF_Transactional);
	GroomBinding->SetGroom(nullptr);
	GroomBinding->SetTargetSkeletalMesh(nullptr);
	GroomBinding->SetSourceSkeletalMesh(nullptr);
	GroomBinding->SetNumInterpolationPoints(100);
	GroomBinding->SetMatchingSection(0);

	return GroomBinding;
}
