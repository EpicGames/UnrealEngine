// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheActions.h"
#include "EditorFramework/AssetImportData.h"
#include "GroomCustomAssetEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomCacheActions)

FLinearColor UAssetDefinition_GroomCacheAsset::GetAssetColor() const
{
	return FColor::White;
}

void UAssetDefinition_GroomCacheAsset::GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const
{
	for (UObject* Asset : TypeAssets)
	{
		const UGroomCache* GroomCache = Cast<UGroomCache>(Asset);
		if (GroomCache && GroomCache->AssetImportData)
		{
			GroomCache->AssetImportData->ExtractFilenames(OutSourceFilePaths);
		}
	}
}
