// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheImportSettingsCustomization.h"
#include "GroomCacheImportOptions.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"

TSharedRef<IPropertyTypeCustomization> FGroomCacheImportSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FGroomCacheImportSettingsCustomization);
}

void FGroomCacheImportSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData(StructPtrs);
	if (StructPtrs.Num() == 1)
	{
		Settings = (FGroomCacheImportSettings*)StructPtrs[0];
	}

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		int32 VisibleType = 0; // Always visible
		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGroomCacheImportSettings, GroomAsset))
		{
			// Visibility depends on other settings
			VisibleType = 1;
		}
		else if	(ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGroomCacheImportSettings, ImportType) ||
			ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGroomCacheImportSettings, FrameStart) ||
			ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGroomCacheImportSettings, FrameEnd) ||
			ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGroomCacheImportSettings, bSkipEmptyFrames))
		{
			// Visibility depends on Import Groom Cache
			VisibleType = 2;
		}

		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);
		Property.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FGroomCacheImportSettingsCustomization::ArePropertiesVisible, VisibleType)));
	}
}

EVisibility FGroomCacheImportSettingsCustomization::ArePropertiesVisible(const int32 VisibleType) const
{
	if (VisibleType == 1 && Settings)
	{
		// GroomAsset property must be shown if GroomCache is imported but not GroomAsset so that a replacement can be specified
		// If GroomCache is not imported, then this property is not relevant
		return Settings->bImportGroomAsset || !Settings->bImportGroomCache ? EVisibility::Collapsed : EVisibility::Visible;
	}
	else if (VisibleType == 2 && Settings)
	{
		return !Settings->bImportGroomCache ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Visible;
}
