// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"

class FGroomCacheImportSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FGroomCacheImportSettingsCustomization() : Settings(nullptr) { }

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

protected:
	EVisibility ArePropertiesVisible(const int32 VisibleType) const;

	struct FGroomCacheImportSettings* Settings;
};