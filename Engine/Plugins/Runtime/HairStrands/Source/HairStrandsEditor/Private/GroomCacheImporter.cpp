// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheImporter.h"
#include "GroomAsset.h"
#include "GroomBuilder.h"
#include "GroomCache.h"
#include "GroomCacheImportOptions.h"
#include "GroomImportOptions.h"
#include "HairStrandsImporter.h"
#include "HairStrandsTranslator.h"
#include "Logging/LogMacros.h"
#include "Misc/ScopedSlowTask.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "RenderUtils.h"

#define LOCTEXT_NAMESPACE "GroomCacheImporter"

DEFINE_LOG_CATEGORY_STATIC(LogGroomCacheImporter, Log, All);

static UGroomCache* CreateGroomCache(EGroomCacheType Type, UObject*& InParent, const FString& ObjectName, const EObjectFlags Flags)
{
	// Don't do any package handling if the parent is the transient package; just use it
	UPackage* Package = nullptr;
	FString SanitizedObjectName = ObjectName;
	if (InParent != GetTransientPackage())
	{
		// Setup package name and create one accordingly
		FString NewPackageName = InParent->GetOutermost()->GetName();
		if (!NewPackageName.EndsWith(ObjectName))
		{
			// Remove the cache suffix if the parent is from a reimport
			if (NewPackageName.EndsWith("_strands_cache"))
			{
				NewPackageName.RemoveFromEnd("_strands_cache");
			}
			else if (NewPackageName.EndsWith("_guides_cache"))
			{
				NewPackageName.RemoveFromEnd("_guides_cache");
			}

			// Append the correct suffix
			NewPackageName += TEXT("_") + ObjectName;
		}

		NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);

		// Parent package to place new GroomCache
		Package = CreatePackage(*NewPackageName);

		FString CompoundObjectName = FPackageName::GetShortName(NewPackageName);
		SanitizedObjectName = ObjectTools::SanitizeObjectName(CompoundObjectName);

		UGroomCache* ExistingTypedObject = FindObject<UGroomCache>(Package, *SanitizedObjectName);
		UObject* ExistingObject = FindObject<UObject>(Package, *SanitizedObjectName);

		if (ExistingTypedObject != nullptr)
		{
			ExistingTypedObject->PreEditChange(nullptr);
			return ExistingTypedObject;
		}
		else if (ExistingObject != nullptr)
		{
			// Replacing an object.  Here we go!
			// Delete the existing object
			const bool bDeleteSucceeded = ObjectTools::DeleteSingleObject(ExistingObject);

			if (bDeleteSucceeded)
			{
				// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

				// Create a package for each mesh
				Package = CreatePackage(*NewPackageName);
				InParent = Package;
			}
			else
			{
				// failed to delete
				return nullptr;
			}
		}
	}
	else
	{
		Package = GetTransientPackage();
	}

	UGroomCache* GroomCache = NewObject<UGroomCache>(Package, FName(*SanitizedObjectName), Flags | RF_Public);
	GroomCache->Initialize(Type);

	return GroomCache;
}

UGroomCache* FGroomCacheImporter::ProcessToGroomCache(FGroomCacheProcessor& Processor, const FGroomAnimationInfo& AnimInfo, FHairImportContext& ImportContext, const FString& ObjectNameSuffix)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheImporter::ProcessToGroomCache);
	if (UGroomCache* GroomCache = CreateGroomCache(Processor.GetType(), ImportContext.Parent, ObjectNameSuffix, ImportContext.Flags))
	{
		UE::Groom::BuildGroomCache(Processor, AnimInfo, GroomCache);
		return GroomCache;
	}
	return nullptr;
}

TArray<UGroomCache*> FGroomCacheImporter::ImportGroomCache(const FString& SourceFilename, TSharedPtr<IGroomTranslator> Translator, const FGroomAnimationInfo& AnimInfo, FHairImportContext& HairImportContext, UGroomAsset* GroomAssetForCache, EGroomCacheImportType ImportType)
{
	bool bSuccess = true;
	bool bGuidesOnly = false;

	const bool bImportStrandsCache = (EnumHasAnyFlags(ImportType, EGroomCacheImportType::Strands));
	const bool bImportGuidesCache = (EnumHasAnyFlags(ImportType, EGroomCacheImportType::Guides));

	FGroomCacheProcessor StrandsProcessor(EGroomCacheType::Strands, AnimInfo.Attributes);
	FGroomCacheProcessor GuidesProcessor(EGroomCacheType::Guides, AnimInfo.Attributes);
	if (Translator->BeginTranslation(SourceFilename))
	{
		// Sample one extra frame so that we can interpolate between EndFrame - 1 and EndFrame
		const int32 NumFrames = AnimInfo.NumFrames + 1;
		FScopedSlowTask SlowTask(NumFrames, LOCTEXT("ImportGroomCache", "Importing GroomCache frames"));
		SlowTask.MakeDialog();

		const TArray<FHairGroupPlatformData>& GroupPlatformData = GroomAssetForCache->GetHairGroupsPlatformData();

		// Each frame is translated into a HairDescription and processed into HairGroupData
		for (int32 FrameIndex = AnimInfo.StartFrame; FrameIndex < AnimInfo.EndFrame + 1; ++FrameIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FGroomCacheImporter::ImportGroomCache::OneFrame);

			const uint32 CurrentFrame = FrameIndex - AnimInfo.StartFrame;

			FTextBuilder TextBuilder;
			TextBuilder.AppendLineFormat(LOCTEXT("ImportGroomCacheFrame", "Importing GroomCache frame {0} of {1}"), FText::AsNumber(CurrentFrame), FText::AsNumber(NumFrames));
			SlowTask.EnterProgressFrame(1.f, TextBuilder.ToText());

			FHairDescription FrameHairDescription;
			if (Translator->Translate(FrameIndex * AnimInfo.SecondsPerFrame, FrameHairDescription, HairImportContext.ImportOptions->ConversionSettings))
			{
				FHairDescriptionGroups HairDescriptionGroups;
				// Do not add extra control points at the end of curve when hair strip geometry is enabled. This is because groom cache data are 
				// serialize within the uasset (i.e. do not used intermediate cached/build data), and need the asset to be compatible with or without hair strip geometry.
				if (!FGroomBuilder::BuildHairDescriptionGroups(FrameHairDescription, HairDescriptionGroups, false /*bAllowAddEndControlPoint*/))
				{
					bSuccess = false;
					break;
				}

				const uint32 GroupCount = HairDescriptionGroups.HairGroups.Num();

				if (GroupCount != GroupPlatformData.Num())
				{
					bSuccess = false;
					UE_LOGF(LogGroomCacheImporter, Warning, "GroomCache does not have the same number of groups as the static groom (%d instead of %d). Aborting GroomCache import.",
						GroupCount, GroupPlatformData.Num());
					break;
				}

				TArray<FHairGroupInfoWithVisibility> HairGroupsInfo = GroomAssetForCache->GetHairGroupsInfo();
				TArray<FGroomCacheInputData> CacheInputDatas;
				CacheInputDatas.SetNum(GroupCount);
				for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
				{
					FGroomBuilder::BuildData(
						HairDescriptionGroups.HairGroups[GroupIndex], 
						GroomAssetForCache->GetHairGroupsInterpolation()[GroupIndex], 
						HairGroupsInfo[GroupIndex], 
						CacheInputDatas[GroupIndex].Strands, 
						CacheInputDatas[GroupIndex].Guides);
				}

				// Validate that the GroomCache has the same topology as the static groom
				for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
				{
					if (bImportStrandsCache)
					{
						// When UsesTriangleStrips is enabled, we add an extra control point at the end of each curve in the groom asset.
						// Since the groom cache needs to be independent of this settings (since the data is serialized directly into the asset
						// and not rebuilt from a description), it does not contains these extra control points.
						// We account for them for validation here
						const bool bHasExtraPoint = GroupPlatformData[GroupIndex].Strands.BulkData.HasFlags(FHairStrandsBulkData::DataFlags_HasExtraEndingPoint);
						const uint32 ExtraControlPointCount = bHasExtraPoint ? CacheInputDatas[GroupIndex].Strands.GetNumCurves() : 0u;

						const bool bCurveCountMatch = CacheInputDatas[GroupIndex].Strands.GetNumCurves() == GroupPlatformData[GroupIndex].Strands.BulkData.GetNumCurves();
						const bool bPointCountMatch = CacheInputDatas[GroupIndex].Strands.GetNumPoints() + ExtraControlPointCount == GroupPlatformData[GroupIndex].Strands.BulkData.GetNumPoints();

						if (!bCurveCountMatch || !bPointCountMatch)
						{
							bSuccess = false;
							UE_LOG(LogGroomCacheImporter, Warning, TEXT("GroomCache frame %d does not have the same number of curves (%u) \
								or vertices (%u) for the strands as the static groom (%u and %u respectively). Aborting GroomCache import."),
								FrameIndex, CacheInputDatas[GroupIndex].Strands.GetNumCurves(), CacheInputDatas[GroupIndex].Strands.GetNumPoints(),
								GroupPlatformData[GroupIndex].Strands.BulkData.GetNumCurves(), GroupPlatformData[GroupIndex].Strands.BulkData.GetNumPoints());
							break;
						}
					}

					if (bImportGuidesCache)
					{
						if (CacheInputDatas[GroupIndex].Strands.GetNumPoints() == 0)
						{
							bGuidesOnly = true;
						}

						const bool bHasExtraPoint = GroupPlatformData[GroupIndex].Guides.BulkData.HasFlags(FHairStrandsBulkData::DataFlags_HasExtraEndingPoint);
						const uint32 ExtraControlPointCount = bHasExtraPoint ? CacheInputDatas[GroupIndex].Guides.GetNumCurves() : 0u;

						const bool bCurveCountMatch = CacheInputDatas[GroupIndex].Guides.GetNumCurves() == GroupPlatformData[GroupIndex].Guides.BulkData.GetNumCurves();
						const bool bPointCountMatch = CacheInputDatas[GroupIndex].Guides.GetNumPoints() + ExtraControlPointCount == GroupPlatformData[GroupIndex].Guides.BulkData.GetNumPoints();

						if (!bCurveCountMatch || !bPointCountMatch)
						{
							bSuccess = false;
							UE_LOG(LogGroomCacheImporter, Warning, TEXT("GroomCache frame %d does not have the same number of curves (%u) \
								or vertices (%u) for the guides as the static groom (%u and %u respectively). Aborting GroomCache import."),
								FrameIndex, CacheInputDatas[GroupIndex].Guides.GetNumCurves(), CacheInputDatas[GroupIndex].Guides.GetNumPoints(),
								GroupPlatformData[GroupIndex].Guides.BulkData.GetNumCurves(), GroupPlatformData[GroupIndex].Guides.BulkData.GetNumPoints());
							break;
						}
					}
				}

				if (!bSuccess)
				{
					break;
				}

				// The HairGroupData is converted into animated groom data by the GroomCacheProcessor
				if (bImportStrandsCache && !bGuidesOnly)
				{
					StrandsProcessor.AddGroomSample(MoveTemp(CacheInputDatas));
				}

				if (bImportGuidesCache)
				{
					GuidesProcessor.AddGroomSample(MoveTemp(CacheInputDatas));
				}
			}
		}
	}
	else
	{
		bSuccess = false;
	}
	Translator->EndTranslation();

	TArray<UGroomCache*> GroomCaches;
	if (bSuccess)
	{
		// Once the processing has completed successfully, the data is transferred to the GroomCache
		if (bImportStrandsCache && !bGuidesOnly)
		{
			UGroomCache* GroomCache = ProcessToGroomCache(StrandsProcessor, AnimInfo, HairImportContext, "strands_cache");
			if (GroomCache)
			{
				GroomCaches.Add(GroomCache);
			}
		}

		if (bImportGuidesCache)
		{
			UGroomCache* GroomCache = ProcessToGroomCache(GuidesProcessor, AnimInfo, HairImportContext, "guides_cache");
			if (GroomCache)
			{
				GroomCaches.Add(GroomCache);
			}
		}
	}
	return GroomCaches;
}

void FGroomCacheImporter::SetupImportSettings(FGroomCacheImportSettings& ImportSettings, const FGroomAnimationInfo& AnimInfo)
{
	// Prepare the import settings for display
	// GroomCache options are only shown if there's a valid groom animation
	ImportSettings.bImportGroomCache = ImportSettings.bImportGroomCache && AnimInfo.IsValid();

	if (ImportSettings.bImportGroomCache)
	{
		if (ImportSettings.FrameEnd == 0)
		{
			ImportSettings.FrameEnd = AnimInfo.EndFrame;
		}
	}
}

void FGroomCacheImporter::ApplyImportSettings(FGroomCacheImportSettings& ImportSettings, FGroomAnimationInfo& AnimInfo)
{
	// Harmonize the values between what's in the settings (set by the user) and the animation info (extracted from the Alembic) used for importing
	// The user settings usually take precedence over the animation info
	if (ImportSettings.bImportGroomCache)
	{
		if (ImportSettings.bSkipEmptyFrames)
		{
			// Skipping empty frames will start from the beginning of the animation range or beyond if specified by the user
			if (ImportSettings.FrameStart > AnimInfo.StartFrame)
			{
				AnimInfo.StartFrame = ImportSettings.FrameStart;
			}
			else
			{
				ImportSettings.FrameStart = AnimInfo.StartFrame;
			}
		}
		else
		{
			// Otherwise, just take the value set by the user
			AnimInfo.StartFrame = ImportSettings.FrameStart;
		}

		if (ImportSettings.FrameEnd == 0)
		{
			// If the user manually set the end to 0, use the actual end of the animation range
			ImportSettings.FrameEnd = AnimInfo.EndFrame;
		}
		else
		{
			// Otherwise, just take the value set by the user
			AnimInfo.EndFrame = ImportSettings.FrameEnd;
		}

		// Sanity check
		if (ImportSettings.FrameEnd <= ImportSettings.FrameStart)
		{
			ImportSettings.FrameEnd = ImportSettings.FrameStart + 1;
			AnimInfo.EndFrame = ImportSettings.FrameStart + 1;
		}

		// EndFrame is not included and must have at least 1 frame
		AnimInfo.NumFrames = FMath::Max(AnimInfo.EndFrame - AnimInfo.StartFrame, 1);

		// Compute the duration as it is not known yet
		AnimInfo.Duration = AnimInfo.NumFrames * AnimInfo.SecondsPerFrame;
	}
}

#undef LOCTEXT_NAMESPACE