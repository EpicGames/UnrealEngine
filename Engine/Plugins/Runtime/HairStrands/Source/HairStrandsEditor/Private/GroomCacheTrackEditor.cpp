// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheTrackEditor.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GroomCache.h"
#include "GroomComponent.h"
#include "LevelSequence.h"
#include "MovieSceneGroomCacheTrack.h"
#include "MovieSceneGroomCacheSection.h"
#include "SequencerSectionPainter.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Styling/SlateIconFinder.h"
#include "TimeToPixel.h"

namespace GroomCacheEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	const uint32 AnimationTrackHeight = 28;
}

#define LOCTEXT_NAMESPACE "FGroomCacheTrackEditor"

static UGroomComponent* AcquireGroomComponentFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (UGroomComponent* GroomComp = Cast<UGroomComponent>(Component))
			{
				return GroomComp;
			}
		}
	}
	else if (UGroomComponent* GroomComp = Cast<UGroomComponent>(BoundObject))
	{
		return GroomComp;
	}

	return nullptr;
}

FGroomCacheSection::FGroomCacheSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: Section(*CastChecked<UMovieSceneGroomCacheSection>(&InSection))
	, Sequencer(InSequencer)
	, InitialFirstLoopStartOffsetDuringResize(0)
	, InitialStartTimeDuringResize(0)
{ }

UMovieSceneSection* FGroomCacheSection::GetSectionObject()
{
	return &Section;
}

FText FGroomCacheSection::GetSectionTitle() const
{
	if (Section.Params.GroomCache != nullptr)
	{
		return FText::FromString(Section.Params.GroomCache->GetName());
	
	}
	return LOCTEXT("NoGroomCacheSection", "No GroomCache");
}

float FGroomCacheSection::GetSectionHeight() const
{
	return (float)GroomCacheEditorConstants::AnimationTrackHeight;
}

int32 FGroomCacheSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	using namespace UE::Sequencer;

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	int32 LayerId = Painter.PaintSectionBackground();

	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	if (!Section.HasStartFrame() || !Section.HasEndFrame())
	{
		return LayerId;
	}

	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	// Add lines where the animation starts and ends/loops
	const float AnimPlayRate = FMath::IsNearlyZero(Section.Params.PlayRate) ? 1.0f : Section.Params.PlayRate;
	const float Duration = Section.Params.GetSequenceLength();
	const float SeqLength = Duration - TickResolution.AsSeconds(Section.Params.StartFrameOffset + Section.Params.EndFrameOffset) / AnimPlayRate;
	const float FirstLoopSeqLength = SeqLength - TickResolution.AsSeconds(Section.Params.FirstLoopStartFrameOffset) / AnimPlayRate;

	if (!FMath::IsNearlyZero(SeqLength, KINDA_SMALL_NUMBER) && SeqLength > 0)
	{
		float MaxOffset = Section.GetRange().Size<FFrameTime>() / TickResolution;
		float OffsetTime = FirstLoopSeqLength;
		float StartTime = Section.GetInclusiveStartFrame() / TickResolution;

		while (OffsetTime < MaxOffset)
		{
			float OffsetPixel = TimeToPixelConverter.SecondsToPixel(StartTime + OffsetTime) - TimeToPixelConverter.SecondsToPixel(StartTime);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.SectionGeometry.MakeChild(
					FVector2D(2.f, Painter.SectionGeometry.Size.Y - 2.f),
					FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
				).ToPaintGeometry(),
				GenericDivider,
				DrawEffects
			);

			OffsetTime += SeqLength;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (Painter.bIsSelected && SequencerPtr.IsValid())
	{
		FFrameTime CurrentTime = SequencerPtr->GetLocalTime().Time;
		if (Section.GetRange().Contains(CurrentTime.FrameNumber) && Section.Params.GroomCache != nullptr)
		{
			const float Time = TimeToPixelConverter.FrameToPixel(CurrentTime);

			UGroomCache* GroomCache = Section.Params.GroomCache;

			// Draw the current time next to the scrub handle
			const float AnimTime = Section.MapTimeToAnimation(Duration, CurrentTime, TickResolution);
			int32 FrameTime = GroomCache->GetFrameNumberAtTime(AnimTime, false);
			FString FrameString = FString::FromInt(FrameTime);

			const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			static const float TextOffsetPx = 10.f;
			bool  bDrawLeft = (Painter.SectionGeometry.Size.X - Time) < (TextSize.X + 22.f) - TextOffsetPx;
			float TextPosition = bDrawLeft ? Time - TextSize.X - TextOffsetPx : Time + TextOffsetPx;
			//handle mirrored labels
			const float MajorTickHeight = 9.0f;
			FVector2D TextOffset(TextPosition, Painter.SectionGeometry.Size.Y - (MajorTickHeight + TextSize.Y));

			const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());
			const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
			// draw time string

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId + 5,
				Painter.SectionGeometry.ToPaintGeometry(TextSize + 2.0f * BoxPadding, FSlateLayoutTransform(TextOffset - BoxPadding)),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);

			FSlateDrawElement::MakeText(
				Painter.DrawElements,
				LayerId + 6,
				Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				DrawEffects,
				DrawColor
			);

		}
	}

	return LayerId;
}

void FGroomCacheSection::BeginResizeSection()
{
	InitialFirstLoopStartOffsetDuringResize = Section.Params.FirstLoopStartFrameOffset;
	InitialStartTimeDuringResize = Section.HasStartFrame() ? Section.GetInclusiveStartFrame() : 0;
}

void FGroomCacheSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	// Adjust the start offset when resizing from the beginning
	if (ResizeMode == SSRM_LeadingEdge)
	{
		FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber StartOffset = FrameRate.AsFrameNumber((ResizeTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

		StartOffset += InitialFirstLoopStartOffsetDuringResize;

		if (StartOffset < 0)
		{
			// Ensure start offset is not less than 0 and adjust ResizeTime
			ResizeTime = ResizeTime - StartOffset;

			StartOffset = FFrameNumber(0);
		}
		else
		{
			// If the start offset exceeds the length of one loop, trim it back.
			const FFrameNumber SeqLength = FrameRate.AsFrameNumber(Section.Params.GetSequenceLength()) - Section.Params.StartFrameOffset - Section.Params.EndFrameOffset;
			StartOffset = StartOffset % SeqLength;
		}

		Section.Params.FirstLoopStartFrameOffset = StartOffset;
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FGroomCacheSection::BeginSlipSection()
{
	BeginResizeSection();
}

void FGroomCacheSection::SlipSection(FFrameNumber SlipTime)
{
	FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameNumber StartOffset = FrameRate.AsFrameNumber((SlipTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

	StartOffset += InitialFirstLoopStartOffsetDuringResize;

	if (StartOffset < 0)
	{
		// Ensure start offset is not less than 0 and adjust ResizeTime
		SlipTime = SlipTime - StartOffset;

		StartOffset = FFrameNumber(0);
	}
	else
	{
		// If the start offset exceeds the length of one loop, trim it back.
		const FFrameNumber SeqLength = FrameRate.AsFrameNumber(Section.Params.GetSequenceLength()) - Section.Params.StartFrameOffset - Section.Params.EndFrameOffset;
		StartOffset = StartOffset % SeqLength;
	}

	Section.Params.FirstLoopStartFrameOffset = StartOffset;

	ISequencerSection::SlipSection(SlipTime);
}

void FGroomCacheSection::BeginDilateSection()
{
	Section.PreviousPlayRate = Section.Params.PlayRate; //make sure to cache the play rate
}
void FGroomCacheSection::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	Section.Params.PlayRate = Section.PreviousPlayRate / DilationFactor;
	Section.SetRange(NewRange);
}

FGroomCacheTrackEditor::FGroomCacheTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ }

TSharedRef<ISequencerTrackEditor> FGroomCacheTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FGroomCacheTrackEditor(InSequencer));
}

bool FGroomCacheTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneGroomCacheTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FGroomCacheTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneGroomCacheTrack::StaticClass();
}

TSharedRef<ISequencerSection> FGroomCacheTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FGroomCacheSection(SectionObject, GetSequencer()));
}

FText FGroomCacheTrackEditor::GetDisplayName() const
{
	return LOCTEXT("GroomCacheTrackEditor_DisplayName", "Groom");
}

void FGroomCacheTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UGroomComponent::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass()))
	{
		UGroomComponent* GroomComp = AcquireGroomComponentFromObjectGuid(ObjectBindings[0], GetSequencer());

		if (GroomComp)
		{
			UMovieSceneTrack* Track = nullptr;

			MenuBuilder.AddMenuEntry(
				NSLOCTEXT("Sequencer", "AddGroomCache", "Groom Cache"),
				NSLOCTEXT("Sequencer", "AddGroomCacheTooltip", "Adds a Groom Cache track."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FGroomCacheTrackEditor::BuildGroomCacheTrack, ObjectBindings, Track)
				)
			);
		}
	}
}

void FGroomCacheTrackEditor::BuildGroomCacheTrack(TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SequencerPtr.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("AddGroomCache_Transaction", "Add Groom Cache"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			if (ObjectBinding.IsValid())
			{
				UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);

				UGroomComponent* GroomComp = AcquireGroomComponentFromObjectGuid(ObjectBinding, GetSequencer());

				if (Object && GroomComp)
				{
					AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FGroomCacheTrackEditor::AddKeyInternal, Object, GroomComp, Track));
				}
			}
		}
	}
}

FKeyPropertyResult FGroomCacheTrackEditor::AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UGroomComponent* GroomComp, UMovieSceneTrack* Track)
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		if (!Track)
		{
			Track = AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectHandle, UMovieSceneGroomCacheTrack::StaticClass(), NAME_None);
			KeyPropertyResult.bTrackCreated = true;
		}

		if (ensure(Track))
		{
			Track->Modify();

			UMovieSceneSection* NewSection = Cast<UMovieSceneGroomCacheTrack>(Track)->AddNewAnimation(KeyTime, GroomComp);
			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}

	return KeyPropertyResult;
}

TSharedPtr<SWidget> FGroomCacheTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UGroomComponent* GroomComp = AcquireGroomComponentFromObjectGuid(ObjectBinding, GetSequencer());

	if (GroomComp)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();

		auto SubMenuCallback = [this, ObjectBinding, Track]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			TArray<FGuid> ObjectBindings;
			ObjectBindings.Add(ObjectBinding);

			BuildGroomCacheTrack(ObjectBindings, Track);
			
			return MenuBuilder.MakeWidget();
		};

		return UE::Sequencer::MakeAddButton(LOCTEXT("GroomCacheText", "Groom Cache"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}

}

const FSlateBrush* FGroomCacheTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(UGroomComponent::StaticClass()).GetIcon();
}

#undef LOCTEXT_NAMESPACE
