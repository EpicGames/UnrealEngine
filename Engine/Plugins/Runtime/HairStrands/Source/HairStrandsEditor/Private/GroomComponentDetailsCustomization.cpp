// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomComponentDetailsCustomization.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "EditorModeManager.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"

#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"

#include "IDetailsView.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"

#include "ScopedTransaction.h"
#include "IPropertyUtilities.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "GroomComponent"

//////////////////////////////////////////////////////////////////////////
// FGroomComponentDetailsCustomization

TSharedRef<IDetailCustomization> FGroomComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FGroomComponentDetailsCustomization);
}

void FGroomComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout.GetSelectedObjects();
	MyDetailLayout = nullptr;
	
	FNotifyHook* NotifyHook = DetailLayout.GetPropertyUtilities()->GetNotifyHook();

	bool bEditingActor = false;

	UGroomComponent* GroomComponent = nullptr;
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		UObject* TestObject = SelectedObjects[ObjectIndex].Get();
		if (AActor* CurrentActor = Cast<AActor>(TestObject))
		{
			if (UGroomComponent* CurrentComponent = CurrentActor->FindComponentByClass<UGroomComponent>())
			{
				bEditingActor = true;
				GroomComponent = CurrentComponent;
				break;
			}
		}
		else if (UGroomComponent* TestComponent = Cast<UGroomComponent>(TestObject))
		{
			GroomComponent = TestComponent;
			break;
		}
	}
	GroomComponentPtr = GroomComponent;

	if (GroomComponentPtr.IsValid())
	{
		// Ensure the group desc is update to date
		GroomComponentPtr->UpdateHairGroupsDesc();

		// Add a delegate to refresh details panel when the groom asset changed
		TSharedRef<IPropertyHandle> GroomAssetHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomComponent, GroomAsset));
		FSimpleDelegate OnGroomAssetChangedDelegate = FSimpleDelegate::CreateSP(this, &FGroomComponentDetailsCustomization::OnGroomAssetChanged, &DetailLayout);
		GroomAssetHandle->SetOnPropertyValueChanged(OnGroomAssetChangedDelegate);
	}

	IDetailCategoryBuilder& HairGroupCategory = DetailLayout.EditCategory("GroomGroupsDesc", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	CustomizeDescGroupProperties(DetailLayout, HairGroupCategory);
}

void FGroomComponentDetailsCustomization::CustomizeDescGroupProperties(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& StrandsGroupFilesCategory)
{
	TSharedRef<IPropertyHandle> GroupDescAssetsProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UGroomComponent, GroomGroupsDesc), UGroomComponent::StaticClass());
	if (GroupDescAssetsProperty->IsValidHandle())
	{
		TSharedRef<FDetailArrayBuilder> GroupDescPropertyBuilder = MakeShareable(new FDetailArrayBuilder(GroupDescAssetsProperty, false, false, false));
		GroupDescPropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FGroomComponentDetailsCustomization::OnGenerateElementForHairGroup, &DetailLayout));
		GroupDescPropertyBuilder->SetDisplayName(FText::FromString(TEXT("Hair Groups")));
		StrandsGroupFilesCategory.AddCustomBuilder(GroupDescPropertyBuilder, false);
	}
}

template <typename T>
bool AssignIfDifferentHairComponent(T& Dest, const T& Src, bool bSetValue)
{
	const bool bHasChanged = Dest != Src;
	if (bHasChanged && bSetValue)
	{
		Dest = Src;
	}
	return bHasChanged;
}
#define HAIR_RESET_COMPONENT(MemberName) { if (PropertyName == GET_MEMBER_NAME_CHECKED(FHairGroupDesc, MemberName)) { bHasChanged = AssignIfDifferentHairComponent(GroomComponentPtr->GroomGroupsDesc[GroupIndex].MemberName, Default.MemberName, bSetValue); } }

bool FGroomComponentDetailsCustomization::CommonResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, bool bSetValue)
{
	bool bHasChanged = false;
	if (ChildHandle == nullptr || GroomComponentPtr == nullptr || GroupIndex < 0 || GroupIndex >= GroomComponentPtr->GroomGroupsDesc.Num())
	{
		return bHasChanged;
	}
	
	FName PropertyName = ChildHandle->GetProperty()->GetFName();

	FHairGroupDesc Default;
	HAIR_RESET_COMPONENT(HairWidth);
	HAIR_RESET_COMPONENT(HairRootScale);
	HAIR_RESET_COMPONENT(HairTipScale);
	HAIR_RESET_COMPONENT(HairLengthScale);
	HAIR_RESET_COMPONENT(HairShadowDensity);
	HAIR_RESET_COMPONENT(HairRaytracingRadiusScale);
	HAIR_RESET_COMPONENT(bUseHairRaytracingGeometry);
	HAIR_RESET_COMPONENT(bUseStableRasterization);
	HAIR_RESET_COMPONENT(bScatterSceneLighting);
	HAIR_RESET_COMPONENT(LODBias);

	if (bSetValue && bHasChanged)
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("UnrealEd", "PropertyWindowResetToDefault", "Reset to Default"));
		GroomComponentPtr->UpdateHairGroupsDescAndInvalidateRenderState();
	}

	return bHasChanged;
}

bool FGroomComponentDetailsCustomization::ShouldResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex)
{
	return CommonResetToDefault(ChildHandle, GroupIndex, false);
}

void FGroomComponentDetailsCustomization::ResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex)
{
	CommonResetToDefault(ChildHandle, GroupIndex, true);
}

void FGroomComponentDetailsCustomization::AddPropertyWithCustomReset(TSharedPtr<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& Builder, int32 GroupIndex)
{
	FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FGroomComponentDetailsCustomization::ShouldResetToDefault, GroupIndex);
	FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FGroomComponentDetailsCustomization::ResetToDefault, GroupIndex);
	FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
	Builder.AddProperty(PropertyHandle.ToSharedRef()).OverrideResetToDefault(ResetOverride);
}

static TSharedRef<SUniformGridPanel> MakeHairInfoGrid(const FSlateFontInfo& DetailFontInfo, const FHairGroupInfo& Infos)
{
	TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);

	// Header
	Grid->AddSlot(0, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Curves", "Curves"))
	];

	Grid->AddSlot(1, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Guides", "Guides"))
	];

	Grid->AddSlot(2, 0) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(LOCTEXT("HairInfo_Length", "Max. Length"))
	];

	// Value
	Grid->AddSlot(0, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(Infos.NumCurves))
	];
	Grid->AddSlot(1, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(Infos.NumGuides))
	];
	Grid->AddSlot(2, 1) // x, y
	.HAlign(HAlign_Right)
	[
		SNew(STextBlock)
		.Font(DetailFontInfo)
		.Text(FText::AsNumber(Infos.MaxCurveLength))
	];

	return Grid;
}

// Hair group custom display
TSharedRef<SWidget> GetGroupNameWidget(const UGroomAsset* GroomAsset, int32 GroupIndex, const FLinearColor& GroupColor);
void FGroomComponentDetailsCustomization::OnGenerateElementForHairGroup(TSharedRef<IPropertyHandle> StructProperty, int32 GroupIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();

	static const FSlateBrush* GenericBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");
	float OtherMargin = 2.0f;
	float RightMargin = 10.0f;

	const FLinearColor GroupColorBlock = GetHairGroupDebugColor(GroupIndex) * 0.75f;

	ChildrenBuilder.AddCustomRow(LOCTEXT("HairInfo_Separator", "Separator"))
	.WholeRowContent()
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(GenericBrush)
			.ColorAndOpacity(GroupColorBlock)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(OtherMargin, OtherMargin, RightMargin, OtherMargin)
			[
				GetGroupNameWidget(GroomComponentPtr.IsValid() ? GroomComponentPtr->GroomAsset : nullptr, GroupIndex, FLinearColor::White)
			]			
		]
	];

	if (GroomComponentPtr != nullptr && GroupIndex>=0 && GroupIndex < GroomComponentPtr->GroomGroupsDesc.Num())
	{
		FHairGroupInfo Infos;
		if (GroomComponentPtr->GroomAsset && GroupIndex < GroomComponentPtr->GroomAsset->GetHairGroupsInfo().Num())
		{
			Infos = GroomComponentPtr->GroomAsset->GetHairGroupsInfo()[GroupIndex];
		}
		ChildrenBuilder.AddCustomRow(LOCTEXT("HairInfo_Separator", "Separator"))
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			MakeHairInfoGrid(DetailFontInfo, Infos)
		];
	}

	uint32 ChildrenCount = 0;
	StructProperty->GetNumChildren(ChildrenCount);
	for (uint32 ChildIt = 0; ChildIt < ChildrenCount; ++ChildIt)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructProperty->GetChildHandle(ChildIt);
		AddPropertyWithCustomReset(ChildHandle, ChildrenBuilder, GroupIndex);
	}
}

void FGroomComponentDetailsCustomization::OnGroomAssetChanged(IDetailLayoutBuilder* LayoutBuilder)
{
	LayoutBuilder->ForceRefreshDetails();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
