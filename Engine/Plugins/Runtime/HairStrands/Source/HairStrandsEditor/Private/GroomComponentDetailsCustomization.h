// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"
#include "GroomComponent.h"

class IDetailLayoutBuilder;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;

//////////////////////////////////////////////////////////////////////////
// FGroomComponentDetailsCustomization

class FGroomComponentDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	TWeakObjectPtr<class UGroomComponent> GroomComponentPtr;
	IDetailLayoutBuilder* MyDetailLayout;

	void CustomizeDescGroupProperties(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& StrandsGroupFilesCategory);
	void OnGenerateElementForHairGroup(TSharedRef<IPropertyHandle> StructProperty, int32 GroupIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);

	bool CommonResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex, bool bSetValue);
	bool ShouldResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex);
	void ResetToDefault(TSharedPtr<IPropertyHandle> ChildHandle, int32 GroupIndex);
	void AddPropertyWithCustomReset(TSharedPtr<IPropertyHandle>& PropertyHandle, IDetailChildrenBuilder& Builder, int32 GroupIndex);
	void OnGroomAssetChanged(IDetailLayoutBuilder* LayoutBuilder);
};
