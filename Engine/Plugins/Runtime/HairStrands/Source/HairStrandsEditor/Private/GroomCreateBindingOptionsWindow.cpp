// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCreateBindingOptionsWindow.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "GroomCreateBindingOptions.h"
#include "IDetailsView.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor.h"
#include "GroomBindingAsset.h"
#include "GroomAsset.h"

#define LOCTEXT_NAMESPACE "GroomCreateBindingOptionsWindow"

void SGroomCreateBindingOptionsWindow::Construct(const FArguments& InArgs)
{
	BindingOptions = InArgs._BindingOptions;
	WidgetWindow = InArgs._WidgetWindow;

	const FSlateFontInfo AttributeFont = FAppStyle::GetFontStyle("CurveEd.InfoFont");
	const FSlateFontInfo AttributeResultFont = FAppStyle::GetFontStyle("CurveEd.InfoFont");
	const FLinearColor AttributeColor(0.0f, 0.80f, 0.0f);
	const FText TrueText = LOCTEXT("GroomBindingOptionsWindow_AttributeTrue", "True");
	const FText FalseText = LOCTEXT("GroomBindingOptionsWindow_AttributeFalse", "False");

	bool bHasRootUV = false;
	FText HasRootUVText = FalseText;
	FLinearColor ValidationColor(1, 1, 1);

	// Check if UV transfer are possible
	BindingOptions->GroomAsset->ConditionalPostLoad();
	if (UGroomAsset* GroomAsset = BindingOptions->GroomAsset.Get())
	{
		const FHairDescription HairDescription = GroomAsset->GetHairDescription();
		bHasRootUV = HairDescription.HasAttribute(EHairAttribute::RootUV);
	}

	if (bHasRootUV)
	{
		HasRootUVText = TrueText;
		if (FProperty* TransferProperty = FindFProperty<FProperty>(BindingOptions->GetClass(), GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, SourceSkeletalMesh))) { TransferProperty->SetMetaData(TEXT("Category"), TEXT("Conversion")); }
		if (FProperty* TransferProperty = FindFProperty<FProperty>(BindingOptions->GetClass(), GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, MatchingSection))) { TransferProperty->SetMetaData(TEXT("Category"), TEXT("Conversion")); }
	}
	else
	{
		if (FProperty* TransferProperty = FindFProperty<FProperty>(BindingOptions->GetClass(), GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, SourceSkeletalMesh))) { TransferProperty->SetMetaData(TEXT("Category"), TEXT("Hidden")); }
		if (FProperty* TransferProperty = FindFProperty<FProperty>(BindingOptions->GetClass(), GET_MEMBER_NAME_CHECKED(UGroomCreateBindingOptions, MatchingSection)))   { TransferProperty->SetMetaData(TEXT("Category"), TEXT("Hidden")); }
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(BindingOptions);

	this->ChildSlot
	[
		SNew(SVerticalBox)

		// Root UV
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(AttributeFont)
					.Text(LOCTEXT("GroomOptionsWindow_HasRootUV", "Root UV: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(AttributeResultFont)
					.Text(HasRootUVText)
					.ColorAndOpacity(AttributeColor)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("CurrentFile", "Current File: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(InArgs._FullPath)
				]
			]
		]

		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			+ SUniformGridPanel::Slot(0, 0)
			[
				SAssignNew(ImportButton, SButton)
				.HAlign(HAlign_Center)
				.Text(InArgs._ButtonLabel)
				.IsEnabled(this, &SGroomCreateBindingOptionsWindow::CanCreateBinding)
				.OnClicked(this, &SGroomCreateBindingOptionsWindow::OnCreateBinding)
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SGroomCreateBindingOptionsWindow::OnCancel)
			]
		]
	];
}

bool SGroomCreateBindingOptionsWindow::CanCreateBinding()  const
{
	return (BindingOptions->GroomBindingType == EGroomBindingMeshType::SkeletalMesh && BindingOptions->TargetSkeletalMesh != nullptr) ||
		   (BindingOptions->GroomBindingType == EGroomBindingMeshType::GeometryCache && BindingOptions->TargetGeometryCache != nullptr);
}

enum class EGroomBindingOptionsVisibility : uint8
{
	None = 0x00,
	ConversionOptions = 0x01,
	BuildOptions = 0x02,
	All = ConversionOptions | BuildOptions
};

ENUM_CLASS_FLAGS(EGroomBindingOptionsVisibility);

TSharedPtr<SGroomCreateBindingOptionsWindow> DisplayOptions(UGroomCreateBindingOptions* BindingOptions, EGroomBindingOptionsVisibility VisibilityFlag, FText WindowTitle, FText InButtonLabel)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.SizingRule(ESizingRule::Autosized);

	TSharedPtr<SGroomCreateBindingOptionsWindow> OptionsWindow;

	Window->SetContent
	(
		SAssignNew(OptionsWindow, SGroomCreateBindingOptionsWindow)
		.BindingOptions(BindingOptions)
		.WidgetWindow(Window)
//		.FullPath(FText::FromString(FileName))
		.ButtonLabel(InButtonLabel)
	);

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	return OptionsWindow;
}

TSharedPtr<SGroomCreateBindingOptionsWindow> SGroomCreateBindingOptionsWindow::DisplayCreateBindingOptions(UGroomCreateBindingOptions* BindingOptions)
{
	return DisplayOptions(BindingOptions, EGroomBindingOptionsVisibility::BuildOptions, LOCTEXT("GroomBindingRebuildWindowTitle", "Groom Binding Options"), LOCTEXT("Build", "Create"));
}

#undef LOCTEXT_NAMESPACE
