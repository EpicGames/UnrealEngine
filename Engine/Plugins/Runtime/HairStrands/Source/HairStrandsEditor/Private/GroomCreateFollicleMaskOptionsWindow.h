// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class SButton;
class UGroomCreateFollicleMaskOptions;

class SGroomCreateFollicleMaskOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGroomCreateFollicleMaskOptionsWindow)
		: _FollicleMaskOptions(nullptr)
		, _WidgetWindow()
		, _FullPath()
		, _ButtonLabel()
	{}

	SLATE_ARGUMENT(UGroomCreateFollicleMaskOptions*, FollicleMaskOptions)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(FText, FullPath)
	SLATE_ARGUMENT(FText, ButtonLabel)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	static TSharedPtr<SGroomCreateFollicleMaskOptionsWindow> DisplayCreateFollicleMaskOptions(UGroomCreateFollicleMaskOptions* FollicleMaskOptions);

	FReply OnCreateFollicleMask()
	{
		bShouldCreate = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldCreate = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldCreate() const
	{
		return bShouldCreate;
	}

	SGroomCreateFollicleMaskOptionsWindow() 
		: FollicleMaskOptions(nullptr)
		, bShouldCreate(false)
	{}

private:

	bool CanCreateFollicleMask() const;

private:
	UGroomCreateFollicleMaskOptions* FollicleMaskOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ImportButton;
	bool bShouldCreate;
};
