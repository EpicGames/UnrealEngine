// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCreateStrandsTexturesOptionsWindow.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "GroomCreateStrandsTexturesOptions.h"
#include "GroomTextureBuilder.h"
#include "IDetailsView.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "IContentBrowserSingleton.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "GroomCreateStrandsTexturesOptionsWindow"

void SGroomCreateStrandsTexturesOptionsWindow::Construct(const FArguments& InArgs)
{
	StrandsTexturesOptions = InArgs._StrandsTexturesOptions;
	WidgetWindow = InArgs._WidgetWindow;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(StrandsTexturesOptions);

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("CurrentFile", "Current File: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(InArgs._FullPath)
				]
			]
		]

		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			+ SUniformGridPanel::Slot(0, 0)
			[
				SAssignNew(ImportButton, SButton)
				.HAlign(HAlign_Center)
				.Text(InArgs._ButtonLabel)
				.IsEnabled(this, &SGroomCreateStrandsTexturesOptionsWindow::CanCreateStrandsTextures)
				.OnClicked(this, &SGroomCreateStrandsTexturesOptionsWindow::OnCreateStrandsTextures)
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SGroomCreateStrandsTexturesOptionsWindow::OnCancel)
			]
		]
	];
}

bool SGroomCreateStrandsTexturesOptionsWindow::CanCreateStrandsTextures()  const
{
	return StrandsTexturesOptions &&
		((StrandsTexturesOptions->SkeletalMesh && StrandsTexturesOptions->MeshType == EStrandsTexturesMeshType::Skeletal) ||
		 (StrandsTexturesOptions->StaticMesh && StrandsTexturesOptions->MeshType == EStrandsTexturesMeshType::Static));
}

TSharedPtr<SGroomCreateStrandsTexturesOptionsWindow> DisplayOptions(UGroomCreateStrandsTexturesOptions* StrandsTexturesOptions, FText WindowTitle, FText InButtonLabel)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.SizingRule(ESizingRule::Autosized);

	TSharedPtr<SGroomCreateStrandsTexturesOptionsWindow> OptionsWindow;

	Window->SetContent
	(
		SAssignNew(OptionsWindow, SGroomCreateStrandsTexturesOptionsWindow)
		.StrandsTexturesOptions(StrandsTexturesOptions)
		.WidgetWindow(Window)
		.ButtonLabel(InButtonLabel)
	);

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	return OptionsWindow;
}

TSharedPtr<SGroomCreateStrandsTexturesOptionsWindow> SGroomCreateStrandsTexturesOptionsWindow::DisplayCreateStrandsTexturesOptions(UGroomCreateStrandsTexturesOptions* StrandsTexturesOptions)
{
	return DisplayOptions(StrandsTexturesOptions, LOCTEXT("GroomStrandsTexturesWindowTitle", "Groom Strands Textures Options"), LOCTEXT("Build", "Create"));
}

#undef LOCTEXT_NAMESPACE
