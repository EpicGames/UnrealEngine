// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "HairStrandsInterface.h"

class SButton;
class UGroomCreateStrandsTexturesOptions;

class SGroomCreateStrandsTexturesOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGroomCreateStrandsTexturesOptionsWindow)
		: _StrandsTexturesOptions(nullptr)
		, _WidgetWindow()
		, _FullPath()
		, _ButtonLabel()
	{}

	SLATE_ARGUMENT(UGroomCreateStrandsTexturesOptions*, StrandsTexturesOptions)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(FText, FullPath)
	SLATE_ARGUMENT(FText, ButtonLabel)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	static TSharedPtr<SGroomCreateStrandsTexturesOptionsWindow> DisplayCreateStrandsTexturesOptions(UGroomCreateStrandsTexturesOptions* StrandsTexturesOptions);

	FReply OnCreateStrandsTextures()
	{
		bShouldCreate = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldCreate = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldCreate() const
	{
		return bShouldCreate;
	}

	SGroomCreateStrandsTexturesOptionsWindow() 
		: StrandsTexturesOptions(nullptr)
		, bShouldCreate(false)
	{}

private:

	bool CanCreateStrandsTextures() const;

private:
	UGroomCreateStrandsTexturesOptions* StrandsTexturesOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ImportButton;
	bool bShouldCreate;
};