// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCustomAssetEditorToolkit.h"
#include "Animation/Skeleton.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor.h"
#include "AssetEditorModeManager.h"
#include "GroomAsset.h"
#include "GroomComponent.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "HairStrandsEditor.h"
#include "GroomEditorCommands.h"
#include "GroomEditorStyle.h"
#include "GroomAssetDetails.h"
#include "GroomMaterialDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "AssetThumbnail.h"
#include "AdvancedPreviewSceneModule.h"

#include "Misc/AssetRegistryInterface.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#define GROOMEDITOR_ENABLE_COMPONENT_PANEL 0
#define LOCTEXT_NAMESPACE "GroomCustomAssetEditor"
const FName FGroomCustomAssetEditorToolkit::ToolkitFName(TEXT("GroomEditor"));

const FName FGroomCustomAssetEditorToolkit::TabId_Viewport(TEXT("GroomCustomAssetEditor_Render"));
const FName FGroomCustomAssetEditorToolkit::TabId_LODProperties(TEXT("GroomCustomAssetEditor_LODProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_InterpolationProperties(TEXT("GroomCustomAssetEditor_InterpolationProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_RenderingProperties(TEXT("GroomCustomAssetEditor_RenderProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_CardsProperties(TEXT("GroomCustomAssetEditor_CardsProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_MeshesProperties(TEXT("GroomCustomAssetEditor_MeshesProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_MaterialProperties(TEXT("GroomCustomAssetEditor_MaterialProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_PhysicsProperties(TEXT("GroomCustomAssetEditor_PhysicsProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_PreviewGroomComponent(TEXT("GroomCustomAssetEditor_PreviewGroomComponent"));
const FName FGroomCustomAssetEditorToolkit::TabId_BindingProperties(TEXT("GroomCustomAssetEditor_BindingProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_PreviewSceneProperties(TEXT("GroomCustomAssetEditor_PreviewSceneProperties"));
const FName FGroomCustomAssetEditorToolkit::TabId_DataflowProperties(TEXT("GroomCustomAssetEditor_DataflowProperties"));

void FGroomCustomAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenuGroomEditor", "Hair Strands Asset Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(TabId_Viewport, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnViewportTab))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Render"));

	InTabManager->RegisterTabSpawner(TabId_LODProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_LODProperties))
		.SetDisplayName(LOCTEXT("LODPropertiesTab", "LOD"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.LOD"));

	InTabManager->RegisterTabSpawner(TabId_InterpolationProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_InterpolationProperties))
		.SetDisplayName(LOCTEXT("InterpolationPropertiesTab", "Interpolation"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Blueprint"));

	InTabManager->RegisterTabSpawner(TabId_RenderingProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_RenderingProperties))
		.SetDisplayName(LOCTEXT("RenderingPropertiesTab", "Strands"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CurveBase"));

	InTabManager->RegisterTabSpawner(TabId_PhysicsProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_PhysicsProperties))
		.SetDisplayName(LOCTEXT("PhysicsPropertiesTab", "Physics"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "CollisionAnalyzer.TabIcon"));

	InTabManager->RegisterTabSpawner(TabId_CardsProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_CardsProperties))
		.SetDisplayName(LOCTEXT("CardsPropertiesTab", "Cards"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ConvertToStaticMesh"));

	InTabManager->RegisterTabSpawner(TabId_MeshesProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_MeshesProperties))
		.SetDisplayName(LOCTEXT("MeshesPropertiesTab", "Meshes"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ConvertToStaticMesh"));

	InTabManager->RegisterTabSpawner(TabId_MaterialProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_MaterialProperties))
		.SetDisplayName(LOCTEXT("MaterialPropertiesTab", "Material"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Material"));

	InTabManager->RegisterTabSpawner(TabId_DataflowProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_DataflowProperties))
		.SetDisplayName(LOCTEXT("DataflowPropertiesTab", "Dataflow"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FName("DataflowEditorStyle"), "ClassIcon.Dataflow"));

#if GROOMEDITOR_ENABLE_COMPONENT_PANEL
	InTabManager->RegisterTabSpawner(TabId_PreviewGroomComponent, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_PreviewGroomComponent))
		.SetDisplayName(LOCTEXT("PreviewGroomComponentTab", "Preview Component"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
#endif

	InTabManager->RegisterTabSpawner(TabId_BindingProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_BindingProperties))
		.SetDisplayName(LOCTEXT("BindingPropertiesTab", "Binding"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.SkeletalMesh"));

	InTabManager->RegisterTabSpawner(TabId_PreviewSceneProperties, FOnSpawnTab::CreateSP(this, &FGroomCustomAssetEditorToolkit::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSettingsTab", "Preview Scene Settings"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FGroomCustomAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(TabId_Viewport);
	InTabManager->UnregisterTabSpawner(TabId_LODProperties);
	InTabManager->UnregisterTabSpawner(TabId_InterpolationProperties);
	InTabManager->UnregisterTabSpawner(TabId_RenderingProperties);
	InTabManager->UnregisterTabSpawner(TabId_CardsProperties);
	InTabManager->UnregisterTabSpawner(TabId_MeshesProperties);
	InTabManager->UnregisterTabSpawner(TabId_MaterialProperties);
	InTabManager->UnregisterTabSpawner(TabId_PhysicsProperties);
#if GROOMEDITOR_ENABLE_COMPONENT_PANEL
	InTabManager->UnregisterTabSpawner(TabId_PreviewGroomComponent);
#endif
	InTabManager->UnregisterTabSpawner(TabId_BindingProperties);
	InTabManager->UnregisterTabSpawner(TabId_PreviewSceneProperties);
	InTabManager->UnregisterTabSpawner(TabId_DataflowProperties);
}

void FGroomCustomAssetEditorToolkit::DocPropChanged(UObject *InObject, FPropertyChangedEvent &Property)
{
	if (!GroomAsset.Get())
	{
		return;
	}
#if 0 //TODO
	UGroomDocument *Doc = FGroomDataIO::GetDocumentForAsset(GroomAsset.Get());

	if (Doc != nullptr && InObject==Doc)
	{		
		if (Property.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UGroomDocument, GroomTarget))
		{
			FObjectProperty* prop = CastField<FObjectProperty>(Property.Property);
			UStaticMesh		*AsStaticMesh = Cast<UStaticMesh>(prop->GetPropertyValue_InContainer(InObject));
			USkeletalMesh	*AsSkeletalMesh = Cast<USkeletalMesh>(prop->GetPropertyValue_InContainer(InObject));
			
			if (AsStaticMesh)
			{
				OnStaticGroomTargetChanged(AsStaticMesh);
			} else if (AsSkeletalMesh)
			{
				OnSkeletalGroomTargetChanged(AsSkeletalMesh);
			}
			else
			{
				OnSkeletalGroomTargetChanged(nullptr);
				OnSkeletalGroomTargetChanged(nullptr);
			}
		}
	}	
#endif
}

void FGroomCustomAssetEditorToolkit::OnSkeletalGroomTargetChanged(USkeletalMesh *NewTarget)
{	
	if (PreviewSkeletalMeshComponent != nullptr)
	{
		PreviewSkeletalMeshComponent->SetSkeletalMesh(NewTarget);
		PreviewSkeletalMeshComponent->SetVisibility(NewTarget != nullptr);
	}	
}

bool FGroomCustomAssetEditorToolkit::OnShouldFilterAnimAsset(const FAssetData& AssetData)
{
	// Check the compatible skeletons.
	if (PreviewSkeletalMeshComponent != nullptr)
	{
		USkeleton* Skeleton = PreviewSkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton();
		if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData))
		{
			return false;
		}
	}
	return true;
}

void FGroomCustomAssetEditorToolkit::OnObjectChangedAnimAsset(const FAssetData& AssetData)
{
	if (UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(AssetData.GetAsset()))
	{
		if (PreviewSkeletalMeshComponent != nullptr)
		{
			PreviewSkeletalAnimationAsset = AnimationAsset;
			PreviewSkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
			PreviewSkeletalMeshComponent->PlayAnimation(AnimationAsset, true);
		}
	}
}

FString FGroomCustomAssetEditorToolkit::GetCurrentAnimAssetPath() const
{
	if (PreviewSkeletalAnimationAsset != nullptr)
	{
		return PreviewSkeletalAnimationAsset->GetPathName();
	}
	return FString();
}

bool FGroomCustomAssetEditorToolkit::OnIsEnabledAnimAsset()
{
	return true;
}

void FGroomCustomAssetEditorToolkit::ExtendToolbar()
{
	// Disable simulation toolbar as it is currently not hooked
	struct Local
	{
		static TSharedRef<SWidget> FillSimulationOptionsMenu(FGroomCustomAssetEditorToolkit* Toolkit)
		{
			FMenuBuilder MenuBuilder(true, Toolkit->GetToolkitCommands());

			MenuBuilder.AddMenuEntry( FGroomEditorCommands::Get().PlaySimulation );
			MenuBuilder.AddMenuEntry( FGroomEditorCommands::Get().PauseSimulation );
			MenuBuilder.AddMenuEntry( FGroomEditorCommands::Get().ResetSimulation );

			return MenuBuilder.MakeWidget();
		}

		static TSharedRef<SWidget> FillAnimationOptionsMenu(FGroomCustomAssetEditorToolkit* Toolkit)
		{
			FMenuBuilder MenuBuilder(true, Toolkit->GetToolkitCommands());

			MenuBuilder.AddMenuEntry(FGroomEditorCommands::Get().PlayAnimation);
			MenuBuilder.AddMenuEntry(FGroomEditorCommands::Get().StopAnimation);

			return MenuBuilder.MakeWidget();
		}

		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FGroomCustomAssetEditorToolkit* Toolkit)
		{			
			ToolbarBuilder.BeginSection( "Animation" );
			{
				FSlateIcon PlayIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Toolbar.Play");
				FSlateIcon StopIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Toolbar.Stop");
				ToolbarBuilder.AddToolBarButton(FGroomEditorCommands::Get().PlayAnimation, NAME_None, TAttribute<FText>(FText()), TAttribute<FText>(), TAttribute<FSlateIcon>(PlayIcon));
				ToolbarBuilder.AddToolBarButton(FGroomEditorCommands::Get().StopAnimation, NAME_None, TAttribute<FText>(FText()), TAttribute<FText>(), TAttribute<FSlateIcon>(StopIcon));

				TAttribute<FText> AnimWidgetText(LOCTEXT("AnimationOptions", "Animation"));
				
				FIntPoint ThumbnailOverride = FIntPoint(64, 64);

				TSharedRef<SWidget> PropWidget = SNew(SObjectPropertyEntryBox)
					.ThumbnailPool(Toolkit->ThumbnailPool)
					.ObjectPath(Toolkit, &FGroomCustomAssetEditorToolkit::GetCurrentAnimAssetPath)
					.AllowedClass(UAnimationAsset::StaticClass())
					.AllowClear(true)
					.DisplayBrowse(false)
					.DisplayUseSelected(false)
					.DisplayThumbnail(true)
					.DisplayCompactSize(true)
					.ThumbnailSizeOverride(ThumbnailOverride)
					.OnObjectChanged(FOnAssetSelected::CreateSP(Toolkit, &FGroomCustomAssetEditorToolkit::OnObjectChangedAnimAsset))
					.OnShouldFilterAsset(FOnShouldFilterAsset::CreateSP(Toolkit, &FGroomCustomAssetEditorToolkit::OnShouldFilterAnimAsset))
					.OnIsEnabled(FOnIsEnabled::CreateSP(Toolkit, &FGroomCustomAssetEditorToolkit::OnIsEnabledAnimAsset));
				ToolbarBuilder.AddToolBarWidget(PropWidget);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this)
	);

	AddToolbarExtender(ToolbarExtender);

	FGroomEditor& GroomEditorModule = FModuleManager::LoadModuleChecked<FGroomEditor>("HairStrandsEditor"); // GroomEditor
}

void FGroomCustomAssetEditorToolkit::InitPreviewComponents()
{
	check(GroomAsset!=nullptr);
//	check(FGroomDataIO::GetDocumentForAsset(GroomAsset) != nullptr); TODO
	check(PreviewGroomComponent == nullptr);
	check(PreviewSkeletalMeshComponent == nullptr);
	
#if 0 //TODO
	UGroomDocument *Doc = FGroomDataIO::GetDocumentForAsset(GroomAsset);

	// Update the document from the groom asset
	FGroomDataIO::UpdateDocumentFromGroomAsset(GroomAsset.Get(), Doc);
#endif

	const bool bHasValidBindingAsset = GroomBindingAsset.IsValid();
	PreviewGroomComponent = NewObject<UGroomComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	PreviewGroomComponent->CastShadow = 1;
	PreviewGroomComponent->bCastDynamicShadow = 1;
	PreviewGroomComponent->SetPreviewMode(true);
	PreviewGroomComponent->SetGroomAsset(GroomAsset.Get());
	PreviewGroomComponent->Activate(true);
	PreviewBinding(ActiveGroomBindingIndex);
}

void FGroomCustomAssetEditorToolkit::OnClose() 
{
	// Remove all delegates
	if (GroomAsset.IsValid() && PropertyListenDelegatesResourceChanged.Num() > 0)
	{
		if(PropertyListenDelegatesResourceChanged.Num() > 0)
		{
			for (FDelegateHandle Handle : PropertyListenDelegatesResourceChanged)
			{
				if (Handle.IsValid())
				{
					GroomAsset->GetOnGroomAssetResourcesChanged().Remove(Handle);
				}
			}
			PropertyListenDelegatesResourceChanged.Empty();
		}
		if (PropertyListenDelegatesAssetChanged.Num() > 0)
        {
         	for (FDelegateHandle Handle : PropertyListenDelegatesAssetChanged)
         	{
         		if (Handle.IsValid())
         		{
         			GroomAsset->GetOnGroomAssetChanged().Remove(Handle);
         		}
         	}
         	PropertyListenDelegatesAssetChanged.Empty();
        }
	}

	PropertiesTab.Reset();
	ViewportTab.Reset();

	DetailView_LODProperties.Reset();
	DetailView_InterpolationProperties.Reset();
	DetailView_RenderingProperties.Reset();
	DetailView_PhysicsProperties.Reset();
	DetailView_CardsProperties.Reset();
	DetailView_MeshesProperties.Reset();
	DetailView_MaterialProperties.Reset();
#if GROOMEDITOR_ENABLE_COMPONENT_PANEL
	DetailView_PreviewGroomComponent.Reset();
#endif
	DetailView_BindingProperties.Reset();
	DetailView_PreviewSceneProperties.Reset();
	DetailView_DataflowProperties.Reset();
}

static void ListAllBindingAssets(const UGroomAsset* InGroomAsset, TWeakObjectPtr<UGroomBindingAssetList>& Out)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> BindingAssetData;

	// 1. Use tagged properties for searching compatible binding assets (for assets saved after AssetRegistrySearchable has been added)
	{
		const FName GroomAssetProperty = UGroomBindingAsset::GetGroomMemberName();
		FString GroomAssetName = FAssetData(InGroomAsset).GetExportTextName();
		FARFilter Filter;
		Filter.ClassPaths.Add(UGroomBindingAsset::StaticClass()->GetClassPathName());
		Filter.TagsAndValues.Add(GroomAssetProperty, GroomAssetName);
		AssetRegistryModule.Get().GetAssets(Filter, BindingAssetData);

		for (FAssetData& Asset : BindingAssetData)
		{
			if (UGroomBindingAsset* Binding = (UGroomBindingAsset*)Asset.GetAsset(/*bLoad*/))
			{
				if (Binding->GetGroom() == InGroomAsset)
				{
					Out->Bindings.Add(Binding);
				}
			}
		}
	}

	// 2. Use name matching for searching compatible binding assets
	if (BindingAssetData.Num() == 0)
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(UGroomBindingAsset::StaticClass()->GetClassPathName());
		AssetRegistryModule.Get().GetAssets(Filter, BindingAssetData);
	
		// Filter binding asset which match the groom asset (as the tag/value filter above does not work)
		// Avoid to load binding asset to slow down panel opening
		for (FAssetData& Asset :  BindingAssetData)
		{
			UGroomBindingAsset* Binding = (UGroomBindingAsset*)Asset.FastGetAsset(false /*bLoad*/);
			if (Binding)
			{
				if (Binding->GetGroom() == InGroomAsset)
				{
					Out->Bindings.Add(Binding);
				}
			}
			else 
			{
				// Use heuristic that binding asset usually contain groom asset name (when preserving the auto-generated binding name)
				const FString BindingAssetName = Asset.AssetName.GetPlainNameString();
				const FString GroomAssetName = InGroomAsset->GetName();
				if (TCString<TCHAR>::Strfind(*BindingAssetName, *GroomAssetName, true))
				{
					Binding = (UGroomBindingAsset*)Asset.GetAsset(/*bLoad*/);
					if (Binding->GetGroom() == InGroomAsset)
					{
						Out->Bindings.Add(Binding);
					}
				}
			}
		}
	}
}

static UAnimationAsset* GetFirstCompatibleAnimAsset(const USkeletalMeshComponent* InComponent)
{
	if (!InComponent)
	{
		return nullptr;
	}

	USkeleton* InSkeleton = InComponent->GetSkeletalMeshAsset()->GetSkeleton();
	if (!InSkeleton)
	{
		return nullptr;
	}

	const FString SkeletonString = FAssetData(InSkeleton).GetExportTextName();

	FARFilter Filter;
	Filter.ClassPaths.Add(UAnimationAsset::StaticClass()->GetClassPathName());
	//Filter.TagsAndValues.Add(USkeletalMesh::GetSkeletonMemberName(), SkeletonString);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> AnimAssetData;
	AssetRegistryModule.Get().GetAssets(Filter, AnimAssetData);

	// Filter binding asset which match the groom asset (as the tag/value filter above does not work)
	for (FAssetData& Asset : AnimAssetData)
	{
		if (InSkeleton->IsCompatibleForEditor(Asset))
		{
			return (UAnimationAsset*)Asset.GetAsset();
		}
	}
	return nullptr;
}

void FGroomCustomAssetEditorToolkit::InitCustomAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UGroomAsset* InCustomAsset)
{
	PreviewGroomComponent = nullptr;
	PreviewSkeletalMeshComponent = nullptr;
	PreviewSkeletalAnimationAsset = nullptr;
	GroomBindingAssetList = nullptr;

	ViewportTab = SNew(SGroomEditorViewport);
	ThumbnailPool = MakeShared<FAssetThumbnailPool>(64);
	GroomEditorStyle = MakeShareable(new FGroomEditorStyle());

	// Automatically affect the first skelal mesh compatible with the groom asset
	#if 0
	if (GroomBindingAssetList->Bindings.Num() > 0)
	{
		ActiveGroomBindingIndex = 0;
		GroomBindingAsset = GroomBindingAssetList->Bindings[ActiveGroomBindingIndex];
	}
	else
	#endif
	{
		ActiveGroomBindingIndex = -1;
	}
	SetCustomAsset(InCustomAsset);
	InitPreviewComponents();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	DetailView_LODProperties			= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_InterpolationProperties	= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_RenderingProperties		= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_PhysicsProperties		= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_CardsProperties			= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_MeshesProperties			= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_MaterialProperties		= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
#if GROOMEDITOR_ENABLE_COMPONENT_PANEL
	DetailView_PreviewGroomComponent	= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
#endif
	DetailView_BindingProperties		= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailView_PreviewSceneProperties	= PropertyEditorModule.CreateDetailView(DetailsViewArgs); 
	DetailView_DataflowProperties	= PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	// Customization
	DetailView_CardsProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Cards));
	DetailView_MeshesProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Meshes));
	DetailView_RenderingProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Strands));
	DetailView_InterpolationProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Interpolation));
	DetailView_PhysicsProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Physics));
	DetailView_LODProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::LODs));
	DetailView_MaterialProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomMaterialDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this));
	DetailView_BindingProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Bindings));
	DetailView_DataflowProperties->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FGroomRenderingDetails::MakeInstance, (IGroomCustomAssetEditorToolkit*)this, EMaterialPanelType::Dataflow));
	
	SEditorViewport::FArguments args;
	
	// Default layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_GroomAssetEditor_Layout_v15b")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(true)
					->AddTab(TabId_Viewport, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(TabId_LODProperties,			ETabState::OpenedTab)
					->AddTab(TabId_InterpolationProperties,	ETabState::OpenedTab)
					->AddTab(TabId_RenderingProperties,		ETabState::OpenedTab)
					->AddTab(TabId_CardsProperties,			ETabState::OpenedTab)
					->AddTab(TabId_MeshesProperties,		ETabState::OpenedTab)
					->AddTab(TabId_MaterialProperties,		ETabState::OpenedTab)
					->AddTab(TabId_PhysicsProperties,		ETabState::OpenedTab)
				#if GROOMEDITOR_ENABLE_COMPONENT_PANEL
					->AddTab(TabId_PreviewGroomComponent,	ETabState::OpenedTab)
				#endif
					->AddTab(TabId_BindingProperties,		ETabState::OpenedTab)
					->AddTab(TabId_PreviewSceneProperties,	ETabState::OpenedTab)
					->AddTab(TabId_DataflowProperties,		ETabState::OpenedTab)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;

	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		FGroomEditor::GroomEditorAppIdentifier,
		StandaloneDefaultLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		(UObject*)InCustomAsset);
	
	bIsTabManagerInitialized = true;

	// Initialize the content of the binding tab/
	// This initialization happens after the tab spawning to check it the binding tab is actually active/visible. 
	// This avoids to loading the binding assets if the tab is not visibile.
	{
		TSharedPtr<SDockTab> BindingTab = TabManager->FindExistingLiveTab(TabId_BindingProperties);
		if (BindingTab && BindingTab->IsActive())
		{
			InitializeBindingAssetTabContent();
		}
	}

	FProperty* P0 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsInterpolationMemberName());
	FProperty* P1 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsRenderingMemberName());
	FProperty* P2 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsPhysicsMemberName());
	FProperty* P3 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsCardsMemberName());
	FProperty* P4 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsLODMemberName());
	FProperty* P5 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsMeshesMemberName());
	FProperty* P6 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsMaterialsMemberName());
	FProperty* P7 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetHairGroupsInfoMemberName());
	FProperty* P9 = FindFProperty<FProperty>(GroomAsset->GetClass(), GET_MEMBER_NAME_CHECKED(UGroomAsset, AssetUserData));
	FProperty* P10 = FindFProperty<FProperty>(GroomAsset->GetClass(), UGroomAsset::GetDataflowSettingsMemberName());

	P7->SetMetaData(TEXT("Category"), TEXT("Hidden"));
	P0->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P1->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P2->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P3->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P4->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P5->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P6->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P7->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P9->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
	P10->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));

	// Override the display name so that it show correct/readable name when the details view is displayed as grid view
	P0->SetMetaData(TEXT("DisplayName"), TEXT("Interpolation"));
	P1->SetMetaData(TEXT("DisplayName"), TEXT("Strands"));
	P2->SetMetaData(TEXT("DisplayName"), TEXT("Physics"));
	P3->SetMetaData(TEXT("DisplayName"), TEXT("Cards"));
	P4->SetMetaData(TEXT("DisplayName"), TEXT("LOD"));
	P5->SetMetaData(TEXT("DisplayName"), TEXT("Meshes"));
	P6->SetMetaData(TEXT("DisplayName"), TEXT("Materials"));
	P7->SetMetaData(TEXT("DisplayName"), TEXT("Info"));
	P10->SetMetaData(TEXT("DisplayName"), TEXT("Dataflow"));

	// Set the asset we are editing in the details view
	if (DetailView_InterpolationProperties.IsValid())
	{
		DetailView_InterpolationProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_RenderingProperties.IsValid())
	{
		DetailView_RenderingProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_PhysicsProperties.IsValid())
	{
		DetailView_PhysicsProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_DataflowProperties.IsValid())
	{
		DetailView_DataflowProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_CardsProperties.IsValid())
	{
		DetailView_CardsProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_MeshesProperties.IsValid())
	{
		DetailView_MeshesProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_LODProperties.IsValid())
	{
		DetailView_LODProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	if (DetailView_MaterialProperties.IsValid())
	{
		DetailView_MaterialProperties->SetObject(Cast<UObject>(GroomAsset));
	}

#if GROOMEDITOR_ENABLE_COMPONENT_PANEL
	if (DetailView_PreviewGroomComponent.IsValid())
	{
		DetailView_PreviewGroomComponent->SetObject(PreviewGroomComponent.Get());
	}
#endif
	if (DetailView_BindingProperties.IsValid())
	{
		DetailView_BindingProperties->SetObject(Cast<UObject>(GroomBindingAssetList));
	}

	if (DetailView_PreviewSceneProperties.IsValid())
	{
		DetailView_PreviewSceneProperties->SetObject(Cast<UObject>(GroomAsset));
	}

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	const TSharedRef<FUICommandList>& CommandList = GetToolkitCommands();
		
	CommandList->MapAction(FGroomEditorCommands::Get().ResetSimulation,
		FExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::OnResetSimulation),
		FCanExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::CanResetSimulation));

	CommandList->MapAction(FGroomEditorCommands::Get().PauseSimulation,
		FExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::OnPauseSimulation),
		FCanExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::CanResetSimulation));

	CommandList->MapAction(FGroomEditorCommands::Get().PlaySimulation,
		FExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::OnPlaySimulation),
		FCanExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::CanPlaySimulation));

	CommandList->MapAction(FGroomEditorCommands::Get().PlayAnimation,
		FExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::OnPlayAnimation),
		FCanExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::CanPlayAnimation));

	CommandList->MapAction(FGroomEditorCommands::Get().StopAnimation,
		FExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::OnStopAnimation),
		FCanExecuteAction::CreateSP(this, &FGroomCustomAssetEditorToolkit::CanStopAnimation));

	// Delegate to refresh the panels (and the stats) when the groom asset change
	if (GroomAsset.IsValid())
	{
		FGroomCustomAssetEditorToolkit* LocalToolKit = this;
		auto InvalidateDetailViews = [LocalToolKit]()
		{
			if (LocalToolKit->DetailView_LODProperties)				{ LocalToolKit->DetailView_LODProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_InterpolationProperties)	{ LocalToolKit->DetailView_InterpolationProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_RenderingProperties)		{ LocalToolKit->DetailView_RenderingProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_PhysicsProperties)			{ LocalToolKit->DetailView_PhysicsProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_CardsProperties)			{ LocalToolKit->DetailView_CardsProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_MeshesProperties)			{ LocalToolKit->DetailView_MeshesProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_MaterialProperties)		{ LocalToolKit->DetailView_MaterialProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_BindingProperties)			{ LocalToolKit->DetailView_BindingProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_PreviewSceneProperties)	{ LocalToolKit->DetailView_PreviewSceneProperties->ForceRefresh(); }
			if (LocalToolKit->DetailView_DataflowProperties)		{ LocalToolKit->DetailView_DataflowProperties->ForceRefresh(); }
		};

		PropertyListenDelegatesResourceChanged.Add(GroomAsset->GetOnGroomAssetResourcesChanged().AddLambda(InvalidateDetailViews));
		PropertyListenDelegatesAssetChanged.Add(GroomAsset->GetOnGroomAssetChanged().AddLambda(InvalidateDetailViews));
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Simulation

void FGroomCustomAssetEditorToolkit::OnPlaySimulation()
{
	//if (NiagaraComponent != nullptr)
	//{
	//	NiagaraComponent->SetPaused(false);
	//}
}
bool FGroomCustomAssetEditorToolkit::CanPlaySimulation() const
{
	return true;
}

void FGroomCustomAssetEditorToolkit::OnPauseSimulation()
{
	//if (NiagaraComponent != nullptr)
	//{
	//	NiagaraComponent->SetPaused(true);
	//}
}

bool FGroomCustomAssetEditorToolkit::CanPauseSimulation() const
{
	return true;
}

void FGroomCustomAssetEditorToolkit::OnResetSimulation()
{
	//if (NiagaraComponent != nullptr)
	//{
	//	NiagaraComponent->Activate(true);
	//}	
}

bool FGroomCustomAssetEditorToolkit::CanResetSimulation() const
{
	return true;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Animation

void FGroomCustomAssetEditorToolkit::OnPlayAnimation()
{
	if (PreviewSkeletalMeshComponent != nullptr && PreviewSkeletalAnimationAsset != nullptr)
	{
		PreviewSkeletalMeshComponent->PlayAnimation(PreviewSkeletalAnimationAsset.Get(), true);
	}
}

bool FGroomCustomAssetEditorToolkit::CanPlayAnimation() const
{
	return PreviewSkeletalMeshComponent != nullptr && PreviewSkeletalAnimationAsset != nullptr && !PreviewSkeletalMeshComponent->IsPlaying();
}

void FGroomCustomAssetEditorToolkit::OnStopAnimation()
{
	if (PreviewSkeletalMeshComponent.Get())
	{
		PreviewSkeletalMeshComponent->Stop();
	}
}

bool FGroomCustomAssetEditorToolkit::CanStopAnimation() const
{
	return PreviewSkeletalMeshComponent != nullptr && PreviewSkeletalMeshComponent->IsPlaying();
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Editor

FGroomCustomAssetEditorToolkit::FGroomCustomAssetEditorToolkit()
{
	GroomAsset						= nullptr;
	PreviewGroomComponent			= nullptr;
	PreviewSkeletalMeshComponent	= nullptr;
	PreviewSkeletalAnimationAsset	= nullptr;
}

FGroomCustomAssetEditorToolkit::~FGroomCustomAssetEditorToolkit()
{

}

FText FGroomCustomAssetEditorToolkit::GetToolkitName() const
{
	return FText::FromString(GroomAsset->GetName());
}

FName FGroomCustomAssetEditorToolkit::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FGroomCustomAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Groom Asset Editor");
}

FText FGroomCustomAssetEditorToolkit::GetToolkitToolTipText() const
{
	return LOCTEXT("ToolTip", "Groom Asset Editor");
}

FString FGroomCustomAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "AnimationDatabase ").ToString();
}

FLinearColor FGroomCustomAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FColor::Red;
}

UGroomAsset* FGroomCustomAssetEditorToolkit::GetCustomAsset() const
{
	return GroomAsset.Get();
}

void FGroomCustomAssetEditorToolkit::SetCustomAsset(UGroomAsset* InCustomAsset)
{
	GroomAsset = InCustomAsset;
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_CardsProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_CardsProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("CardsPropertiesTab", "Cards"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_CardsProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_MeshesProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_MeshesProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("MeshesPropertiesTab", "Meshes"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_MeshesProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_MaterialProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_MaterialProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("MaterialPropertiesTab", "Material"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_MaterialProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_PhysicsProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_PhysicsProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("PhysicsPropertiesTab", "Physics"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_PhysicsProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_DataflowProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_DataflowProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("DataflowPropertiesTab", "Dataflow"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_DataflowProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_RenderingProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_RenderingProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("RenderingPropertiesTab", "Strands"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_RenderingProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab>  FGroomCustomAssetEditorToolkit::SpawnTab_InterpolationProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_InterpolationProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("InterpolationPropertiesTab", "Interpolation"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_InterpolationProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab>  FGroomCustomAssetEditorToolkit::SpawnTab_LODProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_LODProperties);

	return SNew(SDockTab)
		.Label(LOCTEXT("LODPropertiesTab", "LOD"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_LODProperties.ToSharedRef()
		];
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_PreviewGroomComponent(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_PreviewGroomComponent);

	return SNew(SDockTab)
		.Label(LOCTEXT("GroomComponentTab", "Component"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_PreviewGroomComponent.ToSharedRef()
		];
}

void FGroomCustomAssetEditorToolkit::InitializeBindingAssetTabContent()
{
	UGroomAsset* LocalGroomAsset = GetCustomAsset();
	if (GroomBindingAssetList == nullptr && LocalGroomAsset)
	{
		GroomBindingAssetList = NewObject<UGroomBindingAssetList>(GetTransientPackage(), NAME_None, RF_Transient);
		ListAllBindingAssets(LocalGroomAsset, GroomBindingAssetList);
		DetailView_BindingProperties->SetObject(Cast<UObject>(GroomBindingAssetList));
		DetailView_BindingProperties->ForceRefresh();
	}
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_BindingProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_BindingProperties);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.Label(LOCTEXT("BindingPropertiesTab", "Binding"))
		.TabColorScale(GetTabColorScale())
		[
			DetailView_BindingProperties.ToSharedRef()
		];

	if (bIsTabManagerInitialized)
	{
		InitializeBindingAssetTabContent();
	}
	else
	{	
		DockTab->SetOnTabActivated(SDockTab::FOnTabActivatedCallback::CreateLambda([this](TSharedRef<SDockTab> Input, ETabActivationCause)
		{
			InitializeBindingAssetTabContent();
		}));
	}

	return DockTab;
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>(TEXT("AdvancedPreviewScene"));

	TSharedRef<SWidget> PreviewSceneSettingsWidget = SNullWidget::NullWidget;
	TSharedPtr<class FAdvancedPreviewScene> PreviewScene = ViewportTab->GetAdvancedPreviewScene();
	if (PreviewScene.IsValid())
	{
		PreviewSceneSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());
	}

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
	.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
	[
		SNew(SBox)
		[
			PreviewSceneSettingsWidget
		]
	];

	return SpawnedTab;
}

UGroomComponent *FGroomCustomAssetEditorToolkit::GetPreview_GroomComponent() const
{	
	return PreviewGroomComponent.Get();
}

USkeletalMeshComponent *FGroomCustomAssetEditorToolkit::GetPreview_SkeletalMeshComponent() const
{
	return PreviewSkeletalMeshComponent.Get();
}

TSharedRef<SDockTab> FGroomCustomAssetEditorToolkit::SpawnViewportTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TabId_Viewport);
	check(GetPreview_GroomComponent());

	ViewportTab->SetSkeletalMeshComponent(GetPreview_SkeletalMeshComponent());
	ViewportTab->SetGroomComponent(GetPreview_GroomComponent());

	return SNew(SDockTab)
		.Label(LOCTEXT("RenderTitle", "Render"))
		.TabColorScale(GetTabColorScale())
		[
			ViewportTab.ToSharedRef()
		];
}

void FGroomCustomAssetEditorToolkit::PreviewBinding(int32 BindingIndex)
{
	if (BindingIndex >= 0 && GroomBindingAssetList.Get() && BindingIndex < GroomBindingAssetList->Bindings.Num())
	{
		GroomBindingAsset = GroomBindingAssetList->Bindings[BindingIndex];
	}
	else
	{
		GroomBindingAsset = nullptr;
	}

	if (GroomBindingAsset.Get())
	{
		PreviewSkeletalMeshComponent = NewObject<USkeletalMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		PreviewSkeletalMeshComponent->SetSkeletalMesh(GroomBindingAsset->GetTargetSkeletalMesh());
		PreviewSkeletalMeshComponent->SetVisibility(true);
		PreviewSkeletalMeshComponent->Activate(true);
		PreviewGroomComponent->AttachToComponent(PreviewSkeletalMeshComponent.Get(), FAttachmentTransformRules::KeepRelativeTransform);
		PreviewGroomComponent->SetGroomAsset(GroomAsset.Get(), GroomBindingAsset.Get());
		PreviewSkeletalAnimationAsset = GetFirstCompatibleAnimAsset(PreviewSkeletalMeshComponent.Get());
		ActiveGroomBindingIndex = BindingIndex;
	}
	else
	{
		PreviewGroomComponent->SetBindingAsset(nullptr);
		PreviewGroomComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		PreviewSkeletalMeshComponent = nullptr;
		PreviewSkeletalAnimationAsset = nullptr;
		ActiveGroomBindingIndex = -1;
	}

	ViewportTab->SetSkeletalMeshComponent(GetPreview_SkeletalMeshComponent());
}

int32 FGroomCustomAssetEditorToolkit::GetActiveBindingIndex() const
{
	return ActiveGroomBindingIndex;
}

FGroomEditorStyle* FGroomCustomAssetEditorToolkit::GetSlateStyle() const 
{ 
	return GroomEditorStyle.Get();
}

#undef LOCTEXT_NAMESPACE
