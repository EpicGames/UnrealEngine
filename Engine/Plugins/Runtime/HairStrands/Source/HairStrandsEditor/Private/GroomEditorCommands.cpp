// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomEditorCommands.h"
#include "GroomEditorStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "GroomEditorCommands"

FGroomEditorCommands::FGroomEditorCommands() :
	TCommands<FGroomEditorCommands>(
		"GroomEditorCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "GroomEditorMode", "Groom Tools"), // Localized context name for displaying
		NAME_None, // Parent
		FGroomEditorStyle::GetStyleName() // Icon Style Set
		)
{
}

void FGroomEditorCommands::RegisterCommands()
{
	UI_COMMAND(BeginHairPlaceTool,	"HairPlace","Start the Hair Placement Tool",		EUserInterfaceActionType::Button,	FInputChord());
	UI_COMMAND(Simulate,			"Simulate", "Start simulating Hair",				EUserInterfaceActionType::Button,	FInputChord(EKeys::S, EModifierKey::Alt));
	UI_COMMAND(ResetSimulation,		"Reset",	"Reset the running hair simulation",	EUserInterfaceActionType::Button,	FInputChord());
	UI_COMMAND(PauseSimulation,		"Pause",	"Pause the running hair simulation",	EUserInterfaceActionType::Button,	FInputChord());
	UI_COMMAND(PlaySimulation,		"Play",		"Play the hair simulation",				EUserInterfaceActionType::Button,	FInputChord());

	UI_COMMAND(PlayAnimation,		"Play",		"Play animation",						EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopAnimation,		"Stop",		"Stop animation",						EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "GroomViewportLODCommands"

FGroomViewportLODCommands::FGroomViewportLODCommands() :
	TCommands<FGroomViewportLODCommands>(
		TEXT("GroomViewportLODCmd"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "GroomViewportLODCmd", "Groom Viewport LOD Command"), // Localized context name for displaying
		NAME_None, // Parent context name. 
		FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
{
}

void FGroomViewportLODCommands::RegisterCommands()
{
	UI_COMMAND(LODAuto, "LOD Auto", "Automatically select LOD", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LOD0, "LOD 0", "Force select LOD 0", EUserInterfaceActionType::RadioButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
