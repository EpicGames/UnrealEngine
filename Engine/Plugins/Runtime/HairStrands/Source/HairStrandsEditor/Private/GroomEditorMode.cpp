// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomEditorMode.h"
#include "GroomEditorCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomEditorMode)

#define LOCTEXT_NAMESPACE "GroomEditor"

const FEditorModeID UGroomEditorMode::EM_GroomEditorModeId = TEXT("EM_GroomEditorMode");

UGroomEditorMode::UGroomEditorMode()
{
	Info = FEditorModeInfo(
		EM_GroomEditorModeId,
		LOCTEXT("GroomEditorMode", "Groom Editor"),
		FSlateIcon(),
		false);
}

bool UGroomEditorMode::AllowWidgetMove()
{ 
	return false; 
}

void UGroomEditorMode::Enter()
{
	Super::Enter();

	const FGroomEditorCommands& ToolManagerCommands = FGroomEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	// register tool set

	//
	// make shape tools
	////
	//auto HairPlaceToolBuilder = NewObject<UHairPlaceToolBuilder>();
	//HairPlaceToolBuilder->AssetAPI = ToolsContext->GetAssetAPI();
	//RegisterTool(ToolManagerCommands.BeginHairPlaceTool, TEXT("HairPlaceTool"), HairPlaceToolBuilder);
			
	//ToolsContext->ToolManager->SelectActiveToolType(EToolSide::Left, TEXT("HairPlaceTool"));

#ifdef TOOLED_ENABLE_VIEWPORT_INTERACTION
	///
	// Viewport Interaction
	///
	UEditorWorldExtensionCollection* ExtensionCollection = GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld());
	check(ExtensionCollection != nullptr);
	this->ViewportWorldInteraction = NewObject<UViewportWorldInteraction>(ExtensionCollection);
	ExtensionCollection->AddExtension(this->ViewportWorldInteraction);
		//Cast<UViewportWorldInteraction>(ExtensionCollection->AddExtension(UViewportWorldInteraction::StaticClass()));
	check(ViewportWorldInteraction != nullptr);
	//this->ViewportWorldInteraction->UseLegacyInteractions();
	//this->ViewportWorldInteraction->AddMouseCursorInteractor();
	this->ViewportWorldInteraction->SetUseInputPreprocessor(true);
	this->ViewportWorldInteraction->SetGizmoHandleType(EGizmoHandleTypes::All);

	// Set the current viewport.
	{
		const TSharedRef< ILevelEditor >& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstLevelEditor().ToSharedRef();

		// Do we have an active perspective viewport that is valid for VR?  If so, go ahead and use that.
		TSharedPtr<FEditorViewportClient> ViewportClient;
		{
			TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
			if (ActiveLevelViewport.IsValid())
			{
				ViewportClient = StaticCastSharedRef<SLevelViewport>(ActiveLevelViewport->AsWidget())->GetViewportClient();
			}
		}

		this->ViewportWorldInteraction->SetDefaultOptionalViewportClient(ViewportClient);
	}
#endif  // TOOLED_ENABLE_VIEWPORT_INTERACTION
}

void UGroomEditorMode::Exit()
{
#ifdef TOOLED_ENABLE_VIEWPORT_INTERACTION
	///
	// Viewport Interaction
	//
	if (IViewportInteractionModule::IsAvailable())
	{
		if (ViewportWorldInteraction != nullptr)
		{
			ViewportWorldInteraction->ReleaseMouseCursorInteractor();

			// Make sure gizmo is visible.  We may have hidden it
			ViewportWorldInteraction->SetTransformGizmoVisible(true);

			// Unregister mesh element transformer
			//ViewportWorldInteraction->SetTransformer(nullptr);

			UEditorWorldExtensionCollection* ExtensionCollection = GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld());
			if (ExtensionCollection != nullptr)
			{
				ExtensionCollection->RemoveExtension(ViewportWorldInteraction);
			}

			ViewportWorldInteraction = nullptr;
		}
	}
#endif // TOOLED_ENABLE_VIEWPORT_INTERACTION

	Super::Exit();
}

#undef LOCTEXT_NAMESPACE
