// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomFactory.h"
#include "GroomAsset.h"
#include "HairStrandsFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomFactory)

UGroomFactory::UGroomFactory()
{
	SupportedClass = UGroomAsset::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
	bText = false;
	bEditorImport = true;
}

UObject* UGroomFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UGroomAsset* Groom = NewObject<UGroomAsset>(InParent, InName, Flags | RF_Transactional);

	// Groom has to have at least one group to edit with one cards LOD
	Groom->SetNumGroup(1);
	Groom->GetHairGroupsLOD()[0].LODs[0].GeometryType = EGroomGeometryType::Cards;
	Groom->GetHairGroupsInterpolation()[0].InterpolationSettings.bUseUniqueGuide = true;

	// Group must have at least 1 LOD for cards/meshes even if it has no valid data to pass validation
	Groom->GetHairGroupsPlatformData()[0].Cards.LODs.SetNum(1);
	Groom->GetHairGroupsPlatformData()[0].Meshes.LODs.SetNum(1);

	Groom->UpdateHairGroupsInfo();
	Groom->UpdateCachedSettings();
	Groom->InitResources();

	UE::Groom::BuildGroomDataflowAsset(Groom);

	return Groom;
}

FString UGroomFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("GA_NewGroomAsset"));
}

