// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomImportOptionsWindow.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "GroomCacheImportOptions.h"
#include "GroomImportOptions.h"
#include "IDetailsView.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "GroomAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomImportOptionsWindow)

#define LOCTEXT_NAMESPACE "GroomImportOptionsWindow"

FGroomImportStatus GetGroomImportStatus(UGroomHairGroupsPreview* InDescription, UGroomCacheImportOptions* InGroomCacheImportOptions, UGroomHairGroupsMapping* InGroupMapping)
{
	FGroomImportStatus Out;
	Out.Status = EHairDescriptionStatus::None;
	if (!InDescription)
	{
		EnumAddFlags(Out.Status, EHairDescriptionStatus::Unknown);
		return Out;
	}

	const bool bImportGroomAsset = !InGroomCacheImportOptions || InGroomCacheImportOptions->ImportSettings.bImportGroomAsset;
	const bool bImportGroomCache = InGroomCacheImportOptions && InGroomCacheImportOptions->ImportSettings.bImportGroomCache;
	if (!bImportGroomAsset && !bImportGroomCache)
	{
		EnumAddFlags(Out.Status, EHairDescriptionStatus::Unknown);
		return Out;
	}

	if (InDescription->Groups.Num() == 0)
	{
		EnumAddFlags(Out.Status, EHairDescriptionStatus::NoGroup);
		return Out;
	}

	// Check the validity of the groom to import

	bool bGuidesOnly = false;
	for (const FGroomHairGroupPreview& Group : InDescription->Groups)
	{
		if (Group.CurveCount == 0)
		{
			EnumAddFlags(Out.Status, EHairDescriptionStatus::NoCurve);
			if (Group.GuideCount > 0)
			{
				bGuidesOnly = true;
			}
			break;
		}
	}

	// Check if any curve or point have been trimmed
	for (const FGroomHairGroupPreview& Group : InDescription->Groups)
	{
		if (Group.Flags & uint32(EHairGroupInfoFlags::HasTrimmedCurve))
		{
			EnumAddFlags(Out.Status, EHairDescriptionStatus::CurveLimit);
		}
		if (Group.Flags & uint32(EHairGroupInfoFlags::HasTrimmedPoint))
		{
			EnumAddFlags(Out.Status, EHairDescriptionStatus::PointLimit);
		}
		if (Group.Flags & uint32(EHairGroupInfoFlags::HasInvalidPoint))
		{
			EnumAddFlags(Out.Status, EHairDescriptionStatus::InvalidPoint);
		}		
	}

	if (InGroupMapping && !InGroupMapping->HasValidMapping())
	{
		EnumAddFlags(Out.Status, EHairDescriptionStatus::InvalidGroupMapping);
	}

	if (!bImportGroomCache)
	{
		EnumAddFlags(Out.Status, EHairDescriptionStatus::GroomValid);
		return Out;
	}

	// Update the states of the properties being monitored
	Out.bImportGroomAssetState = InGroomCacheImportOptions->ImportSettings.bImportGroomAsset;
	Out.bImportGroomCacheState = InGroomCacheImportOptions->ImportSettings.bImportGroomCache;
	Out.GroomAsset = InGroomCacheImportOptions->ImportSettings.GroomAsset;

	if (!InGroomCacheImportOptions->ImportSettings.bImportGroomAsset)
	{
		// When importing a groom cache with a provided groom asset, check their compatibility
		UGroomAsset* GroomAssetForCache = Cast<UGroomAsset>(InGroomCacheImportOptions->ImportSettings.GroomAsset.TryLoad());
		if (!GroomAssetForCache)
		{
			// No groom asset provided or loaded but one is needed with this setting
			EnumAddFlags(Out.Status, bGuidesOnly ? EHairDescriptionStatus::GuidesOnly : EHairDescriptionStatus::GroomCache);
			return Out;
		}

		const TArray<FHairGroupPlatformData>& GroomHairGroupsData = GroomAssetForCache->GetHairGroupsPlatformData();
		if (GroomHairGroupsData.Num() != InDescription->Groups.Num())
		{
			EnumAddFlags(Out.Status, bGuidesOnly ? EHairDescriptionStatus::GuidesOnlyIncompatible : EHairDescriptionStatus::GroomCacheIncompatible);
			return Out;
		}

		for (int32 Index = 0; Index < GroomHairGroupsData.Num(); ++Index)
		{
			// Check the strands compatibility
			if (!bGuidesOnly && InDescription->Groups[Index].CurveCount != GroomHairGroupsData[Index].Strands.BulkData.GetNumCurves())
			{
				EnumAddFlags(Out.Status, EHairDescriptionStatus::GroomCacheIncompatible);
				break;
			}

			// Check the guides compatibility if there were strands tagged as guides
			// Otherwise, guides will be generated according to the groom asset interpolation settings
			// and compatibility cannot be determined here
			if (InDescription->Groups[Index].GuideCount > 0 &&
				InDescription->Groups[Index].GuideCount != GroomHairGroupsData[Index].Guides.BulkData.GetNumCurves())
			{
				EnumAddFlags(Out.Status, bGuidesOnly ? EHairDescriptionStatus::GuidesOnlyIncompatible : EHairDescriptionStatus::GroomCacheIncompatible);
				break;
			}
		}

		EnumAddFlags(Out.Status, bGuidesOnly ? EHairDescriptionStatus::GuidesOnlyCompatible : EHairDescriptionStatus::GroomCacheCompatible);
	}
	else
	{
		// A guides-only groom cannot be imported as asset, but otherwise the imported groom asset
		// is always compatible with the groom cache since they are from the same file
		EnumAddFlags(Out.Status, bGuidesOnly ? EHairDescriptionStatus::GuidesOnly : EHairDescriptionStatus::GroomValid);
	}

	return Out;
}

void SGroomImportOptionsWindow::UpdateStatus(UGroomHairGroupsPreview* Description) const
{
	ImportStatus = GetGroomImportStatus(Description, GroomCacheImportOptions, GroupsMapping);
	// If a groom is a groom guide-cache and has groom cache compatible asset, remove the 'NoCurve' flags 
	// which would cause the asset to be reported as invalid
	if (EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::NoCurve) &&
		EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::GuidesOnlyCompatible))
	{
		EnumRemoveFlags(ImportStatus.Status, EHairDescriptionStatus::NoCurve);
	}
}

FText GetGroomImportStatusText(FGroomImportStatus In, bool bAddPrefix)
{
	FString Out;
	if (bAddPrefix)
	{
		if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::Error)) 						{ Out += LOCTEXT("GroomOptionsWindow_ValidationText0",  "Error\n").ToString(); }
		else if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::Warning)) 				{ Out += LOCTEXT("GroomOptionsWindow_ValidationText1",  "Warning\n").ToString(); }
		else if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::Valid))					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText2",  "Valid\n").ToString(); }
	}

	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::NoCurve)) 					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText3",  "Some groups have 0 curves.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::NoGroup)) 					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText4",  "The groom does not contain any group.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::GroomCache))					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText5",  "A compatible groom asset must be provided to import the groom cache.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::GroomCacheCompatible)) 		{ Out += LOCTEXT("GroomOptionsWindow_ValidationText6",  "The groom cache is compatible with the groom asset provided.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::GroomCacheIncompatible))		{ Out += LOCTEXT("GroomOptionsWindow_ValidationText7",  "The groom cache is incompatible with the groom asset provided.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::GuidesOnly)) 				{ Out += LOCTEXT("GroomOptionsWindow_ValidationText8",  "Only guides were detected. A compatible groom asset must be provided.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::GuidesOnlyCompatible)) 		{ Out += LOCTEXT("GroomOptionsWindow_ValidationText9",  "Only guides were detected. The groom asset provided is compatible.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::GuidesOnlyIncompatible))		{ Out += LOCTEXT("GroomOptionsWindow_ValidationText10", "Only guides were detected. The groom asset provided is incompatible.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::CurveLimit))					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText11", "At least one group contains more curves than allowed limit (Max:4M). Curves beyond that limit will be trimmed.\n").ToString(); static_assert(HAIR_MAX_NUM_CURVE_PER_GROUP == 4194303); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::PointLimit))					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText12", "At least one group contains more control points per curve than the allowed limit (Max:255). Control points beyond that limit will be trimmed.\n").ToString(); static_assert(HAIR_MAX_NUM_POINT_PER_CURVE == 255); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::InvalidPoint)) 				{ Out += LOCTEXT("GroomOptionsWindow_ValidationText13", "At least one group contains a curve with invalid points. These curves will be trimmed from the asset.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::InvalidGroupMapping)) 		{ Out += LOCTEXT("GroomOptionsWindow_ValidationText15", "No mapping found using group names.\n").ToString(); }
	if (EnumHasAnyFlags(In.Status, EHairDescriptionStatus::Unknown)) 					{ Out += LOCTEXT("GroomOptionsWindow_ValidationText14", "Unknown\n").ToString(); }

	return FText::FromString(Out);
}

FText SGroomImportOptionsWindow::GetStatusText() const
{
	return GetGroomImportStatusText(ImportStatus, true);
}

FSlateColor SGroomImportOptionsWindow::GetStatusColor() const
{
	if (EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::Error))  	{ return FLinearColor(0.80f, 0, 0, 1); }
	if (EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::Warning))  { return FLinearColor(0.80f, 0.80f, 0, 1); }
	if (EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::Valid))	{ return FLinearColor(0, 0.80f, 0, 1); }

	return FLinearColor(1, 1, 1);
}

static void AddAttribute(SVerticalBox::FScopedWidgetSlotArguments& Slot, FText AttributeLegend)
{
	const FLinearColor AttributeColor(0.72f, 0.72f, 0.20f);
	const FSlateFontInfo AttributeFont = FAppStyle::GetFontStyle("CurveEd.InfoFont");
	const FSlateFontInfo AttributeResultFont = FAppStyle::GetFontStyle("CurveEd.InfoFont");

	Slot
	.AutoHeight()
	.Padding(2)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10, 0, 0, 0)
			[
				SNew(STextBlock)
				.Font(AttributeFont)
				.Text(AttributeLegend)
				.ColorAndOpacity(AttributeColor)
			]				
		]
	];
}

static void AddGroupRemappingNoFoundMessage(SVerticalBox::FScopedWidgetSlotArguments& Slot, UGroomHairGroupsMapping* InMapping)
{
	const FLinearColor AttributeColor(0.0f, 0.72f, 0.0f);
	const FSlateFontInfo AttributeFont = FAppStyle::GetFontStyle("CurveEd.InfoFont");
	const FSlateFontInfo AttributeResultFont = FAppStyle::GetFontStyle("CurveEd.InfoFont");

	FText MessageText = LOCTEXT("GroomGroupMapping_NotFound", "No mapping found using group names. Edit the mapping manually or use default values.");
	FLinearColor MessageColor = FLinearColor::Red;

	Slot
	.AutoHeight()
	.Padding(2)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10, 0, 0, 0)
			[
				SNew(STextBlock)
				.Font(AttributeFont)
				.Text(MessageText)
				.ColorAndOpacity(MessageColor)
			]
		]
	];
}

FText GetHairAttributeLocText(EHairAttribute In, uint32 InFlags)
{
	// If a new optional attribute is added, please add its UI/text description here
	static_assert(uint32(EHairAttribute::Count) == 8);

	switch (In)
	{
	case EHairAttribute::Width:						return LOCTEXT("GroomOptionsWindow_HasWidth", "Width");
	case EHairAttribute::RootUV:					return HasHairAttributeFlags(InFlags, EHairAttributeFlags::HasRootUDIM) ? LOCTEXT("GroomOptionsWindow_HasRootUDIM", "Root UV (UDIM)") : LOCTEXT("GroomOptionsWindow_HasRootUV", "Root UV");
	case EHairAttribute::ClumpID:					return HasHairAttributeFlags(InFlags, EHairAttributeFlags::HasMultipleClumpIDs) ? LOCTEXT("GroomOptionsWindow_HasClumpIDs", "Clump IDs (3)") : LOCTEXT("GroomOptionsWindow_HasClumpID", "Clump ID");
	case EHairAttribute::StrandID:					return LOCTEXT("GroomOptionsWindow_HasStrandID", "Strand ID");
	case EHairAttribute::PrecomputedGuideWeights:	return LOCTEXT("GroomOptionsWindow_HasPercomputedGuideWeights", "Pre-Computed Guide Weights");
	case EHairAttribute::Color:						return LOCTEXT("GroomOptionsWindow_HasColor", "Color");
	case EHairAttribute::Roughness:					return LOCTEXT("GroomOptionsWindow_HasRoughness", "Roughness");
	case EHairAttribute::AO:						return LOCTEXT("GroomOptionsWindow_HasAO", "AO");
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> SGroomImportOptionsWindow::AddImportButtons(FText InMainButtonLabel, bool bShowImportAllButton)
{
	if (bShowImportAllButton)
	{
		return SNew(SUniformGridPanel)
		.SlotPadding(2)
		+ SUniformGridPanel::Slot(0, 0)
		[
			SAssignNew(ImportButton, SButton)
			.HAlign(HAlign_Center)
			.Text(InMainButtonLabel)
			.IsEnabled(this, &SGroomImportOptionsWindow::CanImport)
			.OnClicked(this, &SGroomImportOptionsWindow::OnImport)
		]
		+ SUniformGridPanel::Slot(1, 0)
		[
			SAssignNew(ImportButton, SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("GroomOptionWindow_ImportAll", "Import All"))
			.ToolTipText(LOCTEXT("GroomOptionWindow_ImportAll_ToolTip", "Import all files with these same settings"))
			.IsEnabled(this, &SGroomImportOptionsWindow::CanImport)
			.OnClicked(this, &SGroomImportOptionsWindow::OnImportAll)
		]
		+ SUniformGridPanel::Slot(2, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("GroomOptionWindow_Cancel", "Cancel"))
			.OnClicked(this, &SGroomImportOptionsWindow::OnCancel)
		];
	}
	else
	{
		return SNew(SUniformGridPanel)
		.SlotPadding(2)
		+ SUniformGridPanel::Slot(0, 0)
		[
			SAssignNew(ImportButton, SButton)
			.HAlign(HAlign_Center)
			.Text(InMainButtonLabel)
			.IsEnabled(this, &SGroomImportOptionsWindow::CanImport)
			.OnClicked(this, &SGroomImportOptionsWindow::OnImport)
		]
		+ SUniformGridPanel::Slot(1, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("GroomOptionWindow_Cancel", "Cancel"))
			.OnClicked(this, &SGroomImportOptionsWindow::OnCancel)
		];
	}
}

void SGroomImportOptionsWindow::Construct(const FArguments& InArgs)
{
	ImportOptions = InArgs._ImportOptions;
	GroomCacheImportOptions = InArgs._GroomCacheImportOptions;
	GroupsPreview = InArgs._GroupsPreview;
	GroupsMapping = InArgs._GroupsMapping;
	WidgetWindow = InArgs._WidgetWindow;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(ImportOptions);
	
	DetailsView2 = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView2->SetObject(GroupsPreview);

	DetailsView3 = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView3->SetObject(GroupsMapping);

	GroomCacheDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	GroomCacheDetailsView->SetObject(GroomCacheImportOptions);

	ImportStatus.Status = EHairDescriptionStatus::None;
	UpdateStatus(GroupsPreview);

	// Aggregate attributes from all groups (ideally we should display each group attribute separately, to check if one groom is not missing data)
	uint32 Attributes = 0;
	uint32 AttributeFlags = 0;
	for (const FGroomHairGroupPreview& Group : GroupsPreview->Groups)
	{
		Attributes |= Group.Attributes;
		AttributeFlags |= Group.AttributeFlags;
	}

	FText bHasAttributeText = LOCTEXT("GroomOptionsWindow_HasAttributeNone", "None");
	FLinearColor bHasAttributeColor = FLinearColor(0.80f, 0, 0, 1);
	if (Attributes != 0)
	{
		bHasAttributeText = LOCTEXT("GroomOptionsWindow_HasAttributeValid", "Valid");
		bHasAttributeColor = FLinearColor(0, 0.80f, 0, 1);
	}

	auto VerticalSlot = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("CurrentFile", "Current File: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(InArgs._FullPath)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("GroomOptionsWindow_StatusFile", "Status File: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SGroomImportOptionsWindow::GetStatusText)))
					.ColorAndOpacity(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SGroomImportOptionsWindow::GetStatusColor)))
				]
			]
		]

		
		// Insert title of for the attributes
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
					.Text(LOCTEXT("GroomOptionsWindow_Attribute", "Attributes: "))
				]
				+ SHorizontalBox::Slot()
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
					.Text(bHasAttributeText)
					.ColorAndOpacity(bHasAttributeColor)
				]
			]
		]

		// All optional attribute will be inserted here
		// The widget are inserted at the end of this function

		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)		
		[
			GroomCacheDetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)		
		[
			DetailsView2->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			DetailsView3->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			AddImportButtons(InArgs._ButtonLabel, InArgs._bShowImportAllButton)
		];

	// Insert all the optional attributes
	uint32 AttributeSlotIndex = 3;
	for (uint32 AttributeIt = 0; AttributeIt < uint32(EHairAttribute::Count); ++AttributeIt)
	{
		const EHairAttribute AttributeType = (EHairAttribute)AttributeIt;
		if (HasHairAttribute(Attributes, AttributeType))
		{
			SVerticalBox::FScopedWidgetSlotArguments SlotArg = VerticalSlot->InsertSlot(AttributeSlotIndex++);
			AddAttribute(SlotArg, GetHairAttributeLocText(AttributeType, AttributeFlags));
		}
	}

	if (GroupsMapping && !GroupsMapping->HasValidMapping())
	{
		SVerticalBox::FScopedWidgetSlotArguments SlotArg = VerticalSlot->InsertSlot(AttributeSlotIndex++);
		AddGroupRemappingNoFoundMessage(SlotArg, GroupsMapping);
	}

	this->ChildSlot
	[
		VerticalSlot
	];
}

bool SGroomImportOptionsWindow::CanImport() const
{
	bool bNeedUpdate = ImportStatus.Status == EHairDescriptionStatus::None;
	if (GroomCacheImportOptions)
	{
		bNeedUpdate |= ImportStatus.bImportGroomAssetState != GroomCacheImportOptions->ImportSettings.bImportGroomAsset;
		bNeedUpdate |= ImportStatus.bImportGroomCacheState != GroomCacheImportOptions->ImportSettings.bImportGroomCache;
		bNeedUpdate |= ImportStatus.GroomAsset != GroomCacheImportOptions->ImportSettings.GroomAsset;
	}

	if (bNeedUpdate)
	{
		UpdateStatus(GroupsPreview);
	}

	return EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::Valid | EHairDescriptionStatus::Warning);
}

enum class EGroomOptionsVisibility : uint8
{
	None = 0x00,
	ConversionOptions = 0x01,
	BuildOptions = 0x02,
	All = ConversionOptions | BuildOptions
};

ENUM_CLASS_FLAGS(EGroomOptionsVisibility);

TSharedPtr<SGroomImportOptionsWindow> DisplayOptions(
	UGroomImportOptions* ImportOptions, 
	UGroomCacheImportOptions* GroomCacheImportOptions, 
	UGroomHairGroupsPreview* GroupsPreview,
	UGroomHairGroupsMapping* GroupsMapping,
	const FString& FilePath, 
	EGroomOptionsVisibility VisibilityFlag, 
	FText WindowTitle, 
	FText InButtonLabel,
	bool bInShowImportAllButton)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.SizingRule(ESizingRule::Autosized);

	TSharedPtr<SGroomImportOptionsWindow> OptionsWindow;

	FProperty* ConversionOptionsProperty = FindFProperty<FProperty>(ImportOptions->GetClass(), GET_MEMBER_NAME_CHECKED(UGroomImportOptions, ConversionSettings));
	if (ConversionOptionsProperty)
	{
		if (EnumHasAnyFlags(VisibilityFlag, EGroomOptionsVisibility::ConversionOptions))
		{
			ConversionOptionsProperty->SetMetaData(TEXT("ShowOnlyInnerProperties"), TEXT("1"));
			ConversionOptionsProperty->SetMetaData(TEXT("Category"), TEXT("Conversion"));
		}
		else
		{
			// Note that UGroomImportOptions HideCategories named "Hidden",
			// but the hiding doesn't work with ShowOnlyInnerProperties 
			ConversionOptionsProperty->RemoveMetaData(TEXT("ShowOnlyInnerProperties"));
			ConversionOptionsProperty->SetMetaData(TEXT("Category"), TEXT("Hidden"));
		}
	}

	FString FileName = FPaths::GetCleanFilename(FilePath);
	Window->SetContent
	(
		SAssignNew(OptionsWindow, SGroomImportOptionsWindow)
		.ImportOptions(ImportOptions)
		.GroomCacheImportOptions(GroomCacheImportOptions)
		.GroupsPreview(GroupsPreview)
		.GroupsMapping(GroupsMapping)
		.WidgetWindow(Window)
		.FullPath(FText::FromString(FileName))
		.ButtonLabel(InButtonLabel)
		.bShowImportAllButton(bInShowImportAllButton)
	);

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	return OptionsWindow;
}

TSharedPtr<SGroomImportOptionsWindow> SGroomImportOptionsWindow::DisplayImportOptions(UGroomImportOptions* ImportOptions, UGroomCacheImportOptions* GroomCacheImportOptions, UGroomHairGroupsPreview* GroupsPreview, UGroomHairGroupsMapping* GroupsMapping, const FString& FilePath, bool bShowImportAllButton)
{
	// If there's no groom cache to import, don't show its import options
	UGroomCacheImportOptions* GroomCacheOptions = GroomCacheImportOptions && GroomCacheImportOptions->ImportSettings.bImportGroomCache ? GroomCacheImportOptions : nullptr;
	return DisplayOptions(ImportOptions, GroomCacheOptions, GroupsPreview, GroupsMapping, FilePath, EGroomOptionsVisibility::All, LOCTEXT("GroomImportWindowTitle", "Groom Import Options"), LOCTEXT("Import", "Import"), bShowImportAllButton);
}

TSharedPtr<SGroomImportOptionsWindow> SGroomImportOptionsWindow::DisplayRebuildOptions(UGroomImportOptions* ImportOptions, UGroomHairGroupsPreview* GroupsPreview, UGroomHairGroupsMapping* GroupsMapping, const FString& FilePath)
{
	return DisplayOptions(ImportOptions, nullptr, GroupsPreview, GroupsMapping, FilePath, EGroomOptionsVisibility::BuildOptions, LOCTEXT("GroomRebuildWindowTitle ", "Groom Build Options"), LOCTEXT("Build", "Build"), false /*bShowImportAllButton*/);
}


#undef LOCTEXT_NAMESPACE
