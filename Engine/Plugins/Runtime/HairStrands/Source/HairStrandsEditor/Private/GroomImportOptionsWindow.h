// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

#include "GroomImportOptionsWindow.generated.h"

class SButton;
class UGroomImportOptions;
class UGroomCacheImportOptions;
class UGroomHairGroupsPreview;
class UGroomHairGroupsMapping;
struct FHairGroupInfo;

UENUM()
enum class EHairDescriptionStatus : uint32
{
	None					= 0,
	GroomValid				= 1u<<1,
	NoGroup					= 1u<<2,
	NoCurve					= 1u<<3,
	GroomCache				= 1u<<4, // groom cache with unspecified groom asset
	GroomCacheCompatible	= 1u<<5,
	GroomCacheIncompatible	= 1u<<6,
	GuidesOnly				= 1u<<7, // guides-only with unspecified groom asset
	GuidesOnlyCompatible	= 1u<<8,
	GuidesOnlyIncompatible	= 1u<<9,
	PointLimit				= 1u<<10,
	CurveLimit				= 1u<<11,
	InvalidPoint			= 1u<<12,
	InvalidGroupMapping		= 1u<<13,
	Unknown					= 1u<<14,

	Warning = CurveLimit | PointLimit | InvalidPoint | InvalidGroupMapping,
	Error   = NoCurve | NoGroup | GroomCacheIncompatible | GuidesOnlyIncompatible | GroomCache | GuidesOnly | Unknown,
	Valid   = GroomValid | GroomCacheCompatible | GuidesOnlyCompatible
};
ENUM_CLASS_FLAGS(EHairDescriptionStatus);

struct FGroomImportStatus
{
	EHairDescriptionStatus Status;
	FSoftObjectPath GroomAsset;
	bool bImportGroomAssetState = false;
	bool bImportGroomCacheState = false;
};

FGroomImportStatus GetGroomImportStatus(UGroomHairGroupsPreview* InDescription, UGroomCacheImportOptions* InGroomCacheImportOptions, UGroomHairGroupsMapping* InGroupMapping);
FText GetGroomImportStatusText(FGroomImportStatus In, bool bAddPrefix);

class SGroomImportOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGroomImportOptionsWindow)
		: _ImportOptions(nullptr)
		, _GroomCacheImportOptions(nullptr)
		, _GroupsPreview(nullptr)
		, _GroupsMapping(nullptr)
		, _WidgetWindow()
		, _FullPath()
		, _ButtonLabel()
		, _bShowImportAllButton(false)
	{}

	SLATE_ARGUMENT(UGroomImportOptions*, ImportOptions)
	SLATE_ARGUMENT(UGroomCacheImportOptions*, GroomCacheImportOptions)
	SLATE_ARGUMENT(UGroomHairGroupsPreview*, GroupsPreview)
	SLATE_ARGUMENT(UGroomHairGroupsMapping*, GroupsMapping)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(FText, FullPath)
	SLATE_ARGUMENT(FText, ButtonLabel)
	SLATE_ARGUMENT(bool, bShowImportAllButton)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	static TSharedPtr<SGroomImportOptionsWindow> DisplayImportOptions(
		UGroomImportOptions* ImportOptions, 
		UGroomCacheImportOptions* GroomCacheImportOptions, 
		UGroomHairGroupsPreview* GroupsPreview,
		UGroomHairGroupsMapping* GroupsMapping,
		const FString& FilePath,
		bool bShowImportAllButton);

	static TSharedPtr<SGroomImportOptionsWindow> DisplayRebuildOptions(
		UGroomImportOptions* ImportOptions,
		UGroomHairGroupsPreview* GroupsPreview,
		UGroomHairGroupsMapping* GroupsMapping,
		const FString& FilePath);

	FReply OnImport()
	{
		bShouldImport = true;
		bShouldImportAll = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnImportAll()
	{
		bShouldImport = true;
		bShouldImportAll = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		bShouldImportAll = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}

	bool ShouldImportAll() const
	{
		return bShouldImportAll;
	}

	SGroomImportOptionsWindow() 
		: ImportOptions(nullptr)
		, GroomCacheImportOptions(nullptr)
		, GroupsPreview(nullptr)
		, GroupsMapping(nullptr)
	{}

private:

	bool CanImport() const;
	void UpdateStatus(UGroomHairGroupsPreview* Description) const;
	FText GetStatusText() const;
	FSlateColor GetStatusColor() const;
	TSharedRef<SWidget> AddImportButtons(FText InMainButtonLabel, bool bShowImportAllButton);

	// Properties that are monitored for changes
	mutable FGroomImportStatus ImportStatus;

private:
	UGroomImportOptions* ImportOptions;
	UGroomCacheImportOptions* GroomCacheImportOptions;
	TSharedPtr<class IDetailsView> DetailsView;
	TSharedPtr<class IDetailsView> GroomCacheDetailsView;
	TSharedPtr<class IDetailsView> DetailsView2;
	TSharedPtr<class IDetailsView> DetailsView3;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ImportButton;
	bool bShouldImport = false;
	bool bShouldImportAll = false;
public:
	UGroomHairGroupsPreview* GroupsPreview;
	UGroomHairGroupsMapping* GroupsMapping;
};
