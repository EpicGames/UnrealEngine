// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomMaterialDetails.h"
#include "Widgets/Input/SCheckBox.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IDetailGroup.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "GroomComponent.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorDirectories.h"
#include "UnrealEdGlobals.h"
#include "IDetailsView.h"
#include "MaterialList.h"
#include "PropertyCustomizationHelpers.h"
#include "Interfaces/IMainFrameModule.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Rendering/SkeletalMeshModel.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "EditorFramework/AssetImportData.h"
#include "Logging/LogMacros.h"

#include "MeshDescription.h"
#include "MeshAttributes.h"
#include "MeshAttributeArray.h"

#include "Widgets/Input/STextComboBox.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "IDocumentation.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "ComponentReregisterContext.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "SKismetInspector.h"
#include "PropertyEditorDelegates.h"
#include "PropertyCustomizationHelpers.h"
#include "GroomCustomAssetEditorToolkit.h"
#include "JsonObjectConverter.h"

#define LOCTEXT_NAMESPACE "GroomMaterialDetails"

/////////////////////////////////////////////////////////////////////////////////////////////////////////
// FGroomMaterialDetails

FGroomMaterialDetails::FGroomMaterialDetails(IGroomCustomAssetEditorToolkit* InToolkit)
: GroomDetailLayout(nullptr)
{
	if (InToolkit)
	{
		GroomAsset = InToolkit->GetCustomAsset();// InGroomAsset;
	}
	bDeleteWarningConsumed = false;
}

FGroomMaterialDetails::~FGroomMaterialDetails()
{

}

TSharedRef<IDetailCustomization> FGroomMaterialDetails::MakeInstance(IGroomCustomAssetEditorToolkit* InToolkit)
{
	return MakeShareable(new FGroomMaterialDetails(InToolkit));
}

void FGroomMaterialDetails::OnCopyMaterialList()
{
}

void FGroomMaterialDetails::OnPasteMaterialList()
{
}

bool FGroomMaterialDetails::OnCanCopyMaterialList() const
{
	return false;
}

void FGroomMaterialDetails::AddMaterials(IDetailLayoutBuilder& DetailLayout)
{
	if (!GroomAsset)
	{
		return;
	}

	// Create material list panel to let users control the materials array
	{
		FString MaterialCategoryName = FString(TEXT("Material Slots"));
		IDetailCategoryBuilder& MaterialCategory = DetailLayout.EditCategory(*MaterialCategoryName, FText::GetEmpty(), ECategoryPriority::Important);
		MaterialCategory.AddCustomRow(LOCTEXT("AddLODLevelCategories_MaterialArrayOperationAdd", "Materials Operation Add Material Slot"))
			.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FGroomMaterialDetails::OnCopyMaterialList), FCanExecuteAction::CreateSP(this, &FGroomMaterialDetails::OnCanCopyMaterialList)))
			.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FGroomMaterialDetails::OnPasteMaterialList)))
			.NameContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("AddLODLevelCategories_MaterialArrayOperations", "Material Slots"))
			]
			.ValueContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &FGroomMaterialDetails::GetMaterialArrayText)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 1.0f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.Text(LOCTEXT("AddLODLevelCategories_MaterialArrayOpAdd", "Add Material Slot"))
						.ToolTipText(LOCTEXT("AddLODLevelCategories_MaterialArrayOpAdd_Tooltip", "Add Material Slot at the end of the Material slot array. Those Material slots can be used to override a LODs section, (not the base LOD)"))
						.ContentPadding(4.0f)
						.ForegroundColor(FSlateColor::UseForeground())
						.OnClicked(this, &FGroomMaterialDetails::AddMaterialSlot)
						.IsEnabled(true)
						.IsFocusable(false)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			];
		{
			FMaterialListDelegates MaterialListDelegates;

			MaterialListDelegates.OnGetMaterials.BindSP(this, &FGroomMaterialDetails::OnGetMaterialsForArray, 0);
			MaterialListDelegates.OnMaterialChanged.BindSP(this, &FGroomMaterialDetails::OnMaterialArrayChanged, 0);
			MaterialListDelegates.OnGenerateCustomNameWidgets.BindSP(this, &FGroomMaterialDetails::OnGenerateCustomNameWidgetsForMaterialArray);
			MaterialListDelegates.OnGenerateCustomMaterialWidgets.BindSP(this, &FGroomMaterialDetails::OnGenerateCustomMaterialWidgetsForMaterialArray, 0);
			MaterialListDelegates.OnMaterialListDirty.BindSP(this, &FGroomMaterialDetails::OnMaterialListDirty);

			MaterialListDelegates.OnCopyMaterialItem.BindSP(this, &FGroomMaterialDetails::OnCopyMaterialItem);
			MaterialListDelegates.OnCanCopyMaterialItem.BindSP(this, &FGroomMaterialDetails::OnCanCopyMaterialItem);
			MaterialListDelegates.OnPasteMaterialItem.BindSP(this, &FGroomMaterialDetails::OnPasteMaterialItem);

			//Pass an empty material list owner (owner can be use by the asset picker filter. In this case we do not need it)
			TArray<FAssetData> MaterialListOwner;
			MaterialListOwner.Add(GroomAsset);
			MaterialCategory.AddCustomBuilder(MakeShareable(new FMaterialList(MaterialCategory.GetParentLayout(), MaterialListDelegates, MaterialListOwner, false, true)));
		}
	}
}

void FGroomMaterialDetails::OnCopyMaterialItem(int32 CurrentSlot)
{
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	if (GroomAsset->GetHairGroupsMaterials().IsValidIndex(CurrentSlot))
	{
		const FHairGroupsMaterial& Material = GroomAsset->GetHairGroupsMaterials()[CurrentSlot];

		FStaticMaterial TmpMaterial;
		TmpMaterial.MaterialInterface = Material.Material;
		TmpMaterial.MaterialSlotName  = Material.SlotName;
		FJsonObjectConverter::UStructToJsonObject(FStaticMaterial::StaticStruct(), &TmpMaterial, RootJsonObject, 0, 0);
	}

	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

	FString CopyStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

bool FGroomMaterialDetails::OnCanCopyMaterialItem(int32 CurrentSlot) const
{
	return GroomAsset->GetHairGroupsMaterials().IsValidIndex(CurrentSlot);
}

void FGroomMaterialDetails::OnPasteMaterialItem(int32 CurrentSlot)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		FProperty* Property = UGroomAsset::StaticClass()->FindPropertyByName(UGroomAsset::GetHairGroupsMaterialsMemberName());
		check(Property != nullptr);

		GroomAsset->PreEditChange(Property);

		FScopedTransaction Transaction(LOCTEXT("GroomAssetChangedPasteMaterialItem", "GroomAsset editor: Pasted material item"));
		GroomAsset->Modify();

		if (GroomAsset->GetHairGroupsMaterials().IsValidIndex(CurrentSlot))
		{
			FStaticMaterial TmpMaterial;
			FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FStaticMaterial::StaticStruct(), &TmpMaterial, 0, 0);
			GroomAsset->GetHairGroupsMaterials()[CurrentSlot].Material = TmpMaterial.MaterialInterface;
			GroomAsset->GetHairGroupsMaterials()[CurrentSlot].SlotName= TmpMaterial.MaterialSlotName;
		}

		CallPostEditChange(Property);
	}
}

void FGroomMaterialDetails::CallPostEditChange(FProperty* PropertyChanged/*=nullptr*/)
{
	if (PropertyChanged)
	{
		FPropertyChangedEvent PropertyUpdateStruct(PropertyChanged);
		GroomAsset->PostEditChangeProperty(PropertyUpdateStruct);
	}
	else
	{
		GroomAsset->Modify();
		GroomAsset->PostEditChange();
	}
	GroomDetailLayout->ForceRefreshDetails();
}

void FGroomMaterialDetails::ApplyChanges()
{
	GroomDetailLayout->ForceRefreshDetails();
}

FText FGroomMaterialDetails::GetMaterialSlotNameText(int32 MaterialIndex) const
{	
	if (IsMaterialValid(MaterialIndex))
	{
		return FText::FromName(GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName);
	}

	return LOCTEXT("HairMaterial_InvalidIndex", "Invalid Material Index");
}

void FGroomMaterialDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailLayout.GetSelectedObjects();
	check(SelectedObjects.Num()<=1); // The OnGenerateCustomWidgets delegate will not be useful if we try to process more than one object.

	GroomAsset = SelectedObjects.Num() > 0 ? Cast<UGroomAsset>(SelectedObjects[0].Get()) : nullptr;

	// Hide all properties
	{
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsRenderingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsPhysicsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsCardsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMeshesMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsMaterialsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairGroupsInfoMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableGlobalInterpolationMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetHairInterpolationTypeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetAutoLODBiasMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDisableBelowMinLodStrippingMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetMinLODMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetLODModeMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetEnableSimulationCacheMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetRiggedSkeletalMeshMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideProperty(DetailLayout.GetProperty(UGroomAsset::GetDataflowSettingsMemberName(), UGroomAsset::StaticClass()));
		DetailLayout.HideCategory(UGroomAsset::GetDataflowBuilderCategoryName());
	}

	GroomDetailLayout = &DetailLayout;
	AddMaterials(DetailLayout);
}

void FGroomMaterialDetails::OnGetMaterialsForArray(class IMaterialListBuilder& OutMaterials, int32 LODIndex)
{
	if (!GroomAsset)
		return;

	for (int32 MaterialIndex = 0; MaterialIndex < GroomAsset->GetHairGroupsMaterials().Num(); ++MaterialIndex)
	{
		OutMaterials.AddMaterial(MaterialIndex, GroomAsset->GetHairGroupsMaterials()[MaterialIndex].Material, true);
	}
}

void FGroomMaterialDetails::OnMaterialArrayChanged(UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll, int32 LODIndex)
{
	if (!GroomAsset)
		return;

	// Whether or not we made a transaction and need to end it
	bool bMadeTransaction = false;

	FProperty* MaterialProperty = FindFProperty<FProperty>(UGroomAsset::StaticClass(), "HairGroupsMaterials");
	check(MaterialProperty);
	GroomAsset->PreEditChange(MaterialProperty);
	check(GroomAsset->GetHairGroupsMaterials().Num() > SlotIndex)

	if (NewMaterial != PrevMaterial)
	{
		GEditor->BeginTransaction(LOCTEXT("GroomEditorMaterialChanged", "Groom editor: material changed"));
		bMadeTransaction = true;
		GroomAsset->Modify();
		GroomAsset->GetHairGroupsMaterials()[SlotIndex].Material = NewMaterial;

		// Add a default name to the material slot if this slot was manually add and there is no name yet
		if (NewMaterial != nullptr && GroomAsset->GetHairGroupsMaterials()[SlotIndex].SlotName == NAME_None)
		{
			if (GroomAsset->GetHairGroupsMaterials()[SlotIndex].SlotName == NAME_None)
			{					
				GroomAsset->GetHairGroupsMaterials()[SlotIndex].SlotName = NewMaterial->GetFName();
			}

			//Ensure the imported material slot name is unique
			if (GroomAsset->GetHairGroupsMaterials()[SlotIndex].SlotName == NAME_None)
			{
				UGroomAsset* LocalGroomAsset = GroomAsset;
				auto IsMaterialNameUnique = [&LocalGroomAsset, SlotIndex](const FName TestName)
				{
					for (int32 MaterialIndex = 0; MaterialIndex < LocalGroomAsset->GetHairGroupsMaterials().Num(); ++MaterialIndex)
					{
						if (MaterialIndex == SlotIndex)
						{
							continue;
						}
						if (LocalGroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName == TestName)
						{
							return false;
						}
					}
					return true;
				};
				int32 MatchNameCounter = 0;
				//Make sure the name is unique for imported material slot name
				bool bUniqueName = false;
				FString MaterialSlotName = NewMaterial->GetName();
				while (!bUniqueName)
				{
					bUniqueName = true;
					if (!IsMaterialNameUnique(FName(*MaterialSlotName)))
					{
						bUniqueName = false;
						MatchNameCounter++;
						MaterialSlotName = NewMaterial->GetName() + TEXT("_") + FString::FromInt(MatchNameCounter);
					}
				}
				GroomAsset->GetHairGroupsMaterials()[SlotIndex].SlotName = FName(*MaterialSlotName);
			}
		}
	}

	FPropertyChangedEvent PropertyChangedEvent(MaterialProperty);
	GroomAsset->PostEditChangeProperty(PropertyChangedEvent);

	if (bMadeTransaction)
	{
		// End the transation if we created one
		GEditor->EndTransaction();
		// Redraw viewports to reflect the material changes 
		GUnrealEd->RedrawLevelEditingViewports();
	}
}

FReply FGroomMaterialDetails::AddMaterialSlot()
{
	if (!GroomAsset)
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("PersonaAddMaterialSlotTransaction", "Persona editor: Add material slot"));
	GroomAsset->Modify();
	FHairGroupsMaterial NewMaterial;
	NewMaterial.SlotName = FName(TEXT("Material"));

	// Build a unique name
	FName SlotName = NewMaterial.SlotName;
	uint32 UniqueId = 0;
	bool bHasUniqueName = true;
	do
	{
		bHasUniqueName = true;
		for (const FHairGroupsMaterial& Group : GroomAsset->GetHairGroupsMaterials())
		{
			if (Group.SlotName == SlotName)
			{
				bHasUniqueName = false;				
				FString NewSlotName = NewMaterial.SlotName.ToString() + FString::FromInt(++UniqueId);
				SlotName = FName(*NewSlotName);
				break;
			}
		}
	} while (!bHasUniqueName);

	// Add new material
	NewMaterial.SlotName = SlotName;
	GroomAsset->GetHairGroupsMaterials().Add(NewMaterial);
	GroomAsset->PostEditChange();

	return FReply::Handled();
}

FText FGroomMaterialDetails::GetMaterialArrayText() const
{
	FString MaterialArrayText = TEXT(" Material Slots");
	int32 SlotNumber = 0;
	if (GroomAsset)
	{
		SlotNumber = GroomAsset->GetHairGroupsMaterials().Num();
	}
	MaterialArrayText = FString::FromInt(SlotNumber) + MaterialArrayText;
	return FText::FromString(MaterialArrayText);
}

FText FGroomMaterialDetails::GetMaterialNameText(int32 MaterialIndex) const
{
	if (IsMaterialValid(MaterialIndex))
	{
		return FText::FromName(GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName);
	}
	return FText::FromName(NAME_None);
}

void FGroomMaterialDetails::OnMaterialNameCommitted(const FText& InValue, ETextCommit::Type CommitType, int32 MaterialIndex)
{
	FName NewSlotName = FName(*(InValue.ToString()));
	if (IsMaterialValid(MaterialIndex) && NewSlotName != GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName)
	{
		FScopedTransaction ScopeTransaction(LOCTEXT("PersonaMaterialSlotNameChanged", "Persona editor: Material slot name change"));

		FProperty* ChangedProperty = FindFProperty<FProperty>(UGroomAsset::StaticClass(), "HairGroupsMaterials");
		check(ChangedProperty);
		GroomAsset->PreEditChange(ChangedProperty);

		// Rename group which were using the old slot name
		FName PreviousSlotName = GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName;
		for (FHairGroupsRendering& Group : GroomAsset->GetHairGroupsRendering())
		{
			if (PreviousSlotName == Group.MaterialSlotName)
			{
				Group.MaterialSlotName = NewSlotName;
			}
		}
		for (FHairGroupsCardsSourceDescription& Group : GroomAsset->GetHairGroupsCards())
		{
			if (PreviousSlotName == Group.MaterialSlotName)
			{
				Group.MaterialSlotName = NewSlotName;
			}
		}
		for (FHairGroupsMeshesSourceDescription& Group : GroomAsset->GetHairGroupsMeshes())
		{
			if (PreviousSlotName == Group.MaterialSlotName)
			{
				Group.MaterialSlotName = NewSlotName;
			}
		}

		GroomAsset->GetHairGroupsMaterials()[MaterialIndex].SlotName = NewSlotName;

		FPropertyChangedEvent PropertyUpdateStruct(ChangedProperty);
		GroomAsset->PostEditChangeProperty(PropertyUpdateStruct);
	}
}

TSharedRef<SWidget> FGroomMaterialDetails::OnGenerateCustomNameWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex)
{
	return SNew(SVerticalBox);
}

TSharedRef<SWidget> FGroomMaterialDetails::OnGenerateCustomMaterialWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex, int32 LODIndex)
{
	bool bMaterialIsUsed = GroomAsset && GroomAsset->IsMaterialUsed(MaterialIndex);

	return
		SNew(SMaterialSlotWidget, MaterialIndex, bMaterialIsUsed)
		.MaterialName(this, &FGroomMaterialDetails::GetMaterialNameText, MaterialIndex)
		.OnMaterialNameCommitted(this, &FGroomMaterialDetails::OnMaterialNameCommitted, MaterialIndex)
		.CanDeleteMaterialSlot(this, &FGroomMaterialDetails::CanDeleteMaterialSlot, MaterialIndex)
		.OnDeleteMaterialSlot(this, &FGroomMaterialDetails::OnDeleteMaterialSlot, MaterialIndex);
}

bool FGroomMaterialDetails::IsMaterialValid(int32 MaterialIndex) const
{
	return GroomAsset && MaterialIndex >= 0 && MaterialIndex < GroomAsset->GetHairGroupsMaterials().Num();
}

bool FGroomMaterialDetails::CanDeleteMaterialSlot(int32 MaterialIndex) const
{
	if (!GroomAsset)
	{
		return false;
	}

	return !GroomAsset->IsMaterialUsed(MaterialIndex);
}
	
void FGroomMaterialDetails::OnDeleteMaterialSlot(int32 MaterialIndex)
{
	if (!GroomAsset || !CanDeleteMaterialSlot(MaterialIndex))
	{
		return;
	}

	if (!bDeleteWarningConsumed)
	{
		EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("FPersonaMeshDetails_DeleteMaterialSlot", "WARNING - Deleting a material slot can break the game play blueprint or the game play code. All indexes after the delete slot will change"));
		if (Answer == EAppReturnType::Cancel)
		{
			return;
		}
		bDeleteWarningConsumed = true;
	}
	GroomAsset->GetHairGroupsMaterials().RemoveAt(MaterialIndex);

	FScopedTransaction Transaction(LOCTEXT("PersonaOnDeleteMaterialSlotTransaction", "Persona editor: Delete material slot"));
	GroomAsset->Modify();
}

bool FGroomMaterialDetails::OnMaterialListDirty()
{
	bool ForceMaterialListRefresh = false;
	return ForceMaterialListRefresh;
}

TSharedRef<SWidget> FGroomMaterialDetails::OnGenerateCustomNameWidgetsForSection(int32 LodIndex, int32 SectionIndex)
{
	bool IsSectionChunked = false;
	TSharedRef<SVerticalBox> SectionWidget = SNew(SVerticalBox);
	SectionWidget->AddSlot()
		.AutoHeight()
		.Padding(0, 2, 0, 0);
	return SectionWidget;
}

TSharedRef<SWidget> FGroomMaterialDetails::OnGenerateCustomSectionWidgetsForSection(int32 LODIndex, int32 SectionIndex)
{
	TSharedRef<SVerticalBox> SectionWidget = SNew(SVerticalBox);
	SectionWidget->AddSlot()
	.AutoHeight()
	.Padding(0, 2, 0, 0);
	return SectionWidget;
}

EVisibility FGroomMaterialDetails::ShowEnabledSectionDetail(int32 LodIndex, int32 SectionIndex) const
{
	return EVisibility::All;
}

EVisibility FGroomMaterialDetails::ShowDisabledSectionDetail(int32 LodIndex, int32 SectionIndex) const
{
	return EVisibility::All;
}

void FGroomMaterialDetails::OnMaterialSelectedChanged(ECheckBoxState NewState, int32 MaterialIndex)
{

}

ECheckBoxState FGroomMaterialDetails::IsIsolateMaterialEnabled(int32 MaterialIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	return State;
}

void FGroomMaterialDetails::OnMaterialIsolatedChanged(ECheckBoxState NewState, int32 MaterialIndex)
{

}

ECheckBoxState FGroomMaterialDetails::IsSectionSelected(int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	return State;
}

void FGroomMaterialDetails::OnSectionSelectedChanged(ECheckBoxState NewState, int32 SectionIndex)
{

}

ECheckBoxState FGroomMaterialDetails::IsIsolateSectionEnabled(int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	return State;
}

void FGroomMaterialDetails::OnSectionIsolatedChanged(ECheckBoxState NewState, int32 SectionIndex)
{

}


#undef LOCTEXT_NAMESPACE