// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "EngineDefines.h"
#include "PropertyHandle.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Widgets/Input/SComboBox.h"

class UGroomAsset;
class UMaterialInterface;
class IGroomCustomAssetEditorToolkit;

class FGroomMaterialDetails : public IDetailCustomization
{
	UGroomAsset* GroomAsset = nullptr;

public:
	FGroomMaterialDetails(IGroomCustomAssetEditorToolkit* InToolkit);
	~FGroomMaterialDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(IGroomCustomAssetEditorToolkit* InToolkit);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:

	FReply AddMaterialSlot();

	FText GetMaterialArrayText() const;
	bool IsMaterialValid(int32 MaterialIndex) const;

	/**
	* Called by the material list widget when we need to get new materials for the list
	*
	* @param OutMaterials	Handle to a material list builder that materials should be added to
	*/
	void OnGetMaterialsForArray(class IMaterialListBuilder& OutMaterials, int32 LODIndex);

	/**
	* Called when a user drags a new material over a list item to replace it
	*
	* @param NewMaterial	The material that should replace the existing material
	* @param PrevMaterial	The material that should be replaced
	* @param SlotIndex		The index of the slot on the component where materials should be replaces
	* @param bReplaceAll	If true all materials in the slot should be replaced not just ones using PrevMaterial
	*/
	void OnMaterialArrayChanged(UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll, int32 LODIndex);

	/**
	 * Called by the material list widget on generating each name widget
	 *
	 * @param Material		The material that is being displayed
	 * @param SlotIndex		The index of the material slot
	 */
	TSharedRef<SWidget> OnGenerateCustomNameWidgetsForSection(int32 LodIndex, int32 SectionIndex);

	/**
	 * Called by the material list widget on generating each thumbnail widget
	 *
	 * @param Material		The material that is being displayed
	 * @param SlotIndex		The index of the material slot
	 */
	TSharedRef<SWidget> OnGenerateCustomSectionWidgetsForSection(int32 LODIndex, int32 SectionIndex);

	EVisibility ShowEnabledSectionDetail(int32 LodIndex, int32 SectionIndex) const;
	EVisibility ShowDisabledSectionDetail(int32 LodIndex, int32 SectionIndex) const;

	FText GetMaterialNameText(int32 MaterialIndex)const;
	void OnMaterialNameCommitted(const FText& InValue, ETextCommit::Type CommitType, int32 MaterialIndex);

	/**
	* Called by the material list widget on generating name side content
	*
	* @param Material		The material that is being displayed
	* @param MaterialIndex	The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateCustomNameWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex);

	/**
	* Called by the material list widget on generating each thumbnail widget
	*
	* @param Material		The material that is being displayed
	* @param MaterialIndex	The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateCustomMaterialWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex, int32 LODIndex);

	/* If the material list is dirty this function will return true */
	bool OnMaterialListDirty();

	bool CanDeleteMaterialSlot(int32 MaterialIndex) const;

	void OnDeleteMaterialSlot(int32 MaterialIndex);

	/**
	* Handler for changing highlight status on a material
	*
	* @param MaterialIndex	The material index that is being selected
	*/
	void OnMaterialSelectedChanged(ECheckBoxState NewState, int32 MaterialIndex);

	/**
	* Handler for check box display based on whether the material is isolated
	*
	* @param MaterialIndex	The material index that is being isolate
	*/
	ECheckBoxState IsIsolateMaterialEnabled(int32 MaterialIndex) const;

	/**
	* Handler for changing isolated status on a material
	*
	* @param MaterialIndex	The material index that is being isolate
	*/
	void OnMaterialIsolatedChanged(ECheckBoxState NewState, int32 MaterialIndex);


	/**
	 * Handler for check box display based on whether the material is highlighted
	 *
	 * @param SectionIndex	The material section that is being tested
	 */
	ECheckBoxState IsSectionSelected(int32 SectionIndex) const;

	/**
	 * Handler for changing highlight status on a material
	 *
	 * @param SectionIndex	The material section that is being tested
	 */
	void OnSectionSelectedChanged(ECheckBoxState NewState, int32 SectionIndex);

	/**
	* Handler for check box display based on whether the material is isolated
	*
	* @param SectionIndex	The material section that is being tested
	*/
	ECheckBoxState IsIsolateSectionEnabled(int32 SectionIndex) const;

	/**
	* Handler for changing isolated status on a material
	*
	* @param SectionIndex	The material section that is being tested
	*/
	void OnSectionIsolatedChanged(ECheckBoxState NewState, int32 SectionIndex);

	/** Creates the UI for Current LOD panel */
	void AddMaterials(IDetailLayoutBuilder& DetailLayout);

	FText GetMaterialSlotNameText(int32 MaterialIndex) const;

	void OnCopyMaterialList();
	bool OnCanCopyMaterialList() const;
	void OnPasteMaterialList();

	void OnCopyMaterialItem(int32 CurrentSlot);
	bool OnCanCopyMaterialItem(int32 CurrentSlot) const;
	void OnPasteMaterialItem(int32 CurrentSlot);
public:
	void ApplyChanges();

private:
	void CallPostEditChange(FProperty* PropertyChanged=nullptr);

	IDetailLayoutBuilder* GroomDetailLayout;
	/*
	 * This prevent showing the delete material slot warning dialog more then once per editor session
	 */
	bool bDeleteWarningConsumed;
};