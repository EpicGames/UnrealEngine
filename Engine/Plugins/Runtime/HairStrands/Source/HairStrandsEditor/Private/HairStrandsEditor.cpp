// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsEditor.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HairStrandsCore.h"

#include "GroomActions.h"
#include "GroomBindingActions.h"
#include "GroomCacheActions.h"

#include "PropertyEditorModule.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "ToolMenus.h"

#include "GroomAsset.h"
#include "GroomAssetThumbnailRenderer.h"
#include "GroomBindingAsset.h"
#include "GroomBindingAssetThumbnailRenderer.h"
#include "GroomBindingDetailsCustomization.h"
#include "GroomCacheImportOptions.h"
#include "GroomCacheImportSettingsCustomization.h"
#include "GroomCacheTrackEditor.h"
#include "GroomComponentDetailsCustomization.h"
#include "GroomCreateBindingOptions.h"
#include "GroomEditorCommands.h"
#include "GroomEditorMode.h"
#include "GroomImportOptions.h"
#include "GroomPluginSettings.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "ISequencerModule.h"
#include "Delegates/DelegateCombinations.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/UObjectIterator.h"
#include "HairStrandsFactory.h"

#include "Dataflow/AssetDefinition_DataflowAsset.h"

IMPLEMENT_MODULE(FGroomEditor, HairStrandsEditor);

#define LOCTEXT_NAMESPACE "GroomEditor"

LLM_DEFINE_TAG(GroomEditor);

FName FGroomEditor::GroomEditorAppIdentifier(TEXT("GroomEditor"));


///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper functions

void CreateFilename(const FString& InAssetName, const FString& Suffix, FString& OutPackageName, FString& OutAssetName)
{
	// Get a unique package and asset name
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(InAssetName, Suffix, OutPackageName, OutAssetName);
}

void RegisterAsset(UObject* Out)
{
	FAssetRegistryModule::AssetCreated(Out);
}

void SaveAsset(UObject* Object)
{
	UPackage* Package = Object->GetOutermost();
	Object->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(Object);

	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(Package);
	bool bCheckDirty = true;
	bool bPromptToSave = false;
	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FGroomEditor::StartupModule()
{
	LLM_SCOPE_BYTAG(GroomEditor)

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FGroomEditor::RegisterMenus));
	
	// Any attempt to use GEditor right now will fail as it hasn't been initialized yet. Waiting for post engine init resolves that.
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FGroomEditor::OnPostEngineInit);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FGroomEditor::OnPreExit);

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	// Only register once
	if (!StyleSet.IsValid())
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);
		FString HairStrandsContent = IPluginManager::Get().FindPlugin(TEXT("HairStrands"))->GetBaseDir() + "/Content";

		StyleSet = MakeShared<FSlateStyleSet>("Groom");
		StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		
		StyleSet->Set("ClassIcon.GroomComponent", new FSlateImageBrush(HairStrandsContent + "/Icons/S_Groom_16.png", Icon16x16));
		StyleSet->Set("ClassThumbnail.GroomComponent", new FSlateImageBrush(HairStrandsContent + "/Icons/S_Groom_64.png", Icon64x64));
		
		StyleSet->Set("ClassIcon.GroomActor", new FSlateImageBrush(HairStrandsContent + "/Icons/S_Groom_16.png", Icon16x16));
		StyleSet->Set("ClassThumbnail.GroomActor", new FSlateImageBrush(HairStrandsContent + "/Icons/S_Groom_64.png", Icon64x64));

		StyleSet->Set("ClassIcon.GroomAsset", new FSlateImageBrush(HairStrandsContent + "/Icons/S_Groom_16.png", Icon16x16));
		StyleSet->Set("ClassThumbnail.GroomAsset", new FSlateImageBrush(HairStrandsContent + "/Icons/S_Groom_64.png", Icon64x64));

		StyleSet->Set("ClassIcon.GroomBindingAsset", new FSlateImageBrush(HairStrandsContent + "/Icons/S_GroomBinding_16.png", Icon16x16));
		StyleSet->Set("ClassThumbnail.GroomBindingAsset", new FSlateImageBrush(HairStrandsContent + "/Icons/S_GroomBinding_64.png", Icon64x64));
		
		StyleSet->Set("GroomEditor.SimulationOptions", new FSlateImageBrush(HairStrandsContent + "/Icons/S_SimulationOptions_40x.png", Icon40x40));
		StyleSet->Set("GroomEditor.SimulationOptions.Small", new FSlateImageBrush(HairStrandsContent + "/Icons/S_SimulationOptions_40x.png", Icon20x20));

		StyleSet->Set("ClassIcon.GroomCache", new FSlateImageBrush(HairStrandsContent + "/Icons/S_GroomCache_64.png", Icon16x16));
		StyleSet->Set("ClassThumbnail.GroomCache", new FSlateImageBrush(HairStrandsContent + "/Icons/S_GroomCache_64.png", Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

		// Custom widget for groom component (Group desc override, ...)
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UGroomComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGroomComponentDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(UGroomBindingAsset::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGroomBindingDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(UGroomCreateBindingOptions::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGroomCreateBindingDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomClassLayout(UGroomHairGroupsMapping::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGroomHairGroomRemappingDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(FGroomCacheImportSettings::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGroomCacheImportSettingsCustomization::MakeInstance));
	}

	FGroomEditorCommands::Register();

	// Asset create/edition helper/wrapper for creating/edition asset within the HairStrandsCore 
	// project without any editor dependencies
	FHairAssetHelper Helper;
	Helper.CreateFilename = CreateFilename;
	Helper.RegisterAsset = RegisterAsset;
	Helper.SaveAsset = SaveAsset;
	FHairStrandsCore::RegisterAssetHelper(Helper);

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	TrackEditorBindingHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FGroomCacheTrackEditor::CreateTrackEditor));

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "Groom",
			LOCTEXT("GroomPluginSettingsName", "Groom"),
			LOCTEXT("GroomPluginSettingsDescription", "Configure the Groom plug-in."),
			GetMutableDefault<UGroomPluginSettings>()
		);
	}

	UThumbnailManager::Get().RegisterCustomRenderer(UGroomAsset::StaticClass(), UGroomAssetThumbnailRenderer::StaticClass());
	UThumbnailManager::Get().RegisterCustomRenderer(UGroomBindingAsset::StaticClass(), UGroomBindingAssetThumbnailRenderer::StaticClass());

	UE::Groom::RegisterGroomDataflowTemplatePath(TEXT("/HairStrands/Dataflow/Templates"));

	DataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(UGroomAsset::StaticClass());
}

void FGroomEditor::ShutdownModule()
{
	UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(DataflowMenusHandle);

	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(UE_MODULE_NAME);

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Groom");
	}

	ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModulePtr)
	{
		SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
	}

	if (UObjectInitialized())
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule)
		{
			PropertyModule->UnregisterCustomClassLayout(UGroomComponent::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UGroomBindingAsset::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UGroomCreateBindingOptions::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UGroomHairGroupsMapping::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomPropertyTypeLayout(FGroomCacheImportSettings::StaticStruct()->GetFName());
		}

		UThumbnailManager::Get().UnregisterCustomRenderer(UGroomAsset::StaticClass());
		UThumbnailManager::Get().UnregisterCustomRenderer(UGroomBindingAsset::StaticClass());
	}

	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}

	UE::Groom::UnregisterGroomDataflowTemplate("SelectNoDataflowTemplate");
	UE::Groom::UnregisterGroomDataflowTemplatePath(TEXT("/HairStrands/Dataflow/Templates"));
}

void FGroomEditor::RegisterMenus()
{

}

TArray<TSharedPtr<IGroomTranslator>> FGroomEditor::GetHairTranslators()
{
	TArray<TSharedPtr<IGroomTranslator>> Translators;
	for (TFunction<TSharedPtr<IGroomTranslator>()>& SpawnTranslator : TranslatorSpawners)
	{
		Translators.Add(SpawnTranslator());
	}

	return Translators;
}

void FGroomEditor::OnPostEngineInit()
{
	// The editor should be valid at this point.. log a warning if not!
	if (GEditor)
	{
		UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
		PreviewPlatformChangedHandle = EditorEngine->OnPreviewShaderPlatformChanged().AddRaw(this, &FGroomEditor::OnPreviewPlatformChanged);
	}
}

void FGroomEditor::OnPreExit()
{
	if (GEditor)
	{
		CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PreviewPlatformChangedHandle);
	}
}

void FGroomEditor::OnPreviewPlatformChanged(EShaderPlatform NewShaderPlatform)
{
#if WITH_EDITOR
	for (TObjectIterator<UGroomComponent> It; It; ++It)
	{
		if (UGroomComponent* Component = *It)
		{
			Component->HandlePlatformPreviewChanged(NewShaderPlatform);
		}
	}
#endif
}

#undef LOCTEXT_NAMESPACE