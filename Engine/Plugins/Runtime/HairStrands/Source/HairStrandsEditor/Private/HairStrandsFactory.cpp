// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsFactory.h"

#include "AssetImportTask.h"
#include "EditorFramework/AssetImportData.h"
#include "GroomAsset.h"
#include "GroomAssetImportData.h"
#include "GroomCache.h"
#include "GroomCacheData.h"
#include "GroomCacheImporter.h"
#include "GroomImportOptions.h"
#include "GroomCacheImportOptions.h"
#include "GroomBuilder.h"
#include "GroomImportOptionsWindow.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "Dataflow/DataflowObject.h"
#include "Dialog/SMessageDialog.h"
#include "HairDescription.h"
#include "HairStrandsEditor.h"
#include "HairStrandsImporter.h"
#include "HairStrandsTranslator.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HairStrandsFactory)

#define LOCTEXT_NAMESPACE "HairStrandsFactory"

namespace UE::Groom
{ 
	struct FGroomDataflowTemplateManager
	{
		/** Singleton dataflow template manager*/
		static FGroomDataflowTemplateManager& GetInstance()
		{
			static FGroomDataflowTemplateManager TemplateManager;
			return TemplateManager;
		}

		/** Build dataflow template buttons from data*/
		TArray<SMessageDialog::FButton> BuildDataflowTemplateButtons(TFunction<void(FString)> OnButtonClicked) const
		{
			TArray<SMessageDialog::FButton> DataflowButtons;
			DataflowButtons.Reserve(TemplateData.Num());
			
			TArray<FGroomDataflowTemplateData> OutAllTemplateData;
			GetFixedAndDynamicallyLoadedTemplateData(OutAllTemplateData);

			for(const FGroomDataflowTemplateData& DataflowTemplate : OutAllTemplateData)
			{
				const FString DataflowTitleID = DataflowTemplate.TemplateName + FString("Title");
				const FString DataflowTooltipID = DataflowTemplate.TemplateName + FString("Tooltip");
				DataflowButtons.Add(
					SMessageDialog::FButton(FText::AsLocalizable_Advanced(TEXT(LOCTEXT_NAMESPACE), *DataflowTitleID, *DataflowTemplate.TemplateTitle))
					.SetToolTipText(FText::AsLocalizable_Advanced(TEXT(LOCTEXT_NAMESPACE), *DataflowTooltipID, *DataflowTemplate.TemplateTooltip))
					.SetPrimary(DataflowTemplate.bIsPrimaryTemplate)
					.SetOnClicked(FSimpleDelegate::CreateLambda(OnButtonClicked, DataflowTemplate.TemplatePath))
				);
				
			}
			return DataflowButtons;
		}

		/** Get the template path given an index */
		FString GetDataflowTemplatePath(const int32 TemplateIndex) const
		{
			return TemplateData.IsValidIndex(TemplateIndex) ? TemplateData[TemplateIndex].TemplatePath : "";
		}

		/** Add dataflow template data to the manager */
		void AddDataflowTemplateData(const FGroomDataflowTemplateData& DataflowTemplate)
		{
			TemplateData.Add(DataflowTemplate);
		}

		/** Add dataflow template data to the manager */
		void RemoveDataflowTemplateData(const FString& TemplateName)
		{
			for(int32 TemplateIndex = TemplateData.Num()-1; TemplateIndex >= 0; --TemplateIndex)
			{ 
				if (TemplateData[TemplateIndex].TemplateName == TemplateName)
				{
					TemplateData.RemoveAt(TemplateIndex);
				}
			}
		}

		void AddDataflowTemplatePath(const FString& DataflowTemplatePath)
		{
			DynamicallyLoadedTemplatePaths.Add(DataflowTemplatePath);
		}

		void RemoveDataflowTemplatePath(const FString& DataflowTemplatePath)
		{
			DynamicallyLoadedTemplatePaths.Remove(DataflowTemplatePath);
		}

		void GetFixedAndDynamicallyLoadedTemplateData(TArray<FGroomDataflowTemplateData>& OutAllTemplateData) const
		{
			// first fixed ones
			OutAllTemplateData = TemplateData;

			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

			// now load on the spot the ones from dynamically loaded paths
			TArray<FAssetData> AssetDataArray;
			for (const FString& TemplatePath : DynamicallyLoadedTemplatePaths)
			{
				AssetDataArray.Reset();
				AssetRegistryModule.Get().GetAssetsByPath(FName(TemplatePath), AssetDataArray, /*bRecursive*/true, /*bIncludeOnlyOnDiskAssets*/true);

				for (const FAssetData& AssetData : AssetDataArray)
				{
					const FString AssetNameStr =
						AssetData.AssetName.ToString()
						.Replace(TEXT("DF_Groom"), TEXT(""))
						.Replace(TEXT("Template"), TEXT(""))
						;


					UE::Groom::FGroomDataflowTemplateData LoadedTemplateData
					{
						.TemplateName = AssetNameStr,
						.TemplateTitle = AssetNameStr,
						.TemplateTooltip = FString::Format(TEXT("Use {0} Dataflow Template"), { AssetData.AssetName.ToString() }),
						.TemplatePath = AssetData.GetObjectPathString(),
						.bIsPrimaryTemplate = false,
					};
					OutAllTemplateData.Emplace(MoveTemp(LoadedTemplateData));
				}
			}
		}

	private:
		/** List of dataflow template data*/
		TArray<FGroomDataflowTemplateData>	TemplateData;
		TArray<FString> DynamicallyLoadedTemplatePaths;
	};
	
	TArray<SMessageDialog::FButton> BuildGroomDataflowTemplateButtons(TFunction<void(FString)> OnButtonClicked)
	{
		return FGroomDataflowTemplateManager::GetInstance().BuildDataflowTemplateButtons(OnButtonClicked);
	}

	void RegisterGroomDataflowTemplate(const FGroomDataflowTemplateData& DataflowTemplate)
	{
		FGroomDataflowTemplateManager::GetInstance().AddDataflowTemplateData(DataflowTemplate);
		FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
			UGroomAsset::StaticClass(),
			{
				.AssetPath = FSoftObjectPath(DataflowTemplate.TemplatePath),
				.DisplayName = FText::FromString(DataflowTemplate.TemplateTitle),
				.Tooltip = FText::FromString(DataflowTemplate.TemplateTooltip),
				.Icon = FSlateIcon("Groom", "ClassThumbnail.GroomAsset")
			});
	}

	void UnregisterGroomDataflowTemplate(const FString& TemplateName)
	{
		FGroomDataflowTemplateManager::GetInstance().RemoveDataflowTemplateData(TemplateName);
	}

	void RegisterGroomDataflowTemplatePath(const FString& DataflowTemplatePath)
	{
		FGroomDataflowTemplateManager::GetInstance().AddDataflowTemplatePath(DataflowTemplatePath);
		FDataflowTemplateRegistry::Get().RegisterTemplateFolder(UGroomAsset::StaticClass(),
			{ DataflowTemplatePath, FSlateIcon("Groom", "ClassThumbnail.GroomAsset") });
	}

	void UnregisterGroomDataflowTemplatePath(const FString& DataflowTemplatePath)
	{
		FGroomDataflowTemplateManager::GetInstance().RemoveDataflowTemplatePath(DataflowTemplatePath);
	}

	bool BuildGroomDataflowAsset(UGroomAsset* GroomAsset)
	{
		return UE::DataflowAssetDefinitionHelpers::SetDataflowFromTemplatePicker(GroomAsset);
	}
}

UHairStrandsFactory::UHairStrandsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UGroomAsset::StaticClass();
	bCreateNew = false;		// manual creation not allow
	bEditAfterNew = false;
	bEditorImport = true;	// only allow import

	// Slightly increased priority to allow its translators to check if they can translate the file
	ImportPriority += 1;

	// Lazy init the translators to let them register themselves before the CDO is used
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ImportOptions = NewObject<UGroomImportOptions>();
		GroomCacheImportOptions = NewObject<UGroomCacheImportOptions>();

		InitTranslators();
	}
}

void UHairStrandsFactory::InitTranslators()
{
	Formats.Reset();

	Translators = FGroomEditor::Get().GetHairTranslators();
	for (TSharedPtr<IGroomTranslator> Translator : Translators)
	{
		Formats.Add(Translator->GetSupportedFormat());
	}
}

void UHairStrandsFactory::GetSupportedFileExtensions(TArray<FString>& OutExtensions) const
{
	if (HasAnyFlags(RF_ClassDefaultObject) && Formats.Num() == 0)
	{
		// Init the translators the first time the CDO is used
		UHairStrandsFactory* Factory = const_cast<UHairStrandsFactory*>(this);
		Factory->InitTranslators();
	}

	Super::GetSupportedFileExtensions(OutExtensions);
}

UObject* UHairStrandsFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, 
	const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) 
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UHairStrandsFactory::FactoryCreateFile);

	bOutOperationCanceled = false;

	const bool bIsUnattended = (IsAutomatedImport()
		|| FApp::IsUnattended()
		|| IsRunningCommandlet()
		|| GIsRunningUnattendedScript);
	const bool ShowImportDialog = !bIsUnattended && !bImportAll;

	// Translate the hair data from the file
	TSharedPtr<IGroomTranslator> SelectedTranslator = GetTranslator(Filename);
	if (!SelectedTranslator.IsValid())
	{
		return nullptr;
	}

	// Use the settings from the script if provided
	if (AssetImportTask)
	{
		if (UGroomImportOptions* GroomOptions = Cast<UGroomImportOptions>(AssetImportTask->Options))
		{
			ImportOptions = GroomOptions;
		}

		if (UGroomCacheImportOptions* CacheOptions = Cast<UGroomCacheImportOptions>(AssetImportTask->Options))
		{
			GroomCacheImportOptions = CacheOptions;
		}
		else
		{
			GroomCacheImportOptions->ImportSettings.bImportGroomCache = false;
		}
	}

	bool bGuidesOnly = false;
	FGroomAnimationInfo AnimInfo;
	{
		// Load the alembic file upfront to preview & report any potential issue
		FHairDescriptionGroups OutDescription;
		{
			FScopedSlowTask Progress((float)1, LOCTEXT("ImportHairAssetForPreview", "Importing hair asset for preview..."), true);
			Progress.MakeDialog(true);

			FHairDescription HairDescription;
			if (!SelectedTranslator->Translate(Filename, HairDescription, ImportOptions->ConversionSettings, &AnimInfo))
			{
				return nullptr;
			}

			FGroomBuilder::BuildHairDescriptionGroups(HairDescription, OutDescription);

			// Populate the interpolation settings based on the group count, as this is used later during the ImportHair() to define 
			// the exact number of group to create
			const uint32 GroupCount = OutDescription.HairGroups.Num();
			if (GroupCount != uint32(ImportOptions->InterpolationSettings.Num()))
			{
				ImportOptions->InterpolationSettings.Init(FHairGroupsInterpolation(), GroupCount);
			}
		}

		// Convert the process hair description into hair groups
		UGroomHairGroupsPreview* GroupsPreview = NewObject<UGroomHairGroupsPreview>();
		{
			for (const FHairDescriptionGroup& Group : OutDescription.HairGroups)
			{
				FGroomHairGroupPreview& OutGroup = GroupsPreview->Groups.AddDefaulted_GetRef();
				OutGroup.GroupName  		= Group.Info.GroupName;
				OutGroup.GroupID			= Group.Info.GroupID;
				OutGroup.GroupIndex			= Group.Info.GroupIndex;
				OutGroup.CurveCount 		= Group.Info.NumCurves;
				OutGroup.GuideCount 		= Group.Info.NumGuides;
				OutGroup.Attributes 		= Group.GetHairAttributes();
				OutGroup.AttributeFlags 	= Group.GetHairAttributeFlags();
				OutGroup.Flags      		= Group.Info.Flags;
				bGuidesOnly |= (OutGroup.CurveCount == 0 && OutGroup.GuideCount > 0);

				if (OutGroup.GroupIndex < OutDescription.HairGroups.Num())
				{				
					OutGroup.InterpolationSettings = ImportOptions->InterpolationSettings[OutGroup.GroupIndex];
				}
			}
		}

		FGroomCacheImporter::SetupImportSettings(GroomCacheImportOptions->ImportSettings, AnimInfo);

		if (bGuidesOnly)
		{
			// Guides-only groom cache cannot import groom asset and must specify an existing one
			GroomCacheImportOptions->ImportSettings.bImportGroomAsset = false;

			// Hint since All will import guides too
			GroomCacheImportOptions->ImportSettings.ImportType = EGroomCacheImportType::Guides;
		}

		// Don't bother saving the options coming from script
		if (ShowImportDialog)
		{
			// Display import options and handle user cancellation
			TSharedPtr<SGroomImportOptionsWindow> GroomOptionWindow = SGroomImportOptionsWindow::DisplayImportOptions(ImportOptions, GroomCacheImportOptions, GroupsPreview, nullptr/*GroupsMapping*/, Filename, true /*bShowImportAllButton*/);

			if (!GroomOptionWindow->ShouldImport())
			{
				bOutOperationCanceled = true;
				return nullptr;
			}

			if (GroomOptionWindow->ShouldImportAll())
			{
				bImportAll = true;
			}

			// Save the options as the new default
			for (const FGroomHairGroupPreview& GroupPreview : GroupsPreview->Groups)
			{
				if (GroupPreview.GroupIndex < OutDescription.HairGroups.Num())
				{
					ImportOptions->InterpolationSettings[GroupPreview.GroupIndex] = GroupPreview.InterpolationSettings;
				}
			}
			ImportOptions->SaveConfig();
		}
	}

	FGroomCacheImporter::ApplyImportSettings(GroomCacheImportOptions->ImportSettings, AnimInfo);

	FScopedSlowTask Progress((float) 1, LOCTEXT("ImportHairAsset", "Importing hair asset..."), true);
	Progress.MakeDialog(true);

	FHairDescription HairDescription;
	if (!SelectedTranslator->Translate(Filename, HairDescription, ImportOptions->ConversionSettings))
	{
		return nullptr;
	}

	UObject* CurrentAsset = nullptr;
	UGroomAsset* GroomAssetForCache = nullptr;
	FHairImportContext HairImportContext(ImportOptions, InParent, InClass, InName, Flags);
	if (GroomCacheImportOptions->ImportSettings.bImportGroomAsset && !bGuidesOnly)
	{
		// Might try to import the same file in the same folder, so if an asset already exists there, reuse and update it
		// Since we are importing (not reimporting) we reset the object completely. All previous settings will be lost.
		UGroomAsset* ExistingAsset = FindObject<UGroomAsset>(InParent, *InName.ToString());
		if (ExistingAsset)
		{
			ExistingAsset->SetNumGroup(0);
		}

		UGroomAsset* ImportedAsset = FHairStrandsImporter::ImportHair(HairImportContext, HairDescription, ExistingAsset);
		if (ImportedAsset)
		{
			// Setup asset import data
			if (!ImportedAsset->AssetImportData || !ImportedAsset->AssetImportData->IsA<UGroomAssetImportData>())
			{
				ImportedAsset->AssetImportData = NewObject<UGroomAssetImportData>(ImportedAsset);
			}
			ImportedAsset->AssetImportData->Update(Filename);

			UGroomAssetImportData* GroomAssetImportData = Cast<UGroomAssetImportData>(ImportedAsset->AssetImportData);
			GroomAssetImportData->ImportOptions = DuplicateObject<UGroomImportOptions>(ImportOptions, GroomAssetImportData);

			GroomAssetForCache = ImportedAsset;
			CurrentAsset = ImportedAsset;
		}
	}
	else
	{
		GroomAssetForCache = Cast<UGroomAsset>(GroomCacheImportOptions->ImportSettings.GroomAsset.TryLoad());
	}

	if (GroomCacheImportOptions->ImportSettings.bImportGroomCache && GroomAssetForCache)
	{
		if (GroomCacheImportOptions->ImportSettings.bOverrideConversionSettings)
		{
			HairImportContext.ImportOptions->ConversionSettings = GroomCacheImportOptions->ImportSettings.ConversionSettings;
		}
		TArray<UGroomCache*> GroomCaches = FGroomCacheImporter::ImportGroomCache(Filename, SelectedTranslator, AnimInfo, HairImportContext, GroomAssetForCache, GroomCacheImportOptions->ImportSettings.ImportType);

		// Setup asset import data
		for (UGroomCache* GroomCache : GroomCaches)
		{
			if (!GroomCache->AssetImportData || !GroomCache->AssetImportData->IsA<UGroomCacheImportData>())
			{
				UGroomCacheImportData* ImportData = NewObject<UGroomCacheImportData>(GroomCache);
				ImportData->Settings = GroomCacheImportOptions->ImportSettings;
				GroomCache->AssetImportData = ImportData;
			}
			GroomCache->AssetImportData->Update(Filename);
		}

		// GroomAsset was not imported so return one of the GroomCache as the asset that was created
		if (!CurrentAsset && GroomCaches.Num() > 0)
		{
			CurrentAsset = GroomCaches[0];
		}
	} 

	return CurrentAsset;
}

bool UHairStrandsFactory::FactoryCanImport(const FString& Filename)
{
	for (TSharedPtr<IGroomTranslator> Translator : Translators)
	{
		if (Translator->CanTranslate(Filename))
		{
			return true;
		}
	}
	return false;
}

TSharedPtr<IGroomTranslator> UHairStrandsFactory::GetTranslator(const FString& Filename)
{
	FString Extension = FPaths::GetExtension(Filename);
	for (TSharedPtr<IGroomTranslator> Translator : Translators)
	{
		if (Translator->IsFileExtensionSupported(Extension))
		{
			return Translator;
		}
	}
	return {};
}


void UHairStrandsFactory::CleanUp() 
{
	bImportAll = false;
	Super::CleanUp();
}

#undef LOCTEXT_NAMESPACE

