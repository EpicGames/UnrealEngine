// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsImporter.h"

#include "HairDescription.h"
#include "GroomAsset.h"
#include "GroomBuilder.h"
#include "GroomImportOptions.h"

DEFINE_LOG_CATEGORY_STATIC(LogHairImporter, Log, All);

static int32 GHairStrandsReimportGroupRemapping = 1;
static FAutoConsoleVariableRef CVarHairStrandsLoadAsset(TEXT("r.HairStrands.ReimportGroupRemapping"), GHairStrandsReimportGroupRemapping, TEXT("Remap hair group settings/parameters when reimporting a groom asset (experimental)"));

///////////////////////////////////////////////////////////////////////////////////////////////////
// Remapping of settings when reimporting groom asset

struct FHairGroupSettings
{
	FHairGroupInfoWithVisibility Info;
	FHairGroupsRendering Rendering;
	FHairGroupsPhysics Physics;
	FHairGroupsInterpolation Interpolation;
	FHairGroupsLOD LOD;
	float EffectiveLODBias = 0;
};

// Extract all the settings from a groom asset
static TArray<FHairGroupSettings> GetHairGroupSettings(const UGroomAsset* In)
{
	TArray<FHairGroupSettings> Out;
	for (uint32 GroupIndex = 0, GroupCount = In->GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		FHairGroupSettings& OutGroup = Out.AddDefaulted_GetRef();
		OutGroup.Info				= In->GetHairGroupsInfo()[GroupIndex];
		OutGroup.Rendering			= In->GetHairGroupsRendering()[GroupIndex];
		OutGroup.Physics			= In->GetHairGroupsPhysics()[GroupIndex];
		OutGroup.Interpolation		= In->GetHairGroupsInterpolation()[GroupIndex];
		OutGroup.LOD				= In->GetHairGroupsLOD()[GroupIndex];
		OutGroup.EffectiveLODBias	= In->GetEffectiveLODBias()[GroupIndex];
		check(GroupIndex == OutGroup.Info.GroupIndex);
	}
	return Out;
}

// Apply all the settings to a groom asset
static void SetHairGroupSettings(UGroomAsset* OutAsset, const TArray<FHairGroupSettings>& InSettings, const TArray<int32>& OldToNewGroupIndexMapping)
{
	for (uint32 GroupIndex = 0, GroupCount = OutAsset->GetNumHairGroups(); GroupIndex < GroupCount; ++GroupIndex)
	{
		const FHairGroupSettings& Settings = InSettings[GroupIndex];
		OutAsset->GetHairGroupsInfo()[GroupIndex]			= Settings.Info;
		OutAsset->GetHairGroupsRendering()[GroupIndex]		= Settings.Rendering;
		OutAsset->GetHairGroupsPhysics()[GroupIndex]		= Settings.Physics;
		OutAsset->GetHairGroupsInterpolation()[GroupIndex]	= Settings.Interpolation;
		OutAsset->GetHairGroupsLOD()[GroupIndex]			= Settings.LOD;
		OutAsset->GetEffectiveLODBias()[GroupIndex]			= Settings.EffectiveLODBias;
	}

	// Update cards description with new group index
	for (FHairGroupsCardsSourceDescription& Desc : OutAsset->GetHairGroupsCards())
	{
		Desc.GroupIndex = OldToNewGroupIndexMapping[Desc.GroupIndex];
	}

	// Update mesh description with new group index
	for (FHairGroupsMeshesSourceDescription& Desc : OutAsset->GetHairGroupsMeshes())
	{
		Desc.GroupIndex = OldToNewGroupIndexMapping[Desc.GroupIndex];
	}
}

// Remap group settings
static void RemapHairGroupSettings(
	const TArray<int32>& NewToOldGroupIndexMapping,
	const TArray<FHairGroupSettings> InOldSettings,
	const FHairDescriptionGroups& InNewHairDescriptionGroups, 
	const TArray<FHairGroupsInterpolation> InNewImportedInterpolationSettings,
	TArray<FHairGroupSettings>& OutNewSettings)
{
	const FHairGroupsLOD DefaultLOD = FHairGroupsLOD::GetDefault();

	OutNewSettings.SetNum(InNewHairDescriptionGroups.HairGroups.Num());
	for (const FHairDescriptionGroup& NewGroupDesc : InNewHairDescriptionGroups.HairGroups)
	{
		FHairGroupSettings& NewGroup = OutNewSettings[NewGroupDesc.Info.GroupIndex];

		const int32 OldGroupIndex = NewToOldGroupIndexMapping[NewGroupDesc.Info.GroupIndex];
		if (InOldSettings.IsValidIndex(OldGroupIndex))
		{
			NewGroup = InOldSettings[OldGroupIndex];
		}
		else
		{
			NewGroup.LOD = DefaultLOD;
		}


		// Preserve the new group GroupIndex/GroupID/GroupName
		NewGroup.Info.GroupIndex = NewGroupDesc.Info.GroupIndex;
		NewGroup.Info.GroupID    = NewGroupDesc.Info.GroupID;
		NewGroup.Info.GroupName  = NewGroupDesc.Info.GroupName;

		// Use import interpolation settings
		NewGroup.Interpolation   = InNewImportedInterpolationSettings[NewGroupDesc.Info.GroupIndex];
	}
}

void RemapHairGroupInteprolationSettings(
	const UGroomAsset* InOldGroomAsset, 
	const FHairDescriptionGroups& InNewHairDescriptionGroups,
	const UGroomHairGroupsMapping* InGroupsMapping, 
	TArray<FHairGroupsInterpolation>& OutNewInterpolationSettings)
{
	// Remap existing interpolation settings based on GroupName/GroupID if possible, otherwise initialize them to default
	// Populate the interpolation settings based on the group count from the description
	const uint32 NewGroupCount = InNewHairDescriptionGroups.HairGroups.Num();
	OutNewInterpolationSettings.Init(FHairGroupsInterpolation(), NewGroupCount);

	const bool bUseGroupRemapping = GHairStrandsReimportGroupRemapping > 0;
	if (InGroupsMapping && bUseGroupRemapping)
	{
		const TArray<FHairGroupSettings> OldSettings = GetHairGroupSettings(InOldGroomAsset);
		for (const FHairDescriptionGroup& NewGroupDesc : InNewHairDescriptionGroups.HairGroups)
		{			
			const int32 OldGroupIndex = InGroupsMapping->NewToOldGroupIndexMapping[NewGroupDesc.Info.GroupIndex];
			if (OldSettings.IsValidIndex(OldGroupIndex))
			{
				OutNewInterpolationSettings[NewGroupDesc.Info.GroupIndex] = OldSettings[OldGroupIndex].Interpolation;
			}
		}
	}
	else
	{
		// Old path
		for (uint32 GroupIndex = 0; GroupIndex < NewGroupCount; ++GroupIndex)
		{
			OutNewInterpolationSettings[GroupIndex] = InOldGroomAsset->GetHairGroupsInterpolation()[GroupIndex];
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FHairImportContext::FHairImportContext(UGroomImportOptions* InImportOptions, UObject* InParent, UClass* InClass, FName InName, EObjectFlags InFlags)
	: ImportOptions(InImportOptions)
	, Parent(InParent)
	, Class(InClass)
	, Name(InName)
	, Flags(InFlags)
{
}

UGroomAsset* FHairStrandsImporter::ImportHair(const FHairImportContext& ImportContext, FHairDescription& NewHairDescription, UGroomAsset* OldExistingHair, const UGroomHairGroupsMapping* InGroupsMapping)
{
	const uint32 GroupCount = ImportContext.ImportOptions->InterpolationSettings.Num();
	UGroomAsset* OutHairAsset = nullptr;
	if (OldExistingHair)
	{
		OutHairAsset = OldExistingHair;
	}
	else
	{
		OutHairAsset = NewObject<UGroomAsset>(ImportContext.Parent, ImportContext.Class, ImportContext.Name, ImportContext.Flags);
		if (!OutHairAsset)
		{
			UE_LOGF(LogHairImporter, Warning, "Failed to import hair: Could not allocate memory to create asset.");
			return nullptr;
		}
	}

	const bool bUseGroupRemapping = GHairStrandsReimportGroupRemapping > 0;
	if (OldExistingHair && OldExistingHair->CanRebuildFromDescription() && bUseGroupRemapping) // make sure OldExistingHair is not a new empty groom asset
	{
		FHairDescriptionGroups NewHairDescriptionGroups;
		FGroomBuilder::BuildHairDescriptionGroups(NewHairDescription, NewHairDescriptionGroups);

		// 1. Extract/Build group remapping
		TArray<int32> OldToNewGroupIndexMapping;
		TArray<int32> NewToOldGroupIndexMapping;
		if (InGroupsMapping)
		{
			OldToNewGroupIndexMapping = InGroupsMapping->OldToNewGroupIndexMapping;
			NewToOldGroupIndexMapping = InGroupsMapping->NewToOldGroupIndexMapping;
		}
		else
		{
			UGroomHairGroupsMapping::RemapHairDescriptionGroups(OldExistingHair->GetHairDescriptionGroups(), NewHairDescriptionGroups, OldToNewGroupIndexMapping);
			UGroomHairGroupsMapping::RemapHairDescriptionGroups(NewHairDescriptionGroups, OldExistingHair->GetHairDescriptionGroups(), NewToOldGroupIndexMapping);
		}

		// 2. Remap group settings when reimporting the asset and group number/order mismatch
		TArray<FHairGroupSettings> NewSettings;	
		RemapHairGroupSettings(
			NewToOldGroupIndexMapping,
			GetHairGroupSettings(OldExistingHair),
			NewHairDescriptionGroups, 
			ImportContext.ImportOptions->InterpolationSettings, // New interpolation settings, already remapped
			NewSettings);

		// 3. Apply new settings to the new asset
		OutHairAsset->ClearNumGroup(GroupCount);
		SetHairGroupSettings(OutHairAsset, NewSettings, OldToNewGroupIndexMapping);
	}
	else
	{
		OutHairAsset->SetNumGroup(GroupCount);

		// Populate the interpolation settings with the new settings from the importer	
		for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
		{
			// Insure the interpolation settings matches between the importer and the actual asset
			const FHairGroupsInterpolation& InterpolationSettings = ImportContext.ImportOptions->InterpolationSettings[GroupIndex];
			OutHairAsset->GetHairGroupsInterpolation()[GroupIndex] = InterpolationSettings;
		}
	}

	// Sanity check
	check(OutHairAsset->AreGroupsValid());
	check(uint32(OutHairAsset->GetNumHairGroups()) == GroupCount);

	OutHairAsset->CommitHairDescription(MoveTemp(NewHairDescription), EHairDescriptionType::Source);

	const bool bSucceeded = OutHairAsset->CacheDerivedDatas();
	if (!bSucceeded)
	{
		// Purge the newly created asset that failed to import
		if (OutHairAsset != OldExistingHair)
		{
			OutHairAsset->ClearFlags(RF_Standalone);
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
		return nullptr;
	}

	return OutHairAsset;
}
