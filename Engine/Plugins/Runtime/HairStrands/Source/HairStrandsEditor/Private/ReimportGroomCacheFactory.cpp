// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReimportGroomCacheFactory.h"
#include "EditorFramework/AssetImportData.h"
#include "GroomCache.h"
#include "ReimportHairStrandsFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReimportGroomCacheFactory)

UReimportGroomCacheFactory::UReimportGroomCacheFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEditorImport = false;

	// The ReimportGroomCacheFactory comes after the ReimportHairStrandsFactory since it's using the latter to do the actual reimport
	ImportPriority -= 2;
}

bool UReimportGroomCacheFactory::FactoryCanImport(const FString& Filename)
{
	// This factory doesn't import directly since that is managed by the HairStrandsFactory
	// This factory handles the Reimport action on GroomCache and redirects it to the ReimportHairStrandsFactory
	return false;
}

int32 UReimportGroomCacheFactory::GetPriority() const
{
	return ImportPriority;
}

bool UReimportGroomCacheFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	// Lazy init the translators before first use of the CDO
	if (HasAnyFlags(RF_ClassDefaultObject) && Formats.Num() == 0)
	{
		InitTranslators();
	}

	UAssetImportData* ImportData = nullptr;
	if (UGroomCache* GroomCache = Cast<UGroomCache>(Obj))
	{
		ImportData = GroomCache->AssetImportData;
	}

	if (ImportData)
	{
		if (GetTranslator(ImportData->GetFirstFilename()).IsValid())
		{
			ImportData->ExtractFilenames(OutFilenames);
			return true;
		}
	}

	return false;
}

void UReimportGroomCacheFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UGroomCache* Asset = Cast<UGroomCache>(Obj);
	if (Asset && Asset->AssetImportData && ensure(NewReimportPaths.Num() == 1))
	{
		Asset->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportGroomCacheFactory::Reimport(UObject* Obj)
{
	// Let the ReimportHairStrandsFactory do the reimport
	UReimportHairStrandsFactory* ReimportFactory = UReimportHairStrandsFactory::StaticClass()->GetDefaultObject<UReimportHairStrandsFactory>();
	return ReimportFactory->Reimport(Obj);
}

