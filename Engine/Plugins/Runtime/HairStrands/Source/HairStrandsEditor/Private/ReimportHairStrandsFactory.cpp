// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReimportHairStrandsFactory.h"

#include "EditorFramework/AssetImportData.h"
#include "GroomAsset.h"
#include "GroomAssetImportData.h"
#include "GroomBuilder.h"
#include "GroomCache.h"
#include "GroomCacheData.h"
#include "GroomCacheImporter.h"
#include "GroomCacheImportOptions.h"
#include "GroomImportOptions.h"
#include "GroomImportOptionsWindow.h"
#include "HairDescription.h"
#include "HairStrandsImporter.h"
#include "HairStrandsTranslator.h"
#include "Logging/LogMacros.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/Package.h"
#include "Editor/EditorPerProjectUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReimportHairStrandsFactory)

#define LOCTEXT_NAMESPACE "ReimportHairStrandsFactory"

DEFINE_LOG_CATEGORY_STATIC(LogReimportHairStrandsFactory, Log, All);

UReimportHairStrandsFactory::UReimportHairStrandsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Needed for "Reimport With New File" on GroomCache
	SupportedClass = UGroomCache::StaticClass();
	bEditorImport = true;

	// The HairStrandsFactory should come before the Reimport factory
	ImportPriority -= 1;
}

bool UReimportHairStrandsFactory::FactoryCanImport(const FString& Filename)
{
	return false;
}

int32 UReimportHairStrandsFactory::GetPriority() const
{
	return ImportPriority;
}

bool UReimportHairStrandsFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	// Lazy init the translators before first use of the CDO
	if (HasAnyFlags(RF_ClassDefaultObject) && Formats.Num() == 0)
	{
		InitTranslators();
	}

	UAssetImportData* ImportData = nullptr;
	if (UGroomAsset* HairAsset = Cast<UGroomAsset>(Obj))
	{
		ImportData = HairAsset->AssetImportData;
	}

	if (ImportData)
	{
		if (GetTranslator(ImportData->GetFirstFilename()).IsValid())
		{
			ImportData->ExtractFilenames(OutFilenames);
			return true;
		}
	}

	return false;
}

void UReimportHairStrandsFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UGroomAsset* Asset = Cast<UGroomAsset>(Obj);
	if (Asset && Asset->AssetImportData && ensure(NewReimportPaths.Num() == 1))
	{
		Asset->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

namespace ReimportGroomAssetHelpers
{
	// Given a GroomAsset, try to find the corresponding GroomCache import settings, if any
	void FindGroomCacheImportSettings(UGroomAsset* HairAsset, FGroomCacheImportSettings& Settings)
	{
		// Try to find the associated GroomCache if any
		FString GroomAssetPath = HairAsset->GetPathName();
		FString Path = FPackageName::GetLongPackagePath(GroomAssetPath);
		FString ObjectName = FPackageName::GetShortName(GroomAssetPath);

		FString PackageName;
		FString AssetName;
		ObjectName.Split(TEXT("."), &PackageName, &AssetName);

		// Try finding the strands cache
		FString StrandsCache = AssetName + TEXT("_strands_cache");
		FString GroomCacheName = FString::Printf(TEXT("%s.%s"), *StrandsCache, *StrandsCache);
		FString GroomCachePath = FString::Printf(TEXT("%s/%s"), *Path, *GroomCacheName);
		UGroomCache* GroomCache = nullptr;
		UPackage* Package = LoadPackage(nullptr, *GroomCachePath, LOAD_None);
		if (Package)
		{
			GroomCache = FindObject<UGroomCache>(Package, *StrandsCache);
		}

		// Strands cache not found, try finding the guides cache
		if (!GroomCache)
		{
			FString GuidesCache = AssetName + TEXT("_guides_cache");
			GroomCacheName = FString::Printf(TEXT("%s.%s"), *GuidesCache, *GuidesCache);
			GroomCachePath = FString::Printf(TEXT("%s/%s"), *Path, *GroomCacheName);
			Package = LoadPackage(nullptr, *GroomCachePath, LOAD_None);
			if (Package)
			{
				GroomCache = FindObject<UGroomCache>(Package, *GuidesCache);
			}
		}

		if (GroomCache)
		{
			UGroomCacheImportData* GroomCacheImportData = Cast<UGroomCacheImportData>(GroomCache->AssetImportData);
			if (GroomCacheImportData)
			{
				Settings = GroomCacheImportData->Settings;
			}
		}
	}

	UGroomAsset* ReimportGroom(const FString& SourceFilename, FMD5Hash& FileHash, TSharedPtr<IGroomTranslator> Translator, FHairImportContext& HairImportContext, FHairDescription& HairDescription, UGroomAsset* HairAsset, const FGroomCacheImportSettings& Settings, const UGroomHairGroupsMapping* InGroupsMapping)
	{
		if (Settings.bImportGroomAsset)
		{
			UGroomAsset* ReimportedHair = FHairStrandsImporter::ImportHair(HairImportContext, HairDescription, HairAsset, InGroupsMapping);
			if (ReimportedHair)
			{
				// Move the transient ImportOptions to the asset package and set it on the GroomAssetImportData for serialization
				UGroomAssetImportData* GroomAssetImportData = Cast<UGroomAssetImportData>(HairAsset->AssetImportData);
				if (GroomAssetImportData)
				{
					HairImportContext.ImportOptions->Rename(nullptr, GroomAssetImportData);
					GroomAssetImportData->ImportOptions = HairImportContext.ImportOptions;

					// Update the asset import data with the new file. This will hash the file, so cache the hash for later use if needed
					GroomAssetImportData->Update(SourceFilename);
					FileHash = GroomAssetImportData->GetSourceData().SourceFiles[0].FileHash;
				}

				if (HairAsset->GetOuter())
				{
					HairAsset->GetOuter()->MarkPackageDirty();
				}
				else
				{
					HairAsset->MarkPackageDirty();
				}
			}
			else
			{
				UE_LOGF(LogReimportHairStrandsFactory, Error, "Failed to reimport groom asset.");
			}

			return ReimportedHair;
		}
		else
		{
			return Cast<UGroomAsset>(Settings.GroomAsset.TryLoad());
		}
	}
}

namespace ReimportGroomCacheHelpers
{
	// Given a GroomCache and its import settings, try to find its corresponding GroomAsset
	UGroomAsset* FindAssociatedGroom(UGroomCache* SourceGroomCache, const FGroomCacheImportSettings& Settings)
	{
		UGroomAsset* HairAsset = nullptr;

		// Find the associated GroomAsset, either the one imported along with the GroomCache or the one that was manually referenced
		if (Settings.bImportGroomAsset)
		{
			// The GroomCache was imported along with the GroomAsset so try to find the associated GroomAsset by name
			// The GroomAsset name is extracted from the GroomCache without the name suffix
			FString GroomCachePath = SourceGroomCache->GetPathName();

			FString Path = FPackageName::GetLongPackagePath(GroomCachePath);
			FString ObjectName = FPackageName::GetShortName(GroomCachePath);

			FString PackageName;
			FString AssetName;
			if (ObjectName.Split(TEXT("."), &PackageName, &AssetName))
			{
				if (SourceGroomCache->GetType() == EGroomCacheType::Strands)
				{
					AssetName.RemoveFromEnd("_strands_cache");
				}
				else if (SourceGroomCache->GetType() == EGroomCacheType::Guides)
				{
					AssetName.RemoveFromEnd("_guides_cache");
				}
			}

			FString HairAssetPath = FString::Printf(TEXT("%s/%s.%s"), *Path, *AssetName, *AssetName);
			UPackage* Package = LoadPackage(nullptr, *HairAssetPath, LOAD_None);
			if (Package)
			{
				FString HairAssetName = FString::Printf(TEXT("%s.%s"), *AssetName, *AssetName);
				HairAsset = FindObject<UGroomAsset>(Package, *AssetName);
			}
		}
		else if (Settings.GroomAsset.IsValid())
		{
			HairAsset = Cast<UGroomAsset>(Settings.GroomAsset.TryLoad());
		}
		return HairAsset;
	}

	void ReimportGroomCaches(const FString& SourceFilename, FMD5Hash& FileHash, TSharedPtr<IGroomTranslator> Translator, FHairImportContext& HairImportContext, FGroomAnimationInfo& AnimInfo, UGroomAsset* GroomAsset, const FGroomCacheImportSettings& Settings, UGroomCache* SourceGroomCache)
	{
		if (Settings.bImportGroomCache && GroomAsset)
		{
			// If the reimport was from a GroomCache, set it as the parent to preserve the package name
			// Otherwise, use the previously set GroomAsset package as the base for the package name
			if (SourceGroomCache)
			{
				HairImportContext.Parent = SourceGroomCache;
			}
			if (Settings.bOverrideConversionSettings)
			{
				HairImportContext.ImportOptions->ConversionSettings = Settings.ConversionSettings;
			}
			TArray<UGroomCache*> GroomCaches = FGroomCacheImporter::ImportGroomCache(SourceFilename, Translator, AnimInfo, HairImportContext, GroomAsset, Settings.ImportType);

			// Update asset import data
			for (UGroomCache* GroomCache : GroomCaches)
			{
				if (!GroomCache->AssetImportData || !GroomCache->AssetImportData->IsA<UGroomCacheImportData>())
				{
					GroomCache->AssetImportData = NewObject<UGroomCacheImportData>(GroomCache);
				}
				UGroomCacheImportData* ImportData = Cast<UGroomCacheImportData>(GroomCache->AssetImportData);
				ImportData->Settings = Settings;

				if (FileHash.IsValid())
				{
					GroomCache->AssetImportData->Update(SourceFilename, &FileHash);
				}
				else
				{
					GroomCache->AssetImportData->Update(SourceFilename);
					FileHash = GroomCache->AssetImportData->GetSourceData().SourceFiles[0].FileHash;
				}
			}
		}
	}
}

void RemapHairGroupInteprolationSettings(
	const UGroomAsset* InOldGroomAsset, 
	const FHairDescriptionGroups& InNewHairDescriptionGroups,
	const UGroomHairGroupsMapping* InGroupsMapping, 
	TArray<FHairGroupsInterpolation>& OutNewInterpolationSettings);

EReimportResult::Type UReimportHairStrandsFactory::Reimport(UObject* Obj)
{
	// The reimport can start either from a GroomAsset or from a GroomCache
	// First step is to validate the source object and retrieve the corresponding GroomCacheImportSettings
	UGroomAsset* HairAsset = Cast<UGroomAsset>(Obj);
	UGroomCache* SourceGroomCache = nullptr;
	UGroomCacheImportOptions* GroomCacheReimportOptions = NewObject<UGroomCacheImportOptions>();
	if (!HairAsset)
	{
		// Check if it's a valid GroomCache
		SourceGroomCache = Cast<UGroomCache>(Obj);
		if (!SourceGroomCache)
		{
			UE_LOGF(LogReimportHairStrandsFactory, Error, " Not the correct asset type to reimport.");
			return EReimportResult::Failed;
		}

		UGroomCacheImportData* GroomCacheImportData = Cast<UGroomCacheImportData>(SourceGroomCache->AssetImportData);
		if (!GroomCacheImportData)
		{
			UE_LOGF(LogReimportHairStrandsFactory, Error, "Asset import data missing.");
			return EReimportResult::Failed;
		}

		CurrentFilename = SourceGroomCache->AssetImportData->GetFirstFilename();

		// Retrieve the previous import settings to display
		GroomCacheReimportOptions->ImportSettings = GroomCacheImportData->Settings;

		HairAsset = ReimportGroomCacheHelpers::FindAssociatedGroom(SourceGroomCache, GroomCacheImportData->Settings);
		if (!HairAsset)
		{
			// In case where the associated GroomAsset is not found
			UE_LOGF(LogReimportHairStrandsFactory, Error, "Associated groom asset missing.");
			return EReimportResult::Failed;
		}
	}
	else if (!HairAsset->AssetImportData)
	{
		UE_LOGF(LogReimportHairStrandsFactory, Error, "Asset import data missing.");
		return EReimportResult::Failed;
	}
	else
	{
		CurrentFilename = HairAsset->AssetImportData->GetFirstFilename();

		// Try to find the corresponding GroomCache import settings
		// Defaults are used if none was found
		ReimportGroomAssetHelpers::FindGroomCacheImportSettings(HairAsset, GroomCacheReimportOptions->ImportSettings);
	}

	UGroomAssetImportData* GroomAssetImportData = Cast<UGroomAssetImportData>(HairAsset->AssetImportData);
	UGroomImportOptions* GroomReimportOptions = nullptr;
	if (GroomAssetImportData)
	{
		// Duplicate the options to prevent dirtying the asset when they are modified but the re-import is cancelled
		GroomReimportOptions = DuplicateObject<UGroomImportOptions>(GroomAssetImportData->ImportOptions, nullptr);
	}
	else
	{
		// Convert the AssetImportData to a GroomAssetImportData
		GroomAssetImportData = NewObject<UGroomAssetImportData>(HairAsset);
		HairAsset->AssetImportData = GroomAssetImportData;
	}

	if (!GroomReimportOptions)
	{
		// Make sure to have ImportOptions. Could happen if we just converted the AssetImportData
		GroomReimportOptions = NewObject<UGroomImportOptions>();
	}

	TSharedPtr<IGroomTranslator> SelectedTranslator = GetTranslator(CurrentFilename);
	if (!SelectedTranslator.IsValid())
	{
		UE_LOGF(LogReimportHairStrandsFactory, Error, "File format not supported.");
		return EReimportResult::Failed;
	}

	FGroomAnimationInfo AnimInfo;

	UGroomHairGroupsMapping* GroupsMapping = NewObject<UGroomHairGroupsMapping>();

	// Load the alembic file upfront to preview & report any potential issue
	FHairDescriptionGroups OutDescription;
	{
		FScopedSlowTask Progress((float)1, LOCTEXT("ReimportHairAsset", "Reimporting hair asset for preview..."), true);
		Progress.MakeDialog(true);

		FHairDescription HairDescription;
		if (!SelectedTranslator->Translate(CurrentFilename, HairDescription, GroomReimportOptions->ConversionSettings, &AnimInfo))
		{
			UE_LOGF(LogReimportHairStrandsFactory, Error, "Error translating file %ls.", *CurrentFilename);
			return EReimportResult::Failed;
		}

		FGroomBuilder::BuildHairDescriptionGroups(HairDescription, OutDescription);

		// Group remapping
		GroupsMapping->Map(HairAsset->GetHairDescriptionGroups(), OutDescription);

		// Remap existing interpolation settings based on GroupName if possible, otherwise initialize them to default
		RemapHairGroupInteprolationSettings(HairAsset, OutDescription, GroupsMapping, GroomReimportOptions->InterpolationSettings);
	}

	FGroomCacheImporter::SetupImportSettings(GroomCacheReimportOptions->ImportSettings, AnimInfo);

	// Import asset with or without import window
	{
		// Convert the process hair description into hair groups
		UGroomHairGroupsPreview* GroupsPreview = NewObject<UGroomHairGroupsPreview>();
		{
			for (const FHairDescriptionGroup& Group : OutDescription.HairGroups)
			{
				FGroomHairGroupPreview& OutGroup = GroupsPreview->Groups.AddDefaulted_GetRef();
				OutGroup.GroupName  	= Group.Info.GroupName;
				OutGroup.GroupID		= Group.Info.GroupID;
				OutGroup.GroupIndex		= Group.Info.GroupIndex;
				OutGroup.CurveCount 	= Group.Info.NumCurves;
				OutGroup.GuideCount 	= Group.Info.NumGuides;
				OutGroup.Attributes 	= Group.GetHairAttributes();
				OutGroup.AttributeFlags = Group.GetHairAttributeFlags();
				OutGroup.Flags			= Group.Info.Flags;
				if (OutGroup.GroupIndex < OutDescription.HairGroups.Num())
				{
					OutGroup.InterpolationSettings = GroomReimportOptions->InterpolationSettings[OutGroup.GroupIndex];
				}
			}
		}

		// Prevent any UI for automation, unattended and commandlet
		const bool IsUnattended = IsAutomatedImport() || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
		const bool ShowImportDialogAtReimport = (GetDefault<UEditorPerProjectUserSettings>()->bShowImportDialogAtReimport || bForceShowDialog) && !IsUnattended;
		if (ShowImportDialogAtReimport)
		{
			// Not need to show ImportAll button as this is managed by UEditorPerProjectUserSettings
			TSharedPtr<SGroomImportOptionsWindow> GroomOptionWindow = SGroomImportOptionsWindow::DisplayImportOptions(GroomReimportOptions, GroomCacheReimportOptions, GroupsPreview, GroupsMapping, CurrentFilename, false /*bShowImportAllButton*/);

			if (!GroomOptionWindow->ShouldImport())
			{
				return EReimportResult::Cancelled;
			}

			// Save the options as the new default
			for (const FGroomHairGroupPreview& GroupPreview : GroupsPreview->Groups)
			{
				if (GroupPreview.GroupIndex < OutDescription.HairGroups.Num())
				{
					GroomReimportOptions->InterpolationSettings[GroupPreview.GroupIndex] = GroupPreview.InterpolationSettings;
				}
			}
		}
		else
		{
			FGroomImportStatus ImportStatus = GetGroomImportStatus(GroupsPreview, nullptr /*GroomCacheImportOptions*/, GroupsMapping);

			// Display Warning and error in log to not break automatic import flow
			if (EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::Error))
			{
				UE_LOGF(LogReimportHairStrandsFactory, Error, "Error during groom reimport (file %ls) : %ls", *CurrentFilename, *GetGroomImportStatusText(ImportStatus, false /*bAddPrefix*/).ToString());
				return EReimportResult::Failed;
			}
			else if (EnumHasAnyFlags(ImportStatus.Status, EHairDescriptionStatus::Warning))
			{
				UE_LOGF(LogReimportHairStrandsFactory, Warning, "Warning during groom reimport (file %ls) : %ls", *CurrentFilename, *GetGroomImportStatusText(ImportStatus, false /*bAddPrefix*/).ToString());
			}
		}
	}

	FGroomCacheImporter::ApplyImportSettings(GroomCacheReimportOptions->ImportSettings, AnimInfo);

	FHairDescription HairDescription;
	if (!SelectedTranslator->Translate(CurrentFilename, HairDescription, GroomReimportOptions->ConversionSettings))
	{
		UE_LOGF(LogReimportHairStrandsFactory, Error, "Error translating file %ls.", *CurrentFilename);
		return EReimportResult::Failed;
	}

	// Reimport the GroomAsset
	FMD5Hash ReimportedFileHash;
	FHairImportContext HairImportContext(GroomReimportOptions, HairAsset, nullptr, FName(), EObjectFlags::RF_Public | EObjectFlags::RF_Standalone | EObjectFlags::RF_Transactional);
	UGroomAsset* AssociatedGroomAsset = ReimportGroomAssetHelpers::ReimportGroom(CurrentFilename, ReimportedFileHash, SelectedTranslator, HairImportContext, HairDescription, HairAsset, GroomCacheReimportOptions->ImportSettings, GroupsMapping);

	// Reimport the GroomCaches
	ReimportGroomCacheHelpers::ReimportGroomCaches(CurrentFilename, ReimportedFileHash, SelectedTranslator, HairImportContext, AnimInfo, AssociatedGroomAsset, GroomCacheReimportOptions->ImportSettings, SourceGroomCache);

	return EReimportResult::Succeeded;
}

#undef LOCTEXT_NAMESPACE

