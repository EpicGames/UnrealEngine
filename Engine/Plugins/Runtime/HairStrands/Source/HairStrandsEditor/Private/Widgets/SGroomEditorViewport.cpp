// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/SGroomEditorViewport.h"
#include "FinalPostProcessSettings.h"
#include "SceneView.h"
#include "Widgets/Layout/SBox.h"
#include "Editor/UnrealEdEngine.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UnrealEdGlobals.h"
#include "ComponentReregisterContext.h"
#include "Slate/SceneViewport.h"
#include "Engine/TextureCube.h"
#include "ImageUtils.h"
#include "Engine/Canvas.h"
#include "Engine/Font.h"
#include "CanvasItem.h"
#include "ToolMenus.h"
#include "DrawDebugHelpers.h"
#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "GroomComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "GroomEditorCommands.h"
#include "GroomVisualizationMenuCommands.h"
#include "EditorViewportCommands.h"
#include "AssetViewerSettings.h"
#include "PreviewProfileController.h"
#include "SLevelViewport.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Tests/ToolMenusTestUtilities.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "SGroomEditorViewport"

class FAdvancedPreviewScene;

/** Viewport Client for the preview viewport */
class FGroomEditorViewportClient : public FEditorViewportClient
{
public:
	FGroomEditorViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SGroomEditorViewport>& InGroomEditorViewport);
	virtual ~FGroomEditorViewportClient() override;

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override {
		return true;
	};
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }

	void SetShowGrid(bool bShowGrid);
	
	virtual void SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)override;
	
	TWeakPtr<SGroomEditorViewport> GroomEditorViewportPtr;	
};

FGroomEditorViewportClient::FGroomEditorViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SGroomEditorViewport>& InGroomEditorViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InGroomEditorViewport))
{
	GroomEditorViewportPtr = InGroomEditorViewport;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = false;
	DrawHelper.GridColorAxis = FColor(80,80,80);
	DrawHelper.GridColorMajor = FColor(72,72,72);
	DrawHelper.GridColorMinor = FColor(64,64,64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	ShowWidget(false);

	SetViewMode(VMI_Lit);
	
	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetTemporalAA(true);
	EngineShowFlags.SetShaderPrint(true);
	
	OverrideNearClipPlane(1.0f);	

	//This seems to be needed to get the correct world time in the preview.
	SetIsSimulateInEditorViewport(true);	

	UEditorPerProjectUserSettings* PerProjectSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	UAssetViewerSettings* DefaultSettings = UAssetViewerSettings::Get();
	PerProjectSettings->AssetViewerProfileIndex = DefaultSettings->Profiles.IsValidIndex(PerProjectSettings->AssetViewerProfileIndex) ? PerProjectSettings->AssetViewerProfileIndex : 0;

	FAdvancedPreviewScene* PreviewSceneCasted = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	PreviewSceneCasted->SetProfileIndex(PerProjectSettings->AssetViewerProfileIndex);
}

FGroomEditorViewportClient::~FGroomEditorViewportClient()
{
}

void FGroomEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);
	
	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

void FGroomEditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	TSharedPtr<SGroomEditorViewport> GroomEditorViewport = GroomEditorViewportPtr.Pin();
	FEditorViewportClient::Draw(InViewport, Canvas);
}

FLinearColor FGroomEditorViewportClient::GetBackgroundColor() const
{
	FLinearColor BackgroundColor = FLinearColor::Black;
	return BackgroundColor;
}

FSceneView* FGroomEditorViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	FSceneView* SceneView = FEditorViewportClient::CalcSceneView(ViewFamily);
	FFinalPostProcessSettings::FCubemapEntry& CubemapEntry = *new(SceneView->FinalPostProcessSettings.ContributingCubemaps) FFinalPostProcessSettings::FCubemapEntry;
	CubemapEntry.AmbientCubemap = GUnrealEd->GetThumbnailManager()->AmbientCubemap;
	CubemapEntry.AmbientCubemapTintMulScaleValue = FLinearColor::White;
	return SceneView;
}

void FGroomEditorViewportClient::SetShowGrid(bool bShowGrid)
{
	DrawHelper.bDrawGrid = bShowGrid;
}

void FGroomEditorViewportClient::SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)
{
	bIsSimulateInEditorViewport = bInIsSimulateInEditorViewport;
}

//////////////////////////////////////////////////////////////////////////

void SGroomEditorViewport::Construct(const FArguments& InArgs)
{	
	bShowGrid			= true;
	GroomComponent		= nullptr;
	StaticGroomTarget	= nullptr;
	SkeletalGroomTarget = nullptr;

	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
	AdvancedPreviewScene->SetFloorVisibility(false);

	FGroomViewportLODCommands::Register();

	SEditorViewport::Construct( SEditorViewport::FArguments() );
	
	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(AdvancedPreviewScene, Client);
}

void SGroomEditorViewport::AddReferencedObjects( FReferenceCollector& Collector )
{
	if (GroomComponent != nullptr)
	{
		Collector.AddReferencedObject(GroomComponent);
	}

	if (StaticGroomTarget != nullptr)
	{
		Collector.AddReferencedObject(StaticGroomTarget);
	}
}

void SGroomEditorViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SEditorViewport::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );
}

bool SGroomEditorViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground()) && SEditorViewport::IsVisible() ;
}

void SGroomEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FGroomViewportLODCommands& ViewportLODMenuCommands = FGroomViewportLODCommands::Get();

	//LOD Auto
	CommandList->MapAction(
		ViewportLODMenuCommands.LODAuto,
		FExecuteAction::CreateSP(this, &SGroomEditorViewport::SetLODLevel, -1),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SGroomEditorViewport::IsLODSelected, -1));

	// LOD 0
	CommandList->MapAction(
		ViewportLODMenuCommands.LOD0,
		FExecuteAction::CreateSP(this, &SGroomEditorViewport::SetLODLevel, 0),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SGroomEditorViewport::IsLODSelected, 0));
	// all other LODs will be added dynamically
	
	const FGroomVisualizationMenuCommands& GroomCommands = FGroomVisualizationMenuCommands::Get();
	GroomCommands.BindCommands(*CommandList, SystemViewportClient);
}

void SGroomEditorViewport::RefreshViewport()
{
	// Invalidate the viewport's display.
	SceneViewport->Invalidate();
}

void SGroomEditorViewport::SetLODLevel(int32 LODIndex)
{
	if (GroomComponent)
	{
		GroomComponent->SetForcedLOD(LODIndex);
		RefreshViewport();
	}	
}

bool SGroomEditorViewport::IsLODSelected(int32 InLODSelection) const
{
	if (GroomComponent)
	{
		return GroomComponent->GetForcedLOD() == InLODSelection;
	}
	return false;
}

int32 SGroomEditorViewport::GetCurrentLOD() const
{
	if (GroomComponent)
	{
		return GroomComponent->GetForcedLOD();
	}
	return -1;
}

int32 SGroomEditorViewport::GetLODCount() const
{
	if (GroomComponent)
	{
		return GroomComponent->GetNumLODs();
	}
	return -1;
}

void SGroomEditorViewport::FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands)
{
	Commands.Add(FGroomViewportLODCommands::Get().LODAuto);
	Commands.Add(FGroomViewportLODCommands::Get().LOD0);
}

void SGroomEditorViewport::OnFocusViewportToSelection()
{
	if (GroomComponent)
	{
		SystemViewportClient->FocusViewportOnBox(GroomComponent->Bounds.GetBox());
	}
}

void SGroomEditorViewport::TogglePreviewGrid()
{
	bShowGrid = !bShowGrid;
	SystemViewportClient->SetShowGrid(bShowGrid);
}

bool SGroomEditorViewport::IsTogglePreviewGridChecked() const
{
	return bShowGrid;
}

void SGroomEditorViewport::SetStaticMeshComponent(UStaticMeshComponent *Target)
{
	if (StaticGroomTarget != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(StaticGroomTarget);
	}
	StaticGroomTarget = Target;

	if (StaticGroomTarget != nullptr)
	{		
		AdvancedPreviewScene->AddComponent(StaticGroomTarget, StaticGroomTarget->GetRelativeTransform());
	}
}

void SGroomEditorViewport::SetSkeletalMeshComponent(USkeletalMeshComponent *Target)
{
	if (SkeletalGroomTarget != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(SkeletalGroomTarget);
	}
	SkeletalGroomTarget = Target;

	if (SkeletalGroomTarget != nullptr)
	{
		AdvancedPreviewScene->AddComponent(SkeletalGroomTarget, SkeletalGroomTarget->GetRelativeTransform());
	}
}

void  SGroomEditorViewport::SetGroomComponent(UGroomComponent* InGroomComponent)
{
	if (GroomComponent != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(GroomComponent);
	}
	GroomComponent = InGroomComponent;

	if (GroomComponent != nullptr)
	{		
		GroomComponent->PostLoad();
		AdvancedPreviewScene->AddComponent(GroomComponent, GroomComponent->GetRelativeTransform());
	}

	if (GroomComponent != nullptr && SystemViewportClient)
	{
		SystemViewportClient->FocusViewportOnBox(GroomComponent->Bounds.GetBox());
	}

	RefreshViewport();
}

TSharedRef<FEditorViewportClient> SGroomEditorViewport::MakeEditorViewportClient() 
{
	SystemViewportClient = MakeShareable(new FGroomEditorViewportClient(*AdvancedPreviewScene.Get(), SharedThis(this)));
		 
	SystemViewportClient->SetViewLocation( FVector::ZeroVector );
	SystemViewportClient->SetViewRotation( FRotator::ZeroRotator );
	SystemViewportClient->SetViewLocationForOrbiting( FVector::ZeroVector );
	SystemViewportClient->bSetListenerPosition = false;

	SystemViewportClient->SetRealtime( true );
	SystemViewportClient->VisibilityDelegate.BindSP( this, &SGroomEditorViewport::IsVisible );
	
	return SystemViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SGroomEditorViewport::BuildViewportToolbar()
{
	const FName ViewportToolbarName = "GroomEditor.ViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(ViewportToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolMenu->StyleName = "ViewportToolbar";
		
		ToolMenu->AddSection("Left");
		
		FToolMenuSection& RightSection = ToolMenu->AddSection("Right");
		RightSection.Alignment = EToolMenuSectionAlign::Last;
		
		
		RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu());
		
		{
			// View Modes
			RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			UToolMenu* ViewMenu = UToolMenus::Get()->ExtendMenu(UToolMenus::JoinMenuPaths(ViewportToolbarName, "ViewModes"));
			FToolMenuSection& GroomViewSection = ViewMenu->FindOrAddSection("Groom", LOCTEXT("GroomViewSectionName", "Groom"));
			
			GroomViewSection.AddDynamicEntry("GroomDynamicViewModes", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				const UUnrealEdViewportToolbarContext* ToolbarContext = Section.Context.FindContext<UUnrealEdViewportToolbarContext>();
				if (!ToolbarContext || !ToolbarContext->Viewport.IsValid())
				{
					return;
				}
				
				FToolMenuEntry Entry = FGroomVisualizationMenuCommands::BuildVisualizationSubMenuItemForGroomEditor(ToolbarContext->Viewport); 
				
				Entry.SetShowInToolbarTopLevel(true);
				
				// Customize the top-level menu to include the option to return to lit
				Entry.ToolBarData.ComboButtonContextMenuGenerator.NewMenuLegacy.BindLambda([](FMenuBuilder& MenuBuilder)
				{
					MenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().LitMode, NAME_None, UViewModeUtils::GetViewModeDisplayName(VMI_Lit));
					MenuBuilder.AddSeparator();
					FGroomVisualizationMenuCommands::BuildVisualisationSubMenuForGroomEditor(MenuBuilder);
				});
				
				Section.AddEntry(Entry);
			}));
		}
		
		{
			// LOD Menu
			RightSection.AddDynamicEntry("DynamicLOD", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				if (const UUnrealEdViewportToolbarContext* Context = Section.FindContext<UUnrealEdViewportToolbarContext>())
				{
					const TWeakPtr<SGroomEditorViewport> GroomViewport = StaticCastWeakPtr<SGroomEditorViewport>(Context->Viewport);
					Section.AddEntry(UE::UnrealEd::CreatePreviewLODSelectionSubmenu(GroomViewport));
				}
			}));
		}
		
		RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());
		RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
	}
	
	FToolMenuContext ViewportToolbarContext;
	ViewportToolbarContext.AppendCommandList(GetCommandList());
	ViewportToolbarContext.AddExtender(GetExtenders());
	
	UUnrealEdViewportToolbarContext* const ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
	
	ViewportToolbarContext.AddObject(ContextObject);
	
	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

TSharedPtr<IPreviewProfileController> SGroomEditorViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

EVisibility SGroomEditorViewport::OnGetViewportContentVisibility() const
{
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<class SEditorViewport> SGroomEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SGroomEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SGroomEditorViewport::OnFloatingButtonClicked()
{

}

#undef LOCTEXT_NAMESPACE