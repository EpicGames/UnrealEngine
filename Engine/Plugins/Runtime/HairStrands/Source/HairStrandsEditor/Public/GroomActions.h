// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomAsset.h"
#include "AssetDefinitionDefault.h"

#include "GroomActions.generated.h"

UCLASS()
class UAssetDefinition_GroomAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetDefinition", "AssetDefinition_Groom", "Groom Asset");
	}

	virtual FLinearColor GetAssetColor() const override;
	
	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UGroomAsset::StaticClass();
	}

	virtual bool CanImport() const override
	{
		return true;
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "AssetDefinition_GroomSubMenu", "Other"), ECategoryMenuType::Section)
		};
		return Categories;
	}

	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End	

	void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const;
};
