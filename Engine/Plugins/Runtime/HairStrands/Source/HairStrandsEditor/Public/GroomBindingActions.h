// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomBindingAsset.h"
#include "AssetDefinitionDefault.h"

#include "GroomBindingActions.generated.h"

UCLASS()
class UAssetDefinition_GroomBindingAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_GroomBinding", "GroomBinding"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UGroomBindingAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Misc };
		return Categories;
	}

	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	// UAssetDefinition End
};
