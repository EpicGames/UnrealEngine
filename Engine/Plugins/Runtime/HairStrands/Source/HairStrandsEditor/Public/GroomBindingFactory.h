// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "GroomBindingFactory.generated.h"

UCLASS(MinimalAPI)
class UGroomBindingFactory : public UFactory
{
	GENERATED_BODY()

public:
	UGroomBindingFactory();

	//~ Begin UFactory Interface
	virtual bool ShouldShowInNewMenu() const { return false;  } // Not shown in the Menu, only exposed for code/python access.
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ End UFactory Interface
};



