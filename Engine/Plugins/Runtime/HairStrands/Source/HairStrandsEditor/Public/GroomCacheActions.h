// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomCache.h"
#include "AssetDefinitionDefault.h"

#include "GroomCacheActions.generated.h"

UCLASS()
class UAssetDefinition_GroomCacheAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_GroomCache", "Groom Cache"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UGroomCache::StaticClass(); }
	virtual bool CanImport() const override { return true; }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Misc };
		return Categories;
	}
	// UAssetDefinition End	

	void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths) const;
};
