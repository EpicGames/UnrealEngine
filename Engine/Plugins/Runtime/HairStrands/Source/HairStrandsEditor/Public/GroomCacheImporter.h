// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API HAIRSTRANDSEDITOR_API

struct FGroomAnimationInfo;
struct FHairImportContext;
class FGroomCacheProcessor;
class IGroomTranslator;
class UGroomAsset;
class UGroomCache;

enum class EGroomCacheImportType : uint8;

struct FGroomCacheImporter
{
	static UE_API TArray<UGroomCache*> ImportGroomCache(const FString& SourceFilename, TSharedPtr<IGroomTranslator> Translator, const FGroomAnimationInfo& InAnimInfo, FHairImportContext& HairImportContext, UGroomAsset* GroomAssetForCache, EGroomCacheImportType ImportType);
	static UE_API void SetupImportSettings(struct FGroomCacheImportSettings& ImportSettings, const FGroomAnimationInfo& AnimInfo);
	static UE_API void ApplyImportSettings(struct FGroomCacheImportSettings& ImportSettings, FGroomAnimationInfo& AnimInfo);
	static UE_API UGroomCache* ProcessToGroomCache(FGroomCacheProcessor& Processor, const FGroomAnimationInfo& AnimInfo, FHairImportContext& ImportContext, const FString& ObjectNameSuffix);
};

#undef UE_API
