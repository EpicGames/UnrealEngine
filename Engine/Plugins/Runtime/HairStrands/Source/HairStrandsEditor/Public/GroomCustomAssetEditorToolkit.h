// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Editor/PropertyEditor/Public/PropertyEditorDelegates.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/SGroomEditorViewport.h"
#include "HairStrandsInterface.h"
#include "GroomEditorStyle.h"

#define UE_API HAIRSTRANDSEDITOR_API

class FAssetThumbnailPool;
class UGroomAsset; 
class UGroomComponent;
class IDetailsView;
class SDockableTab;
class SGroomEditorViewport;
class USkeletalMesh;
class UStaticMesh;
class USkeletalMeshComponent;
class UGroomAsset;
class UGroomBindingAsset;
class UGroomBindingAssetList;
class UAnimationAsset;

class IGroomCustomAssetEditorToolkit : public FAssetEditorToolkit
{
public:
	/** Retrieves the current custom asset. */
	virtual UGroomAsset* GetCustomAsset() const = 0;

	/** Set the current custom asset. */
	virtual void SetCustomAsset(UGroomAsset* InCustomAsset) = 0;

	/** Set preview of a particular binding. */
	virtual void PreviewBinding(int32 BindingIndex) = 0;

	virtual int32 GetActiveBindingIndex() const =0;

	virtual FGroomEditorStyle* GetSlateStyle() const { return nullptr; };
};

class FGroomCustomAssetEditorToolkit : public IGroomCustomAssetEditorToolkit
{
public:

	UE_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;

	/**
	 * Edits the specified asset object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InCustomAsset			The Custom Asset to Edit
	 */
	UE_API void InitCustomAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UGroomAsset* InCustomAsset);

	UE_API FGroomCustomAssetEditorToolkit();

	/** Destructor */
	UE_API virtual ~FGroomCustomAssetEditorToolkit();

	/** Begin IToolkit interface */
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual bool IsPrimaryEditor() const override { return true; }
	UE_API virtual void OnClose() override;
	/** End IToolkit interface */

	/** Retrieves the current custom asset. */
	UE_API virtual UGroomAsset* GetCustomAsset() const override;

	/** Set the current custom asset. */
	UE_API virtual void SetCustomAsset(UGroomAsset* InCustomAsset) override;	

	/** Set preview of a particular binding. */
	UE_API virtual void PreviewBinding(int32 BindingIndex) override;

	/** Return the index of the active binding. */
	UE_API virtual int32 GetActiveBindingIndex() const override;

	/** Return groom editor style. */
	UE_API FGroomEditorStyle* GetSlateStyle() const override;

private:

	// called when The Play simulation button is pressed
	UE_API void OnPlaySimulation();
	UE_API bool CanPlaySimulation() const;
	
	// Called when the pause simulation button is pressed
	UE_API void OnPauseSimulation();
	UE_API bool CanPauseSimulation() const;

	// Called when the reset simulation button is pressed
	UE_API void OnResetSimulation();
	UE_API bool CanResetSimulation() const;

	// Called when the play animation button is pressed
	UE_API void OnPlayAnimation();
	UE_API bool CanPlayAnimation() const;

	// Called when the stop animation button is pressed
	UE_API void OnStopAnimation();
	UE_API bool CanStopAnimation() const;

	// Add buttons to to the toolbar
	UE_API void ExtendToolbar();

	// THis should be called when the properties of the Document are changed
	UE_API void DocPropChanged(UObject *, FPropertyChangedEvent &);

	// THis is called when the groom target object changes and needs updating
	UE_API void OnSkeletalGroomTargetChanged(USkeletalMesh *NewTarget);

	// Return true if the animation asset should be filtered out (not compatible with the preview skel. mesh)
	UE_API bool OnShouldFilterAnimAsset(const FAssetData& AssetData);
	UE_API void OnObjectChangedAnimAsset(const FAssetData& AssetData);
	UE_API bool OnIsEnabledAnimAsset();
	UE_API FString GetCurrentAnimAssetPath() const;

	// create the custom components we need
	UE_API void InitPreviewComponents();

	// Initialized the content of the binding tab
	UE_API void InitializeBindingAssetTabContent();

	// return a pointer to the groom preview component
	UE_API UGroomComponent*		GetPreview_GroomComponent() const;
	UE_API USkeletalMeshComponent*	GetPreview_SkeletalMeshComponent() const;

	UE_API TSharedRef<SDockTab> SpawnViewportTab(const FSpawnTabArgs& Args);

	UE_API TSharedRef<SDockTab> SpawnTab_LODProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_InterpolationProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_RenderingProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_CardsProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_MeshesProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_MaterialProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_PhysicsProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_PreviewGroomComponent(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_BindingProperties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_DataflowProperties(const FSpawnTabArgs& Args);
	
private:

	/** Dockable tab for properties */
	TSharedPtr< SDockableTab > PropertiesTab;
	TSharedPtr< SGroomEditorViewport > ViewportTab;

	TSharedPtr<class IDetailsView> DetailView_LODProperties;
	TSharedPtr<class IDetailsView> DetailView_InterpolationProperties;
	TSharedPtr<class IDetailsView> DetailView_RenderingProperties;
	TSharedPtr<class IDetailsView> DetailView_CardsProperties;
	TSharedPtr<class IDetailsView> DetailView_MeshesProperties;
	TSharedPtr<class IDetailsView> DetailView_MaterialProperties;
	TSharedPtr<class IDetailsView> DetailView_PhysicsProperties;
	TSharedPtr<class IDetailsView> DetailView_PreviewGroomComponent;
	TSharedPtr<class IDetailsView> DetailView_BindingProperties;
	TSharedPtr<class IDetailsView> DetailView_PreviewSceneProperties;
	TSharedPtr<class IDetailsView> DetailView_DataflowProperties;

	static UE_API const FName ToolkitFName;
	static UE_API const FName TabId_Viewport;

	static UE_API const FName TabId_LODProperties;
	static UE_API const FName TabId_InterpolationProperties;
	static UE_API const FName TabId_RenderingProperties;
	static UE_API const FName TabId_CardsProperties;
	static UE_API const FName TabId_MeshesProperties;
	static UE_API const FName TabId_MaterialProperties;
	static UE_API const FName TabId_PhysicsProperties;
	static UE_API const FName TabId_PreviewGroomComponent;
	static UE_API const FName TabId_BindingProperties;
	static UE_API const FName TabId_PreviewSceneProperties;
	static UE_API const FName TabId_DataflowProperties;


	bool bIsTabManagerInitialized = false;
	int32 ActiveGroomBindingIndex = -1;
	TArray<FDelegateHandle> PropertyListenDelegatesAssetChanged;
	TArray<FDelegateHandle> PropertyListenDelegatesResourceChanged;
	TWeakObjectPtr<UGroomAsset> GroomAsset;
	TWeakObjectPtr<UGroomBindingAsset> GroomBindingAsset;
	TWeakObjectPtr<UGroomBindingAssetList> GroomBindingAssetList;

	TWeakObjectPtr<UGroomComponent> PreviewGroomComponent;
	TWeakObjectPtr<USkeletalMeshComponent> PreviewSkeletalMeshComponent;
	TWeakObjectPtr<UAnimationAsset> PreviewSkeletalAnimationAsset;

	TSharedPtr<FAssetThumbnailPool> ThumbnailPool;

	TSharedPtr<FGroomEditorStyle> GroomEditorStyle;
};

#undef UE_API
