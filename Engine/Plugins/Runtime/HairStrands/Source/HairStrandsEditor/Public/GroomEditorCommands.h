// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"
#include "Templates/SharedPointer.h"
#include "Framework/Commands/Commands.h"

class FGroomEditorCommands : public TCommands<FGroomEditorCommands>
{
public:

	FGroomEditorCommands();

	TSharedPtr<FUICommandInfo> BeginHairPlaceTool;
	
	TSharedPtr<FUICommandInfo> ResetSimulation;	
	TSharedPtr<FUICommandInfo> PauseSimulation;
	TSharedPtr<FUICommandInfo> PlaySimulation;

	TSharedPtr<FUICommandInfo> PlayAnimation;
	TSharedPtr<FUICommandInfo> StopAnimation;

	TSharedPtr<FUICommandInfo> Simulate;

	virtual void RegisterCommands() override;
};


class FGroomViewportLODCommands : public TCommands<FGroomViewportLODCommands>
{
public:
	FGroomViewportLODCommands();

	TSharedPtr< FUICommandInfo > LODAuto;
	TSharedPtr< FUICommandInfo > LOD0;

	virtual void RegisterCommands() override;
};
