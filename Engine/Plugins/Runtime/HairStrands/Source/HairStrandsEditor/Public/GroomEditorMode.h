// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"

#include "GroomEditorMode.generated.h"

UCLASS(Transient)
class UGroomEditorMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:
	const static FEditorModeID EM_GroomEditorModeId;

	UGroomEditorMode();

	////////////////
	// UBaseLegacyWidgetEdMode interface
	////////////////

	// these disable the standard gizmo, which is probably want we want in
	// these tools as we can't hit-test the standard gizmo...
	virtual bool AllowWidgetMove() override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	/*
	 * focus events
	 */

	// called when we "start" this editor mode (ie switch to this tab)
	virtual void Enter() override;

	// called when we "end" this editor mode (ie switch to another tab)
	virtual void Exit() override;

	//////////////////
	// End of UBaseLegacyWidgetEdMode interface
	//////////////////

public:
	/** Cached pointer to the viewport world interaction object we're using to interact with mesh elements */
	class UViewportWorldInteraction* ViewportWorldInteraction;

};

