// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPluginManager.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( RootToContentDir( RelativePath, TEXT(".svg") ), __VA_ARGS__ )

/**
 * Implements the visual style of the Groom asset editor UI.
 */
class FGroomEditorStyle	final : public FSlateStyleSet
{
public:

	/** Default constructor. */
	FGroomEditorStyle()
		: FSlateStyleSet(GetStyleName())
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);

		const FString BaseDir = IPluginManager::Get().FindPlugin(TEXT("HairStrands"))->GetBaseDir();
		SetContentRoot(BaseDir / TEXT("Content/Icons"));

		// Set the icon and thumbnail for this class
		Set("Icon16x16.GroomAsset", new IMAGE_BRUSH("S_Groom_16", Icon16x16));
		Set("ClassThumbnail.GroomAsset", new IMAGE_BRUSH("S_Groom_64", Icon40x40));
//		Set("HairToolsManagerCommands.BeginAddPrimitiveTool", new IMAGE_BRUSH("HairSpray128", Icon40x40));
		Set("GroomEditor.SimulationOptions", new IMAGE_BRUSH("S_SimulationOptions_40x", Icon40x40));
		Set("GroomEditor.SimulationOptions.Small", new IMAGE_BRUSH("S_SimulationOptions_40x", Icon20x20));
		Set("GroomEditor.GroomCardGenerator", new IMAGE_BRUSH_SVG("S_GroomCardGenerator", Icon16x16));
		// Set("GroomEditor.GroomCardGenerator", new IMAGE_BRUSH("S_GroomCardGenerator_128", Icon16x16));
		if (FSlateStyleRegistry::FindSlateStyle(GetStyleName()) == nullptr)
		{
			FSlateStyleRegistry::RegisterSlateStyle(*this);
		}
	}

	/** Return the name of this style */
	static FName GetStyleName()
	{
		static FName StyleSetName(TEXT("GroomEditorStyle"));
		return StyleSetName;
	}

	/** Destructor. */
	~FGroomEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};

#undef IMAGE_BRUSH


