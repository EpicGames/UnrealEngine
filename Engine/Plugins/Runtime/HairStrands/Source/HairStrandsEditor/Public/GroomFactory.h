// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "GroomFactory.generated.h"

UCLASS(MinimalAPI)
class UGroomFactory : public UFactory
{
	GENERATED_BODY()

public:
	UGroomFactory();

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ End UFactory Interface
};



