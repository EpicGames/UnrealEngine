// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"
#include "HAL/LowLevelMemTracker.h"
#include "RHIFwd.h"
#include "RHIShaderPlatform.h"

#define UE_API HAIRSTRANDSEDITOR_API

#define HAIRSTRANDSEDITOR_MODULE_NAME TEXT("HairStrandsEditor")

class IGroomTranslator;

LLM_DECLARE_TAG_API(GroomEditor, HAIRSTRANDSEDITOR_API);

/** Implements the HairStrands module  */
class FGroomEditor : public IModuleInterface
{
public:

	//~ IModuleInterface interface

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override { return false; }

	UE_API void OnPostEngineInit();
	UE_API void OnPreviewPlatformChanged(EShaderPlatform NewShaderPlatform);
	UE_API void OnPreExit();

	static inline FGroomEditor& Get()
	{
		LLM_SCOPE_BYTAG(GroomEditor);
		return FModuleManager::LoadModuleChecked<FGroomEditor>(HAIRSTRANDSEDITOR_MODULE_NAME);
	}

	/** Register HairStrandsTranslator to add support for import by the HairStandsFactory */
	template <typename TranslatorType>
	void RegisterHairTranslator()
	{
		TranslatorSpawners.Add([]
		{
			return MakeShared<TranslatorType>();
		});
	}

	/** Get new instances of HairStrandsTranslators */
	UE_API TArray<TSharedPtr<IGroomTranslator>> GetHairTranslators();

private:
	UE_API void RegisterMenus();

	TArray<TFunction<TSharedPtr<IGroomTranslator>()>> TranslatorSpawners;

	TSharedPtr<FSlateStyleSet> StyleSet;

	FDelegateHandle TrackEditorBindingHandle;
	FDelegateHandle PreviewPlatformChangedHandle;
	FDelegateHandle PreviewFeatureLevelChangedHandle;
	FDelegateHandle DataflowMenusHandle;

public:

	static UE_API FName GroomEditorAppIdentifier;
};

#undef UE_API
