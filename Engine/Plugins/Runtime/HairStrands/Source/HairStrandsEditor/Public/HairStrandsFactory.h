// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dialog/SMessageDialog.h"
#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "HairStrandsFactory.generated.h"

class IGroomTranslator;
class UGroomAsset;

namespace UE::Groom
{
	struct FGroomDataflowTemplateData
	{
		/** Dataflow template name */
		FString TemplateName;

		/** Dataflow template title */
		FString TemplateTitle;

		/** Dataflow template tooltip */
		FString TemplateTooltip;

		/** Dataflow template asset path */
		FString TemplatePath;

		/** Template primary flag */
		bool bIsPrimaryTemplate = false;
	};

	/** Build dataflow template buttons */
	HAIRSTRANDSEDITOR_API TArray<SMessageDialog::FButton> BuildGroomDataflowTemplateButtons(TFunction<void(FString)> OnButtonClicked);

	/** Register dataflow template to the manager */
	HAIRSTRANDSEDITOR_API void RegisterGroomDataflowTemplate(const FGroomDataflowTemplateData& DataflowTemplate);

	/** Unregister dataflow template to the manager */
	HAIRSTRANDSEDITOR_API void UnregisterGroomDataflowTemplate(const FString& TemplateName);

	/** Register a dataflow template path to the manager. this will be used to dynamically load template from it whenever the list of templates is requested */
	HAIRSTRANDSEDITOR_API void RegisterGroomDataflowTemplatePath(const FString& DataflowTemplatePath);

	/** Unregister a dataflow template path from the manager */
	HAIRSTRANDSEDITOR_API void UnregisterGroomDataflowTemplatePath(const FString& DataflowTemplatePath);

	/** Build groom dataflow asset from templates */
	HAIRSTRANDSEDITOR_API bool BuildGroomDataflowAsset(UGroomAsset* GroomAsset);
}

/**
 * Implements a factory for UHairStrands objects.
 */
UCLASS(hidecategories = Object)
class UHairStrandsFactory : public UFactory
{
	GENERATED_BODY()

public:

	UHairStrandsFactory(const FObjectInitializer& ObjectInitializer);

	//~ Begin UFactory interface
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
		const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual void GetSupportedFileExtensions(TArray<FString>& OutExtensions) const override;
	virtual void CleanUp() override;
	//~ End UFactory interface

protected:
	void InitTranslators();

	TSharedPtr<IGroomTranslator> GetTranslator(const FString& Filename);

	class UGroomImportOptions* ImportOptions;
	class UGroomCacheImportOptions* GroomCacheImportOptions;

private:
	TArray<TSharedPtr<IGroomTranslator>> Translators;
	bool bImportAll = false;
};

