// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#define UE_API HAIRSTRANDSEDITOR_API

class FHairDescription;
class UGroomAsset;
class UGroomImportOptions;
class UGroomHairGroupsMapping;

struct FHairImportContext
{
	UE_API FHairImportContext(UGroomImportOptions* InImportOptions, UObject* InParent = nullptr, UClass* InClass = nullptr, FName InName = FName(), EObjectFlags InFlags = EObjectFlags::RF_NoFlags);

	UGroomImportOptions* ImportOptions;
	UObject* Parent;
	UClass* Class;
	FName Name;
	EObjectFlags Flags;
};

struct FHairStrandsImporter
{
	static UE_API UGroomAsset* ImportHair(const FHairImportContext& ImportContext, FHairDescription& HairDescription, UGroomAsset* ExistingHair = nullptr, const UGroomHairGroupsMapping* GroupsMapping=nullptr);
};

#undef UE_API
