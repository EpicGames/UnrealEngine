// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairDescription.h"

class IGroomTranslator
{
public:
	virtual ~IGroomTranslator() = default;

	/** Translate a given file into a HairDescription; return true if successful */
	virtual bool Translate(const FString& FilePath, FHairDescription& OutHairDescription, const struct FGroomConversionSettings& ConversionSettings) = 0;

	/** Return true if a given file can be translated (by checking its content if necessary) */
	virtual bool CanTranslate(const FString& FilePath) = 0;

	/** Return true if a given file extension is supported by the translator */
	virtual bool IsFileExtensionSupported(const FString& FileExtension) const = 0;

	/** Return the file format supported by the translator in the form "ext;file format description" */
	virtual FString GetSupportedFormat() const = 0;

	/** Translate a given file into a HairDescription with info about Groom animation if OutAnimInfo is not null; return true if successful */
	virtual bool Translate(const FString& FilePath, FHairDescription& OutHairDescription, const struct FGroomConversionSettings& ConversionSettings, struct FGroomAnimationInfo* OutAnimInfo)
	{
		return Translate(FilePath, OutHairDescription, ConversionSettings);
	}

	/** Open a file for multiple translations at different frame indices */
	virtual bool BeginTranslation(const FString& FilePath) { return false; }

	/** Translate a given file into a HairDescription at the requested frame index; return true if successful */
	UE_DEPRECATED(5.0, "Translate by frame index is deprecated. Use Translate by frame time instead.")
	virtual bool Translate(uint32 FrameIndex, FHairDescription& OutHairDescription, const struct FGroomConversionSettings& ConversionSettings) { return false; }

	/** Translate a given file into a HairDescription at the requested frame time (in seconds); return true if successful */
	virtual bool Translate(float FrameTime, FHairDescription& OutHairDescription, const struct FGroomConversionSettings& ConversionSettings) { return false; }

	/** Clean up after finishing translations */
	virtual void EndTranslation() { }
};
