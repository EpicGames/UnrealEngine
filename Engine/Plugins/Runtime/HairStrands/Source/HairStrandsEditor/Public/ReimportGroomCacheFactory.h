// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "HairStrandsFactory.h"

#include "ReimportGroomCacheFactory.generated.h"

UCLASS(hidecategories = Object)
class UReimportGroomCacheFactory : public UHairStrandsFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface

	//~ FReimportHandler Interface
	virtual int32 GetPriority() const override;
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
};

