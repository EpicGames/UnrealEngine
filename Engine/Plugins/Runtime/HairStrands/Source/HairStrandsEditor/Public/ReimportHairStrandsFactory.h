// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairStrandsFactory.h"
#include "EditorReimportHandler.h"

#include "ReimportHairStrandsFactory.generated.h"

UCLASS(hidecategories = Object)
class UReimportHairStrandsFactory : public UHairStrandsFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface

	//~ FReimportHandler interface
	virtual int32 GetPriority() const override;
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
};

