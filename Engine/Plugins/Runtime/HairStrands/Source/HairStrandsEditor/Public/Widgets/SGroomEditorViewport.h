// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/GCObject.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PreviewScene.h"
#include "Framework/Commands/UICommandList.h"
#include "EditorViewportClient.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "HairStrandsInterface.h"
#include "IPreviewLODController.h"


class UGroomComponent;
class SDockTab;

/**
 * Material Editor Preview viewport widget
 */
class SGroomEditorViewport : public SEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider, public IPreviewLODController
{
public:

	void Construct(const FArguments& InArgs);
	
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SGroomEditorViewport");
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
										
	/** Event handlers */
	void TogglePreviewGrid();
	bool IsTogglePreviewGridChecked() const;
	
	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	// Set the component to preview
	void SetGroomComponent(UGroomComponent* GroomComponent);

	// set the mesh on which we are grooming
	void SetStaticMeshComponent(UStaticMeshComponent *StaticGroomTarget);

	// set the mesh on which we are grooming
	void SetSkeletalMeshComponent(USkeletalMeshComponent *SkeletalGroomTarget);
	
	// IPreviewLODController interface
	virtual int32 GetCurrentLOD() const override;
	virtual int32 GetLODCount() const override;
	virtual bool IsLODSelected(int32 LODIndex) const override;
	virtual void SetLODLevel(int32 LODIndex) override;
	virtual void FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands) override;
	virtual int32 GetAutoLODStartingIndex() const override { return 1; }
	// ~IPreviewLODController interface

	TSharedPtr<class FAdvancedPreviewScene> GetAdvancedPreviewScene() { return AdvancedPreviewScene; }

protected:

	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;

private:

	bool IsVisible() const override;

	void RefreshViewport();

private:
	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

	/** Level viewport client */
	TSharedPtr<class FGroomEditorViewportClient> SystemViewportClient;

	/** Preview Scene - uses advanced preview settings */
	TSharedPtr<class FAdvancedPreviewScene> AdvancedPreviewScene;

	TObjectPtr<class UGroomComponent> GroomComponent;
	TObjectPtr<class UStaticMeshComponent> StaticGroomTarget;
	class USkeletalMeshComponent	*SkeletalGroomTarget;

	/** If true, render grid the preview scene. */
	bool bShowGrid;
	
};
