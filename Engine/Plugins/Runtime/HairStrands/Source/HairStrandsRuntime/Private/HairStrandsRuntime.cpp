// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheStreamingManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/*
 * The HairStrandsRuntime module exists solely because it needs to be initialized after the RHI
 * but the HairStrandsCore module has to be at PostConfigInit (before RHI is initialized)
 * because it contains shaders that need that loading phase.
 */
class FHairStrandsRuntime : public IModuleInterface
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override
	{
		// The GroomCache streaming manager has a RHI dependency because of 
		// FStreamingManagerCollection::AddOrRemoveTextureStreamingManagerIfNeeded
		// which is called when FStreamingManagerCollection is created via IStreamingManager::Get()
		IGroomCacheStreamingManager::Register();
	}

	virtual void ShutdownModule() override
	{
		IGroomCacheStreamingManager::Unregister();
	}
};

IMPLEMENT_MODULE(FHairStrandsRuntime, HairStrandsRuntime);
