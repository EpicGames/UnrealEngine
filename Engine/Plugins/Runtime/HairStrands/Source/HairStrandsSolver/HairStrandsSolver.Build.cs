// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairStrandsSolver : ModuleRules
	{
		public HairStrandsSolver(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"HairStrandsCore",
					"DataflowCore",
					"DataflowEngine",
					"DataflowSimulation",
					"Engine",
					"RHI",
					"RenderCore",
					"OptimusCore",
					"ComputeFramework", 
					"ChaosCaching"
				}
			);
		}
	}
}