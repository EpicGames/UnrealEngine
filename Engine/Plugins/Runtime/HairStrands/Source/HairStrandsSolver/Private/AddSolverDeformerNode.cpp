// Copyright Epic Games, Inc. All Rights Reserved.

#include "AddSolverDeformerNode.h"
#include "GroomSolverComponent.h"
#include "Dataflow/DataflowConnection.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusVariableDescription.h"
#include "OptimusDeformer.h"
#include "OptimusDeformerInstance.h"
#include "OptimusDeformerDynamicInstanceManager.h"
#include "Dataflow/DataflowSimulationManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AddSolverDeformerNode)

namespace UE::Groom::Private
{
	template<typename DataflowType>
	FORCEINLINE UE::Dataflow::TConnectionReference<DataflowType> GetConnectionReference(const TArray<DataflowType>& DataflowInputs, int32 InputIndex)
	{
		return { &DataflowInputs[InputIndex], InputIndex, &DataflowInputs };
	}

	template<typename DataflowType>
	FORCEINLINE void RegisterArrayConnection(const TArray<DataflowType>& DataflowInputs, FDataflowNode& DataflowNode)
	{
		if(!DataflowInputs.IsEmpty())
		{
			for(int32 InputIndex = 0; InputIndex < DataflowInputs.Num(); ++InputIndex)
			{
				DataflowNode.FindOrRegisterInputArrayConnection(GetConnectionReference(DataflowInputs, InputIndex));
			}
		}
	}

	template<typename DataflowType, typename PolicyType = typename DataflowType::FPolicyType>
	FORCEINLINE void UnregisterArrayConnection(TArray<DataflowType>& DataflowInputs, FDataflowNode& DataflowNode)
	{
		TArray<FDataflowInput*> NodeInputs = DataflowNode.GetInputs();
		int32 NumRegisteredInputs = 0;
		for (FDataflowInput* NodeInput: NodeInputs)
		{
			if(PolicyType::SupportsTypeStatic(NodeInput->GetType()))
			{
				NumRegisteredInputs++;
			}
		}
		const int32 NumDataflowInputs = DataflowInputs.Num();
		if(NumRegisteredInputs > NumDataflowInputs)
		{
			DataflowInputs.SetNum(NumRegisteredInputs);
			for (int32 InputIndex = NumDataflowInputs; InputIndex < NumRegisteredInputs; ++InputIndex)
			{
				DataflowNode.UnregisterInputConnection(GetConnectionReference(DataflowInputs, InputIndex));
			}
			DataflowInputs.SetNum(NumDataflowInputs);
		}
	}
	
	template<typename DataflowType>
	FORCEINLINE void RemoveOptionPin(TArray<DataflowType>& DataflowInputs, const FDataflowNode& DataflowNode, const UE::Dataflow::FPin& Pin)
	{
		if(!DataflowInputs.IsEmpty() && (Pin.Direction == UE::Dataflow::FPin::EDirection::INPUT))
		{
			for(int32 InputIndex = 0; InputIndex < DataflowInputs.Num(); ++InputIndex)
			{
				if (const FDataflowInput* const DeformerInput = DataflowNode.FindInput(GetConnectionReference(DataflowInputs, InputIndex)))
				{
					if(Pin.Type == DeformerInput->GetType() && Pin.Name == DeformerInput->GetName() )
					{
						DataflowInputs.RemoveAt(InputIndex);
						break;
					}		
				}
			}
		}
	}

	template<typename DataflowType>
	FORCEINLINE void GatherOptionPins(const TArray<DataflowType>& DataflowInputs, const FDataflowNode& DataflowNode, TArray<UE::Dataflow::FPin>& Pins)
	{
		for(int32 InputIndex = DataflowInputs.Num()-1; InputIndex >= 0; --InputIndex)
		{
			if (const FDataflowInput* const DeformerInput = DataflowNode.FindInput(GetConnectionReference(DataflowInputs, InputIndex)))
			{
				Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, DeformerInput->GetType(), DeformerInput->GetName() });		
			}
		}
	}

	template<typename PinType, typename DataflowType>
	FORCEINLINE void AddOptionPin(TArray<DataflowType>& DataflowInputs, FDataflowNode& DataflowNode, const FName& PinName, TArray<UE::Dataflow::FPin>& Pins)
	{
		const uint32 InputIndex = DataflowInputs.AddDefaulted();
		FDataflowInput& DeformerInput = DataflowNode.RegisterInputArrayConnection(GetConnectionReference(DataflowInputs, InputIndex) );

		DeformerInput.SetName(PinName);
	
		DataflowNode.SetInputConcreteType(GetConnectionReference(DataflowInputs, InputIndex), FName(TDataflowPolicyTypeName<PinType>::GetName()));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, DeformerInput.GetType(), DeformerInput.GetName() });	
	}

	template<typename PinType, typename DataflowType, typename StorageType = typename DataflowType::FStorageType>
	FORCEINLINE PinType GetPinValue(const TArray<DataflowType>& DataflowInputs, const FDataflowNode& DataflowNode, UE::Dataflow::FDataflowSimulationContext& SimulationContext, const int32& InputIndex)
	{
		PinType  ResultValue = PinType();
		const UE::Dataflow::TConnectionReference<DataflowType> InputReference = GetConnectionReference(DataflowInputs, InputIndex);
		if (DataflowNode.IsConnected(InputReference))
		{
			const StorageType PinValue = DataflowNode.GetValue<DataflowType>(SimulationContext, InputReference);
			FDataflowConverter<StorageType>::To(PinValue, ResultValue);
		}
		return ResultValue;
	}

	FORCEINLINE void CreateDeformerInstance(UOptimusDeformer* DeformerGraph, UMeshDeformerInstance* DeformerInstance, const FGuid InstanceGuid, UDataflow* DataflowObject)
	{
		if(DeformerGraph && DeformerInstance && DeformerInstance->GetOuter())
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([DeformerGraph, DeformerInstance, InstanceGuid, DataflowObject]()
			{
				if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(DeformerInstance->GetOuter()))
				{
					if (!MeshComponent->IsBeingDestroyed() && MeshComponent->GetScene())
					{
						if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
						{
							if (UOptimusDeformerDynamicInstanceManager* DeformerInstanceManager = Cast<UOptimusDeformerDynamicInstanceManager>(DeformerInstance))
							{
								if (!DeformerInstanceManager->GetDeformerInstance(InstanceGuid))
								{
									DeformerInstanceManager->AddProducerDeformer(DataflowObject, InstanceGuid, DeformerGraph);
								}
							}
						}
					}
				}
			}, TStatId(), NULL, ENamedThreads::GameThread);
		}
	}
}

void FAddSolverDeformerDataflowNode::EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const
{
	const TArray<FDataflowSimulationProperty> SolverProperties = GetValue(SimulationContext, &PhysicsSolvers);
	const float SimulationDeltaTime = GetValue(SimulationContext, &SimulationTime).DeltaTime;
	
	UOptimusDeformer* OptimusDeformer = Cast<UOptimusDeformer>(MeshDeformer);
	if(!SolverProperties.IsEmpty() && OptimusDeformer)
	{
		for(const FDataflowSimulationProperty& SolverProperty : SolverProperties)
		{
			if(SolverProperty.SimulationProxy)
			{
				if(FDataflowGroomSolverProxy* GroomProxy = SolverProperty.SimulationProxy->AsType<FDataflowGroomSolverProxy>())
				{
					FGuid& DeformerInstanceGuid = GroomProxy->DeformerInstanceGuids.FindOrAdd(GetGuid());
					if (!DeformerInstanceGuid.IsValid())
					{
						// Build the deformer instance given that GUID
						if(UDataflow* DataflowObject = Cast<UDataflow>(SimulationContext.Owner))
						{
							if(GroomProxy->DeformerInstance)
							{
								// Create a new GUID for the new deformer instance
								DeformerInstanceGuid = FGuid::NewGuid();

								// Build a deformer instance given a guid
								UE::Groom::Private::CreateDeformerInstance(OptimusDeformer, GroomProxy->DeformerInstance, DeformerInstanceGuid, DataflowObject);
							}
						}
					}
					else if (UOptimusDeformerDynamicInstanceManager* DeformerInstanceManager = Cast<UOptimusDeformerDynamicInstanceManager>(GroomProxy->DeformerInstance))
					{
						if (UOptimusDeformerInstance* DeformerInstance = DeformerInstanceManager->GetDeformerInstance(DeformerInstanceGuid))
						{
							// Enqueue the execution of the deformer instance
							DeformerInstanceManager->EnqueueProducerDeformer(DeformerInstanceGuid, EOptimusDeformerExecutionPhase::OverrideDefaultDeformer, 1);
							
							// Set the value of the deformer variables
							for(int32 InputIndex = 0; InputIndex < DeformerNumericInputs.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerNumericInputs, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<int32>::GetName())
									{
										DeformerInstance->SetIntVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<int32>(
											DeformerNumericInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<uint32>::GetName())
									{
										DeformerInstance->SetIntVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<uint32>(
											DeformerNumericInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<double>::GetName())
									{
										DeformerInstance->SetFloatVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<double>(
											DeformerNumericInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<float>::GetName())
									{
										DeformerInstance->SetFloatVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<float>(
											DeformerNumericInputs, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerVectorInputs.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerVectorInputs, InputIndex)))
								{
									 if(DeformerInput->GetType() == TDataflowPolicyTypeName<FVector2D>::GetName())
									{
										DeformerInstance->SetVector2Variable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FVector2D>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FVector>::GetName())
									{
										DeformerInstance->SetVectorVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FVector>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FVector4>::GetName())
									{
										DeformerInstance->SetVector4Variable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FVector4>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FQuat>::GetName())
									{
										DeformerInstance->SetQuatVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FQuat>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FLinearColor>::GetName())
									{
										DeformerInstance->SetLinearColorVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FLinearColor>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FIntPoint>::GetName())
									{
										DeformerInstance->SetInt2Variable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FIntPoint>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FIntVector3>::GetName())
									{
										DeformerInstance->SetInt3Variable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FIntVector3>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FIntVector4>::GetName())
									{
										DeformerInstance->SetInt4Variable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FIntVector4>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<FRotator>::GetName())
									{
										DeformerInstance->SetRotatorVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FRotator>(
											DeformerVectorInputs, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerStringInputs.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerStringInputs, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<FName>::GetName())
									{
										DeformerInstance->SetNameVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FName>(
											DeformerStringInputs, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerBoolInputs.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerBoolInputs, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<bool>::GetName())
									{
										DeformerInstance->SetBoolVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<bool>(
											DeformerBoolInputs, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerTransformInputs.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerTransformInputs, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<FTransform>::GetName())
									{
										DeformerInstance->SetTransformVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<FTransform>(
											DeformerTransformInputs, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerNumericArrays.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerNumericArrays, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<int32>>::GetName())
									{
										DeformerInstance->SetIntArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<int32>>(
											DeformerNumericArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<uint32>>::GetName())
									{
										DeformerInstance->SetIntArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<int32>>(
											DeformerNumericArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<double>>::GetName())
									{
										DeformerInstance->SetFloatArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<double>>(
											DeformerNumericArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<float>>::GetName())
									{
										DeformerInstance->SetFloatArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<double>>(
											DeformerNumericArrays, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerVectorArrays.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerVectorArrays, InputIndex)))
								{
									 if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FVector2D>>::GetName())
									{
										DeformerInstance->SetVector2ArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FVector2D>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FVector>>::GetName())
									{
										DeformerInstance->SetVectorArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FVector>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FVector4>>::GetName())
									{
										DeformerInstance->SetVector4ArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FVector4>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FQuat>>::GetName())
									{
										DeformerInstance->SetQuatArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FQuat>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FLinearColor>>::GetName())
									{
										DeformerInstance->SetLinearColorArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FLinearColor>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FIntPoint>>::GetName())
									{
										DeformerInstance->SetInt2ArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FIntPoint>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FIntVector3>>::GetName())
									{
										DeformerInstance->SetInt3ArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FIntVector3>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FIntVector4>>::GetName())
									{
										DeformerInstance->SetInt4ArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FIntVector4>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
									else if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FRotator>>::GetName())
									{
										DeformerInstance->SetRotatorArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FRotator>>(
											DeformerVectorArrays, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerBoolArrays.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerBoolArrays, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FTransform>>::GetName())
									{
										DeformerInstance->SetBoolArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<bool>>(
											DeformerBoolArrays, *this, SimulationContext, InputIndex));
									}
								}
							}
							for(int32 InputIndex = 0; InputIndex < DeformerTransformArrays.Num(); ++InputIndex)
							{
								if (const FDataflowInput* const DeformerInput = FindInput(
									UE::Groom::Private::GetConnectionReference(DeformerTransformArrays, InputIndex)))
								{
									if(DeformerInput->GetType() == TDataflowPolicyTypeName<TArray<FTransform>>::GetName())
									{
										DeformerInstance->SetTransformArrayVariable(DeformerInput->GetName(), UE::Groom::Private::GetPinValue<TArray<FTransform>>(
											DeformerTransformArrays, *this, SimulationContext, InputIndex));
									}
								}
							}
						}
					}
				}
			}
		}
	}
	SetValue(SimulationContext, SolverProperties, &PhysicsSolvers);
}

void FAddSolverDeformerDataflowNode::OnInvalidate()
{}

TArray<UE::Dataflow::FPin> FAddSolverDeformerDataflowNode::AddPins()
{
	if(UOptimusDeformer* OptimusDeformer = Cast<UOptimusDeformer>(MeshDeformer))
	{
		TArray<UE::Dataflow::FPin> Pins;
		for (UOptimusVariableDescription* Variable : OptimusDeformer->GetVariables())
		{
			if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FIntProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<int32>(DeformerNumericInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FUInt32Property::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<uint32>(DeformerNumericInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FDoubleProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<double>(DeformerNumericInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<float>(DeformerNumericInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FVector2D>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FVector2D>(DeformerVectorInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FVector>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FVector>(DeformerVectorInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FVector4>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FVector4>(DeformerVectorInputs,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FQuat>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FQuat>(DeformerVectorInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FLinearColor>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FLinearColor>(DeformerVectorInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntPoint>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FIntPoint>(DeformerVectorInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntVector3>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FIntVector3>(DeformerVectorInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntVector4>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FIntVector4>(DeformerVectorInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FRotator>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FRotator>(DeformerVectorInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<FName>(DeformerStringInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FBoolProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<bool>(DeformerBoolInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FTransform>::Get()))
			{
				UE::Groom::Private::AddOptionPin<FTransform>(DeformerTransformInputs, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FIntProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<TArray<int32>>(DeformerNumericArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FUInt32Property::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<TArray<uint32>>(DeformerNumericArrays,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FDoubleProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<TArray<double>>(DeformerNumericArrays,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FFloatProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<TArray<float>>(DeformerNumericArrays,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FVector2D>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FVector2D>>(DeformerVectorArrays,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FVector>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FVector>>(DeformerVectorArrays,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FVector4>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FVector4>>(DeformerVectorArrays,  *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FQuat>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FQuat>>(DeformerVectorArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FLinearColor>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FLinearColor>>(DeformerVectorArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FIntPoint>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FIntPoint>>(DeformerVectorArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FIntVector3>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FIntVector3>>(DeformerVectorArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FIntVector4>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FIntVector4>>(DeformerVectorArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FRotator>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FRotator>>(DeformerVectorArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FNameProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FName>>(DeformerStringArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FBoolProperty::StaticClass()))
			{
				UE::Groom::Private::AddOptionPin<TArray<bool>>(DeformerBoolArrays, *this, Variable->VariableName, Pins);
			}
			else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FTransform>::Get()))
			{
				UE::Groom::Private::AddOptionPin<TArray<FTransform>>(DeformerTransformArrays, *this, Variable->VariableName, Pins);
			}
			else
			{
				UE_LOGF(LogTemp, Error, "Unsupported Dataflow variable type");
			}			
		}
		return Pins;
	}

	return Super::AddPins();
}

TArray<UE::Dataflow::FPin> FAddSolverDeformerDataflowNode::GetPinsToRemove() const
{
	if(!DeformerNumericInputs.IsEmpty() || !DeformerStringInputs.IsEmpty() || !DeformerVectorInputs.IsEmpty() || !DeformerBoolInputs.IsEmpty() || !DeformerTransformInputs.IsEmpty())
	{
		TArray<UE::Dataflow::FPin> Pins;
		UE::Groom::Private::GatherOptionPins(DeformerNumericInputs, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerVectorInputs,  *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerStringInputs, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerBoolInputs, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerTransformInputs, *this, Pins);

		UE::Groom::Private::GatherOptionPins(DeformerNumericArrays, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerVectorArrays, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerStringArrays, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerBoolArrays, *this, Pins);
		UE::Groom::Private::GatherOptionPins(DeformerTransformArrays, *this, Pins);
		return Pins;
	}
	return Super::GetPinsToRemove();
}

void FAddSolverDeformerDataflowNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	if(!DeformerNumericInputs.IsEmpty() || !DeformerStringInputs.IsEmpty() || !DeformerVectorInputs.IsEmpty() || !DeformerBoolInputs.IsEmpty() || !DeformerTransformInputs.IsEmpty())
	{
		UE::Groom::Private::RemoveOptionPin(DeformerNumericInputs,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerVectorInputs,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerStringInputs,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerBoolInputs,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerTransformInputs,  *this, Pin);

		UE::Groom::Private::RemoveOptionPin(DeformerNumericArrays,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerVectorArrays,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerStringArrays,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerBoolArrays,  *this, Pin);
		UE::Groom::Private::RemoveOptionPin(DeformerTransformArrays,  *this, Pin);
	}
	return Super::OnPinRemoved(Pin);
}

void FAddSolverDeformerDataflowNode::PostSerialize(const FArchive& Ar)
{
	// because we add pins we need to make sure we restore them when loading
	// to make sure they can get properly reconnected

	if (Ar.IsLoading())
	{
		UE::Groom::Private::RegisterArrayConnection(DeformerNumericInputs, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerVectorInputs, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerStringInputs, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerBoolInputs, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerTransformInputs, *this);
		
		UE::Groom::Private::RegisterArrayConnection(DeformerNumericArrays, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerVectorArrays, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerStringArrays, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerBoolArrays, *this);
		UE::Groom::Private::RegisterArrayConnection(DeformerTransformArrays, *this);
		
		if (Ar.IsTransacting())
		{
			UE::Groom::Private::UnregisterArrayConnection(DeformerNumericInputs, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerVectorInputs, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerStringInputs, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerBoolInputs, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerTransformInputs, *this);
			
			UE::Groom::Private::UnregisterArrayConnection(DeformerNumericArrays, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerVectorArrays, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerStringArrays, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerBoolArrays, *this);
			UE::Groom::Private::UnregisterArrayConnection(DeformerTransformArrays, *this);
		}
		else
		{
			ensureAlways((DeformerNumericInputs.Num() + DeformerVectorInputs.Num() + DeformerStringInputs.Num() +
			DeformerBoolInputs.Num() + DeformerTransformInputs.Num() + DeformerNumericArrays.Num() + DeformerVectorArrays.Num() + DeformerStringArrays.Num() +
			DeformerBoolArrays.Num() + DeformerTransformArrays.Num() + 2)== GetNumInputs());
		}
	}
}