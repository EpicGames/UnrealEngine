// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCacheAdapter.h"
#include "GroomBuilder.h"
#include "GroomCache.h"
#include "Chaos/ChaosCache.h"
#include "Chaos/CacheManagerActor.h"
#include "GroomComponent.h"
#include "GroomSolverComponent.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "HairStrandsCore.h"

DEFINE_LOG_CATEGORY(LogGroomCache)

namespace UE::Groom
{
FGroomCacheAdapter::~FGroomCacheAdapter()
{}

Chaos::FComponentCacheAdapter::SupportType FGroomCacheAdapter::SupportsComponentClass(UClass* InComponentClass) const
{
	const UClass* Desired = GetDesiredClass();
	if(InComponentClass == Desired)
	{
		return Chaos::FComponentCacheAdapter::SupportType::Direct;
	}
	else if(InComponentClass->IsChildOf(Desired))
	{
		return Chaos::FComponentCacheAdapter::SupportType::Derived;
	}

	return Chaos::FComponentCacheAdapter::SupportType::None;
}

UClass* FGroomCacheAdapter::GetDesiredClass() const
{
	return UGroomComponent::StaticClass();
}

uint8 FGroomCacheAdapter::GetPriority() const
{
	return EngineAdapterPriorityBegin;
}

FGuid FGroomCacheAdapter::GetGuid() const
{
	FGuid NewGuid;
	checkSlow(FGuid::Parse(TEXT("FC61D2A13092410CBCF2F767C8490986"), NewGuid));
	return NewGuid;
}

Chaos::FPhysicsSolver* FGroomCacheAdapter::GetComponentSolver(UPrimitiveComponent* InComponent) const
{
	return nullptr;
}

Chaos::FPhysicsSolverEvents* FGroomCacheAdapter::BuildEventsSolver(UPrimitiveComponent* InComponent) const
{
	if(UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(InComponent))
	{
		if(GroomComponent->GetGroomSolver())
		{
			if(UGroomSolverComponent* GroomSolver = Cast<UGroomSolverComponent>(GroomComponent->GetGroomSolver()))
			{
				return static_cast<FDataflowGroomSolverProxy*>(GroomSolver->GetSimulationProxy());
			}
		}
	}
	return nullptr;
}

bool FGroomCacheAdapter::ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const
{
	const UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(InComponent);
	return GroomComponent != nullptr;
}

void FGroomCacheAdapter::InitializeForLoad(UPrimitiveComponent* InComponent, FObservedComponent& InObserved)
{
	bIsLoading = true;
	if(UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(InComponent))
	{
		if(UChaosCache* ChaosCache = InObserved.GetChaosCache())
		{
			if(ChaosCache->GetCacheData() && !GroomComponent->GetGroomCache())
			{
				GroomComponent->SetGroomCache(Cast<UGroomCache>(ChaosCache->GetCacheData()));
				GroomComponent->SetManualTick(true);

				if(GroomComponent->GroomAsset && !GroomComponent->GroomAsset->GetEnableSimulationCache())
				{
					GroomComponent->GroomAsset->ValidateSimulationCache();
				}
			}
		}
	}
}

void FGroomCacheAdapter::SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const
{
	if(bIsLoading)
	{
		if(UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(InComponent))
		{
			if(UGroomCache* GroomCache = GroomComponent->GetGroomCache())
			{
				if(InTime < GroomCache->GetDuration())
				{
					// Undo the dataflow editor's +DeltaTime on the groom path: groom cache lookup is
					// floor(T / SecondsPerFrame) and the +DeltaTime over-compensates here.
					const float CacheDt = GroomCache->GetGroomAnimationInfo().SecondsPerFrame;
					const float CorrectedTime = FMath::Max(0.0f, float(InTime) - CacheDt);
					GroomComponent->TickAtThisTime(CorrectedTime, /*bInIsRunning=*/true, /*bInBackwards=*/false, /*bInIsLooping=*/true);
				}
			}
		}
	}
}

bool FGroomCacheAdapter::InitializeForRecord(UPrimitiveComponent* InComponent, FObservedComponent& InObserved)
{
	bIsLoading = false;
	if(UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(InComponent))
	{
		FGroomCacheData& GroomCache = GroomCaches.FindOrAdd(InComponent);
		
		GroomCache.AnimInfo.Attributes = EGroomCacheAttributes::Position;
		GroomCache.AnimInfo.StartFrame = 0;
		GroomCache.AnimInfo.StartTime = 0.0f;
		GroomCache.AnimInfo.EndFrame = 0;
		GroomCache.AnimInfo.EndTime = 0.0f;

		GroomCache.PositionsBuffer.Reset();
		GroomCache.CacheTimes.Reset();
		GroomCache.CacheName = InObserved.CacheName.ToString();
		GroomCache.CacheProcessor = FGroomCacheProcessor(EGroomCacheType::Guides, GroomCache.AnimInfo.Attributes);

		GroomComponent->SetGroomCache(nullptr);
		GroomComponent->SetManualTick(false);

		// SetGroomCache(nullptr) rebuilds the component's instances; force a fresh proxy
		// synchronously so the first Record_PostSolve doesn't read the stale one and crash.
		GroomComponent->RecreateRenderState_Concurrent();

		if(UChaosCache* ChaosCache = InObserved.GetChaosCache())
		{
#if WITH_EDITOR
			if(!ChaosCache->GetCacheData())
			{
				FString PackageNameCache, NameCache;
                FHairStrandsCore::AssetHelper().CreateFilename(GroomComponent->GroomAsset->GetOutermost()->GetName(), TEXT("_GroomCache"), PackageNameCache, NameCache);
                UPackage* PackageCache = CreatePackage(*PackageNameCache);

				UGroomCache* CacheData = NewObject<UGroomCache>(PackageCache, *NameCache, RF_Public | RF_Standalone | RF_Transactional);
				FHairStrandsCore::AssetHelper().RegisterAsset(CacheData);
				
				ChaosCache->SetCacheData(CacheData);
			}
			GroomCache.CacheAsset = Cast<UGroomCache>(ChaosCache->GetCacheData());
#endif
		}
	}
	return true;
}

void FGroomCacheAdapter::Record_PostSolve(UPrimitiveComponent* InComponent, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const
{
	if(!bIsLoading)
	{
		if(const UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(InComponent))
		{
			if(FGroomCacheData* GroomCache = GroomCaches.Find(InComponent))
			{
				TSharedPtr<FStrandsPositionOutput> PositionOutput = MakeShared<FStrandsPositionOutput>();
				GroomCache->PositionsBuffer.Add(PositionOutput);
				GroomCache->CacheTimes.Add(InTime);

#if WITH_EDITOR
				ReadbackTasks.Add(FFunctionGraphTask::CreateAndDispatchWhenReady([GroomComponent, PositionOutput](
					ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
					{
						RequestStrandsPosition(GroomComponent, PositionOutput, true);
					}, TStatId(), nullptr, ENamedThreads::Type::RHIThread));
#endif
				GroomCache->AnimInfo.EndFrame++;
				GroomCache->AnimInfo.EndTime = InTime;
			}
		}
	}
}

bool FGroomCacheAdapter::InitializeForPlayback(UPrimitiveComponent* InComponent, FObservedComponent& InObserved, float InTime)
{
	bIsLoading = false;
	return false;
}

FORCEINLINE void FillGuidesPositions(const FStrandsPositionOutput& PositionsBuffer, TArray<FGroomCacheInputData>& GroupsData)
{
	if(PositionsBuffer.IsValid() && PositionsBuffer.Groups.Num() == GroupsData.Num())
	{
		for(uint32 GroupIndex = 0, NumGroups = GroupsData.Num(); GroupIndex < NumGroups; ++GroupIndex)
		{
			const FStrandsPositionOutput::FGroup& PositionsGroup = PositionsBuffer.Groups[GroupIndex];
			TArray<FVector3f>& GuidesPositions = GroupsData[GroupIndex].Guides.StrandsPoints.PointsPosition;

			const uint32 NumPoints = GuidesPositions.Num();
			uint32 PointIndex = 0;
			for(uint32 GuideIndex = 0, NumGuides = PositionsGroup.Num(); GuideIndex < NumGuides; ++GuideIndex)
			{
				// BuildGroupsData() builds destination with bAllowAddEndControlPoint=false; write all
				// source points (the original code's -1 was dropping the tip of each guide).
				for(int32 VertexIndex = 0, NumVertices = PositionsGroup[GuideIndex].Num();
						  VertexIndex < NumVertices && PointIndex < NumPoints; ++VertexIndex, ++PointIndex)
				{
					GuidesPositions[PointIndex] = PositionsGroup[GuideIndex][VertexIndex];
				}
			}
		}
	}
}

void FGroomCacheAdapter::WaitForSolverTasks(UPrimitiveComponent* InComponent) const
{
	if (ReadbackTasks.Num())
	{
		FTaskGraphInterface::Get().WaitUntilTasksComplete(ReadbackTasks, ENamedThreads::GameThread);
	}
}

#if WITH_EDITORONLY_DATA

void FGroomCacheAdapter::FillCacheProcessor(const UGroomComponent* GroomComponent, FGroomCacheData& GroomCache, const int32 NumFrames, float& MaxTime, int32& MaxFrame)
{
	if(UGroomAsset* GroomAsset = GroomComponent->GroomAsset)
	{
		TArray<FGroomCacheInputData> GroupsData;
		if(UE::Groom::BuildGroupsData(GroomAsset->GetHairDescription(), GroomAsset->GetHairGroupsPlatformData(), GroomAsset->GetHairGroupsInfo(), GroomAsset->GetHairGroupsInterpolation(), GroupsData))
		{
			if(GroomCache.PositionsBuffer.Num() == NumFrames)
			{
				for(int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
				{
					if(GroomCache.PositionsBuffer[FrameIndex]->IsValid())
					{
						MaxTime = FMath::Max(MaxTime, GroomCache.CacheTimes[FrameIndex]);
						MaxFrame = FMath::Max(MaxFrame, FrameIndex+1);

						TArray<FGroomCacheInputData> LocalData = GroupsData;
						FillGuidesPositions(*GroomCache.PositionsBuffer[FrameIndex], LocalData);
						GroomCache.CacheProcessor.AddGroomSample(MoveTemp(LocalData));
					}
				}
			}
		}
	}
}

#endif

void FGroomCacheAdapter::Finalize()
{
	if(!bIsLoading)
	{
		for(TPair<UPrimitiveComponent*,FGroomCacheData>& GroomCache : GroomCaches)
		{
			if(UGroomComponent* GroomComponent = CastChecked<UGroomComponent>(GroomCache.Key))
			{
#if WITH_EDITORONLY_DATA
				if(!GroomCache.Value.PositionsBuffer.IsEmpty())
				{
					float MaxTime = -TNumericLimits<float>::Max();
					int32 MaxFrame = -TNumericLimits<int32>::Max();
					FillCacheProcessor(GroomComponent, GroomCache.Value,
						GroomCache.Value.AnimInfo.EndFrame - GroomCache.Value.AnimInfo.StartFrame, MaxTime, MaxFrame);

					GroomCache.Value.AnimInfo.EndTime = MaxTime;
					GroomCache.Value.AnimInfo.EndFrame = MaxFrame;

					GroomCache.Value.AnimInfo.NumFrames = GroomCache.Value.AnimInfo.EndFrame - GroomCache.Value.AnimInfo.StartFrame;
					GroomCache.Value.AnimInfo.Duration = GroomCache.Value.AnimInfo.EndTime - GroomCache.Value.AnimInfo.StartTime;
					GroomCache.Value.AnimInfo.SecondsPerFrame = GroomCache.Value.AnimInfo.NumFrames > 0
						? GroomCache.Value.AnimInfo.Duration / GroomCache.Value.AnimInfo.NumFrames
						: 0.0f;

					GroomCache.Value.PositionsBuffer.Reset();
					GroomCache.Value.CacheTimes.Reset();
					if(GroomCache.Value.CacheAsset)
					{
						GroomCache.Value.CacheAsset->Initialize(EGroomCacheType::Guides);
						UE::Groom::BuildGroomCache(GroomCache.Value.CacheProcessor, GroomCache.Value.AnimInfo, GroomCache.Value.CacheAsset);

						if(GroomCache.Value.CacheAsset->MarkPackageDirty())
						{
							FHairStrandsCore::SaveAsset(GroomCache.Value.CacheAsset);
						}
					}
				}
#endif
				GroomComponent->SetGroomCache(nullptr);
				GroomComponent->SetManualTick(false);
			}
		}
	}
	GroomCaches.Reset();
	ReadbackTasks.Reset();
}
}    // namespace UE::Groom
