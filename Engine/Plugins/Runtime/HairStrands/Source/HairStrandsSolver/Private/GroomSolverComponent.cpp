// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomSolverComponent.h"
#include "GroomComponent.h"
#include "Animation/MeshDeformer.h"
#include "Animation/MeshDeformerInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroomSolverComponent)

namespace UE::Groom::Private
{
	static void GatherViewLocations(const UWorld* LocalWorld, TArray<FVector>& ViewLocations)
	{
		if(LocalWorld)
		{
			if(LocalWorld->GetPlayerControllerIterator())
			{
				for (FConstPlayerControllerIterator Iterator = LocalWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
				{
					APlayerController* PlayerController = Iterator->Get();
					if (PlayerController && PlayerController->IsLocalPlayerController())
					{
						FVector PlayerLocation; FRotator CameraRotation;
						PlayerController->GetPlayerViewPoint(PlayerLocation, CameraRotation);
								
						ViewLocations.Add(PlayerLocation);
					}
				}
			}
			else
			{
				ViewLocations = LocalWorld->ViewLocationsRenderedLastFrame;
			}
		}
	}

	static FBox GetGroomBounds(const UGroomComponent* GroomComponent, const int32 GroupIndex)
	{
		FBox LocalBounds(EForceInit::ForceInitToZero);
		if(GroomComponent && GroomComponent->GroomAsset)
		{
			const FTransform LocalToWorld = GroomComponent->GetComponentTransform();
			const FHairGroupPlatformData& GroupData = GroomComponent->GroomAsset->GetHairGroupsPlatformData()[GroupIndex];
			if (GroupData.Guides.HasValidData())
			{
				// The hair width is not scaled by the transform scale. Since we increase the local bound by the hair width, 
				// and that amount is then scaled by the local to world matrix which conatins a scale factor, we apply a 
				// inverse scale to the radius computation
				const float InvScale = 1.f / LocalToWorld.GetMaximumAxisScale();

				// Take into account the strands size and add it as a bound 'border'
				const FHairGroupDesc Desc = GroomComponent->GroomGroupsDesc[GroupIndex];
				const FVector HairBorder(0.5f * Desc.HairWidth * FMath::Max3(1.0f, Desc.HairRootScale, Desc.HairTipScale) * InvScale);

				LocalBounds += GroupData.Guides.GetBounds();

				LocalBounds.Min -= HairBorder;
				LocalBounds.Max += HairBorder;
				
				return LocalBounds.TransformBy(LocalToWorld);
			}
		}
		return LocalBounds;
	}
}

UGroomSolverComponent::UGroomSolverComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;
	bAutoActivate = true;
	bSelectable = true;
	Mobility = EComponentMobility::Movable;
	bCanEverAffectNavigation = false;

	// Overlap events are expensive and not needed (at least at the moment) as we don't need to collide against other component.
	SetGenerateOverlapEvents(false);
}

#if WITH_EDITOR

void UGroomSolverComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGroomSolverComponent, MeshDeformer))
	{
		SetDeformerSolver(MeshDeformer);
	}
}

#endif

void UGroomSolverComponent::OnRegister()
{
	Super::OnRegister();

	DeformerInstance = (MeshDeformer != nullptr) ? MeshDeformer->CreateInstance(this, DeformerSettings) : nullptr;
	GroomSolverProxy.DeformerInstance = DeformerInstance;
}

void UGroomSolverComponent::OnUnregister()
{
	Super::OnUnregister();
	
	DeformerInstance = nullptr;
	GroomSolverProxy.DeformerInstance = nullptr;
}

void UGroomSolverComponent::SelectDynamicCurves() 
{
	TArray<FVector> ViewLocations;
	UE::Groom::Private::GatherViewLocations(GetWorld(), ViewLocations);

	SolverSettings.CurveDynamicIndices.Reset();
	SolverSettings.CurveKinematicIndices.Reset();
	SolverSettings.PointDynamicIndices.Reset();
	SolverSettings.PointKinematicIndices.Reset();
	SolverSettings.ObjectDistanceLods.Reset();

	static constexpr int32 GroupSize = 64;

	if (ViewLocations.Num() > 0)
	{
		uint32 CurveOffset = 0, PointOffset = 0;
		for(const TObjectPtr<UGroomComponent>& GroomComponent : GetGroomComponents())
		{
			if(GroomComponent && GroomComponent->GroomAsset)
			{
				const FBoxSphereBounds GroomBounds = GroomComponent->CalcBounds(GroomComponent->GetComponentTransform());
						
				FVector::FReal MinDistance = FMath::Square(WORLD_MAX);
				for (const FVector& ViewLocation : ViewLocations)
				{
					MinDistance = FMath::Min(MinDistance, (GroomBounds.Origin - ViewLocation).SizeSquared());
				}
				MinDistance = FMath::Sqrt(MinDistance);

				const bool bHasValidBounds = GetSolverSettings().MaxLODDistance >= GetSolverSettings().MinLODDistance;
				const FVector::FReal DistanceRatio = bHasValidBounds ?
					FMath::Clamp((MinDistance - GetSolverSettings().MinLODDistance) / (GetSolverSettings().MaxLODDistance - GetSolverSettings().MinLODDistance), 0.0f, 1.0f) : 1.0f;

				if(!bHasValidBounds)
				{
					UE_LOGF(LogTemp, Warning, "Groom solver max distance should be higher than the min distance")
				}
				
				const uint32 NumGroups = GroomComponent->GetGroupCount();

				int32 TotalNumPoints = 0;
				for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
				{
					const FHairGroupInstance* GroupInstance = GroomComponent->GetGroupInstance(GroupIndex);
					if(GroomComponent->IsDeformationEnable(GroupIndex) && GroupInstance && GroupInstance->Guides.IsValid())
					{
						TotalNumPoints += (1.0-DistanceRatio) * GroupInstance->Guides.GetData().GetNumPoints();
					}
				}
				const float PointsScale = FMath::Clamp(GetSolverSettings().MaxPointsCount / static_cast<float>(TotalNumPoints), 0.0f, 1.0f);
				for (uint32 GroupIndex = 0; GroupIndex < NumGroups; ++GroupIndex)
				{
					const FHairGroupInstance* GroupInstance = GroomComponent->GetGroupInstance(GroupIndex);
					if(GroomComponent->IsDeformationEnable(GroupIndex) && GroupInstance && GroupInstance->Guides.IsValid())
					{
						const int32 NumTotalCurves = GroupInstance->Guides.GetData().GetNumCurves();
						const int32 NumTotalPoints = GroupInstance->Guides.GetData().GetNumPoints();

						const int32 NumCurvePoints = NumTotalPoints / NumTotalCurves;
						const int32 NumDynamicCurves = (1.0-DistanceRatio) * NumTotalCurves * PointsScale;
						
						const int32 NumDynamicPoints = NumDynamicCurves * NumCurvePoints;
						const int32 NumKinematicPoints = NumTotalPoints-NumDynamicPoints;
						
						const int32 NumDynamicPadding = FMath::CeilToInt32(NumDynamicPoints / static_cast<float>(GroupSize)) * GroupSize - NumDynamicPoints;
						const int32 NumKinematicPadding = FMath::CeilToInt32(NumKinematicPoints / static_cast<float>(GroupSize)) * GroupSize - NumKinematicPoints;
						
						for (int32 CurveIndex = 0; CurveIndex < NumDynamicCurves; ++CurveIndex)
						{
							SolverSettings.CurveDynamicIndices.Add(CurveOffset + CurveIndex);
						}
						for (int32 CurveIndex = NumDynamicCurves; CurveIndex < NumTotalCurves; ++CurveIndex)
						{
							SolverSettings.CurveKinematicIndices.Add(CurveOffset + CurveIndex);
						}
						for (int32 PointIndex = 0; PointIndex < NumDynamicPoints; ++PointIndex)
						{
							SolverSettings.PointDynamicIndices.Add(PointOffset + PointIndex);
						}
						for (int32 PointIndex = 0; PointIndex < NumDynamicPadding; ++PointIndex)
						{
							SolverSettings.PointDynamicIndices.Add(INDEX_NONE);
						}
						for (int32 PointIndex = NumDynamicPoints; PointIndex < NumTotalPoints; ++PointIndex)
						{
							SolverSettings.PointKinematicIndices.Add(PointOffset + PointIndex);
						}
						for (int32 PointIndex = 0; PointIndex < NumKinematicPadding; ++PointIndex)
						{
							SolverSettings.PointKinematicIndices.Add(INDEX_NONE);
						}

						const int32 NumObjectLODs = FMath::CeilLogTwo(NumTotalCurves);
						const uint32 ObjectLod = NumObjectLODs - 1 - FMath::FloorLog2(NumDynamicCurves);

						SolverSettings.ObjectDistanceLods.Add(ObjectLod);
						
						CurveOffset += NumTotalCurves;
						PointOffset += NumTotalPoints;
						
						// Keep it for now for debugging since it is WIP
                        //UE_LOGF(LogTemp, Log, "Num Curves = %d Distance Ratio = %f Simulated Curves = %d | Num LODs = %d | Points Scale = %f",
						//	NumTotalCurves, DistanceRatio, NumDynamicCurves, FMath::CeilLogTwo(NumTotalCurves), PointsScale);
					}
				}
			}
		}
	}
}

void UGroomSolverComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if(DeformerInstance != nullptr)
	{
		MarkRenderDynamicDataDirty();
	}

	SelectDynamicCurves();
}

void UGroomSolverComponent::SendRenderDynamicData_Concurrent()
{
	Super::SendRenderDynamicData_Concurrent();
	
	if (DeformerInstance != nullptr )
	{
		UMeshDeformerInstance::FEnqueueWorkDesc Desc;
		Desc.Scene = GetScene();
		Desc.OwnerName = GetFName();
		Desc.ExecutionGroup = UMeshDeformerInstance::ExecutionGroup_BeginInitViews;
		DeformerInstance->EnqueueWork(Desc);
	}
}

void UGroomSolverComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	if (DeformerInstance)
	{
		DeformerInstance->AllocateResources();
	}
	
	Super::CreateRenderState_Concurrent(Context);
}

void UGroomSolverComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (DeformerInstance)
	{
		DeformerInstance->ReleaseResources();
	}
}

void UGroomSolverComponent::AddGroomComponent(UGroomComponent* GroomPhysicsComponent)
{
	if(GroomPhysicsComponent)
	{ 
		GroomComponents.Add(GroomPhysicsComponent);
		GroomPhysicsComponent->SetGroomSolver(this);

#if WITH_EDITOR
		GroomPhysicsComponent->InvalidateAndRecreate();
#endif
	}
}

void UGroomSolverComponent::RemoveGroomComponent(UGroomComponent* GroomPhysicsComponent)
{
	if (GroomPhysicsComponent)
	{
		GroomComponents.Remove(GroomPhysicsComponent);
		GroomPhysicsComponent->SetGroomSolver(nullptr);

#if WITH_EDITOR
		GroomPhysicsComponent->InvalidateAndRecreate();
#endif
	}
}

void UGroomSolverComponent::ResetGroomComponents()
{
	for(UGroomComponent* GroomPhysicsComponent : GroomComponents)
	{
		GroomPhysicsComponent->SetGroomSolver(nullptr);
	}
	GroomComponents.Reset();
}

void UGroomSolverComponent::AddCollisionComponent(UMeshComponent* CollisionComponent, const int32 LODIndex)
{
	CollisionComponents.Add(CollisionComponent, LODIndex);
}

void UGroomSolverComponent::RemoveCollisionComponent(UMeshComponent* CollisionComponent)
{
	CollisionComponents.Remove(CollisionComponent);
}

void UGroomSolverComponent::ResetCollisionComponents()
{
	CollisionComponents.Reset();
}

void UGroomSolverComponent::SetDeformerSolver(UMeshDeformer* DeformerSolver)
{
	MeshDeformer = DeformerSolver;
	if(MeshDeformer && !IsBeingDestroyed())
	{
		DeformerSettings = MeshDeformer->CreateSettingsInstance(this);
        DeformerInstance = MeshDeformer->CreateInstance(this, DeformerSettings);
		GroomSolverProxy.DeformerInstance = DeformerInstance;
	}
	else
	{
		DeformerSettings = nullptr;
		DeformerInstance = nullptr;
		GroomSolverProxy.DeformerInstance =  nullptr;
	}
	MarkRenderDynamicDataDirty();
}


