// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsSolverModule.h"
#include "AddSolverDeformerNode.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "HairStrandsSolver"

void FHairStrandsSolverModule::StartupModule()
{
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddSolverDeformerDataflowNode);

	GroomCacheAdapter = MakeUnique<UE::Groom::FGroomCacheAdapter>();
	Chaos::RegisterAdapter(GroomCacheAdapter.Get());
}

void FHairStrandsSolverModule::ShutdownModule()
{
	Chaos::UnregisterAdapter(GroomCacheAdapter.Get());
	GroomCacheAdapter = nullptr;
}

IMPLEMENT_MODULE(FHairStrandsSolverModule, HairStrandsSolver)

#undef LOCTEXT_NAMESPACE
