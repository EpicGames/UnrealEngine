// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSimulationContext.h"
#include "Dataflow/DataflowSimulationNodes.h"

#include "AddSolverDeformerNode.generated.h"

class UMeshDeformer;

/** Add a graph deformer to the groom simulation */
USTRUCT(meta = (DataflowSimulation))
struct FAddSolverDeformerDataflowNode : public FDataflowSimulationNode
{
	GENERATED_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddSolverDeformerDataflowNode, "AddSolverDeformer", "Physics|Solver", UDataflow::SimulationTag)

public:
	
	FAddSolverDeformerDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
	: FDataflowSimulationNode(InParam, InGuid)
	{
		RegisterInputConnection(&SimulationTime);
		RegisterInputConnection(&PhysicsSolvers);
		RegisterOutputConnection(&PhysicsSolvers, &PhysicsSolvers);
	}

	/** Physics solvers to advance in time */
	UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "PhysicsSolvers"))
	TArray<FDataflowSimulationProperty> PhysicsSolvers;
	
    /** Delta time to use to advance the solver*/
    UPROPERTY(Transient, SkipSerialization, Meta = (DataflowInput))
    FDataflowSimulationTime SimulationTime = FDataflowSimulationTime(0.0f,0.0f);
	
	/** Graph deformer solver the component is using. Declared as base UMeshDeformer to be
	 *  compatible with the MeshDeformer property type customization (which asserts the picker's
	 *  AllowedClass is assignable to the property class). Cast to UOptimusDeformer at use sites. */
    UPROPERTY(EditAnywhere, Category = "Groom")
    TObjectPtr<UMeshDeformer> MeshDeformer = nullptr;

	/** List of deformer numeric inputs that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowNumericTypes> DeformerNumericInputs;

	/** List of deformer vector inputs that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowVectorTypes> DeformerVectorInputs;

	/** List of deformer string inputs that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowStringTypes> DeformerStringInputs;

	/** List of deformer bool inputs that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowBoolTypes> DeformerBoolInputs;

	/** List of deformer transform inputs that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowTransformTypes> DeformerTransformInputs;

	/** List of deformer numeric arrays that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowNumericArrayTypes> DeformerNumericArrays;
	
	/** List of deformer vector arrays that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowVectorArrayTypes> DeformerVectorArrays;
	
	/** List of deformer string arrays that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowStringArrayTypes> DeformerStringArrays;
	
	/** List of deformer bool arrays that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowBoolArrayTypes> DeformerBoolArrays;

	/** List of deformer transform arrays that will appear in the option pins */
	UPROPERTY()
	TArray<FDataflowTransformArrayTypes> DeformerTransformArrays;
	
	//~ Begin UDataflowNode interface
	virtual void EvaluateSimulation(UE::Dataflow::FDataflowSimulationContext& SimulationContext, const FDataflowOutput* Output) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return DeformerNumericInputs.IsEmpty() && DeformerVectorInputs.IsEmpty() && DeformerStringInputs.IsEmpty(); }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual bool CanRemovePin() const override { return !CanAddPin(); }
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void OnInvalidate() override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End UDataflowNode interface
};
