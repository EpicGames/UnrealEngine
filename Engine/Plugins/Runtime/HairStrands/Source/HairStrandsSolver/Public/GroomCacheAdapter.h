// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GroomBuilder.h"
#include "Chaos/Adapters/CacheAdapter.h"
#include "GroomCache.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGroomCache, Verbose, All);
class UGroomComponent;
class UGroomCache;

namespace UE::Groom
{
	/**
	 * Groom cache adapter to be able to cache groom simulation data through the chaos cache system
	 */
	class FGroomCacheAdapter : public Chaos::FComponentCacheAdapter
	{
	public:

		virtual ~FGroomCacheAdapter();

		// ~Begin FComponentCacheAdapter interface
		virtual SupportType	SupportsComponentClass(UClass* InComponentClass) const override;
		virtual UClass*	GetDesiredClass() const override;
		virtual uint8 GetPriority() const override;
		virtual FGuid GetGuid() const override;
		virtual void SetRestState(UPrimitiveComponent* InComponent, UChaosCache* InCache, const FTransform& InRootTransform, Chaos::FReal InTime) const override;
		virtual bool InitializeForRecord(UPrimitiveComponent* InComponent, FObservedComponent& InObserved) override;
		virtual bool InitializeForPlayback(UPrimitiveComponent* InComponent, FObservedComponent& InObserved, float InTime) override;
		virtual void InitializeForLoad(UPrimitiveComponent* InComponent, FObservedComponent& InObserved) override;
		virtual bool ValidForPlayback(UPrimitiveComponent* InComponent, UChaosCache* InCache) const override;
		virtual void Finalize() override;
		virtual Chaos::FPhysicsSolver* GetComponentSolver(UPrimitiveComponent* InComponent) const override;
		virtual Chaos::FPhysicsSolverEvents* BuildEventsSolver(UPrimitiveComponent* InComponent) const override;
		virtual void Record_PostSolve(UPrimitiveComponent* InComp, const FTransform& InRootTransform, FPendingFrameWrite& OutFrame, Chaos::FReal InTime) const override;
		virtual void Playback_PreSolve(UPrimitiveComponent* InComponent, UChaosCache* InCache, Chaos::FReal InTime, FPlaybackTickRecord& TickRecord, TArray<Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>*>& OutUpdatedRigids) const override {}
		virtual void WaitForSolverTasks(UPrimitiveComponent* InComponent) const override;
		// ~End FComponentCacheAdapter interface
		
	private:

		/** Groom cache aata  that will store the readback results */
		struct FGroomCacheData
		{
			FGroomCacheData() : CacheProcessor(EGroomCacheType::Guides, EGroomCacheAttributes::Position)
			{}
			
			/** Cache processor to be used to build the groom cache */
			FGroomCacheProcessor CacheProcessor;

			/** Cache name to create the groom cache */
			FString CacheName;

			/** Animation info to build the cache */
			FGroomAnimationInfo AnimInfo;

			/** Guides positions to be used to build the cache */
			TArray<TSharedPtr<FStrandsPositionOutput>> PositionsBuffer;

			/** Cache asset used to record the positions */
			UGroomCache* CacheAsset = nullptr;

			/** Cache time at which the task is enqueued */
			TArray<float> CacheTimes;
		};
		
#if WITH_EDITORONLY_DATA
		/** Fill the cache processor from the position buffer */
		static void FillCacheProcessor(const UGroomComponent* GroomComponent, FGroomCacheData& GroomCache, const int32 NumFrames, float& MaxTime, int32& MaxFrame);
#endif

		/** List of groom caches that will be used to record/play caches */
		mutable TMap<UPrimitiveComponent*,FGroomCacheData> GroomCaches;

		/** Boolean to check if we are reading the cache or not */
		bool bIsLoading = false;

		/** List of readback tasks to process */
		mutable FGraphEventArray ReadbackTasks;
	};
}    // namespace UE::Groom
