// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GroomComponent.h"
#include "Components/MeshComponent.h"
#include "UObject/ObjectMacros.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"

#include "GroomSolverComponent.generated.h"

#define UE_API HAIRSTRANDSSOLVER_API

class FRHIGPUBufferReadback;
class UMeshDeformer;

/** Solver settings that will be used in dataflow/deformergraph*/
USTRUCT()
struct FGroomSolverSettings
{
	GENERATED_BODY()

	/** Maximum LOD distance (if distance between the component and the views is higher that this threshold, no simulation)*/
	UPROPERTY(EditAnywhere, Category = "LODs", meta = (ClampMin = "0", UIMin = "0"))
	float MaxLODDistance = 1000;
	
	/** Minimum LOD distance (if distance between the component and the views is lower that this threshold, no simulation)*/
	UPROPERTY(EditAnywhere, Category = "LODs", meta = (ClampMin = "0", UIMin = "0"))
	float MinLODDistance = 100;

	/** Maximum number of points the solver is going to simulate */
	UPROPERTY(EditAnywhere, Category = "LODs")
	float MaxPointsCount = 100000;

	/** List of dynamic curves */
	TArray<int32> CurveDynamicIndices;

	/** List of kinematic curves */
	TArray<int32> CurveKinematicIndices;

	/** List of dynamic points */
	TArray<int32> PointDynamicIndices;

	/** List of kinematic points */
	TArray<int32> PointKinematicIndices;
	
	/** List of object curve lods */
	TArray<int32> ObjectDistanceLods;
};

/** Dataflow groom solver proxy used in dataflow simulation */ 
USTRUCT(BlueprintType)
struct FDataflowGroomSolverProxy : public FDataflowPhysicsSolverProxy
#if CPP
	, public Chaos::FPhysicsSolverEvents
#endif
{
	GENERATED_USTRUCT_BODY()

	FDataflowGroomSolverProxy(FGroomSolverSettings InProp = FGroomSolverSettings()) :
		FDataflowPhysicsSolverProxy()
	{
	}
	virtual ~FDataflowGroomSolverProxy() override
	{
		EventTeardown.Broadcast();
	}

	//~ Begin FPhysicsSolverInterface interface
	virtual void AdvanceSolverDatas(const float DeltaTime) override
	{
		EventPreSolve.Broadcast(DeltaTime);
		EventPreBuffer.Broadcast(DeltaTime);
		EventPostSolve.Broadcast(DeltaTime);
	}
	
	virtual bool IsValid() const override { return false;}
	
	virtual const UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	//~ End FPhysicsSolverInterface interface

	/** Deformer instance coming from the component */
	UMeshDeformerInstance* DeformerInstance = nullptr;

	/** Deformer Instance GUIDs */
	TMap<FGuid, FGuid> DeformerInstanceGuids;
};

template <>
struct TStructOpsTypeTraits<FDataflowGroomSolverProxy> : public TStructOpsTypeTraitsBase2<FDataflowGroomSolverProxy>
{
	enum { WithCopy = false };
};

/** Groom solver component in which groom component could be added to be solver together */
UCLASS(HideCategories = (Object, Physics, Collision, Activation, Mobility, "Components|Activation"), meta = (BlueprintSpawnableComponent), ClassGroup = Physics, MinimalAPI)
class UGroomSolverComponent : public UMeshComponent, public IDataflowPhysicsSolverInterface
{
	GENERATED_BODY()

public:

	UE_API UGroomSolverComponent(const FObjectInitializer& ObjectInitializer);

	// Begin IDataflowPhysicsSolverInterface overrides
	virtual FString GetSimulationName() const override {return GetName();};
	virtual FDataflowSimulationAsset& GetSimulationAsset() override {return SimulationAsset;};
	virtual const FDataflowSimulationAsset& GetSimulationAsset() const override {return SimulationAsset;};
	virtual FDataflowSimulationProxy* GetSimulationProxy() override {return &GroomSolverProxy;}
	virtual const FDataflowSimulationProxy* GetSimulationProxy() const  override {return &GroomSolverProxy;}
	virtual void BuildSimulationProxy() override {}
	virtual void ResetSimulationProxy() override {}
	virtual void WriteToSimulation(const float DeltaTime, const bool bAsyncTask) override {}
	virtual void ReadFromSimulation(const float DeltaTime, const bool bAsyncTask) override {}
	// End IDataflowPhysicsSolverInterface overrides

	//~ Begin UPrimitiveComponent Interface.
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UPrimitiveComponent Interface.
	
	//~ Begin USceneComponent interface 
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	UE_API virtual void SendRenderDynamicData_Concurrent() override;
	UE_API virtual void DestroyRenderState_Concurrent() override;
	//~ End USceneComponent interface

	/** Add a groom component to the solver */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void AddGroomComponent(UGroomComponent* GroomComponent);
	
	/** Remove a groom component to the solver */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void RemoveGroomComponent(UGroomComponent* GroomComponent);

	/** Reset the solver groom components */
    UFUNCTION(BlueprintCallable, Category = "Groom")
    UE_API void ResetGroomComponents();

	/** Add a collision component to the solver */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void AddCollisionComponent(UMeshComponent* CollisionComponent, const int32 LODIndex);

	/** Remove a collision component to the solver */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void RemoveCollisionComponent(UMeshComponent* CollisionComponent);

	/** Reset the solver collision components */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void ResetCollisionComponents();

	/** Get the solver groom components */
	const TSet< TObjectPtr<UGroomComponent> >& GetGroomComponents() const {return GroomComponents;}

	/** Get the solver collision components */
	const TMap< TObjectPtr<UMeshComponent>, int32 >& GetCollisionComponents() const {return CollisionComponents;}

	/* Change the MeshDeformer solver that is used for this Component. */
	UFUNCTION(BlueprintCallable, Category = "Groom")
	UE_API void SetDeformerSolver(UMeshDeformer* DeformerSolver);

	/** Get the groom solver settings */
	const FGroomSolverSettings& GetSolverSettings() const {return SolverSettings;};

	/** Return the mesh deformer instance */
	UMeshDeformerInstance* GetMeshDeformerInstance() const {return DeformerInstance;}

private :

	/** Select the number of dynamic curves based on the distance component-views */
	void SelectDynamicCurves();

	/** Solver settings used to control the simulation */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FGroomSolverSettings SolverSettings;

	/* Solver dataflow asset used to advance in time */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta=(EditConditionHides), AdvancedDisplay)
	FDataflowSimulationAsset SimulationAsset;

	/** List of physics objects registered to the solver */
	UPROPERTY(VisibleAnywhere, Category = "Groom")
	TSet< TObjectPtr<UGroomComponent> > GroomComponents;

	/** Graph deformer solver the component is using */
	UPROPERTY(EditAnywhere, Category = "Groom", meta = (DisplayPriority = 2))
	TObjectPtr<UMeshDeformer> MeshDeformer = nullptr;
	
	/** Object containing state for the MeshDeformer. */
	UPROPERTY(Transient)
	TObjectPtr<UMeshDeformerInstance> DeformerInstance = nullptr;

	/** Object containing instance settings for the MeshDeformer. */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Groom", meta = (DisplayName = "Deformer Settings", EditCondition = "DeformerSettings!=nullptr", ShowOnlyInnerProperties))
	TObjectPtr<UMeshDeformerInstanceSettings> DeformerSettings = nullptr;

	/** Groom solver proxy to be used in dataflow */
	FDataflowGroomSolverProxy GroomSolverProxy;

	/** List of collision components registered to the solver */
	UPROPERTY(VisibleAnywhere, Category = "Groom")
	TMap< TObjectPtr<UMeshComponent>, int32 > CollisionComponents;
};

#undef UE_API 
