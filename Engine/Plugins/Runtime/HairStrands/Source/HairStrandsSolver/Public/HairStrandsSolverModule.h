// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroomCacheAdapter.h"
#include "Modules/ModuleInterface.h"

/** Hair dataflow construction module public interface */
class FHairStrandsSolverModule : public IModuleInterface
{
	public :
	
	//~ IModuleInterface begin interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ IModuleInterface end interface

private:
	/** Cache adapter to be used when caching grooms in dataflow */
	TUniquePtr<UE::Groom::FGroomCacheAdapter> GroomCacheAdapter;
};
